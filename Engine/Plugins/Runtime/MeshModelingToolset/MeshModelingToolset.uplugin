{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mesh Modeling Toolset",
	"Description": "A set of modules implementing 3D mesh creation and editing based on the Interactive Tools Framework",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Hidden": true,
	"Modules": [
		{
			"Name": "MeshModelingTools",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshModelingToolsEditorOnly",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelingComponents",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelingComponentsEditorOnly",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},		
		{
			"Name": "ModelingOperators",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ModelingOperatorsEditorOnly",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "SkeletalMeshModifiers",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "ProxyLODPlugin",
			"Enabled": true
		},
		{
			"Name": "PlanarCut",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		}
	]
}
