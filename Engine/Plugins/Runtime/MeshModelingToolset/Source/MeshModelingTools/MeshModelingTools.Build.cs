// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshModelingTools : ModuleRules
{
	public MeshModelingTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InputCore",
				"InteractiveToolsFramework",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"DynamicMesh",
				"MeshConversion",
				"MeshDescription",
				"StaticMeshDescription",
				"ModelingComponents",
				"ModelingOperators",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"SlateCore",
				// ... add private dependencies that you statically link with here ...
			}
		);

		bDisableAutoRTFMInstrumentation = true;
	}
}
