// Copyright Epic Games, Inc. All Rights Reserved.

#include "AddPrimitiveTool.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "ToolBuilderUtil.h"
#include "InteractiveToolManager.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "Selection/ToolSelectionUtil.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "ModelingObjectsCreationAPI.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"

#include "Generators/CapsuleGenerator.h"
#include "Generators/SweepGenerator.h"
#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Generators/SphereGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/DiscMeshGenerator.h"
#include "Generators/StairGenerator.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "FaceGroupUtil.h"

#include "DynamicMeshEditor.h"
#include "UObject/PropertyIterator.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AddPrimitiveTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UAddPrimitiveTool"

/*
 * ToolBuilder
 */
bool UAddPrimitiveToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UAddPrimitiveToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UAddPrimitiveTool* NewTool = nullptr;
	switch (ShapeType)
	{
	case EMakeMeshShapeType::Box:
		NewTool = NewObject<UAddBoxPrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Cylinder:
		NewTool = NewObject<UAddCylinderPrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Cone:
		NewTool = NewObject<UAddConePrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Arrow:
		NewTool = NewObject<UAddArrowPrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Rectangle:
		NewTool = NewObject<UAddRectanglePrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Disc:
		NewTool = NewObject<UAddDiscPrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Torus:
		NewTool = NewObject<UAddTorusPrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Sphere:
		NewTool = NewObject<UAddSpherePrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Stairs:
		NewTool = NewObject<UAddStairsPrimitiveTool>(SceneState.ToolManager);
		break;
	case EMakeMeshShapeType::Capsule:
		NewTool = NewObject<UAddCapsulePrimitiveTool>(SceneState.ToolManager);
		break;
	default:
		break;
	}
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}

void UAddPrimitiveTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

UAddPrimitiveTool::UAddPrimitiveTool(const FObjectInitializer&)
{
	ShapeSettings = CreateDefaultSubobject<UProceduralShapeToolProperties>(TEXT("ShapeSettings"));
	// CreateDefaultSubobject automatically sets RF_Transactional flag, we need to clear it so that undo/redo doesn't affect tool properties
	ShapeSettings->ClearFlags(RF_Transactional);
}

bool UAddPrimitiveTool::CanAccept() const
{
	return CurrentState == EState::AdjustingSettings;
}

void UAddPrimitiveTool::Setup()
{
	USingleClickTool::Setup();

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	AddToolPropertySource(ShapeSettings);
	
	ShapeSettings->WatchProperty(ShapeSettings->TargetSurface, [this](EMakeMeshPlacementType){UpdateTargetSurface();});
	ShapeSettings->PolygroupMode = GetDefaultPolygroupMode();
	ShapeSettings->RestoreProperties(this);

	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	AddToolPropertySource(MaterialProperties);
	MaterialProperties->RestoreProperties(this);

	// create preview mesh object
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(TargetWorld, FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr); 
	PreviewMesh->SetVisible(false);
	PreviewMesh->SetMaterial(MaterialProperties->Material.Get());
	PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);

	UTransformProxy* TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->OnTransformChanged.AddWeakLambda(this, [this](UTransformProxy*, FTransform NewTransform) 
		{
			PreviewMesh->SetTransform(NewTransform);
		});

	// TODO: It might be nice to use a repositionable gizmo, but the drag alignment mechanic can't currently 
	// hit the preview mesh, which makes middle click repositioning feel broken and not very useful.
	Gizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(),
		ETransformGizmoSubElements::StandardTranslateRotate, this);
	Gizmo->SetActiveTarget(TransformProxy, GetToolManager());

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->AddToGizmo(Gizmo);

	UpdatePreviewMesh();
	SetState(EState::PlacingPrimitive);
}

void UAddPrimitiveTool::SetState(EState NewState)
{
	CurrentState = NewState;

	bool bGizmoActive = (CurrentState == EState::AdjustingSettings);
	Gizmo->SetVisibility(bGizmoActive && ShapeSettings->bShowGizmo);
	ShapeSettings->bShowGizmoOptions = bGizmoActive;
	NotifyOfPropertyChangeByTool(ShapeSettings);

	if (CurrentState == EState::PlacingPrimitive)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartAddPrimitiveTool", "This Tool creates new shapes. Click in the scene to choose initial placement of mesh."),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		// Initialize gizmo to current preview location
		Gizmo->ReinitializeGizmoTransform(PreviewMesh->GetTransform());

		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartAddPrimitiveTool2", "Alter shape settings in the detail panel or modify placement with gizmo, then accept the tool."),
			EToolMessageLevel::UserNotification);
	}
}


void UAddPrimitiveTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset();
	}

	DragAlignmentMechanic->Shutdown();
	DragAlignmentMechanic = nullptr;
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	Gizmo = nullptr;

	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	OutputTypeProperties->SaveProperties(this);
	ShapeSettings->SaveProperties(this);
	MaterialProperties->SaveProperties(this);
}


void UAddPrimitiveTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);
}



void UAddPrimitiveTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	// Because of how the ShapeSettings property set is implemented in this Tool, changes to it are transacted,
	// and if the user exits the Tool and then tries to undo/redo those transactions, this function will end up being called.
	// So we need to ensure that we handle this case.
	if (PreviewMesh)
	{
		PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
		PreviewMesh->SetMaterial(MaterialProperties->Material.Get());
		UpdatePreviewMesh();
	}

	if (Gizmo)
	{
		Gizmo->SetVisibility(CurrentState == EState::AdjustingSettings && ShapeSettings->bShowGizmo);
	}
}




FInputRayHit UAddPrimitiveTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Result(0);
	Result.bHit = (CurrentState == EState::PlacingPrimitive);
	return Result;
}

void UAddPrimitiveTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	UpdatePreviewPosition(DevicePos);
}

bool UAddPrimitiveTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	UpdatePreviewPosition(DevicePos);
	return true;
}

void UAddPrimitiveTool::OnEndHover()
{
	// do nothing
}



void UAddPrimitiveTool::UpdatePreviewPosition(const FInputDeviceRay& DeviceClickPos)
{
	FRay ClickPosWorldRay = DeviceClickPos.WorldRay;

	// hit position (temp)
	bool bHit = false;

	auto RaycastPlaneWithFallback = [](const FVector3d& Origin, const FVector3d& Direction, const FPlane& Plane, double FallbackDistance = 1000) -> FVector3d
	{
		double IntersectTime = FMath::RayPlaneIntersectionParam(Origin, Direction, Plane);
		if (IntersectTime < 0 || !FMath::IsFinite(IntersectTime))
		{
			IntersectTime = FallbackDistance;
		}
		return Origin + Direction * IntersectTime;
	};

	FPlane DrawPlane(FVector::ZeroVector, FVector(0, 0, 1));
	if (ShapeSettings->TargetSurface == EMakeMeshPlacementType::GroundPlane)
	{
		FVector3d DrawPlanePos = RaycastPlaneWithFallback(ClickPosWorldRay.Origin, ClickPosWorldRay.Direction, DrawPlane);
		bHit = true;
		ShapeFrame = FFrame3d(DrawPlanePos);
	}
	else if (ShapeSettings->TargetSurface == EMakeMeshPlacementType::OnScene)
	{
		// cast ray into scene
		FHitResult Result;
		bHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, ClickPosWorldRay);
		if (bHit)
		{
			FVector3d Normal = (FVector3d)Result.ImpactNormal;
			if (!ShapeSettings->bAlignToNormal)
			{
				Normal = FVector3d::UnitZ();
			}
			ShapeFrame = FFrame3d((FVector3d)Result.ImpactPoint, Normal);
			ShapeFrame.ConstrainedAlignPerpAxes();
		}
		else
		{
			// fall back to ground plane if we don't have a scene hit
			FVector3d DrawPlanePos = RaycastPlaneWithFallback(ClickPosWorldRay.Origin, ClickPosWorldRay.Direction, DrawPlane);
			bHit = true;
			ShapeFrame = FFrame3d(DrawPlanePos);
		}
	}
	else
	{
		bHit = true;
		ShapeFrame = FFrame3d();
	}

	// Snap to grid
	USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GetToolManager());
	if (SnapManager)
	{
		FSceneSnapQueryRequest Request;
		Request.RequestType = ESceneSnapQueryType::Position;
		Request.TargetTypes = ESceneSnapQueryTargetType::Grid;
		Request.Position = (FVector)ShapeFrame.Origin;
		TArray<FSceneSnapQueryResult> Results;
		if (SnapManager->ExecuteSceneSnapQuery(Request, Results))
		{
			ShapeFrame.Origin = (FVector3d)Results[0].Position;
		}
	}

	if (ShapeSettings->Rotation != 0)
	{
		ShapeFrame.Rotate(FQuaterniond(ShapeFrame.Z(), ShapeSettings->Rotation, true));
	}

	if (bHit)
	{
		PreviewMesh->SetVisible(true);
		PreviewMesh->SetTransform(ShapeFrame.ToFTransform());
	}
	else
	{
		PreviewMesh->SetVisible(false);
	}
}

void UAddPrimitiveTool::UpdatePreviewMesh() const
{
	FDynamicMesh3 NewMesh;
	GenerateMesh( &NewMesh );

	if (ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerShape)
	{
		FaceGroupUtil::SetGroupID(NewMesh, 0);
	}

	if (MaterialProperties->UVScale != 1.0 || MaterialProperties->bWorldSpaceUVScale)
	{
		FDynamicMeshEditor Editor(&NewMesh);
		const float WorldUnitsInMetersFactor = MaterialProperties->bWorldSpaceUVScale ? .01f : 1.0f;
		Editor.RescaleAttributeUVs(MaterialProperties->UVScale * WorldUnitsInMetersFactor, MaterialProperties->bWorldSpaceUVScale);
	}

	// set mesh position
	const FAxisAlignedBox3d Bounds = NewMesh.GetBounds(true);
	FVector3d TargetOrigin = Bounds.Center();
	if (!ShouldCenterXY())
	{
		TargetOrigin.X = TargetOrigin.Y = 0;
	}
	if (ShapeSettings->PivotLocation == EMakeMeshPivotLocation::Base)
	{
		TargetOrigin.Z = Bounds.Min.Z;
	}
	else if (ShapeSettings->PivotLocation == EMakeMeshPivotLocation::Top)
	{
		TargetOrigin.Z = Bounds.Max.Z;
	}
	for (const int Vid : NewMesh.VertexIndicesItr())
	{
		FVector3d Pos = NewMesh.GetVertex(Vid);
		Pos -= TargetOrigin;
		NewMesh.SetVertex(Vid, Pos);
	}

	PreviewMesh->UpdatePreview(&NewMesh);

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	const bool CalculateTangentsSuccessful = PreviewMesh->CalculateTangents();
	checkSlow(CalculateTangentsSuccessful);
}

void UAddPrimitiveTool::UpdateTargetSurface()
{
	if (ShapeSettings->TargetSurface == EMakeMeshPlacementType::AtOrigin)
	{
		// default ray is used as coordinates will not be needed to set position at origin
		const FInputDeviceRay DefaultRay = FInputDeviceRay();
		UpdatePreviewPosition(DefaultRay);

		SetState(EState::AdjustingSettings);
		GetToolManager()->EmitObjectChange(this, MakeUnique<FStateChange>(PreviewMesh->GetTransform()),
				LOCTEXT("PlaceMeshTransaction", "Place Mesh"));
	} else
	{
		SetState(EState::PlacingPrimitive);
	}
}



bool UAddPrimitiveTool::SupportsWorldSpaceFocusBox()
{
	return PreviewMesh != nullptr;
}

FBox UAddPrimitiveTool::GetWorldSpaceFocusBox()
{
	if (PreviewMesh)
	{
		if (UPrimitiveComponent* Component = PreviewMesh->GetRootComponent())
		{
			return Component->Bounds.GetBox();
		}
	}
	return FBox();
}

bool UAddPrimitiveTool::SupportsWorldSpaceFocusPoint()
{
	return PreviewMesh != nullptr;

}

bool UAddPrimitiveTool::GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut)
{
	if (PreviewMesh)
	{
		FHitResult HitResult;
		if (PreviewMesh->FindRayIntersection(WorldRay, HitResult))
		{
			PointOut = HitResult.ImpactPoint;
			return true;
		}
	}
	return false;
}


FInputRayHit UAddPrimitiveTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Result(0);
	Result.bHit = (CurrentState == EState::PlacingPrimitive);
	return Result;
}

void UAddPrimitiveTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (!ensure(CurrentState == EState::PlacingPrimitive))
	{
		return;
	}

	UpdatePreviewPosition(ClickPos);
	SetState(EState::AdjustingSettings);
	GetToolManager()->EmitObjectChange(this, MakeUnique<FStateChange>(PreviewMesh->GetTransform()),
		LOCTEXT("PlaceMeshTransaction", "Place Mesh"));
}

void UAddPrimitiveTool::GenerateAsset()
{
	UMaterialInterface* Material = PreviewMesh->GetMaterial();

	const FDynamicMesh3* CurMesh = PreviewMesh->GetPreviewDynamicMesh();

	GetToolManager()->BeginUndoTransaction(LOCTEXT("AddPrimitiveToolTransactionName", "Add Primitive Tool"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = PreviewMesh->GetTransform();
	NewMeshObjectParams.BaseName = AssetName;
	NewMeshObjectParams.Materials.Add(Material);
	NewMeshObjectParams.SetMesh(CurMesh);
	OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() )
	{
		if (Result.NewActor != nullptr)
		{
			// select newly-created object
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
		}
	}

	GetToolManager()->EndUndoTransaction();
}

void UAddPrimitiveTool::FStateChange::Apply(UObject* Object)
{
	UAddPrimitiveTool* Tool = Cast<UAddPrimitiveTool>(Object);

	// Set preview transform before changing state so that the adjustment gizmo is initialized properly
	Tool->PreviewMesh->SetTransform(MeshTransform);

	Tool->SetState(EState::AdjustingSettings);
}

void UAddPrimitiveTool::FStateChange::Revert(UObject* Object)
{
	UAddPrimitiveTool* Tool = Cast<UAddPrimitiveTool>(Object);
	Tool->SetState(EState::PlacingPrimitive);
}


UAddBoxPrimitiveTool::UAddBoxPrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralBoxToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Box");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("BoxToolName", "Create Box"));
}

void UAddBoxPrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	FGridBoxMeshGenerator BoxGen;
	const UProceduralBoxToolProperties* BoxSettings = Cast<UProceduralBoxToolProperties>(ShapeSettings);
	BoxGen.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), 0.5*FVector3d(BoxSettings->Depth, BoxSettings->Width, BoxSettings->Height));
	BoxGen.EdgeVertices = FIndex3i(BoxSettings->DepthSubdivisions + 1,
								   BoxSettings->WidthSubdivisions + 1,
								   BoxSettings->HeightSubdivisions + 1);
	BoxGen.bPolygroupPerQuad = ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad;
	BoxGen.Generate();
	OutMesh->Copy(&BoxGen);
}



UAddRectanglePrimitiveTool::UAddRectanglePrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralRectangleToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Rectangle");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("RectToolName", "Create Rectangle"));
}

void UAddRectanglePrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	auto* RectangleSettings = Cast<UProceduralRectangleToolProperties>(ShapeSettings);
	switch (RectangleSettings->RectangleType)
	{
	case EProceduralRectType::Rectangle:
	{
		FRectangleMeshGenerator RectGen;
		RectGen.Width = RectangleSettings->Depth;
		RectGen.Height = RectangleSettings->Width;
		RectGen.WidthVertexCount = RectangleSettings->DepthSubdivisions + 1;
		RectGen.HeightVertexCount = RectangleSettings->WidthSubdivisions + 1;
		RectGen.bSinglePolyGroup = (ShapeSettings->PolygroupMode != EMakeMeshPolygroupMode::PerQuad);
		RectGen.Generate();
		OutMesh->Copy(&RectGen);
		break;
	}
	case EProceduralRectType::RoundedRectangle:
	{
		FRoundedRectangleMeshGenerator RectGen;
		RectGen.Width = RectangleSettings->Depth;
		RectGen.Height = RectangleSettings->Width;
		RectGen.Radius = RectangleSettings->CornerRadius;
		if (RectangleSettings->bMaintainDimension)
		{
			constexpr double MinSide = UE_DOUBLE_KINDA_SMALL_NUMBER;
			constexpr double MinRadius = MinSide * .5;
			double SmallSide = FMath::Min(RectGen.Width, RectGen.Height);
			RectGen.Radius = FMath::Clamp(RectGen.Radius, MinRadius, (SmallSide - MinSide) * .5);
			RectGen.Width = FMath::Max(MinSide, RectGen.Width - RectGen.Radius * 2);
			RectGen.Height = FMath::Max(MinSide, RectGen.Height - RectGen.Radius * 2);
			// Update the Radius in the UI, if it was clamped
			if (RectGen.Radius != RectangleSettings->CornerRadius)
			{
				RectangleSettings->CornerRadius = RectGen.Radius;
				NotifyOfPropertyChangeByTool(RectangleSettings);
			}
		}
		RectGen.WidthVertexCount = RectangleSettings->DepthSubdivisions + 1;
		RectGen.HeightVertexCount = RectangleSettings->WidthSubdivisions + 1;
		RectGen.bSinglePolyGroup = (ShapeSettings->PolygroupMode != EMakeMeshPolygroupMode::PerQuad);
		RectGen.AngleSamples = RectangleSettings->CornerSlices - 1;
		RectGen.Generate();
		OutMesh->Copy(&RectGen);
		break;
	}
	}
}


UAddDiscPrimitiveTool::UAddDiscPrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralDiscToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Disc");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("DiscToolName", "Create Disc"));
}

void UAddDiscPrimitiveTool::Setup()
{
	Super::Setup();


	// add watchers to ensure the hole radius is never larger than the disc radius when updating a punctured disc

	UProceduralDiscToolProperties* DiscSettings = Cast<UProceduralDiscToolProperties>(ShapeSettings);

	int32 HoleRadiusWatchID = DiscSettings->WatchProperty(DiscSettings->HoleRadius, 
															[DiscSettings](float HR) 
															{
																if (DiscSettings->DiscType == EProceduralDiscType::PuncturedDisc)
																{
																	DiscSettings->HoleRadius = FMath::Min(DiscSettings->Radius, HR);
																}
															});
	DiscSettings->WatchProperty(DiscSettings->Radius, 
								[DiscSettings, HoleRadiusWatchID](float R) 
								{
									if (DiscSettings->DiscType == EProceduralDiscType::PuncturedDisc)
									{
										DiscSettings->HoleRadius = FMath::Min(DiscSettings->HoleRadius, R);
										DiscSettings->SilentUpdateWatcherAtIndex(HoleRadiusWatchID);
									}
								});
}

void UAddDiscPrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	const UProceduralDiscToolProperties* DiscSettings = Cast<UProceduralDiscToolProperties>(ShapeSettings);
	switch (DiscSettings->DiscType)
	{
	case EProceduralDiscType::Disc:
	{
		FDiscMeshGenerator Gen;
		Gen.Radius = DiscSettings->Radius;
		Gen.AngleSamples = DiscSettings->RadialSlices;
		Gen.RadialSamples = DiscSettings->RadialSubdivisions;
		Gen.bSinglePolygroup = (ShapeSettings->PolygroupMode != EMakeMeshPolygroupMode::PerQuad);
		Gen.Generate();
		OutMesh->Copy(&Gen);
		break;
	}
	case EProceduralDiscType::PuncturedDisc:
	{
		FPuncturedDiscMeshGenerator Gen;
		Gen.Radius = DiscSettings->Radius;
		Gen.HoleRadius = FMath::Min(DiscSettings->HoleRadius, Gen.Radius * .999f); // hole cannot be bigger than outer radius
		Gen.AngleSamples = DiscSettings->RadialSlices;
		Gen.RadialSamples = DiscSettings->RadialSubdivisions;
		Gen.bSinglePolygroup = (ShapeSettings->PolygroupMode != EMakeMeshPolygroupMode::PerQuad);
		Gen.Generate();
		OutMesh->Copy(&Gen);
		break;
	}
	}
}



UAddTorusPrimitiveTool::UAddTorusPrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralTorusToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Torus");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("TorusToolName", "Create Torus"));
}

void UAddTorusPrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	FGeneralizedCylinderGenerator Gen;
	const UProceduralTorusToolProperties* TorusSettings = Cast<UProceduralTorusToolProperties>(ShapeSettings);
	Gen.CrossSection = FPolygon2d::MakeCircle(TorusSettings->MinorRadius, TorusSettings->MinorSlices);
	FPolygon2d PathCircle = FPolygon2d::MakeCircle(TorusSettings->MajorRadius, TorusSettings->MajorSlices);
	for (int Idx = 0; Idx < PathCircle.VertexCount(); Idx++)
	{
		Gen.Path.Add( FVector3d(PathCircle[Idx].X, PathCircle[Idx].Y, 0) );
	}
	Gen.bLoop = true;
	Gen.bCapped = false;
	Gen.bPolygroupPerQuad = ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad;
	Gen.InitialFrame = FFrame3d(Gen.Path[0]);
	Gen.Generate();
	OutMesh->Copy(&Gen);
}



UAddCylinderPrimitiveTool::UAddCylinderPrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralCylinderToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Cylinder");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("CylinderToolName", "Create Cylinder"));
}

void UAddCylinderPrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	FCylinderGenerator CylGen;
	const UProceduralCylinderToolProperties* CylinderSettings = Cast<UProceduralCylinderToolProperties>(ShapeSettings);
	CylGen.Radius[1] = CylGen.Radius[0] = CylinderSettings->Radius;
	CylGen.Height = CylinderSettings->Height;
	CylGen.AngleSamples = CylinderSettings->RadialSlices;
	CylGen.LengthSamples = CylinderSettings->HeightSubdivisions - 1;
	CylGen.bCapped = true;
	CylGen.bPolygroupPerQuad = ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad;
	CylGen.Generate();
	OutMesh->Copy(&CylGen);
}


UAddCapsulePrimitiveTool::UAddCapsulePrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralCapsuleToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Capsule");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("CapsuleToolName", "Create Capsule"));
}

void UAddCapsulePrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	FCapsuleGenerator CapGen;
	const UProceduralCapsuleToolProperties* CapsuleSettings = Cast<UProceduralCapsuleToolProperties>(ShapeSettings);
	CapGen.Radius = CapsuleSettings->Radius;
	CapGen.SegmentLength = CapsuleSettings->CylinderLength;
	CapGen.NumHemisphereArcSteps = CapsuleSettings->HemisphereSlices;
	CapGen.NumCircleSteps = CapsuleSettings->CylinderSlices;
	CapGen.NumSegmentSteps = CapsuleSettings->CylinderSubdivisions;
	CapGen.bPolygroupPerQuad = ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad;
	CapGen.Generate();
	OutMesh->Copy(&CapGen);
}


UAddConePrimitiveTool::UAddConePrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralConeToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Cone");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("ConeToolName", "Create Cone"));
}

void UAddConePrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	// Unreal's standard cone is just a cylinder with a very small top
	FCylinderGenerator CylGen;
	const UProceduralConeToolProperties* ConeSettings = Cast<UProceduralConeToolProperties>(ShapeSettings);
	CylGen.Radius[0] = ConeSettings->Radius;
	CylGen.Radius[1] = .01f;
	CylGen.Height = ConeSettings->Height;
	CylGen.AngleSamples = ConeSettings->RadialSlices;
	CylGen.LengthSamples = ConeSettings->HeightSubdivisions - 1;
	CylGen.bCapped = true;
	CylGen.bPolygroupPerQuad = ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad;
	CylGen.Generate();
	OutMesh->Copy(&CylGen);
}


UAddArrowPrimitiveTool::UAddArrowPrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralArrowToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Arrow");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("ArrowToolName", "Create Arrow"));
}

void UAddArrowPrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	FArrowGenerator ArrowGen;
	const UProceduralArrowToolProperties* ArrowSettings = Cast<UProceduralArrowToolProperties>(ShapeSettings);
	ArrowGen.StickRadius = ArrowSettings->ShaftRadius;
	ArrowGen.StickLength = ArrowSettings->ShaftHeight;
	ArrowGen.HeadBaseRadius = ArrowSettings->HeadRadius;
	ArrowGen.HeadTipRadius = .01f;
	ArrowGen.HeadLength = ArrowSettings->HeadHeight;
	ArrowGen.AngleSamples = ArrowSettings->RadialSlices;
	ArrowGen.bCapped = true;
	ArrowGen.bPolygroupPerQuad = ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad;
	if (ArrowSettings->HeightSubdivisions > 1)
	{
		const int AdditionalLengthsSamples = ArrowSettings->HeightSubdivisions - 1;
		ArrowGen.AdditionalLengthSamples[0] = AdditionalLengthsSamples;
		ArrowGen.AdditionalLengthSamples[1] = AdditionalLengthsSamples;
		ArrowGen.AdditionalLengthSamples[2] = AdditionalLengthsSamples;
	}
	ArrowGen.Generate();
	OutMesh->Copy(&ArrowGen);
}



UAddSpherePrimitiveTool::UAddSpherePrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralSphereToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Sphere");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("SphereToolName", "Create Sphere"));
}

void UAddSpherePrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	const UProceduralSphereToolProperties* SphereSettings = Cast<UProceduralSphereToolProperties>(ShapeSettings);
	switch (SphereSettings->SubdivisionType)
	{
	case EProceduralSphereType::LatLong:
	{
		FSphereGenerator SphereGen;
		SphereGen.Radius = SphereSettings->Radius;
		SphereGen.NumTheta = SphereSettings->VerticalSlices;
		SphereGen.NumPhi = SphereSettings->HorizontalSlices + 1;
		// In FSphereGenerator, PerFace is effectively ignored, and does the same thing as 
		//  PerShape, which is unlikely to be useful. So we might as well have PerFace do
		//  the same thing as PerQuad.
		SphereGen.bPolygroupPerQuad = (ShapeSettings->PolygroupMode != EMakeMeshPolygroupMode::PerShape);
		SphereGen.Generate();
		OutMesh->Copy(&SphereGen);
		break;
	}
	case EProceduralSphereType::Box:
	{
		FBoxSphereGenerator SphereGen;
		SphereGen.Radius = SphereSettings->Radius;
		SphereGen.Box = FOrientedBox3d(FVector3d::Zero(),
			0.5 * FVector3d(SphereSettings->Subdivisions + 1,
				SphereSettings->Subdivisions + 1,
				SphereSettings->Subdivisions + 1));
		int EdgeNum = SphereSettings->Subdivisions + 1;
		SphereGen.EdgeVertices = FIndex3i(EdgeNum, EdgeNum, EdgeNum);
		SphereGen.bPolygroupPerQuad = (ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad);
		SphereGen.Generate();
		OutMesh->Copy(&SphereGen);
		break;
	}
	}
}



UAddStairsPrimitiveTool::UAddStairsPrimitiveTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<UProceduralStairsToolProperties>(TEXT("ShapeSettings")))
{
	AssetName = TEXT("Stairs");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("StairsToolName", "Create Stairs"));
}

void UAddStairsPrimitiveTool::GenerateMesh(FDynamicMesh3* OutMesh) const
{
	const UProceduralStairsToolProperties* StairSettings = Cast<UProceduralStairsToolProperties>(ShapeSettings);
	switch (StairSettings->StairsType)
	{
	case EProceduralStairsType::Linear:
	{
		FLinearStairGenerator StairGen;
		StairGen.StepWidth = StairSettings->StepWidth;
		StairGen.StepHeight = StairSettings->StepHeight;
		StairGen.StepDepth = StairSettings->StepDepth;
		StairGen.NumSteps = StairSettings->NumSteps;
		StairGen.bPolygroupPerQuad = (ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad);
		StairGen.Generate();
		OutMesh->Copy(&StairGen);
		break;
	}
	case EProceduralStairsType::Floating:
	{
		FFloatingStairGenerator StairGen;
		StairGen.StepWidth = StairSettings->StepWidth;
		StairGen.StepHeight = StairSettings->StepHeight;
		StairGen.StepDepth = StairSettings->StepDepth;
		StairGen.NumSteps = StairSettings->NumSteps;
		StairGen.bPolygroupPerQuad = (ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad);
		StairGen.Generate();
		OutMesh->Copy(&StairGen);
		break;
	}
	case EProceduralStairsType::Curved:
	{
		FCurvedStairGenerator StairGen;
		StairGen.StepWidth = StairSettings->StepWidth;
		StairGen.StepHeight = StairSettings->StepHeight;
		StairGen.NumSteps = StairSettings->NumSteps;
		StairGen.InnerRadius = StairSettings->InnerRadius;
		StairGen.CurveAngle = StairSettings->CurveAngle;
		StairGen.bPolygroupPerQuad = (ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad);
		StairGen.Generate();
		OutMesh->Copy(&StairGen);
		break;
	}
	case EProceduralStairsType::Spiral:
	{
		FSpiralStairGenerator StairGen;
		StairGen.StepWidth = StairSettings->StepWidth;
		StairGen.StepHeight = StairSettings->StepHeight;
		StairGen.NumSteps = StairSettings->NumSteps;
		StairGen.InnerRadius = StairSettings->InnerRadius;
		StairGen.CurveAngle = StairSettings->SpiralAngle;
		StairGen.bPolygroupPerQuad = (ShapeSettings->PolygroupMode == EMakeMeshPolygroupMode::PerQuad);
		StairGen.Generate();
		OutMesh->Copy(&StairGen);
		break;
	}
	}
}


#undef LOCTEXT_NAMESPACE

