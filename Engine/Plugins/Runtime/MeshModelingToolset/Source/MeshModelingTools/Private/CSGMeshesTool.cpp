// Copyright Epic Games, Inc. All Rights Reserved.

#include "CSGMeshesTool.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "CompositionOps/BooleanMeshesOp.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CSGMeshesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UCSGMeshesTool"


void UCSGMeshesTool::EnableTrimMode()
{
	check(OriginalDynamicMeshes.Num() == 0);		// must not have been initialized!
	bTrimMode = true;
}

void UCSGMeshesTool::SetupProperties()
{
	Super::SetupProperties();

	if (bTrimMode)
	{
		TrimProperties = NewObject<UTrimMeshesToolProperties>(this);
		TrimProperties->RestoreProperties(this);
		AddToolPropertySource(TrimProperties);

		TrimProperties->WatchProperty(TrimProperties->WhichMesh, [this](ETrimOperation)
		{
			UpdateGizmoVisibility();
			UpdatePreviewsVisibility();
		});
		TrimProperties->WatchProperty(TrimProperties->bShowTrimmingMesh, [this](bool)
		{
			UpdatePreviewsVisibility();
		});
		TrimProperties->WatchProperty(TrimProperties->ColorOfTrimmingMesh, [this](FLinearColor)
		{
			UpdatePreviewsMaterial();
		});
		TrimProperties->WatchProperty(TrimProperties->OpacityOfTrimmingMesh, [this](float)
		{
			UpdatePreviewsMaterial();
		});

		SetToolDisplayName(LOCTEXT("TrimMeshesToolName", "Trim"));
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartTrimTool", "Trim one mesh with another. Use the transform gizmos to tweak the positions of the input objects (can help to resolve errors/failures)"),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		CSGProperties = NewObject<UCSGMeshesToolProperties>(this);
		CSGProperties->RestoreProperties(this);
		AddToolPropertySource(CSGProperties);

		CSGProperties->WatchProperty(CSGProperties->Operation, [this](ECSGOperation)
		{
			UpdateGizmoVisibility();
			UpdatePreviewsVisibility();
		});
		CSGProperties->WatchProperty(CSGProperties->bShowSubtractedMesh, [this](bool)
		{
			UpdatePreviewsVisibility();
		});
		CSGProperties->WatchProperty(CSGProperties->SubtractedMeshColor, [this](FLinearColor)
		{
			UpdatePreviewsMaterial();
		});
		CSGProperties->WatchProperty(CSGProperties->SubtractedMeshOpacity, [this](float)
		{
			UpdatePreviewsMaterial();
		});

		SetToolDisplayName(LOCTEXT("CSGMeshesToolName", "Mesh Boolean"));
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartTool", "Perform Boolean operations on the input meshes; any interior faces will be removed. Use the transform gizmos to modify the position and orientation of the input objects."),
			EToolMessageLevel::UserNotification);
	}
}

void UCSGMeshesTool::UpdatePreviewsMaterial()
{
	if (!PreviewsGhostMaterial)
	{
		PreviewsGhostMaterial = ToolSetupUtil::GetSimpleCustomMaterial(GetToolManager(), FLinearColor::Black, .2f);
	}
	FLinearColor Color;
	float Opacity;
	if (bTrimMode)
	{
		Color = TrimProperties->ColorOfTrimmingMesh;
		Opacity = TrimProperties->OpacityOfTrimmingMesh;
	}
	else
	{
		Color = CSGProperties->SubtractedMeshColor;
		Opacity = CSGProperties->SubtractedMeshOpacity;
	}
	PreviewsGhostMaterial->SetVectorParameterValue(TEXT("Color"), Color);
	PreviewsGhostMaterial->SetScalarParameterValue(TEXT("Opacity"), Opacity);
}

void UCSGMeshesTool::UpdatePreviewsVisibility()
{
	int32 ShowPreviewIdx = -1;
	if (bTrimMode && TrimProperties->bShowTrimmingMesh)
	{
		ShowPreviewIdx = TrimProperties->WhichMesh == ETrimOperation::TrimA ? OriginalMeshPreviews.Num() - 1 : 0;
	}
	else if (!bTrimMode && CSGProperties->bShowSubtractedMesh)
	{
		if (CSGProperties->Operation == ECSGOperation::DifferenceAB)
		{
			ShowPreviewIdx = OriginalMeshPreviews.Num() - 1;
		}
		else if (CSGProperties->Operation == ECSGOperation::DifferenceBA)
		{
			ShowPreviewIdx = 0;
		}
	}
	for (int32 MeshIdx = 0; MeshIdx < OriginalMeshPreviews.Num(); MeshIdx++)
	{
		OriginalMeshPreviews[MeshIdx]->SetVisible(ShowPreviewIdx == MeshIdx);
	}
}

int32 UCSGMeshesTool::GetHiddenGizmoIndex() const
{
	int32 ParentHiddenIndex = Super::GetHiddenGizmoIndex();
	if (ParentHiddenIndex != -1)
	{
		return ParentHiddenIndex;
	}
	if (bTrimMode)
	{
		return TrimProperties->WhichMesh == ETrimOperation::TrimA ? 0 : 1;
	}
	else if (CSGProperties->Operation == ECSGOperation::DifferenceAB)
	{
		return 0;
	}
	else if (CSGProperties->Operation == ECSGOperation::DifferenceBA)
	{
		return 1;
	}
	else
	{
		return -1;
	}
}

void UCSGMeshesTool::SaveProperties()
{
	Super::SaveProperties();
	if (bTrimMode)
	{
		TrimProperties->SaveProperties(this);
	}
	else
	{
		CSGProperties->SaveProperties(this);
	}
}


void UCSGMeshesTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	OriginalDynamicMeshes.SetNum(Targets.Num());
	FComponentMaterialSet AllMaterialSet;
	TMap<UMaterialInterface*, int> KnownMaterials;
	TArray<TArray<int>> MaterialRemap; MaterialRemap.SetNum(Targets.Num());

	if (bTrimMode || !CSGProperties->bUseFirstMeshMaterials)
	{
		for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			const FComponentMaterialSet ComponentMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[ComponentIdx]);
			for (UMaterialInterface* Mat : ComponentMaterialSet.Materials)
			{
				int* FoundMatIdx = KnownMaterials.Find(Mat);
				int MatIdx;
				if (FoundMatIdx)
				{
					MatIdx = *FoundMatIdx;
				}
				else
				{
					MatIdx = AllMaterialSet.Materials.Add(Mat);
					KnownMaterials.Add(Mat, MatIdx);
				}
				MaterialRemap[ComponentIdx].Add(MatIdx);
			}
		}
	}
	else
	{
		AllMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
		for (int MatIdx = 0; MatIdx < AllMaterialSet.Materials.Num(); MatIdx++)
		{
			MaterialRemap[0].Add(MatIdx);
		}
		for (int ComponentIdx = 1; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			MaterialRemap[ComponentIdx].Init(0, Cast<IMaterialProvider>(Targets[ComponentIdx])->GetNumMaterials());
		}
	}

	UpdatePreviewsMaterial();
	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		OriginalDynamicMeshes[ComponentIdx] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		*OriginalDynamicMeshes[ComponentIdx] = UE::ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx]);

		// ensure materials and attributes are always enabled
		OriginalDynamicMeshes[ComponentIdx]->EnableAttributes();
		OriginalDynamicMeshes[ComponentIdx]->Attributes()->EnableMaterialID();
		FDynamicMeshMaterialAttribute* MaterialIDs = OriginalDynamicMeshes[ComponentIdx]->Attributes()->GetMaterialID();
		int32 NumComponentMaterials = MaterialRemap[ComponentIdx].Num();
		// If component had valid materials, remap them. (If it had no materials, no IDs are valid, so just leave them as they were.)
		if (NumComponentMaterials > 0)
		{
			for (int TID : OriginalDynamicMeshes[ComponentIdx]->TriangleIndicesItr())
			{
				int32 MID = MaterialIDs->GetValue(TID);
				MaterialIDs->SetValue(TID, MaterialRemap[ComponentIdx][MID >= 0 && MID < NumComponentMaterials ? MID : 0]);
			}
		}

		if (bSetPreviewMesh)
		{
			UPreviewMesh* OriginalMeshPreview = OriginalMeshPreviews.Add_GetRef(NewObject<UPreviewMesh>());
			OriginalMeshPreview->CreateInWorld(GetTargetWorld(), (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]));
			OriginalMeshPreview->UpdatePreview(OriginalDynamicMeshes[ComponentIdx].Get());

			OriginalMeshPreview->SetMaterial(0, PreviewsGhostMaterial);
			OriginalMeshPreview->SetVisible(false);
			TransformProxies[ComponentIdx]->AddComponent(OriginalMeshPreview->GetRootComponent());
		}
	}
	Preview->ConfigureMaterials(AllMaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
}


void UCSGMeshesTool::SetPreviewCallbacks()
{	
	DrawnLineSet = NewObject<ULineSetComponent>(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetupAttachment(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager()));
	DrawnLineSet->RegisterComponent();

	Preview->OnOpCompleted.AddLambda(
		[this](const FDynamicMeshOperator* Op)
		{
			const FBooleanMeshesOp* BooleanOp = (const FBooleanMeshesOp*)(Op);
			CreatedBoundaryEdges = BooleanOp->GetCreatedBoundaryEdges();
		}
	);
	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute*)
		{
			GetToolManager()->PostInvalidation();
			UpdateVisualization();
		}
	);
}


void UCSGMeshesTool::UpdateVisualization()
{
	FColor BoundaryEdgeColor(240, 15, 15);
	float BoundaryEdgeThickness = 2.0;
	float BoundaryEdgeDepthBias = 2.0f;

	const FDynamicMesh3* TargetMesh = Preview->PreviewMesh->GetPreviewDynamicMesh();
	FVector3d A, B;

	DrawnLineSet->Clear();
	if (!bTrimMode && CSGProperties->bShowNewBoundaries)
	{
		for (int EID : CreatedBoundaryEdges)
		{
			TargetMesh->GetEdgeV(EID, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}
	}
}


TUniquePtr<FDynamicMeshOperator> UCSGMeshesTool::MakeNewOperator()
{
	TUniquePtr<FBooleanMeshesOp> BooleanOp = MakeUnique<FBooleanMeshesOp>();
	
	BooleanOp->bTrimMode = bTrimMode;
	if (bTrimMode)
	{
		BooleanOp->WindingThreshold = TrimProperties->WindingThreshold;
		BooleanOp->TrimOperation = TrimProperties->WhichMesh;
		BooleanOp->TrimSide = TrimProperties->TrimSide;
		BooleanOp->bAttemptFixHoles = false;
		BooleanOp->bTryCollapseExtraEdges = false;
	}
	else
	{
		BooleanOp->WindingThreshold = CSGProperties->WindingThreshold;
		BooleanOp->CSGOperation = CSGProperties->Operation;
		BooleanOp->bAttemptFixHoles = CSGProperties->bTryFixHoles;
		BooleanOp->bTryCollapseExtraEdges = CSGProperties->bTryCollapseEdges;
	}

	check(OriginalDynamicMeshes.Num() == 2);
	check(Targets.Num() == 2);
	BooleanOp->Transforms.SetNum(2);
	BooleanOp->Meshes.SetNum(2);
	for (int Idx = 0; Idx < 2; Idx++)
	{
		BooleanOp->Meshes[Idx] = OriginalDynamicMeshes[Idx];
		BooleanOp->Transforms[Idx] = TransformProxies[Idx]->GetTransform();
	}

	return BooleanOp;
}



void UCSGMeshesTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCSGMeshesToolProperties, bUseFirstMeshMaterials)))
	{
		if (!AreAllTargetsValid())
		{
			GetToolManager()->DisplayMessage(LOCTEXT("InvalidTargets", "Target meshes are no longer valid"), EToolMessageLevel::UserWarning);
			return;
		}
		ConvertInputsAndSetPreviewMaterials(false);
		Preview->InvalidateResult();
	}
	else if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCSGMeshesToolProperties, bShowNewBoundaries)))
	{
		GetToolManager()->PostInvalidation();
		UpdateVisualization();
	}
	else
	{
		//TODO: UBaseCreateFromSelectedTool::OnPropertyModified below calls Preview->InvalidateResult() which triggers
		//expensive recompute of the boolean operator. We should rethink which code is responsible for invalidating the 
		//preview and make it consistent since some of the code calls Preview->InvalidateResult() manually.
		bool bVisualUpdate = false;
		if (bTrimMode)
		{
			bVisualUpdate = Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTrimMeshesToolProperties, bShowTrimmingMesh) ||
										 Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTrimMeshesToolProperties, OpacityOfTrimmingMesh) ||
										 Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTrimMeshesToolProperties, ColorOfTrimmingMesh) ||
										 Property->GetFName() == FName("R") ||
										 Property->GetFName() == FName("G") ||
										 Property->GetFName() == FName("B") ||
										 Property->GetFName() == FName("A"));
		}
		else 
		{	
			bVisualUpdate = Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCSGMeshesToolProperties, bShowSubtractedMesh) ||
										 Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCSGMeshesToolProperties, SubtractedMeshColor) ||
										 Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCSGMeshesToolProperties, SubtractedMeshOpacity) ||
										 Property->GetFName() == FName("R") ||
										 Property->GetFName() == FName("G") ||
										 Property->GetFName() == FName("B") ||
										 Property->GetFName() == FName("A"));
		}

		// If the property that was changed only affects the visuals, we do not need to recalculate the output 
		// of FBooleanMeshesOp 
		if (bVisualUpdate == false)
		{
			Super::OnPropertyModified(PropertySet, Property);
		}
	}
}


FString UCSGMeshesTool::GetCreatedAssetName() const
{
	if (bTrimMode)
	{
		return TEXT("Trim");
	}
	else
	{
		return TEXT("Boolean");
	}
}


FText UCSGMeshesTool::GetActionName() const
{
	if (bTrimMode)
	{
		return LOCTEXT("TrimActionName", "Trim Meshes");
	}
	else
	{
		return LOCTEXT("BooleanActionName", "Boolean Meshes");
	}
}



void UCSGMeshesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Super::OnShutdown(ShutdownType);

	for (UPreviewMesh* MeshPreview : OriginalMeshPreviews)
	{
		MeshPreview->SetVisible(false);
		MeshPreview->Disconnect();
		MeshPreview = nullptr;
	}
}



#undef LOCTEXT_NAMESPACE

