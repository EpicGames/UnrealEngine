// Copyright Epic Games, Inc. All Rights Reserved.

#include "CombineMeshesTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshTransforms.h"

#include "ModelingObjectsCreationAPI.h"
#include "Selection/ToolSelectionUtil.h"
#include "Physics/ComponentCollisionUtil.h"
#include "ShapeApproximation/SimpleShapeSet3.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CombineMeshesTool)

#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#endif

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UCombineMeshesTool"

namespace CombineMeshesToolLocals
{
	void SetNewMaterialID(int32 ComponentIdx, FDynamicMeshMaterialAttribute* MatAttrib, int32 TID, 
		TArray<TArray<int32>>& MaterialIDRemaps, TArray<UMaterialInterface*>& AllMaterials)
	{
		int MatID = MatAttrib->GetValue(TID);
		if (!ensure(MatID >= 0))
		{
			return;
		}
		if (MatID >= MaterialIDRemaps[ComponentIdx].Num())
		{
			UE_LOGF(LogGeometry, Warning, "UCombineMeshesTool: Component %d had at least one material ID (%d) "
				"that was not in its material list.", ComponentIdx, MatID);
			
			// There are different things we could do here. It's worth noting that out of bounds material indices
			// are handled differently in static meshes and dynamic mesh components, and depend in part on how we
			// got to that state. So trying to preserve a specific behavior is not practical, and probably not 
			// expected if the user is not giving us valid data to begin with.
			// The route we go is to give a separate nullptr material slot to each out of bounds ID. This will give
			// the user a chance to preserve their material assignments while fixing the issue by assigning materials to
			// the slots created in the output (at least, unless they pass through this tool again, at which point any
			// nullptr-pointing IDs will be collapsed to point to the same nullptr slot, due to the way we create the
			// combined material list for in-bounds IDs).
			int32 NumElementsToAdd = MatID - MaterialIDRemaps[ComponentIdx].Num() + 1;
			for (int32 i = 0; i < NumElementsToAdd; ++i)
			{
				MaterialIDRemaps[ComponentIdx].Add(AllMaterials.Num());
				AllMaterials.Add(nullptr);
			}
			checkSlow(MaterialIDRemaps[ComponentIdx].Num() == MatID + 1);
		}
		MatAttrib->SetValue(TID, MaterialIDRemaps[ComponentIdx][MatID]);
	}
}

/*
 * ToolBuilder
 */

bool UCombineMeshesToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return (bIsDuplicateTool) ?
		  (SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1)
		: (SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) > 1);
}

UMultiSelectionMeshEditingTool* UCombineMeshesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UCombineMeshesTool* NewTool = NewObject<UCombineMeshesTool>(SceneState.ToolManager);
	NewTool->SetDuplicateMode(bIsDuplicateTool);
	return NewTool;
}

const FToolTargetTypeRequirements& UCombineMeshesToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements CombineTypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		UDynamicMeshCommitter::StaticClass()
		});
	static FToolTargetTypeRequirements DuplicateTypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return bIsDuplicateTool ? DuplicateTypeRequirements : CombineTypeRequirements;
}


/*
 * Tool
 */


void UCombineMeshesTool::SetDuplicateMode(bool bDuplicateModeIn)
{
	this->bDuplicateMode = bDuplicateModeIn;
}

void UCombineMeshesTool::Setup()
{
	UInteractiveTool::Setup();

	BasicProperties = NewObject<UCombineMeshesToolProperties>(this);
	AddToolPropertySource(BasicProperties);
	BasicProperties->RestoreProperties(this);
	BasicProperties->bIsDuplicateMode = this->bDuplicateMode;

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->InitializeDefaultWithAuto();
	OutputTypeProperties->OutputType = UCreateMeshObjectTypeProperties::AutoIdentifier;
	OutputTypeProperties->RestoreProperties(this, TEXT("OutputTypeFromInputTool"));
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	BasicProperties->WatchProperty(BasicProperties->OutputWriteTo, [&](EBaseCreateFromSelectedTargetType NewType)
	{
		if (NewType == EBaseCreateFromSelectedTargetType::NewObject)
		{
			BasicProperties->OutputExistingName = TEXT("");
			SetToolPropertySourceEnabled(OutputTypeProperties, true);
		}
		else
		{
			int32 Index = (BasicProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::FirstInputObject) ? 0 : Targets.Num() - 1;
			BasicProperties->OutputExistingName = UE::Modeling::GetComponentAssetBaseName(UE::ToolTarget::GetTargetComponent(Targets[Index]), false);
			SetToolPropertySourceEnabled(OutputTypeProperties, false);
		}
	});

	SetToolPropertySourceEnabled(OutputTypeProperties, BasicProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::NewObject);

	if (bDuplicateMode)
	{
		SetToolDisplayName(LOCTEXT("DuplicateMeshesToolName", "Duplicate"));
		BasicProperties->OutputNewName = UE::Modeling::GetComponentAssetBaseName(UE::ToolTarget::GetTargetComponent(Targets[0]));
	}
	else
	{
		SetToolDisplayName(LOCTEXT("CombineMeshesToolName", "Append"));
		BasicProperties->OutputNewName = FString("Combined");
	}

	HandleSourceProperties = bDuplicateMode
		                         ? static_cast<UOnAcceptHandleSourcesPropertiesBase*>(NewObject<UOnAcceptHandleSourcesPropertiesSingle>(this))
		                         : static_cast<UOnAcceptHandleSourcesPropertiesBase*>(NewObject<UOnAcceptHandleSourcesProperties>(this));
	AddToolPropertySource(HandleSourceProperties);
	HandleSourceProperties->RestoreProperties(this);

	if (bDuplicateMode)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartToolDuplicate", "This tool duplicates a single input object to create new objects, and optionally replaces the input object."),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartToolCombine", "This tool appends multiple input object to create new objects, and optionally replaces the one of the input objects."),
			EToolMessageLevel::UserNotification);
	}
}


void UCombineMeshesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	BasicProperties->SaveProperties(this);
	OutputTypeProperties->SaveProperties(this, TEXT("OutputTypeFromInputTool"));
	HandleSourceProperties->SaveProperties(this);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		if (bDuplicateMode || BasicProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::NewObject)
		{
			CreateNewAsset();
		}
		else
		{
			UpdateExistingAsset();
		}
	}
}


void UCombineMeshesTool::CreateNewAsset()
{
	using namespace CombineMeshesToolLocals;

	// Make sure meshes are available before we open transaction. This is to avoid potential stability issues related 
	// to creation/load of meshes inside a transaction, for assets that possibly do not have bulk data currently loaded.
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	TArray<FDynamicMesh3> InputMeshes;
	InputMeshes.Reserve(Targets.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		InputMeshes.Add(UE::ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx], GetMeshParams));
	}

	GetToolManager()->BeginUndoTransaction( bDuplicateMode ? 
		LOCTEXT("DuplicateMeshToolTransactionName", "Duplicate Mesh") :
		LOCTEXT("CombineMeshesToolTransactionName", "Merge Meshes"));

	FBox Box(ForceInit);
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		Box += UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx])->Bounds.GetBox();
	}

	TArray<UMaterialInterface*> AllMaterials;
	TArray<TArray<int32>> MaterialIDRemaps;
	BuildCombinedMaterialSet(AllMaterials, MaterialIDRemaps);

	FDynamicMesh3 AccumulateDMesh;
	AccumulateDMesh.EnableTriangleGroups();
	AccumulateDMesh.EnableAttributes();
	AccumulateDMesh.Attributes()->EnableTangents();
	AccumulateDMesh.Attributes()->EnableMaterialID();
	AccumulateDMesh.Attributes()->EnablePrimaryColors();
	constexpr bool bCenterPivot = false;
	FVector3d Origin = FVector3d::ZeroVector;
	if (bCenterPivot)
	{
		// Place the pivot at the bounding box center
		Origin = Box.GetCenter();
	}
	else if (!Targets.IsEmpty())
	{
		// Use the average pivot
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			Origin += UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]).TransformPosition(FVector3d::ZeroVector);
		}
		Origin /= Targets.Num();
	}
	FTransform3d AccumToWorld(Origin);
	FTransform3d ToAccum(-Origin);

	FSimpleShapeSet3d SimpleCollision;
	UE::Geometry::FComponentCollisionSettings CollisionSettings;

	{
#if WITH_EDITOR
		FScopedSlowTask SlowTask(Targets.Num()+1, 
			bDuplicateMode ? 
			LOCTEXT("DuplicateMeshBuild", "Building duplicate mesh ...") :
			LOCTEXT("CombineMeshesBuild", "Building merged mesh ..."));
		SlowTask.MakeDialog();
#endif
		bool bNeedColorAttr = false;
		int MatIndexBase = 0;
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
#if WITH_EDITOR
			SlowTask.EnterProgressFrame(1);
#endif
			UPrimitiveComponent* PrimitiveComponent = UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]);

			FDynamicMesh3& ComponentDMesh = InputMeshes[ComponentIdx];
			bNeedColorAttr = bNeedColorAttr || (ComponentDMesh.HasAttributes() && ComponentDMesh.Attributes()->HasPrimaryColors());

			if (ComponentDMesh.HasAttributes())
			{
				AccumulateDMesh.Attributes()->EnableMatchingAttributes(*ComponentDMesh.Attributes(), false);
			}

			// update material IDs to account for combined material set
			if (FDynamicMeshMaterialAttribute* MatAttrib = ComponentDMesh.Attributes()->GetMaterialID())
			{
				for (int TID : ComponentDMesh.TriangleIndicesItr())
				{
					SetNewMaterialID(ComponentIdx, MatAttrib, TID, MaterialIDRemaps, AllMaterials);
				}
			}

			FDynamicMeshEditor Editor(&AccumulateDMesh);
			FMeshIndexMappings IndexMapping;
			if (bDuplicateMode) // no transform if duplicating
			{
				Editor.AppendMesh(&ComponentDMesh, IndexMapping);

				if (UE::Geometry::ComponentTypeSupportsCollision(PrimitiveComponent))
				{
					CollisionSettings = UE::Geometry::GetCollisionSettings(PrimitiveComponent);
					UE::Geometry::AppendSimpleCollision(PrimitiveComponent, &SimpleCollision, FTransform3d::Identity);
				}
			}
			else
			{
				FTransformSRT3d XF = (UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]) * ToAccum);
				if (XF.GetDeterminant() < 0)
				{
					ComponentDMesh.ReverseOrientation(false);
				}

				Editor.AppendMesh(&ComponentDMesh, IndexMapping,
					[&XF](int Unused, const FVector3d P) { return XF.TransformPosition(P); },
					[&XF](int Unused, const FVector3d N) { return XF.TransformNormal(N); });
				if (UE::Geometry::ComponentTypeSupportsCollision(PrimitiveComponent))
				{
					UE::Geometry::AppendSimpleCollision(PrimitiveComponent, &SimpleCollision, XF);
				}
			}

			FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[ComponentIdx]);
			MatIndexBase += MaterialSet.Materials.Num();
		}

		if (!bNeedColorAttr)
		{
			AccumulateDMesh.Attributes()->DisablePrimaryColors();
		}

#if WITH_EDITOR
		SlowTask.EnterProgressFrame(1);
#endif

		if (bDuplicateMode)
		{
			// TODO: will need to refactor this when we support duplicating multiple
			check(Targets.Num() == 1);
			AccumToWorld = (FTransform3d)UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
		}

		// max len explicitly enforced here, would ideally notify user
		FString UseBaseName = BasicProperties->OutputNewName.Left(250);
		if (UseBaseName.IsEmpty())
		{
			UseBaseName = (bDuplicateMode) ? TEXT("Duplicate") : TEXT("Merge");
		}

		FCreateMeshObjectParams NewMeshObjectParams;
		NewMeshObjectParams.TargetWorld = GetTargetWorld();
		NewMeshObjectParams.Transform = (FTransform)AccumToWorld;
		NewMeshObjectParams.BaseName = UseBaseName;
		NewMeshObjectParams.Materials = AllMaterials;
		NewMeshObjectParams.SetMesh(&AccumulateDMesh);
		if (OutputTypeProperties->OutputType == UCreateMeshObjectTypeProperties::AutoIdentifier)
		{
			UE::ToolTarget::ConfigureCreateMeshObjectParams(Targets[0], NewMeshObjectParams);
		}
		else
		{
			OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
		}
		FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
		if (Result.IsOK() && Result.NewActor != nullptr)
		{
			// if any inputs have Simple Collision geometry we will forward it to new mesh.
			if (UE::Geometry::ComponentTypeSupportsCollision(Result.NewComponent) && SimpleCollision.TotalElementsNum() > 0)
			{
				UE::Geometry::SetSimpleCollision(Result.NewComponent, &SimpleCollision, CollisionSettings);
			}

			// select the new actor
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
		}
	}
	
	TArray<AActor*> Actors;
	for (int32 Idx = 0; Idx < Targets.Num(); Idx++)
	{
		Actors.Add(UE::ToolTarget::GetTargetActor(Targets[Idx]));
	}
	HandleSourceProperties->ApplyMethod(Actors, GetToolManager());


	GetToolManager()->EndUndoTransaction();
}


void UCombineMeshesTool::UpdateExistingAsset()
{
	using namespace CombineMeshesToolLocals;

	// Make sure meshes are available before we open transaction. This is to avoid potential stability issues related 
	// to creation/load of meshes inside a transaction, for assets that possibly do not have bulk data currently loaded.
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	TArray<FDynamicMesh3> InputMeshes;
	InputMeshes.Reserve(Targets.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		InputMeshes.Add(UE::ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx], GetMeshParams));
	}

	check(!bDuplicateMode);
	GetToolManager()->BeginUndoTransaction(LOCTEXT("CombineMeshesToolTransactionName", "Merge Meshes"));

	AActor* SkipActor = nullptr;

	TArray<UMaterialInterface*> AllMaterials;
	TArray<TArray<int32>> MaterialIDRemaps;
	BuildCombinedMaterialSet(AllMaterials, MaterialIDRemaps);

	FDynamicMesh3 AccumulateDMesh;
	AccumulateDMesh.EnableTriangleGroups();
	AccumulateDMesh.EnableAttributes();
	AccumulateDMesh.Attributes()->EnableTangents();
	AccumulateDMesh.Attributes()->EnableMaterialID();
	AccumulateDMesh.Attributes()->EnablePrimaryColors();

	int32 SkipIndex = (BasicProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::FirstInputObject) ? 0 : (Targets.Num() - 1);
	UPrimitiveComponent* UpdateComponent = UE::ToolTarget::GetTargetComponent(Targets[SkipIndex]);
	SkipActor = UE::ToolTarget::GetTargetActor(Targets[SkipIndex]);

	FTransform3d TargetToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[SkipIndex]);

	FSimpleShapeSet3d SimpleCollision;
	UE::Geometry::FComponentCollisionSettings CollisionSettings;
	bool bOutputComponentSupportsCollision = UE::Geometry::ComponentTypeSupportsCollision(UpdateComponent);
	if (bOutputComponentSupportsCollision)
	{
		CollisionSettings = UE::Geometry::GetCollisionSettings(UpdateComponent);
	}
	TArray<FTransform3d> Transforms;
	Transforms.SetNum(2);

	{
#if WITH_EDITOR
		FScopedSlowTask SlowTask(Targets.Num()+1, 
			bDuplicateMode ? 
			LOCTEXT("DuplicateMeshBuild", "Building duplicate mesh ...") :
			LOCTEXT("CombineMeshesBuild", "Building merged mesh ..."));
		SlowTask.MakeDialog();
#endif
		bool bNeedColorAttr = false;
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
#if WITH_EDITOR
			SlowTask.EnterProgressFrame(1);
#endif
			UPrimitiveComponent* PrimitiveComponent = UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]);

			FDynamicMesh3& ComponentDMesh = InputMeshes[ComponentIdx];
			bNeedColorAttr = bNeedColorAttr || (ComponentDMesh.HasAttributes() && ComponentDMesh.Attributes()->HasPrimaryColors());

			// update material IDs to account for combined material set
			FDynamicMeshMaterialAttribute* MatAttrib = ComponentDMesh.Attributes()->GetMaterialID();
			for (int TID : ComponentDMesh.TriangleIndicesItr())
			{
				SetNewMaterialID(ComponentIdx, MatAttrib, TID, MaterialIDRemaps, AllMaterials);
			}

			if (ComponentIdx != SkipIndex)
			{
				FTransform3d ComponentToWorld = (FTransform3d)UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]);
				MeshTransforms::ApplyTransform(ComponentDMesh, ComponentToWorld, true);
				MeshTransforms::ApplyTransformInverse(ComponentDMesh, TargetToWorld, true);
				Transforms[0] = ComponentToWorld;
				if (TargetToWorld.GetRotation().IsIdentity() || TargetToWorld.GetScale3D().IsUniform())
				{
					// Inverse can be represented by a single FTransform3d
					Transforms[1] = TargetToWorld.Inverse();
				}
				else
				{
					// Separate inverse into a rotation+translation part and a scale part
					FQuat4d WorldToTargetR = TargetToWorld.GetRotation().Inverse();
					FTransform3d WorldToTargetRT(WorldToTargetR, WorldToTargetR * (-TargetToWorld.GetTranslation()), FVector3d::One());
					FTransform3d WorldToTargetS = FTransform3d::Identity;
					WorldToTargetS.SetScale3D(FTransform3d::GetSafeScaleReciprocal(TargetToWorld.GetScale3D()));

					Transforms[1] = WorldToTargetRT;
					Transforms.Add(WorldToTargetS);
				}
				if (bOutputComponentSupportsCollision && UE::Geometry::ComponentTypeSupportsCollision(PrimitiveComponent))
				{
					UE::Geometry::AppendSimpleCollision(PrimitiveComponent, &SimpleCollision, Transforms);
				}
			}
			else
			{
				if (bOutputComponentSupportsCollision && UE::Geometry::ComponentTypeSupportsCollision(PrimitiveComponent))
				{
					UE::Geometry::AppendSimpleCollision(PrimitiveComponent, &SimpleCollision, FTransform3d::Identity);
				}
			}

			FDynamicMeshEditor Editor(&AccumulateDMesh);
			FMeshIndexMappings IndexMapping;
			Editor.AppendMesh(&ComponentDMesh, IndexMapping);
		}

		if (!bNeedColorAttr)
		{
			AccumulateDMesh.Attributes()->DisablePrimaryColors();
		}

#if WITH_EDITOR
		SlowTask.EnterProgressFrame(1);
#endif

		FComponentMaterialSet NewMaterialSet;
		NewMaterialSet.Materials = AllMaterials;
		UE::ToolTarget::CommitDynamicMeshUpdate(Targets[SkipIndex], AccumulateDMesh, true, FConversionToMeshDescriptionOptions(), &NewMaterialSet);

		// CommitDynamicMeshUpdate updates the materials for the underlying asset. However,
		// it does not update the component itself, so address that now.
		UE::ToolTarget::CommitMaterialSetUpdate(Targets[SkipIndex], NewMaterialSet, false);

		if (bOutputComponentSupportsCollision)
		{
			UE::Geometry::SetSimpleCollision(UpdateComponent, &SimpleCollision, CollisionSettings);
		}

		// select the new actor
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), SkipActor);
	}

	
	TArray<AActor*> Actors;
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		AActor* Actor = UE::ToolTarget::GetTargetActor(Targets[Idx]);
		Actors.Add(Actor);
	}
	HandleSourceProperties->ApplyMethod(Actors, GetToolManager(), SkipActor);

	GetToolManager()->EndUndoTransaction();
}


void UCombineMeshesTool::BuildCombinedMaterialSet(TArray<UMaterialInterface*>& NewMaterialsOut, TArray<TArray<int32>>& MaterialIDRemapsOut)
{
	NewMaterialsOut.Reset();

	TMap<UMaterialInterface*, int> KnownMaterials;

	MaterialIDRemapsOut.SetNum(Targets.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[ComponentIdx]);
		int32 NumMaterials = MaterialSet.Materials.Num();
		for (int MaterialIdx = 0; MaterialIdx < NumMaterials; MaterialIdx++)
		{
			UMaterialInterface* Mat = MaterialSet.Materials[MaterialIdx];
			int32 NewMaterialIdx = 0;
			if (KnownMaterials.Contains(Mat) == false)
			{
				NewMaterialIdx = NewMaterialsOut.Num();
				KnownMaterials.Add(Mat, NewMaterialIdx);
				NewMaterialsOut.Add(Mat);
			}
			else
			{
				NewMaterialIdx = KnownMaterials[Mat];
			}
			MaterialIDRemapsOut[ComponentIdx].Add(NewMaterialIdx);
		}
	}
}


#undef LOCTEXT_NAMESPACE

