// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DeleteGeometrySelectionCommand.h"
#include "ToolContextInterfaces.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshEditor.h"
#include "FaceGroupUtil.h"
#include "Changes/MeshChange.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/DynamicMeshSelector.h"
#include "Selections/MeshConnectedComponents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeleteGeometrySelectionCommand)

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "UDeleteGeometrySelectionCommand"


FText UDeleteGeometrySelectionCommand::GetCommandShortString() const
{
	return LOCTEXT("ShortString", "Delete Selection");
}


bool UDeleteGeometrySelectionCommand::CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs)
{
	return SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh);
}


void UDeleteGeometrySelectionCommand::ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs, UInteractiveCommandResult** Result)
{
	IGeometrySelector* BaseSelector = SelectionArgs->SelectionHandle.Selector;
	if (!ensure(BaseSelector != nullptr))
	{
		UE_LOGF(LogGeometry, Warning, "UDeleteGeometrySelectionCommand: Delete Selection requires Selector be provided in Selection Arguments");
		return;
	}

	// delete never returns a new selection
	if (Result != nullptr)
	{
		*Result = nullptr;
	}

	// should have been verified by CanExecute
	check(SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh));

	// TODO: extremely hardcoded behavior right here. Need a way to make this more generic, however
	// having the UpdateAfterGeometryEdit function in the base GeometrySelector does not make sense as 
	// it is specific to meshes. Probably this Command needs to be specialized for Mesh Edits.
	FBaseDynamicMeshSelector* BaseDynamicMeshSelector = static_cast<FBaseDynamicMeshSelector*>(BaseSelector);

	// collect up all our inputs
	UDynamicMesh* MeshObject = SelectionArgs->SelectionHandle.Identifier.GetAsObjectType<UDynamicMesh>();
	check(MeshObject != nullptr);
	const FGeometrySelection* Selection = SelectionArgs->SelectionHandle.Selection;
	if (Selection->Selection.Num() == 0)
	{
		return;
	}

	const bool bTrackChanges = SelectionArgs->HasTransactionsAPI();
	TUniquePtr<FDynamicMeshChange> DynamicMeshChange;	// only initialized if bTrackChanges == true

	// apply the delete operation
	MeshObject->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		FDynamicMeshChangeTracker ChangeTracker(&EditMesh);

		// handles the case of deleting polygroup edge by merging adjoining groups to match behavior in PolyEdit
		if (SelectionArgs->TopologyMode == UE::Geometry::EGeometryTopologyType::Polygroup
			&& SelectionArgs->ElementType == UE::Geometry::EGeometryElementType::Edge)
		{
			FMeshConnectedComponents Components(&EditMesh);

			// retrieve all selected edges
			TSet<int32> EdgeIDs;
			UE::Geometry::EnumeratePolygroupSelectionEdges(*Selection, EditMesh, FPolygroupSet(&EditMesh), 
				[&](const int32 EdgeID) { EdgeIDs.Add(EdgeID); });

			// similar but simplified version of work done in EnumeratePolygroupSelectionTriangles
			// retrieves the TriIDs of the triangles adjacent to all edges in the PolyEdge as they will all be merged into one Polygroup
			TSet<int32> SeedTriangleIDs;
			for (int32 Edge : EdgeIDs.Array())
			{
					FIndex2i AdjacentTriangles = EditMesh.GetEdgeT(Edge);
					EdgeIDs.Add(Edge);
					SeedTriangleIDs.Add(AdjacentTriangles.A);
					if (AdjacentTriangles.B != FDynamicMesh3::InvalidID)
					{
						SeedTriangleIDs.Add(AdjacentTriangles.B);
					}
			}

			// retrieve the rest of the connected components which will be in the merged PolyGroup
			Components.FindTrianglesConnectedToSeeds(SeedTriangleIDs.Array(), [&EditMesh, &EdgeIDs](const int32 Tri0, const int32 Tri1)
			{
				return EditMesh.GetTriangleGroup(Tri0) == EditMesh.GetTriangleGroup(Tri1) || EdgeIDs.Contains(EditMesh.FindEdgeFromTriPair(Tri0,Tri1));
			});

			// mark triangles for change
			if (bTrackChanges)
			{
				ChangeTracker.BeginChange();
	
				for (FMeshConnectedComponents::FComponent& Component : Components.Components)
				{
					ChangeTracker.SaveTriangles(Component.Indices, true);
					int32 NewGroupID = EditMesh.GetTriangleGroup(Component.Indices[0]);
					FaceGroupUtil::SetGroupID(EditMesh, Component.Indices, NewGroupID);
				}
			}
		}
		else
		{
			// build list of triangles from whatever the selection contains
			TSet<int32> TriangleList;
			//UE::Geometry::FPolygroupSet UsePolygroupSet = ...;		// need to support this eventually
			UE::Geometry::EnumerateSelectionTriangles(*Selection, EditMesh,
			[&](const int32 TriangleID) { TriangleList.Add(TriangleID); });

			// mark triangles for change
			if (bTrackChanges)
			{
				ChangeTracker.BeginChange();
				ChangeTracker.SaveTriangles(TriangleList, true);
			}

			// actually delete them
			FDynamicMeshEditor Editor(&EditMesh);
			Editor.RemoveTriangles(TriangleList.Array(), true);
		}
		
		// extract the change record
		if (bTrackChanges)
		{
			DynamicMeshChange = ChangeTracker.EndChange();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);


	// emit change 
	if ( bTrackChanges && DynamicMeshChange.IsValid() )
	{
		SelectionArgs->GetTransactionsAPI()->BeginUndoTransaction(GetCommandShortString());

		BaseDynamicMeshSelector->UpdateAfterGeometryEdit(
			SelectionArgs->GetTransactionsAPI(), true, MoveTemp(DynamicMeshChange), GetCommandShortString());

		SelectionArgs->GetTransactionsAPI()->EndUndoTransaction();
	}

}


#undef LOCTEXT_NAMESPACE
