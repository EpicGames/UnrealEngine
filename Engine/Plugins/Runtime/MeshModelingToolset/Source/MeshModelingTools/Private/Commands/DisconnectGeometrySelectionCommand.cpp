// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/DisconnectGeometrySelectionCommand.h"
#include "ToolContextInterfaces.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshEditor.h"
#include "Changes/MeshChange.h"
#include "Selections/GeometrySelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DisconnectGeometrySelectionCommand)

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "UDisconnectGeometrySelectionCommand"


FText UDisconnectGeometrySelectionCommand::GetCommandShortString() const
{
	return LOCTEXT("ShortString", "Disconnect Selection");
}


bool UDisconnectGeometrySelectionCommand::CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs)
{
	return SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh);
}


void UDisconnectGeometrySelectionCommand::ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs, UInteractiveCommandResult** Result)
{
	if (Result != nullptr)
	{
		*Result = nullptr;
	}

	// should have been verified by CanExecute
	check(SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh));
	
	// collect up all our inputs
	UDynamicMesh* MeshObject = SelectionArgs->SelectionHandle.Identifier.GetAsObjectType<UDynamicMesh>();
	check(MeshObject != nullptr);
	const FGeometrySelection* Selection = SelectionArgs->SelectionHandle.Selection;
	if (Selection->Selection.Num() == 0)
	{
		return;
	}

	bool bTrackChanges = SelectionArgs->HasTransactionsAPI();
	TUniquePtr<FDynamicMeshChange> DynamicMeshChange;	// only initialized if bTrackChanges == true

	// apply the Disconnect operation
	MeshObject->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		// build list of triangles from whatever the selection contains
		TSet<int32> TriangleList;
		//UE::Geometry::FPolygroupSet UsePolygroupSet = ...;		// need to support this eventually
		UE::Geometry::EnumerateSelectionTriangles(*Selection, EditMesh,
			[&](int32 TriangleID) { TriangleList.Add(TriangleID); });

		// mark triangles for change
		FDynamicMeshChangeTracker ChangeTracker(&EditMesh);
		if (bTrackChanges)
		{
			ChangeTracker.BeginChange();
			ChangeTracker.SaveTriangles(TriangleList, true);
		}

		// actually Disconnect them
		FDynamicMeshEditor Editor(&EditMesh);
		Editor.DisconnectTriangles(TriangleList.Array(), false);

		// extract the change record
		if (bTrackChanges)
		{
			DynamicMeshChange = ChangeTracker.EndChange();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);


	// emit change 
	if ( bTrackChanges && DynamicMeshChange.IsValid() )
	{
		SelectionArgs->GetTransactionsAPI()->BeginUndoTransaction(GetCommandShortString());
		SelectionArgs->GetTransactionsAPI()->AppendChange(MeshObject, 
			MakeUnique<FMeshChange>(MoveTemp(DynamicMeshChange)), GetCommandShortString());
		SelectionArgs->GetTransactionsAPI()->EndUndoTransaction();
	}

	if (Result != nullptr)
	{
		UGeometrySelectionEditCommandResult* NewResult = NewObject<UGeometrySelectionEditCommandResult>();
		NewResult->SourceHandle = SelectionArgs->SelectionHandle;
		NewResult->OutputSelection = *NewResult->SourceHandle.Selection;
		*Result = NewResult;
	}

}


#undef LOCTEXT_NAMESPACE
