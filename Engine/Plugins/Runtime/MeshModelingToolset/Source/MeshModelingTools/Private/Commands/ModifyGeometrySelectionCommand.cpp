// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/ModifyGeometrySelectionCommand.h"
#include "ToolContextInterfaces.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshEditor.h"
#include "Changes/MeshChange.h"
#include "Selections/GeometrySelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModifyGeometrySelectionCommand)

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "UModifyGeometrySelectionCommand"


FText UModifyGeometrySelectionCommand::GetCommandShortString() const
{
	switch (GetModificationType())
	{
	default:
	case EModificationType::SelectAll:
		return LOCTEXT("ShortString_SelectAll", "Select All");
	case EModificationType::ExpandToConnected:
		return LOCTEXT("ShortString_ExpandToConnected", "Expand To Connected");
		
	case EModificationType::Invert:
		return LOCTEXT("ShortString_Invert", "Invert Selection");
	case EModificationType::InvertConnected:
		return LOCTEXT("ShortString_InvertConnected", "Invert Selection (Connected)");

	case EModificationType::Expand:
		return LOCTEXT("ShortString_Expand", "Expand/Grow Selection");
	case EModificationType::Contract:
		return LOCTEXT("ShortString_Contract", "Contract/Shrink Selection");
	}
}


bool UModifyGeometrySelectionCommand::CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs)
{
	return SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh);
}


void UModifyGeometrySelectionCommand::ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs, UInteractiveCommandResult** Result)
{
	if (!ensure(Result != nullptr))
	{
		UE_LOGF(LogGeometry, Warning, "UModifyGeometrySelectionCommand: Selection-Modify commands are no-ops unless Result is provided and used");
		return;
	}
	if (!ensure(SelectionArgs->SelectionHandle.Selector != nullptr))
	{
		UE_LOGF(LogGeometry, Warning, "UModifyGeometrySelectionCommand: Selection-Modify commands require Selector be provided in Selection Arguments");
		return;
	}

	// should have been verified by CanExecute
	check(SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh));
	
	// collect up all our inputs
	UDynamicMesh* MeshObject = SelectionArgs->SelectionHandle.Identifier.GetAsObjectType<UDynamicMesh>();
	check(MeshObject != nullptr);
	const FGeometrySelection* InitialSelection = SelectionArgs->SelectionHandle.Selection;

	// some might be no-ops?
	//if (Selection->Selection.Num() == 0)
	//{
	//	return;
	//}

	UGeometrySelectionEditCommandResult* NewResult = NewObject<UGeometrySelectionEditCommandResult>();
	NewResult->SourceHandle = SelectionArgs->SelectionHandle;
	NewResult->OutputSelection.InitializeTypes(*InitialSelection);

	if (GetModificationType() == EModificationType::SelectAll)
	{
		SelectionArgs->SelectionHandle.Selector->InitializeSelectionFromPredicate(
			NewResult->OutputSelection, [&](FGeoSelectionID) { return true;}, IGeometrySelector::EInitializeSelectionMode::All, nullptr );
	}
	else if (GetModificationType() == EModificationType::ExpandToConnected)
	{
		SelectionArgs->SelectionHandle.Selector->InitializeSelectionFromPredicate(
			NewResult->OutputSelection, [&](FGeoSelectionID) { return true;}, 
			IGeometrySelector::EInitializeSelectionMode::Connected, InitialSelection );
	}
	else if (GetModificationType() == EModificationType::Invert)
	{
		FGeometrySelectionEditor TmpReadOnlyEditor;
		TmpReadOnlyEditor.Initialize(const_cast<FGeometrySelection*>(InitialSelection), InitialSelection->TopologyType == EGeometryTopologyType::Polygroup);		// not actually going to edit, just use selected test
		SelectionArgs->SelectionHandle.Selector->InitializeSelectionFromPredicate(
			NewResult->OutputSelection, 
			[&](FGeoSelectionID SelectionID) { return TmpReadOnlyEditor.IsSelected(SelectionID.Encoded()) == false; },
			IGeometrySelector::EInitializeSelectionMode::All, nullptr);
	}
	else if (GetModificationType() == EModificationType::InvertConnected)
	{
		SelectionArgs->SelectionHandle.Selector->InitializeSelectionFromPredicate(
			NewResult->OutputSelection, [&](FGeoSelectionID) { return true;}, 
			IGeometrySelector::EInitializeSelectionMode::Connected, InitialSelection );
		UE::Geometry::CombineSelectionInPlace(NewResult->OutputSelection, *InitialSelection, UE::Geometry::EGeometrySelectionCombineModes::Subtract);
	}
	else if (GetModificationType() == EModificationType::Expand)
	{
		// create border-ring selection
		SelectionArgs->SelectionHandle.Selector->InitializeSelectionFromPredicate(
			NewResult->OutputSelection, [&](FGeoSelectionID) { return true;}, 
			IGeometrySelector::EInitializeSelectionMode::AdjacentToBorder, InitialSelection );
		UE::Geometry::CombineSelectionInPlace(NewResult->OutputSelection, *InitialSelection, UE::Geometry::EGeometrySelectionCombineModes::Add);
	}
	else if (GetModificationType() == EModificationType::Contract)
	{
		// create border-ring selection
		FGeometrySelection TmpSelection;
		TmpSelection.InitializeTypes(NewResult->OutputSelection);
		SelectionArgs->SelectionHandle.Selector->InitializeSelectionFromPredicate(
			TmpSelection, [&](FGeoSelectionID) { return true;}, 
			IGeometrySelector::EInitializeSelectionMode::AdjacentToBorder, InitialSelection );
		NewResult->OutputSelection = *InitialSelection;
		UE::Geometry::CombineSelectionInPlace(NewResult->OutputSelection, TmpSelection, UE::Geometry::EGeometrySelectionCombineModes::Subtract);
	}

	*Result = NewResult;
}


#undef LOCTEXT_NAMESPACE
