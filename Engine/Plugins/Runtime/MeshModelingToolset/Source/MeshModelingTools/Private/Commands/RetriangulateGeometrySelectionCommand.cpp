// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/RetriangulateGeometrySelectionCommand.h"
#include "ToolContextInterfaces.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshEditor.h"
#include "Changes/MeshChange.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Operations/PolygroupRemesh.h"
#include "ConstrainedDelaunay2.h"
#include "Operations/SimpleHoleFiller.h"
#include "MeshRegionBoundaryLoops.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "Selection/DynamicMeshSelector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetriangulateGeometrySelectionCommand)

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "URetriangulateGeometrySelectionCommand"


FText URetriangulateGeometrySelectionCommand::GetCommandShortString() const
{
	return LOCTEXT("ShortString", "Retriangulate Selection");
}


bool URetriangulateGeometrySelectionCommand::CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs)
{
	return SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh)
		&& (SelectionArgs->SelectionHandle.Selection->ElementType == EGeometryElementType::Face)
		&& (SelectionArgs->SelectionHandle.Selection->TopologyType == EGeometryTopologyType::Polygroup);
}


void URetriangulateGeometrySelectionCommand::ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs, UInteractiveCommandResult** Result)
{
	IGeometrySelector* BaseSelector = SelectionArgs->SelectionHandle.Selector;
	if (!ensure(BaseSelector != nullptr))
	{
		UE_LOGF(LogGeometry, Warning, "URetriangulateGeometrySelectionCommand: Retriangulate Selection requires Selector be provided in Selection Arguments");
		return;
	}

	if (Result != nullptr)
	{
		*Result = nullptr;
	}

	// should have been verified by CanExecute
	check(SelectionArgs->IsMatchingType(FGeometryIdentifier::ETargetType::MeshContainer, FGeometryIdentifier::EObjectType::DynamicMesh));
	
	// TODO: extremely hardcoded behavior right here. Need a way to make this more generic, however
	// having the UpdateAfterGeometryEdit function in the base GeometrySelector does not make sense as 
	// it is specific to meshes. Probably this Command needs to be specialized for Mesh Edits.
	FBaseDynamicMeshSelector* BaseDynamicMeshSelector = static_cast<FBaseDynamicMeshSelector*>(BaseSelector);

	// collect up all our inputs
	UDynamicMesh* MeshObject = SelectionArgs->SelectionHandle.Identifier.GetAsObjectType<UDynamicMesh>();
	check(MeshObject != nullptr);
	const FGeometrySelection* Selection = SelectionArgs->SelectionHandle.Selection;

	bool bTrackChanges = SelectionArgs->HasTransactionsAPI();
	TUniquePtr<FDynamicMeshChange> DynamicMeshChange;	// only initialized if bTrackChanges == true

	// apply the Retriangulate operation
	MeshObject->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		// build list of triangles from whatever the selection contains
		TSet<int32> TriangleList;
		//UE::Geometry::FPolygroupSet UsePolygroupSet = ...;		// need to support this eventually
		UE::Geometry::EnumerateSelectionTriangles(*Selection, EditMesh,
			[&](int32 TriangleID) { TriangleList.Add(TriangleID); });

		if (TriangleList.Num() == 0)
		{
			for (int32 tid : EditMesh.TriangleIndicesItr())
			{
				TriangleList.Add(tid);
			}
		}

		// mark triangles for change
		FDynamicMeshChangeTracker ChangeTracker(&EditMesh);
		FGroupTopology Topology(&EditMesh, true);

		if (bTrackChanges)
		{
			ChangeTracker.BeginChange();
		}

		if ( TriangleList.Num() == EditMesh.TriangleCount() )
		{
			if (bTrackChanges)
			{
				ChangeTracker.SaveTriangles(TriangleList, true);
			}

			FPolygroupRemesh Remesh(&EditMesh, &Topology, ConstrainedDelaunayTriangulate<double>);
			bool bSuccess = Remesh.Compute();
		}
		else
		{
			// ported from UEditMeshPolygonsTool::ApplyRetriangulate(), should be moved to a standalone geometry op

			const FDynamicMeshAttributeSet* Attributes = EditMesh.HasAttributes() ? EditMesh.Attributes() : nullptr;

			TSet<int32> SelectedGroupIDs;
			for (int32 tid : TriangleList)
			{
				SelectedGroupIDs.Add( Topology.GetGroupID(tid) );
			}
			
			int32 nCompleted = 0;
			FDynamicMeshEditor Editor(&EditMesh);
			for (int32 GroupID : SelectedGroupIDs)
			{
				const TArray<int32>& Triangles = Topology.GetGroupTriangles(GroupID);
				ChangeTracker.SaveTriangles(Triangles, true);
				FMeshRegionBoundaryLoops RegionLoops(&EditMesh, Triangles, true);
				if (!RegionLoops.bFailed && RegionLoops.Loops.Num() == 1 && Triangles.Num() > 1)
				{
					TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>> VidUVMaps;
					if (Attributes)
					{
						for (int i = 0; i < Attributes->NumUVLayers(); ++i)
						{
							VidUVMaps.Emplace();
							RegionLoops.GetLoopOverlayMap(RegionLoops.Loops[0], *Attributes->GetUVLayer(i), VidUVMaps.Last());
						}
					}

					// We don't want to remove isolated vertices while removing triangles because we don't
					// want to throw away boundary verts. However, this means that we'll have to go back
					// through these vertices later to throw away isolated internal verts.
					TArray<int32> OldVertices;
					UE::Geometry::TriangleToVertexIDs(&EditMesh, Triangles, OldVertices);
					Editor.RemoveTriangles(Topology.GetGroupTriangles(GroupID), false);

					RegionLoops.Loops[0].Reverse();
					FSimpleHoleFiller Filler(&EditMesh, RegionLoops.Loops[0]);
					Filler.FillType = FSimpleHoleFiller::EFillType::PolygonEarClipping;
					Filler.Fill(GroupID);

					// Throw away any of the old verts that are still isolated (they were in the interior of the group)
					for (int32 Vid : OldVertices)
					{
						if ( !EditMesh.IsReferencedVertex(Vid) )
						{
							constexpr bool bPreserveManifold = false;
							EditMesh.RemoveVertex(Vid, bPreserveManifold);
						}
					}

					if (Attributes)
					{
						for (int i = 0; i < Attributes->NumUVLayers(); ++i)
						{
							RegionLoops.UpdateLoopOverlayMapValidity(VidUVMaps[i], *Attributes->GetUVLayer(i));
						}
						Filler.UpdateAttributes(VidUVMaps);
					}

					nCompleted++;
				}
			}

		}

		// extract the change record
		if (bTrackChanges)
		{
			DynamicMeshChange = ChangeTracker.EndChange();
		}

	}, EDynamicMeshChangeType::GeneralEdit, EDynamicMeshAttributeChangeFlags::Unknown, false);

	// emit change 
	if ( bTrackChanges && DynamicMeshChange.IsValid() )
	{
		SelectionArgs->GetTransactionsAPI()->BeginUndoTransaction(GetCommandShortString());
		BaseDynamicMeshSelector->UpdateAfterGeometryEdit(
			SelectionArgs->GetTransactionsAPI(), true, MoveTemp(DynamicMeshChange), GetCommandShortString());
		SelectionArgs->GetTransactionsAPI()->EndUndoTransaction();
	}

	if (Result != nullptr)
	{
		UGeometrySelectionEditCommandResult* NewResult = NewObject<UGeometrySelectionEditCommandResult>();
		NewResult->SourceHandle = SelectionArgs->SelectionHandle;
		// todo...
		//NewResult->OutputSelection = *NewResult->SourceHandle.Selection;
		*Result = NewResult;
	}

}


#undef LOCTEXT_NAMESPACE
