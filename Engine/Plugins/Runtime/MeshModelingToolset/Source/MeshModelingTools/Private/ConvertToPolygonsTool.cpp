// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConvertToPolygonsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "PreviewMesh.h"

#include "MeshOpPreviewHelpers.h"
#include "ModelingOperators.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "DynamicMeshEditor.h"
#include "DynamicSubmesh3.h"
#include "MeshRegionBoundaryLoops.h"
#include "Util/ColorConstants.h"
#include "Polygroups/PolygroupUtil.h"
#include "Util/ColorConstants.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/GeometrySelectionVisualization.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "ToolTargetManager.h"
#include "Polygroups/GroupSetAdapter.h"

#include "Polygroups/PolygroupsGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConvertToPolygonsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UConvertToPolygonsTool"

namespace UE::PolygroupUtilLocals
{
	const TCHAR* NewLayer = TEXT("Create New...");
	const TCHAR* DefaultLayer = TEXT("Default");
}

/*
 * ToolBuilder
 */

USingleTargetWithSelectionTool* UConvertToPolygonsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UConvertToPolygonsTool>(SceneState.ToolManager);
}

bool UConvertToPolygonsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return USingleTargetWithSelectionToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

class FConvertToPolygonsOp : public  FDynamicMeshOperator
{
public:
	// parameters set by the tool
	EConvertToPolygonsMode ConversionMode = EConvertToPolygonsMode::FaceNormalDeviation;
	double AngleTolerance = 0.1f;
	bool bUseAverageGroupNormal = true;
	int32 NumPoints = 10;
	bool bSubdivideExisting = false;
	FPolygroupsGenerator::EWeightingType WeightingType = FPolygroupsGenerator::EWeightingType::None;
	FVector3d WeightingCoeffs = FVector3d::One();

	bool bRespectUVSeams = false;
	bool bRespectHardNormals = false;

	double QuadMetricClamp = 1.0;
	double QuadAdjacencyWeight = 1.0;
	int QuadSearchRounds = 1;

	int32 MinGroupSize = 2;
	int32 InitialGroupID = 0;

	bool bCalculateNormals = false;

	// input mesh
	TSharedPtr<FSharedConstDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	// input polygroups, if available
	TSharedPtr<ITriangleGroupSetAdapter, ESPMode::ThreadSafe> SourcePolyGroups;

	// result
	FPolygroupsGenerator Generator;

	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		if ((Progress && Progress->Cancelled()) || OriginalMesh.IsValid() == false)
		{
			return;
		}

		OriginalMesh->AccessSharedObject([&](const FDynamicMesh3& ReadMesh)
		{
			ResultMesh->Copy(ReadMesh, true, true, true, true);
		});
	
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		if (ConversionMode == EConvertToPolygonsMode::CopyFromLayer)
		{
			if (!ensure(SourcePolyGroups.IsValid())) return;
			for (int32 tid : ResultMesh->TriangleIndicesItr())
			{
				ResultMesh->SetTriangleGroup(tid, SourcePolyGroups->GetGroup(tid));
			}
			return;
		}

		Generator = FPolygroupsGenerator(ResultMesh.Get());
		Generator.MinGroupSize = MinGroupSize;
		Generator.InitialGroupID = InitialGroupID;

		switch (ConversionMode)
		{
		case EConvertToPolygonsMode::FromUVIslands:
		{
			Generator.FindPolygroupsFromUVIslands();
			break;
		}
		case EConvertToPolygonsMode::FromNormalSeams:
		{
			Generator.FindPolygroupsFromHardNormalSeams();
			break;
		}
		case EConvertToPolygonsMode::FromMaterialIDs:
		{
			Generator.FindPolygroupsFromMaterialIDs();
			break;
		}
		case EConvertToPolygonsMode::FromConnectedTris:
		{
			Generator.FindPolygroupsFromConnectedTris();
			break;
		}
		case EConvertToPolygonsMode::FaceNormalDeviation:
		{
			double DotTolerance = 1.0 - FMathd::Cos(AngleTolerance * FMathd::DegToRad);
			Generator.FindPolygroupsFromFaceNormals(
				DotTolerance,
				bRespectUVSeams, 
				bRespectHardNormals,
				bUseAverageGroupNormal);
			break;
		}
		case EConvertToPolygonsMode::FindPolygons:
		{
			Generator.FindSourceMeshPolygonPolygroups(
				bRespectUVSeams, 
				bRespectHardNormals, 
				QuadMetricClamp, 
				QuadAdjacencyWeight,
				FMath::Clamp(QuadSearchRounds, 1, 100)
			);
			break;
		}
		case EConvertToPolygonsMode::FromFurthestPointSampling:
		{
			if (bSubdivideExisting)
			{
				OriginalMesh->AccessSharedObject([&](const FDynamicMesh3& ReadMesh)
				{
					FPolygroupSet InputGroups(&ReadMesh);
					Generator.FindPolygroupsFromFurthestPointSampling(NumPoints, WeightingType, WeightingCoeffs, &InputGroups);
				});
			}
			else
			{
				Generator.FindPolygroupsFromFurthestPointSampling(NumPoints, WeightingType, WeightingCoeffs, nullptr);
			}
			break;
		}
		default:
			check(0);
		}

		Generator.FindPolygroupEdges();

		if (bCalculateNormals && ConversionMode == EConvertToPolygonsMode::FaceNormalDeviation)
		{
			if (ResultMesh->HasAttributes() == false)
			{
				ResultMesh->EnableAttributes();
			}

			FDynamicMeshNormalOverlay* NormalOverlay = ResultMesh->Attributes()->PrimaryNormals();
			NormalOverlay->ClearElements();

			FDynamicMeshEditor Editor(ResultMesh.Get());
			for (const TArray<int>& Polygon : Generator.FoundPolygroups)
			{
				if (Polygon.IsEmpty())
				{
					continue;
				}
				FVector3f Normal = (FVector3f)ResultMesh->GetTriNormal(Polygon[0]);
				Editor.SetTriangleNormals(Polygon, Normal);
			}

			FMeshNormals Normals(ResultMesh.Get());
			Normals.RecomputeOverlayNormals(ResultMesh->Attributes()->PrimaryNormals());
			Normals.CopyToOverlay(NormalOverlay, false);
		}

	}

	void SetTransform(const FTransformSRT3d& Transform)
	{
		ResultTransform = Transform;
	}


};

TUniquePtr<FDynamicMeshOperator> UConvertToPolygonsOperatorFactory::MakeNewOperator()
{
	// backpointer used to populate parameters.
	check(ConvertToPolygonsTool);

	// Create the actual operator type based on the requested operation
	TUniquePtr<FConvertToPolygonsOp>  MeshOp = MakeUnique<FConvertToPolygonsOp>();

	// Operator runs on another thread - copy data over that it needs.
	ConvertToPolygonsTool->UpdateOpParameters(*MeshOp);

	// give the operator
	return MeshOp;
}

/*
 * Tool
 */
UConvertToPolygonsTool::UConvertToPolygonsTool()
{
	SetToolDisplayName(LOCTEXT("ConvertToPolygonsToolName", "Generate PolyGroups"));
}

bool UConvertToPolygonsTool::CanAccept() const
{
	return Super::CanAccept() && (PreviewCompute == nullptr || PreviewCompute->HaveValidResult());
}

void UConvertToPolygonsTool::Setup()
{
	UInteractiveTool::Setup();

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);

	OriginalDynamicMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(Target));

	// initialize triangle ROI if one exists
	SelectionTriangleROI = MakeShared<TSet<int32>, ESPMode::ThreadSafe>();
	TArray<int> TriangleROI;
	if (HasGeometrySelection())
	{
		const FGeometrySelection& InputSelection = GetGeometrySelection();

		UE::Geometry::EnumerateSelectionTriangles(InputSelection, *OriginalDynamicMesh,
			[&](int32 TriangleID) { SelectionTriangleROI->Add(TriangleID); });

		TriangleROI = SelectionTriangleROI->Array();

		OriginalSubmesh = MakeShared<FDynamicSubmesh3, ESPMode::ThreadSafe>(OriginalDynamicMesh.Get(), TriangleROI);
		bUsingSelection = true;
	}

	if (bUsingSelection)
	{
		ComputeOperatorSharedMesh = MakeShared<FSharedConstDynamicMesh3>(&OriginalSubmesh->GetSubmesh());
	}
	else
	{
		ComputeOperatorSharedMesh = MakeShared<FSharedConstDynamicMesh3>(OriginalDynamicMesh.Get());
	}

	Settings = NewObject<UConvertToPolygonsToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	FTransform MeshTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	UE::ToolTarget::HideSourceObject(Target);

	{
		// create the operator factory
		UConvertToPolygonsOperatorFactory* ConvertToPolygonsOperatorFactory = NewObject<UConvertToPolygonsOperatorFactory>(this);
		ConvertToPolygonsOperatorFactory->ConvertToPolygonsTool = this; // set the back pointer

		PreviewCompute = NewObject<UMeshOpPreviewWithBackgroundCompute>(ConvertToPolygonsOperatorFactory);
		PreviewCompute->Setup(GetTargetWorld(), ConvertToPolygonsOperatorFactory);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewCompute->PreviewMesh, Target);
		PreviewCompute->SetIsMeshTopologyConstant(true, EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals);

		// Give the preview something to display
		PreviewCompute->PreviewMesh->SetTransform(MeshTransform);
		PreviewCompute->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
		PreviewCompute->PreviewMesh->UpdatePreview(OriginalDynamicMesh.Get());
		
		PreviewCompute->ConfigureMaterials(MaterialSet.Materials,
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
		);

		// show the preview mesh
		PreviewCompute->SetVisibility(true);

		// something to capture the polygons from the async task when it is done
		PreviewCompute->OnOpCompleted.AddLambda([this](const FDynamicMeshOperator* MeshOp) 
		{ 
			const FConvertToPolygonsOp*  ConvertToPolygonsOp = static_cast<const FConvertToPolygonsOp*>(MeshOp);
			this->PolygonEdges = ConvertToPolygonsOp->Generator.PolygroupEdges;
			UpdateVisualization();
		});

	}
	
	if (bUsingSelection)
	{
		// create the preview object for the unmodified area
		UnmodifiedAreaPreviewMesh = NewObject<UPreviewMesh>();
		UnmodifiedAreaPreviewMesh->CreateInWorld(GetTargetWorld(), MeshTransform);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(UnmodifiedAreaPreviewMesh, Target);
		UnmodifiedAreaPreviewMesh->SetMaterials(MaterialSet.Materials);
		UnmodifiedAreaPreviewMesh->EnableSecondaryTriangleBuffers( [this](const FDynamicMesh3* Mesh, int32 TriangleID) {
			return SelectionTriangleROI->Contains(TriangleID);
		});
		UnmodifiedAreaPreviewMesh->SetSecondaryBuffersVisibility(false);
		UnmodifiedAreaPreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
		UnmodifiedAreaPreviewMesh->UpdatePreview(OriginalDynamicMesh.Get());
	}


	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(GetTargetWorld(), MeshTransform);

	Settings->WatchProperty(Settings->ConversionMode, [this](EConvertToPolygonsMode) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->bShowGroupColors, [this](bool) { UpdateVisualization(); });
	Settings->WatchProperty(Settings->AngleTolerance, [this](float) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->bUseAverageGroupNormal, [this](float) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->NumPoints, [this](int32) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->bSplitExisting, [this](bool) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->bNormalWeighted, [this](bool) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->NormalWeighting, [this](float) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->MinGroupSize, [this](int32) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->QuadAdjacencyWeight, [this](float) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->QuadMetricClamp, [this](float) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->QuadSearchRounds, [this](int) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->bRespectUVSeams, [this](int) { OnSettingsModified(); });
	Settings->WatchProperty(Settings->bRespectHardNormals, [this](int) { OnSettingsModified(); });

	// add group layer picking property set for source groups
	CopyFromLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	CopyFromLayerProperties->bSupportLabels = true;
	CopyFromLayerProperties->InitializeGroupLayers(OriginalDynamicMesh.Get());
	CopyFromLayerProperties->WatchProperty(CopyFromLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedFromGroupLayerChanged(); });
	AddToolPropertySource(CopyFromLayerProperties);
	UpdateFromGroupLayer();
	SetToolPropertySourceEnabled(CopyFromLayerProperties, Settings->ConversionMode == EConvertToPolygonsMode::CopyFromLayer);

	// add picker for output group layer
	Settings->OptionsList.Reset();
	Settings->OptionsList.Add(UE::PolygroupUtilLocals::DefaultLayer);		// always have standard group
	if (FDynamicMeshAttributeSet* Attributes = OriginalDynamicMesh->Attributes())
	{
		for (int32 k = 0; k < Attributes->NumPolygroupLayers(); k++)
		{
			FName Name = Attributes->GetPolygroupLayer(k)->GetName();
			Settings->OptionsList.Add(Name.ToString());
		}
		for (const auto&[Name, LayerAttr]: Attributes->GetTriangleLabelAttributes())
		{
			Settings->OptionsList.Add(Name.ToString());
		}
	}
	
	Settings->OptionsList.Add(UE::PolygroupUtilLocals::NewLayer);
	
	Settings->WatchProperty(Settings->GroupLayer, [&](FName NewName)
	{
		Settings->bShowNewLayerName = (NewName == UE::PolygroupUtilLocals::NewLayer);
		
		if (NewName == UE::PolygroupUtilLocals::DefaultLayer)
		{
			Settings->LayerType = EPerTriangleAttributeType::Integer;
		}
		else if (NewName != UE::PolygroupUtilLocals::NewLayer)
		{
			if (FDynamicMeshAttributeSet* Attributes = OriginalDynamicMesh->Attributes())
			{
				bool bPolygroupFound = false;
				for (int32 k = 0; k < Attributes->NumPolygroupLayers(); k++)
				{
					if (Attributes->GetPolygroupLayer(k)->GetName() == NewName)
					{
						Settings->LayerType = EPerTriangleAttributeType::Integer;
						bPolygroupFound = true;
						break;
					}
				}
		
				if (!bPolygroupFound)
				{
					if (Attributes->FindTriangleLabelAttribute(NewName))
					{
						Settings->LayerType = EPerTriangleAttributeType::Label;
					}
				}
			}
		}
	});

	if (bUsingSelection)
	{
		GeometrySelectionVizProperties = NewObject<UGeometrySelectionVisualizationProperties>(this);
		GeometrySelectionVizProperties->RestoreProperties(this);
		AddToolPropertySource(GeometrySelectionVizProperties);
		GeometrySelectionVizProperties->Initialize(this);
		GeometrySelectionVizProperties->bEnableShowTriangleROIBorder = true;
		GeometrySelectionVizProperties->SelectionElementType = static_cast<EGeometrySelectionElementType>(GeometrySelection.ElementType);
		GeometrySelectionVizProperties->SelectionTopologyType = static_cast<EGeometrySelectionTopologyType>(GeometrySelection.TopologyType);

		GeometrySelectionViz = NewObject<UPreviewGeometry>(this);
		GeometrySelectionViz->CreateInWorld(GetTargetWorld(), MeshTransform);
		UE::Geometry::InitializeGeometrySelectionVisualization(
			GeometrySelectionViz,
			GeometrySelectionVizProperties,
			*OriginalDynamicMesh,
			GeometrySelection,
			nullptr,
			nullptr,
			&TriangleROI);
	}

	// start the compute
	PreviewCompute->InvalidateResult();

	// updates the triangle color visualization
	UpdateVisualization();

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Cluster triangles of the Mesh into PolyGroups using various strategies"),
		EToolMessageLevel::UserNotification);
}

void UConvertToPolygonsTool::UpdateOpParameters(FConvertToPolygonsOp& ConvertToPolygonsOp) const
{
	ConvertToPolygonsOp.bCalculateNormals = Settings->bCalculateNormals;
	ConvertToPolygonsOp.ConversionMode    = Settings->ConversionMode;
	ConvertToPolygonsOp.AngleTolerance    = Settings->AngleTolerance;
	ConvertToPolygonsOp.bUseAverageGroupNormal = Settings->bUseAverageGroupNormal;
	ConvertToPolygonsOp.NumPoints = Settings->NumPoints;
	ConvertToPolygonsOp.bSubdivideExisting = Settings->bSplitExisting;
	ConvertToPolygonsOp.WeightingType = (Settings->bNormalWeighted) ? FPolygroupsGenerator::EWeightingType::NormalDeviation : FPolygroupsGenerator::EWeightingType::None;
	ConvertToPolygonsOp.WeightingCoeffs = FVector3d(Settings->NormalWeighting, 1.0, 1.0);
	ConvertToPolygonsOp.MinGroupSize = Settings->MinGroupSize;
	ConvertToPolygonsOp.QuadMetricClamp = Settings->QuadMetricClamp;
	ConvertToPolygonsOp.QuadAdjacencyWeight = Settings->QuadAdjacencyWeight;
	ConvertToPolygonsOp.QuadSearchRounds = Settings->QuadSearchRounds;
	ConvertToPolygonsOp.bRespectUVSeams = Settings->bRespectUVSeams;
	ConvertToPolygonsOp.bRespectHardNormals = Settings->bRespectHardNormals;

	ConvertToPolygonsOp.OriginalMesh = ComputeOperatorSharedMesh;

	if (bUsingSelection)
	{
		ConvertToPolygonsOp.InitialGroupID = OriginalDynamicMesh->MaxGroupID();
	}

	if (ActiveFromGroupSetAdapter.IsValid())
	{
		ConvertToPolygonsOp.SourcePolyGroups = ActiveFromGroupSetAdapter;
		if (bUsingSelection)
		{
			ConvertToPolygonsOp.InitialGroupID = ActiveFromGroupSetAdapter->GetMaxGroupID();
		}
	}

	FTransform LocalToWorld = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	ConvertToPolygonsOp.SetTransform(LocalToWorld);
}



void UConvertToPolygonsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	UE::ToolTarget::ShowSourceObject(Target);

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
		PreviewGeometry = nullptr;
	}

	if (UnmodifiedAreaPreviewMesh != nullptr)
	{
		UnmodifiedAreaPreviewMesh->Disconnect();
		UnmodifiedAreaPreviewMesh = nullptr;
	}

	if (PreviewCompute)
	{
		FDynamicMeshOpResult Result = PreviewCompute->Shutdown();
		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("ConvertToPolygonsToolTransactionName", "Find PolyGroups"));
			FDynamicMesh3* DynamicMeshResult = Result.Mesh.Get();
			if (ensure(DynamicMeshResult != nullptr))
			{
				if (Settings->GroupLayer != UE::PolygroupUtilLocals::DefaultLayer)
				{
					DoUpdateMeshAttribute(*DynamicMeshResult);
				}
				else
				{
					// todo: have not actually modified topology here, but groups-only update is not supported yet

					if (bUsingSelection)
					{
						for (int32 tid : *SelectionTriangleROI)
						{
							int32 GroupID = DynamicMeshResult->GetTriangleGroup(OriginalSubmesh->MapTriangleToSubmesh(tid));
							OriginalDynamicMesh->SetTriangleGroup(tid, GroupID);
						}
						UE::ToolTarget::CommitDynamicMeshUpdate(Target, *OriginalDynamicMesh, true);
					}
					else
					{
						UE::ToolTarget::CommitDynamicMeshUpdate(Target, *DynamicMeshResult, true);
					}
				}
			}

			if (bUsingSelection)
			{
				// if the input was a group selection, that selection is no longer valid. But if we output
				// a triangle selection it should be converted to the group selection
				FGeometrySelection OutputSelection;
				for (int32 tid : *SelectionTriangleROI)
				{
					OutputSelection.Selection.Add(FGeoSelectionID::MeshTriangle(tid).Encoded());
				}
				UE::Geometry::SetToolOutputGeometrySelectionForTarget(this, Target, OutputSelection);
			}

			GetToolManager()->EndUndoTransaction();
		}
	}

	if (ComputeOperatorSharedMesh.IsValid())
	{
		ComputeOperatorSharedMesh->ReleaseSharedObject();
	}

	Super::OnShutdown(ShutdownType);
}

void UConvertToPolygonsTool::OnSettingsModified()
{
	SetToolPropertySourceEnabled(CopyFromLayerProperties, Settings->ConversionMode == EConvertToPolygonsMode::CopyFromLayer);

	PreviewCompute->InvalidateResult();
}


void UConvertToPolygonsTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	PreviewCompute->Tick(DeltaTime);
}


void UConvertToPolygonsTool::OnSelectedFromGroupLayerChanged()
{
	UpdateFromGroupLayer();
	PreviewCompute->InvalidateResult();
}

void UConvertToPolygonsTool::UpdateFromGroupLayer()
{
	ComputeOperatorSharedMesh->AccessSharedObject([&](const FDynamicMesh3& ReadMesh)
	{
		if (CopyFromLayerProperties->HasSelectedPolygroup() == false)
		{
			ActiveFromGroupSetAdapter = MakeShared<FPolygroupLayerAdapter, ESPMode::ThreadSafe>(&ReadMesh, nullptr);
		}
		else
		{
			const FName SelectedName = CopyFromLayerProperties->ActiveGroupLayer;
			const FDynamicMeshPolygroupAttribute* FoundAttrib = FindPolygroupLayerByName(ReadMesh, SelectedName);
			if (!FoundAttrib)
			{
				if (const FDynamicMeshTriangleLabelAttribute* Labels = FindTriangleLabelLayerByName(ReadMesh, SelectedName))
				{
					ActiveFromGroupSetAdapter = MakeShared<FTriangleReadOnlyLabelAdapter>(&ReadMesh, Labels);
					return;
				}
			}
			
			ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
			ActiveFromGroupSetAdapter = MakeShared<FPolygroupLayerAdapter, ESPMode::ThreadSafe>(&ReadMesh, FoundAttrib);
		}
	});
}


void UConvertToPolygonsTool::UpdateVisualization()
{
	if (!PreviewCompute)
	{
		return;
	}

	IMaterialProvider* MaterialTarget = Cast<IMaterialProvider>(Target);
	FComponentMaterialSet MaterialSet;
	if (Settings->bShowGroupColors)
	{
		int32 NumMaterials = MaterialTarget->GetNumMaterials();
		for (int32 i = 0; i < NumMaterials; ++i)
		{ 
			MaterialSet.Materials.Add(ToolSetupUtil::GetVertexColorMaterial(GetToolManager()));
		}
		PreviewCompute->PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
		{
			return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
		}, 
		UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	else
	{
		MaterialTarget->GetMaterialSet(MaterialSet);
		PreviewCompute->PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	PreviewCompute->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

	FColor GroupLineColor = FColor::Red;
	float GroupLineThickness = 2.0f;

	ComputeOperatorSharedMesh->AccessSharedObject([&](const FDynamicMesh3& ReadMesh)
	{
		PreviewGeometry->CreateOrUpdateLineSet(TEXT("GroupBorders"), PolygonEdges.Num(), 
			[&](int32 k, TArray<FRenderableLine>& LinesOut) {
				FVector3d A, B;
				ReadMesh.GetEdgeV(PolygonEdges[k], A, B);
				LinesOut.Add(FRenderableLine(A, B, GroupLineColor, GroupLineThickness));
			}, 1);
	});
}

void UConvertToPolygonsTool::DoUpdateMeshAttribute(const FDynamicMesh3& InMesh) const
{
	if (Settings->GroupLayer == UE::PolygroupUtilLocals::DefaultLayer)
	{
		return;
	}

	FDynamicMesh3 UseResultMesh = *OriginalDynamicMesh;
	if (UseResultMesh.HasAttributes() == false)
	{
		UseResultMesh.EnableAttributes();
	}

	switch(Settings->LayerType)
	{
	case EPerTriangleAttributeType::Integer:
	{
		// if we want to write to any layer other than default, we have to find or create it
		FDynamicMeshPolygroupAttribute* UseAttribLayer = nullptr;
		if (Settings->GroupLayer == UE::PolygroupUtilLocals::NewLayer)
		{
			// append new group layer and set it's name
			int32 TargetLayerIdx = UseResultMesh.Attributes()->NumPolygroupLayers();
			UseResultMesh.Attributes()->SetNumPolygroupLayers(TargetLayerIdx + 1);
			UseAttribLayer = UseResultMesh.Attributes()->GetPolygroupLayer(TargetLayerIdx);
			FString UseUniqueName = MakeUniqueGroupLayerName(UseResultMesh, Settings->NewLayerName);
			UseAttribLayer->SetName( FName(UseUniqueName) );
		}
		else
		{
			UseAttribLayer = FindPolygroupLayerByName(UseResultMesh, Settings->GroupLayer);
		}

		if (UseAttribLayer)
		{
			// copy the generated groups from the Op output mesh (stored in primary groups) to the target layer
			if (bUsingSelection)
			{
				for (int32 tid : *SelectionTriangleROI)
				{
					int32 GroupID = InMesh.GetTriangleGroup(OriginalSubmesh->MapTriangleToSubmesh(tid));
					UseAttribLayer->SetValue(tid, GroupID);
				}
			}
			else
			{
				for (int32 tid : UseResultMesh.TriangleIndicesItr())
				{
					UseAttribLayer->SetValue(tid, InMesh.GetTriangleGroup(tid));
				}
			}
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, UseResultMesh, true);
		}
		else
		{
			// If we can't find or create the layer (which should not be possible) the tool is going to do nothing,
			// this is the safest option at this point
			UE_LOGF(LogGeometry, Warning, "Output group layer missing?");
		}
		return;
	}
	case EPerTriangleAttributeType::Label:
	{
		// if we want to write to any layer other than default, we have to find or create it
		FDynamicMeshTriangleLabelAttribute* LabelLayer = [&UseResultMesh, LayerName = Settings->GroupLayer, NewName = Settings->NewLayerName]
		{
			if (LayerName == UE::PolygroupUtilLocals::NewLayer)
			{
				FName NewAttrName(NewName);
				int32 NumberCounter = 0;	
				while (UseResultMesh.Attributes()->FindTriangleLabelAttribute(NewAttrName))
				{
					NewAttrName.SetNumber(NumberCounter++);
				}
				FDynamicMeshTriangleLabelAttribute* NewAttr = new FDynamicMeshTriangleLabelAttribute(&UseResultMesh);
				UseResultMesh.Attributes()->AttachTriangleLabelAttribute(NewAttrName, NewAttr);
				return NewAttr;
			}
			
			return UseResultMesh.Attributes()->FindTriangleLabelAttribute(LayerName);
		}();

		if (LabelLayer)
		{
			FTriangleLabelAdapter LabelAdapter(&UseResultMesh, LabelLayer);
						
			// copy the generated groups from the Op output mesh (stored in primary groups) to the target layer
			if (bUsingSelection)
			{
				for (int32 tid : *SelectionTriangleROI)
				{
					int32 GroupID = InMesh.GetTriangleGroup(OriginalSubmesh->MapTriangleToSubmesh(tid));
					LabelAdapter.SetGroup(tid, GroupID);
				}
			}
			else
			{
				for (int32 tid : UseResultMesh.TriangleIndicesItr())
				{
					LabelAdapter.SetGroup(tid, InMesh.GetTriangleGroup(tid));
				}
			}
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, UseResultMesh, true);
		}
		else
		{
			// If we can't find or create the layer (which should not be possible) the tool is going to do nothing,
			// this is the safest option at this point
			UE_LOGF(LogGeometry, Warning, "Output group layer missing?");
		}
		return;
	}
	default:
		break;
	}
}


#undef LOCTEXT_NAMESPACE

