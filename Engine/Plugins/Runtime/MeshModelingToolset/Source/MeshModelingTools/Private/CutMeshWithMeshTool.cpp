// Copyright Epic Games, Inc. All Rights Reserved.

#include "CutMeshWithMeshTool.h"
#include "CompositionOps/BooleanMeshesOp.h"
#include "ToolSetupUtil.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "Async/Async.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CutMeshWithMeshTool)

using namespace UE::Geometry;

namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution CutMeshWithMeshToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution CutMeshWithMeshToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}


#define LOCTEXT_NAMESPACE "UCutMeshWithMeshTool"

void UCutMeshWithMeshTool::SetupProperties()
{
	Super::SetupProperties();

	CutProperties = NewObject<UCutMeshWithMeshToolProperties>(this);
	CutProperties->RestoreProperties(this);
	AddToolPropertySource(CutProperties);

	SetToolDisplayName(LOCTEXT("CutMeshWithMeshToolName", "Cut With Mesh"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Cut the first input mesh with the second input mesh. Use the transform gizmos to modify the position and orientation of the input objects."),
		EToolMessageLevel::UserNotification);


	// create intersection preview mesh object
	IntersectPreviewMesh = NewObject<UPreviewMesh>(this);
	IntersectPreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(IntersectPreviewMesh, nullptr); 
	IntersectPreviewMesh->SetVisible(true);
	IntersectPreviewMesh->SetMaterial(ToolSetupUtil::GetDefaultBrushVolumeMaterial(GetToolManager()));
}

void UCutMeshWithMeshTool::SaveProperties()
{
	Super::SaveProperties();
	CutProperties->SaveProperties(this);

	IntersectPreviewMesh->Disconnect();
}


void UCutMeshWithMeshTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	// disable output options
	// (this property set is not registered yet in SetupProperties() above)
	SetToolPropertySourceEnabled(HandleSourcesProperties, false);
	SetToolPropertySourceEnabled(OutputTypeProperties, false);

	FComponentMaterialSet AllMaterialSet;
	TArray<TArray<int>> MaterialRemap; MaterialRemap.SetNum(Targets.Num());

	if (!CutProperties->bUseFirstMeshMaterials)
	{
		
		TMap<UMaterialInterface*, int> KnownMaterials;
		for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			const FComponentMaterialSet ComponentMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[ComponentIdx]);
			for (UMaterialInterface* Mat : ComponentMaterialSet.Materials)
			{
				int* FoundMatIdx = KnownMaterials.Find(Mat);
				int MatIdx;
				if (FoundMatIdx)
				{
					MatIdx = *FoundMatIdx;
				}
				else
				{
					MatIdx = AllMaterialSet.Materials.Add(Mat);
					KnownMaterials.Add(Mat, MatIdx);
				}
				MaterialRemap[ComponentIdx].Add(MatIdx);
			}
		}
	}
	else
	{
		AllMaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
		for (int MatIdx = 0; MatIdx < AllMaterialSet.Materials.Num(); MatIdx++)
		{
			MaterialRemap[0].Add(MatIdx);
		}
		for (int ComponentIdx = 1; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			MaterialRemap[ComponentIdx].Init(0, Cast<IMaterialProvider>(Targets[ComponentIdx])->GetNumMaterials());
		}
	}

	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> Mesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		*Mesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx]);

		// ensure materials and attributes are always enabled
		Mesh->EnableAttributes();
		Mesh->Attributes()->EnableMaterialID();
		FDynamicMeshMaterialAttribute* MaterialIDs = Mesh->Attributes()->GetMaterialID();
		int32 NumComponentMaterials = MaterialRemap[ComponentIdx].Num();
		// If component had valid materials, remap them. (If it had no materials, no IDs are valid, so just leave them as they were.)
		if (NumComponentMaterials > 0)
		{
			for (int TID : Mesh->TriangleIndicesItr())
			{
				int32 MID = MaterialIDs->GetValue(TID);
				MaterialIDs->SetValue(TID, MaterialRemap[ComponentIdx][MID >= 0 && MID < NumComponentMaterials ? MID : 0]);
			}
		}

		if (ComponentIdx == 0)
		{
			OriginalTargetMesh = Mesh;
		}
		else
		{
			OriginalCuttingMesh = Mesh;
		}
	}
	Preview->ConfigureMaterials(AllMaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

	// check if we have the same mesh on both inputs
	if (Cast<IAssetBackedTarget>(Targets[0]) != nullptr && Cast<IAssetBackedTarget>(Targets[0])->HasSameSourceData(Targets[1]))
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("SameSourceError", "WARNING: Both input meshes have the same Asset; both inputs will be affected."),
			EToolMessageLevel::UserWarning);
	}
}




class FCutMeshWithMeshOp : public FDynamicMeshOperator
{
public:
	virtual ~FCutMeshWithMeshOp() {}

	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> TargetMesh;
	FTransform TargetMeshTransform;
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> CuttingMesh;
	FTransform CuttingMeshTransform;

	bool bAttemptToFixHoles = true;
	bool bCollapseExtraEdges = true;
	double WindingThreshold = 0.5;

	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		TUniquePtr<FBooleanMeshesOp> SubtractOp = MakeUnique<FBooleanMeshesOp>();
		SubtractOp->CSGOperation = ECSGOperation::DifferenceAB;
		SubtractOp->bAttemptFixHoles = bAttemptToFixHoles;
		SubtractOp->bTryCollapseExtraEdges = bCollapseExtraEdges;
		SubtractOp->WindingThreshold = WindingThreshold;
		SubtractOp->Meshes.Add(TargetMesh);
		SubtractOp->Transforms.Add(TargetMeshTransform);
		SubtractOp->Meshes.Add(CuttingMesh);
		SubtractOp->Transforms.Add(CuttingMeshTransform);

		TUniquePtr<FBooleanMeshesOp> IntersectOp = MakeUnique<FBooleanMeshesOp>();
		IntersectOp->CSGOperation = ECSGOperation::Intersect;
		IntersectOp->bAttemptFixHoles = bAttemptToFixHoles;
		IntersectOp->bTryCollapseExtraEdges = bCollapseExtraEdges;
		IntersectOp->WindingThreshold = WindingThreshold;
		IntersectOp->Meshes.Add(TargetMesh);
		IntersectOp->Transforms.Add(TargetMeshTransform);
		IntersectOp->Meshes.Add(CuttingMesh);
		IntersectOp->Transforms.Add(CuttingMeshTransform);

		TFuture<void> SubtractFuture = Async(CutMeshWithMeshToolAsyncExecTarget, [&]()
		{
			SubtractOp->CalculateResult(Progress);
		});
		TFuture<void> IntersectFuture = Async(CutMeshWithMeshToolAsyncExecTarget, [&]()
		{
			IntersectOp->CalculateResult(Progress);
		});

		SubtractFuture.Wait();
		IntersectFuture.Wait();

		this->ResultMesh = SubtractOp->ExtractResult();
		SetResultTransform(SubtractOp->GetResultTransform());

		IntersectMesh = IntersectOp->ExtractResult();

		CreatedSubtractBoundaryEdges = SubtractOp->GetCreatedBoundaryEdges();
		CreatedIntersectBoundaryEdges = IntersectOp->GetCreatedBoundaryEdges();
	}

	TUniquePtr<FDynamicMesh3> IntersectMesh;
	TArray<int> CreatedSubtractBoundaryEdges;
	TArray<int> CreatedIntersectBoundaryEdges;
};





void UCutMeshWithMeshTool::SetPreviewCallbacks()
{	
	DrawnLineSet = NewObject<ULineSetComponent>(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetupAttachment(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager()));
	DrawnLineSet->RegisterComponent();

	Preview->OnOpCompleted.AddLambda(
		[this](const FDynamicMeshOperator* Op)
		{
			const FCutMeshWithMeshOp* CuttingOp = (const FCutMeshWithMeshOp*)(Op);
			CreatedSubtractBoundaryEdges = CuttingOp->CreatedSubtractBoundaryEdges;
			CreatedIntersectBoundaryEdges = CuttingOp->CreatedIntersectBoundaryEdges;
			IntersectionMesh = *CuttingOp->IntersectMesh;		// cannot steal this here because it is const...
			IntersectPreviewMesh->UpdatePreview(&IntersectionMesh);
			IntersectPreviewMesh->SetTransform((FTransform)Op->GetResultTransform());
		}
	);
	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute*)
		{
			GetToolManager()->PostInvalidation();
			UpdateVisualization();
		}
	);
}


void UCutMeshWithMeshTool::UpdateVisualization()
{
	constexpr FColor BoundaryEdgeColor(240, 15, 15);
	constexpr float BoundaryEdgeThickness = 2.0;
	constexpr float BoundaryEdgeDepthBias = 2.0f;

	DrawnLineSet->Clear();
	if (CutProperties->bShowNewBoundaries)
	{
		const FDynamicMesh3* TargetMesh = Preview->PreviewMesh->GetPreviewDynamicMesh();
		FVector3d A, B;
		for (int EID : CreatedSubtractBoundaryEdges)
		{
			TargetMesh->GetEdgeV(EID, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}

		for (int EID : CreatedIntersectBoundaryEdges)
		{
			IntersectionMesh.GetEdgeV(EID, A, B);
			DrawnLineSet->AddLine((FVector)A, (FVector)B, BoundaryEdgeColor, BoundaryEdgeThickness, BoundaryEdgeDepthBias);
		}
	}
}



TUniquePtr<FDynamicMeshOperator> UCutMeshWithMeshTool::MakeNewOperator()
{
	TUniquePtr<FCutMeshWithMeshOp> CuttingOp = MakeUnique<FCutMeshWithMeshOp>();
	
	CuttingOp->TargetMesh = OriginalTargetMesh;
	CuttingOp->TargetMeshTransform = TransformProxies[0]->GetTransform();

	CuttingOp->CuttingMesh = OriginalCuttingMesh;
	CuttingOp->CuttingMeshTransform = TransformProxies[1]->GetTransform();

	CuttingOp->bAttemptToFixHoles = CutProperties->bTryFixHoles;
	CuttingOp->bCollapseExtraEdges = CutProperties->bTryCollapseEdges;
	CuttingOp->WindingThreshold = CutProperties->WindingThreshold;

	return CuttingOp;
}



void UCutMeshWithMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCutMeshWithMeshToolProperties, bUseFirstMeshMaterials)))
	{
		if (!AreAllTargetsValid())
		{
			GetToolManager()->DisplayMessage(LOCTEXT("InvalidTargets", "Input meshes are no longer valid"), EToolMessageLevel::UserWarning);
			return;
		}
		ConvertInputsAndSetPreviewMaterials(false);
		Preview->InvalidateResult();
	}
	else if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCutMeshWithMeshToolProperties, bShowNewBoundaries)))
	{
		GetToolManager()->PostInvalidation();
		UpdateVisualization();
	}
	else
	{
		Super::OnPropertyModified(PropertySet, Property);
	}
}


FString UCutMeshWithMeshTool::GetCreatedAssetName() const
{
	return TEXT("Boolean");
}


FText UCutMeshWithMeshTool::GetActionName() const
{
	return LOCTEXT("CutMeshWithMeshActionName", "Cut Mesh");
}


void UCutMeshWithMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	SaveProperties();
	HandleSourcesProperties->SaveProperties(this);
	TransformProperties->SaveProperties(this);

	FDynamicMeshOpResult OpResult = Preview->Shutdown();
	// Restore (unhide) the source meshes
	for ( int32 ci = 0; ci < Targets.Num(); ++ci)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ci]);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(GetActionName());

		TArray<AActor*> SelectActors;

		FComponentMaterialSet MaterialSet;
		MaterialSet.Materials = GetOutputMaterials();

		// update subtract asset
		FTransform3d TargetToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
		{
			if (OpResult.Mesh->TriangleCount() > 0)
			{
				MeshTransforms::ApplyTransform(*OpResult.Mesh, OpResult.Transform, true);
				MeshTransforms::ApplyTransformInverse(*OpResult.Mesh, TargetToWorld, true);
				UE::ToolTarget::CommitMeshDescriptionUpdateViaDynamicMesh(Targets[0], *OpResult.Mesh, true);
				Cast<IMaterialProvider>(Targets[0])->CommitMaterialSetUpdate(MaterialSet, true);
			}
		}
		SelectActors.Add(UE::ToolTarget::GetTargetActor(Targets[0]));

		// create intersection asset
		if ( IntersectionMesh.TriangleCount() > 0)
		{
			MeshTransforms::ApplyTransform(IntersectionMesh, OpResult.Transform, true);
			MeshTransforms::ApplyTransformInverse(IntersectionMesh, TargetToWorld, true);
			FTransform3d NewTransform = TargetToWorld;

			FString CurName = UE::Modeling::GetComponentAssetBaseName(UE::ToolTarget::GetTargetComponent(Targets[0]));
			FString UseBaseName = FString::Printf(TEXT("%s_%s"), *CurName, TEXT("CutPart") );

			FCreateMeshObjectParams NewMeshObjectParams;
			NewMeshObjectParams.TargetWorld = GetTargetWorld();
			NewMeshObjectParams.Transform = (FTransform)NewTransform;
			NewMeshObjectParams.BaseName = UseBaseName;
			NewMeshObjectParams.Materials = GetOutputMaterials();
			NewMeshObjectParams.SetMesh(&IntersectionMesh);
			// note: CutMeshWithMeshTool does not support converting types currently
			UE::ToolTarget::ConfigureCreateMeshObjectParams(Targets[0], NewMeshObjectParams);
			FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
			if (Result.IsOK() && Result.NewActor != nullptr)
			{
				SelectActors.Add(Result.NewActor);
			}
		}

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), SelectActors);
		GetToolManager()->EndUndoTransaction();
	}

	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	GizmoManager->DestroyAllGizmosByOwner(this);
}


#undef LOCTEXT_NAMESPACE

