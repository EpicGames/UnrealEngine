// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformMeshPolygonsTool.h"

#include "Curves/RichCurve.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ModelingTaskTypes.h"
#include "ModelingToolTargetUtil.h"
#include "Solvers/ConstrainedMeshDeformer.h"
#include "ToolBuilderUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeformMeshPolygonsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDeformMeshPolygonsTool"

class FDeformTask;


//Stores per-vertex data needed by the laplacian deformer object
//TODO: May be a candidate for a subclass of the FGroupTopologyLaplacianDeformer
struct FDeformerVertexConstraintData
{
	FDeformerVertexConstraintData& operator=(const FDeformerVertexConstraintData& other)
	{
		Position = other.Position;
		Weight   = other.Weight;
		bPostFix = other.bPostFix;
		return *this;
	}

	FVector3d Position;
	double Weight{0.0};
	bool bPostFix{false};
};

/**
*	FDeformTask is an object which wraps an asynchronous task to be run multiple times on a separate thread.
*	The Laplacian deformation process requires the use of potentially large sparse matrices and sparse multiplication.
*
*   Expected usage:
*
*
*   // define constraints.  Need Constraints[VertID] to hold the constraints for the corresponding vertex.
*   TArray<FDeformerVertexConstraintData> Constraints;
*   ....
*
*   // populate with the VertexIDs of the vertices that are in the region you wish to deform.
*   TArray<int32> SrcVertIDs;  //Basically a mini-index buffer.
*   ...
*
*   // Create or reuse a laplacian deformation task.
*   FDeformTask*   DeformTask = New FDeformTask(WeightScheme);
*
*   // the deformer will have to build a new mesh that represents the regions in SrcVertIDs;
*   // but set this to false on subsequent calls to UpdateDeformer if the SrcVertIDs array hasn't changed.
*   bool bRequiresRegion = true;

*   DefTask->UpdateDeformer(WeightScheme, Mesh, Constraints, SrcVertIDs, bRequiresRegion);
*
*   DeformTask->DoWork();  or DeformTask->StartBackgroundTask(); //which calls DoWork on background thread.
*
*  // wheh DeformTask->IsDone == true; you can copy the results back to the mesh
*  DeformTask->ExportResults(Mesh);
*
* Note: if only the positions in the Constraints change (e.g. handle positions) then subsequent calls
* to UpdateDeformer() and DoWork() will be much faster as the matrix system will not be rebuilt or re-factored
*/
class FConstrainedMeshDeformerTask : public FNonAbandonableTask
{
	friend class FAsyncTask<FDeformTask>;

public:
	enum
	{
		INACTIVE_SUBSET_ID = -1
	};

	FConstrainedMeshDeformerTask(const ELaplacianWeightScheme SelectedWeightScheme)
		: LaplacianWeightScheme(SelectedWeightScheme)
	{
	}

	virtual ~FConstrainedMeshDeformerTask(){};

	//NO idea what this is meant to do. Performance analysis maybe? Scheduling data?
	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FConstrainedMeshDeformerTask, STATGROUP_ThreadPoolAsyncTasks);
	}


	/** Called by the main thread in the tool, this copies the Constraint buffer right before the task begins on another thread.
	  * Ensures the FConstrainedMeshDeformer is using correct mesh subset and the selected settings, then updates on change in properties, i.e. weight scheme */
	void UpdateDeformer(const ELaplacianWeightScheme SelectedWeightScheme, const FDynamicMesh3& Mesh,
	                    const TArray<FDeformerVertexConstraintData>& ConstraintArray,
	                    const TArray<int32>& SrcIDBufferSubset, bool bNewTransaction, const FRichCurve* Curve);

	/** Required by the FAsyncTaskExecutor */
	void SetAbortSource(bool* bAbort)
	{
		bAbortSource = bAbort;
	};

	/** Called by the FAsyncTask<FDeformTask> object for background computation. */
	void DoWork();


	/** Updates the positions in the target mesh for regions that correspond to the subset mesh */
	void ExportResults(FDynamicMesh3& TargetMesh) const;

private:
	/** Creates the mesh (i.e. SubsetMesh) that corresponds to the region of the SrcMesh defined by the partial index buffer SrcIDBufferSubset */
	void InitializeSubsetMesh(const FDynamicMesh3& SrcMesh, const TArray<int32>& SrcIDBufferSubset);

	/** Attenuates the weights of the constraints using the selected curve */
	void ApplyAttenuation();

	/** Denotes the weight scheme being used by the running background task. Changes when selected property changes in editor. */
	ELaplacianWeightScheme LaplacianWeightScheme;

	/** positions for each vertex in the subset mesh - for use in the deformer */
	TArray<FVector3d> SubsetPositionBuffer;

	/** constraint data for each vertex in subset mesh - for use by the deformer*/
	TArray<FDeformerVertexConstraintData> SubsetConstraintBuffer;

	FRichCurve WeightAttenuationCurve;

	/** True only for the first update, and then false for the duration of the Input transaction
	  * It's passed in and copied in UpdateDeformer() */
	bool bIsNewTransaction = true;

	/** When true, the constraint weights will be attenuated based on distance using the provided curve object*/
	bool bAttenuateWeights = false;

	/** The abort bool used by the Task Deleter */
	bool* bAbortSource = nullptr;

	/** Used to initialize the array mapping, updated during the UpdateDeformer() function */
	int SrcMeshMaxVertexID = 0;

	/** A subset of the original mesh */
	FDynamicMesh3 SubsetMesh;

	/** Maps Subset Mesh VertexID to Src Mesh VertexID */
	TArray<int32> SubsetVertexIDToSrcVertexIDMap;

	/** Laplacian deformer object gets rebuilt each new transaction */
	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> ConstrainedDeformer;
};

class FGroupTopologyLaplacianDeformer : public FGroupTopologyDeformer
{
public:
	FGroupTopologyLaplacianDeformer() = default;

	virtual ~FGroupTopologyLaplacianDeformer();

	/** Used to begin a procedural addition of modified vertices */
	inline void ResetModifiedVertices()
	{
		ModifiedVertices.Empty();
	};

	/** Change tracking */
	template<typename ValidSetAppendContainerType>
	void RecordModifiedVertices(const ValidSetAppendContainerType& Container)
	{
		ModifiedVertices.Empty();
		ModifiedVertices.Append(Container);
	}

	/** Used to iteratively add to the active change set (TSet<>)*/
	inline void RecordModifiedVertex(int32 VertexID)
	{
		ModifiedVertices.Add(VertexID);
	};


	void SetActiveHandleFaces(const TArray<int>& FaceGroupIDs) override;
	void SetActiveHandleEdges(const TArray<int>& TopologyEdgeIDs) override;
	void SetActiveHandleCorners(const TArray<int>& TopologyCornerIDs) override;
	void SetActiveHandle(const TArray<int>& FaceGroupIDs, const TArray<int>& TopologyEdgeIDs, const TArray<int>& TopologyCornerIDs) override;


	/** Allocates shared storage for use in task synchronization */
	void InitBackgroundWorker(const ELaplacianWeightScheme WeightScheme);


	/** Coordinates the background tasks. Returns false if the worker was already running */
	bool UpdateAndLaunchdWorker(const ELaplacianWeightScheme WeightScheme, const FRichCurve* Curve = nullptr);

	/** Capture data about background task state.*/
	bool IsTaskInFlight() const;


	/** Sets the SrcMeshConstraintBuffer to have a size of MaxVertexID, and initializes with the current mesh positions, but weight zero*/
	void InitializeConstraintBuffer();

	/** Given an array of Group IDs, update the selection and record vertices */
	void UpdateSelection(const FDynamicMesh3* TargetMesh, const TArray<int>& Groups, bool bLocalizeDeformation);

	/** Updates the mesh preview and/or solvers upon user input, provided a deformation strategy */
	void UpdateSolution(FDynamicMesh3* TargetMesh,
	                    const TFunction<FVector3d(FDynamicMesh3* Mesh, int)>& HandleVertexDeformFunc) override;

	/** Updates the vertex positions of the mesh with the result from the last deformation solve. */
	void ExportDeformedPositions(FDynamicMesh3* TargetMesh);

	/** Returns true if the asynchronous task has finished. */
	inline bool IsDone()
	{
		return AsyncMeshDeformTask == nullptr || AsyncMeshDeformTask->IsDone();
	};

	/** Triggers abort on task and passes off ownership to deleter object */
	inline void Shutdown();

	const TArray<FROIFace>& GetROIFaces() const
	{
		return ROIFaces;
	}

	/** Stores the position of the vertex constraints and corresponding weights for the entire mesh.  This is used as a form of scratch space.*/
	TArray<FDeformerVertexConstraintData> SrcMeshConstraintBuffer;

	/** Array of vertex indices organized in groups of three - basically an index buffer - that defines the subset of the mesh that the deformation task will work on.*/
	TArray<int32> SubsetIDBuffer;

	/** Need to update the task with the current submesh */
	bool bTaskSubmeshIsDirty = true;

	/** Asynchronous task object. This object deals with expensive matrix functionality that computes the deformation of a local mesh. */
	FAsyncTaskExecuterWithAbort<FConstrainedMeshDeformerTask>* AsyncMeshDeformTask = nullptr;


	/** The weight which will be applied to the constraints corresponding to the handle vertices. */
	double HandleWeights = 1.0;

	/** This is set to true whenever the user interacts with the tool under laplacian deformation mode.
	  * It is set to false immediately before beginning a background task and cannot be set to false again until the work is done. */
	bool bDeformerNeedsToRun = false;


	/** When true, tells the solver to attempt to postfix the actual position of the handles to the constrained position */
	bool bPostfixHandles = false;

	//This is set to false only after
	//	1) the asynchronous deformation task is complete
	//	2) the main thread has seen it complete, and
	//	3) the main thread updates the vertex positions of the mesh one last time
	bool bVertexPositionsNeedSync = false;

	bool bLocalize = true;
};


//////////////////////////////
// DEBUG_SETTINGS

//Draw white triangles defining the selection subset
//#define DEBUG_ROI_TRIANGLES

//Draw pink circles around the handles
//#define DEBUG_ROI_HANDLES

//Draw points on the ROI vertices, White => Weight == 0, Black => Weight == 1
//#define DEBUG_ROI_WEIGHTS

//////////////////////////////


/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UDeformMeshPolygonsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UDeformMeshPolygonsTool* DeformTool = NewObject<UDeformMeshPolygonsTool>(SceneState.ToolManager);
	DeformTool->SetWorld(SceneState.World);
	return DeformTool;
}

/*
 * Tool
 */
UDeformMeshPolygonsTransformProperties::UDeformMeshPolygonsTransformProperties()
{
	DeformationStrategy = EGroupTopologyDeformationStrategy::Laplacian;
	TransformMode       = EQuickTransformerMode::AxisTranslation;
	bSelectVertices     = true;
	bSelectFaces        = true;
	bSelectEdges        = true;
	bShowWireframe      = false;
}

/*
 * Asynchronous Task
 */


void FConstrainedMeshDeformerTask::UpdateDeformer(const ELaplacianWeightScheme SelectedWeightScheme,
                                                  const FDynamicMesh3& SrcMesh,
                                                  const TArray<FDeformerVertexConstraintData>& ConstraintArray,
                                                  const TArray<int32>& SrcIDBufferSubset, bool bNewTransaction,
                                                  const FRichCurve* Curve)
{
	bIsNewTransaction  = bNewTransaction;
	SrcMeshMaxVertexID = SrcMesh.MaxVertexID();

	LaplacianWeightScheme = SelectedWeightScheme;

	bAttenuateWeights = (Curve != nullptr);
	if (bAttenuateWeights)
	{
		WeightAttenuationCurve = *Curve;
	}

	// Set-up the subset mesh.
	if (bIsNewTransaction)
	{
		//Copy the part of the mesh we want to deform into the SubsetMesh and create map from Src Mesh to the SubsetMesh.
		InitializeSubsetMesh(SrcMesh, SrcIDBufferSubset);
	}

	// only want the subset of constraints that correspond to our subset mesh.
	{
		const int32 NumSubsetVerts = SubsetVertexIDToSrcVertexIDMap.Num();
		SubsetConstraintBuffer.Empty(NumSubsetVerts);
		SubsetConstraintBuffer.AddUninitialized(NumSubsetVerts);

		for (int32 SubVertexID = 0; SubVertexID < SubsetVertexIDToSrcVertexIDMap.Num(); ++SubVertexID)
		{
			int32 SrcVtxID                      = SubsetVertexIDToSrcVertexIDMap[SubVertexID];
			SubsetConstraintBuffer[SubVertexID] = ConstraintArray[SrcVtxID];
		}
	}

	check(bIsNewTransaction || ConstrainedDeformer.IsValid());
}


void FConstrainedMeshDeformerTask::DoWork()
{
	//TODO: (simple optimization) -
	//	Instead of SrcVertexIDtoSubsetVertexIDMap, use SubsetVertexIDToSetVertexIDMap - then we can use the VertexIndicesItr()
	//	on the SubsetMesh to minimize the quantity of vertex indices we need to iterate at every following step.

	if (*bAbortSource == true)
	{
		return;
	}

	if (bIsNewTransaction) //Will only be true once per input transaction (click+drag)
	{
		// Create a new deformation solver.
		ConstrainedDeformer = UE::MeshDeformation::ConstructConstrainedMeshDeformer(LaplacianWeightScheme, SubsetMesh);

		if (bAttenuateWeights)
		{
			ApplyAttenuation();
		}

		//Update our deformer's constraints before deforming using the copy of the constraint buffer
		for (int32 SubsetVertexID = 0; SubsetVertexID < SubsetConstraintBuffer.Num(); ++SubsetVertexID)
		{
			FDeformerVertexConstraintData& CData = SubsetConstraintBuffer[SubsetVertexID];
			ConstrainedDeformer->AddConstraint(SubsetVertexID, CData.Weight, CData.Position, CData.bPostFix);
		}

		bIsNewTransaction = false;
	}
	else
	{
		//This else block is run every consecutive frame after the start of the input transaction because UpdateConstraintPosition() is very cheap (no factorizing or rebuilding)
		//Update only the positions of the constraints, as the weights cannot change mid-transaction

		for (int32 SubsetVertexID = 0; SubsetVertexID < SubsetConstraintBuffer.Num(); ++SubsetVertexID)
		{
			FDeformerVertexConstraintData& CData = SubsetConstraintBuffer[SubsetVertexID];
			ConstrainedDeformer->UpdateConstraintPosition(SubsetVertexID, CData.Position, CData.bPostFix);
		}
	}

	if (*bAbortSource == true)
	{
		return;
	}

	//Run the deformation process

	const bool bSuccessfulSolve = ConstrainedDeformer->Deform(SubsetPositionBuffer);


	if (bSuccessfulSolve)
	{
		if (*bAbortSource == true) //-V547
		{
			return;
		}
	}
	else
	{
		//UE_LOGF(LogTemp, Warning, "Laplacian deformation failed");
	}
}

inline void FConstrainedMeshDeformerTask::InitializeSubsetMesh(const FDynamicMesh3& SrcMesh,
                                                               const TArray<int32>& SrcIDBufferSubset)
{
	//These can be re-used until the user stops dragging
	SubsetMesh.Clear();
	SubsetPositionBuffer.Reset();

	//Initialize every element to -1, helps us keep track of vertices we've already added while iterating the triangles
	TArray<int32> SrcVertexIDToSubsetVertexIDMap;
	SrcVertexIDToSubsetVertexIDMap.Init(INACTIVE_SUBSET_ID, SrcMeshMaxVertexID);


	//Iterate the triangle array to append vertices, and then triangles to the temporary subset mesh all at once
	for (int32 i = 0; i < SrcIDBufferSubset.Num(); i += 3)
	{
		// Build the triangle
		FIndex3i Triangle;
		for (int32 v = 0; v < 3; ++v)
		{
			//It's the SrcVertexID because every element in the SrcIDBufferSubset is the Vertex ID of a vertex in the original mesh.
			const int32 SrcVertexID = SrcIDBufferSubset[i + v];
			int32& SubsetID         = SrcVertexIDToSubsetVertexIDMap[SrcVertexID];

			if (SubsetID == INACTIVE_SUBSET_ID) // we haven't already visited this vertex
			{
				const FVector3d Vertex = SrcMesh.GetVertex(SrcVertexID);
				SubsetID               = SubsetMesh.AppendVertex(Vertex);
			}

			Triangle[v] = SubsetID;
		}
		SubsetMesh.AppendTriangle(Triangle);
	}

	// create a mapping back to the original vertex IDs from the subset mesh
	int32 MaxSubMeshVertexID = SubsetMesh.MaxVertexID(); // Really MaxID + 1
	SubsetVertexIDToSrcVertexIDMap.Reset(MaxSubMeshVertexID);
	SubsetVertexIDToSrcVertexIDMap.AddUninitialized(MaxSubMeshVertexID);

	for (int32 SrcID = 0; SrcID < SrcVertexIDToSubsetVertexIDMap.Num(); ++SrcID)
	{
		const int32 SubsetVertexID = SrcVertexIDToSubsetVertexIDMap[SrcID];
		if (SubsetVertexID != INACTIVE_SUBSET_ID)
		{
			SubsetVertexIDToSrcVertexIDMap[SubsetVertexID] = SrcID;
		}
	}
}

void FConstrainedMeshDeformerTask::ExportResults(FDynamicMesh3& TargetMesh) const
{
	//Update the position buffer result
	for (int32 SubsetVertexID = 0; SubsetVertexID < SubsetVertexIDToSrcVertexIDMap.Num(); ++SubsetVertexID)
	{
		const int32 SrcVertexID  = SubsetVertexIDToSrcVertexIDMap[SubsetVertexID];
		const FVector3d Position = SubsetPositionBuffer[SubsetVertexID];

		TargetMesh.SetVertex(SrcVertexID, Position);
	}

	FMeshNormals::QuickRecomputeOverlayNormals(TargetMesh);
}

void FConstrainedMeshDeformerTask::ApplyAttenuation()
{
	TSet<int> Handles;

	auto InPlaceMinMaxElements = [](FVector3d& Min, FVector3d& Max, const FVector3d Test) {
		for (uint8 i = 0; i < 3; ++i)
		{
			Min[i] = Test[i] < Min[i] ? Test[i] : Min[i];
			Max[i] = Test[i] > Max[i] ? Test[i] : Max[i];
		}
	};

	//Experimental approach: Just going to try grabbing the bounding box of the entire mesh, then the bounding box of the handles as a point cloud.
	//						 We need a T value to pass to the Weights curve, so let's try finding the distance of each vertex V from line segment formed by the min/max handles
	//						 Divide the distance from the handles to vertex V by the length of the mesh's bounding box extent,
	//						 and that will provide a **ROUGH** approximation of the time value for our curve.
	//
	//											  Distance( LineSegment(MaxHandle,MinHandle) , V )
	// where T(V) is time value at V     T(V) = -----------------------------------------------------
	//   and V is the position								Length(MeshMin - MeshMax)
	//      of each vertex

	FVector3d Min{std::numeric_limits<double>::max(), std::numeric_limits<double>::max(),
	              std::numeric_limits<double>::max()};
	FVector3d Max{std::numeric_limits<double>::min(), std::numeric_limits<double>::min(),
	              std::numeric_limits<double>::min()};
	FVector3d MinHandles = Min;
	FVector3d MaxHandles = Max;
	double LeastWeight   = std::numeric_limits<double>::max();


	for (int32 SubVertexID = 0; SubVertexID < SubsetConstraintBuffer.Num(); ++SubVertexID)
	{
		FDeformerVertexConstraintData& CData = SubsetConstraintBuffer[SubVertexID];
		// Update bounding box
		InPlaceMinMaxElements(Min, Max, CData.Position);

		if (CData.Weight > 0.0)
		{
			LeastWeight = CData.Weight < LeastWeight ? CData.Weight : LeastWeight;

			// update bounding box
			InPlaceMinMaxElements(MinHandles, MaxHandles, CData.Position);
			Handles.Add(SubVertexID);
		}
	}

	double ExtentLength = Distance(Min, Max);

	// Is this why the system has memory?
	for (int32 SubVertexID = 0; SubVertexID < SubsetConstraintBuffer.Num(); ++SubVertexID)
	{
		if (!Handles.Contains(SubVertexID))
		{
			FDeformerVertexConstraintData& CData = SubsetConstraintBuffer[SubVertexID];
			FVector3d OtherPoint = (FVector3d)FMath::ClosestPointOnSegment((FVector)CData.Position, (FVector)MinHandles, (FVector)MaxHandles);
			double T = Distance(CData.Position, OtherPoint) / ExtentLength;
			CData.Weight = WeightAttenuationCurve.Eval(T) * LeastWeight;
		}
	}
}


/*
*	FGroupTopologyLaplacianDeformer methods
*/


void FGroupTopologyLaplacianDeformer::InitBackgroundWorker(const ELaplacianWeightScheme WeightScheme)
{
	//Initialize asynchronous deformation objects
	if (AsyncMeshDeformTask == nullptr)
	{
		AsyncMeshDeformTask = new FAsyncTaskExecuterWithAbort<FConstrainedMeshDeformerTask>(WeightScheme);
	}
}

void FGroupTopologyLaplacianDeformer::InitializeConstraintBuffer()
{
	//MaxVertexID is used because the array is potentially sparse.
	int MaxVertexID = Mesh->MaxVertexID();

	SrcMeshConstraintBuffer.SetNum(MaxVertexID);

	for (int32 VertexID : Mesh->VertexIndicesItr())
	{
		FDeformerVertexConstraintData& CD = SrcMeshConstraintBuffer[VertexID];
		CD.Position                       = Mesh->GetVertex(VertexID);
		CD.Weight                         = 0.0;
		CD.bPostFix                       = false;
	}
}

bool FGroupTopologyLaplacianDeformer::IsTaskInFlight() const
{
	return (AsyncMeshDeformTask != nullptr && !AsyncMeshDeformTask->IsDone());
}

bool FGroupTopologyLaplacianDeformer::UpdateAndLaunchdWorker(const ELaplacianWeightScheme SelectedWeightScheme,
                                                             const FRichCurve* Curve)
{
	/* Deformer needs to run if we've modified the constraints since the last time it finished. */
	if (AsyncMeshDeformTask == nullptr)
	{
		InitBackgroundWorker(SelectedWeightScheme);
	}

	if (bDeformerNeedsToRun && AsyncMeshDeformTask->IsDone())
	{
		bool bRebuildSubsetMesh = bTaskSubmeshIsDirty;

		FConstrainedMeshDeformerTask& Task = AsyncMeshDeformTask->GetTask();

		// Update the deformer's buffers and weight scheme
		// this creates the subset mesh if needed.
		Task.UpdateDeformer(SelectedWeightScheme, *Mesh, SrcMeshConstraintBuffer, SubsetIDBuffer, bRebuildSubsetMesh,
		                    Curve);

		// task now has valid submesh

		bTaskSubmeshIsDirty = false;

		//Launch second thread
		AsyncMeshDeformTask->StartBackgroundTask();

		bDeformerNeedsToRun      = false; // This was set to true above in UpdateSolution()
		bVertexPositionsNeedSync = true; // The task will generate new vertex positions.

		return true;
	}
	return false;
}

void FGroupTopologyLaplacianDeformer::SetActiveHandleFaces(const TArray<int>& FaceGroupIDs)
{
	Reset();

	check(FaceGroupIDs.Num() == 1); // multi-face not supported yet
	int GroupID = FaceGroupIDs[0];

	// find set of vertices in handle
	Topology->CollectGroupVertices(GroupID, HandleVertices);
	Topology->CollectGroupBoundaryVertices(GroupID, HandleBoundaryVertices);
	ModifiedVertices = HandleVertices;


	// list of adj groups.  may contain duplicates.
	TArray<int> AdjGroups;
	for (int BoundaryVert : HandleBoundaryVertices)
	{
		Topology->FindVertexNbrGroups(BoundaryVert, AdjGroups);
	}

	// Local neighborhood - Adjacent groups plus self
	TArray<int> NeighborhoodGroups;

	// Collect the rest of the 1-ring groups that are adjacent to the selected one.
	NeighborhoodGroups.Add(GroupID);
	for (int AdjGroup : AdjGroups)
	{
		NeighborhoodGroups.AddUnique(AdjGroup); // remove duplicates by add unique
	}

	CalculateROI(FaceGroupIDs, NeighborhoodGroups);

	UpdateSelection(Mesh, NeighborhoodGroups, bLocalize);

	// Save the positions of the selected region.
	SaveInitialPositions();
}


void FGroupTopologyLaplacianDeformer::SetActiveHandleEdges(const TArray<int>& TopologyEdgeIDs)
{
	Reset();

	for (int EdgeID : TopologyEdgeIDs)
	{
		const TArray<int>& EdgeVerts = Topology->GetGroupEdgeVertices(EdgeID);
		for (int VertID : EdgeVerts)
		{
			HandleVertices.Add(VertID);
		}
	}
	HandleBoundaryVertices = HandleVertices;
	ModifiedVertices       = HandleVertices;

	TArray<int> HandleGroups;
	TArray<int> NbrGroups;
	Topology->FindEdgeNbrGroups(TopologyEdgeIDs, NbrGroups);

	CalculateROI(HandleGroups, NbrGroups);

	UpdateSelection(Mesh, NbrGroups, bLocalize);

	// Save the positions of the selected region.
	SaveInitialPositions();
}

void FGroupTopologyLaplacianDeformer::SetActiveHandleCorners(const TArray<int>& CornerIDs)
{
	Reset();

	for (int CornerID : CornerIDs)
	{
		int VertID = Topology->GetCornerVertexID(CornerID);
		if (VertID >= 0)
		{
			HandleVertices.Add(VertID);
		}
	}
	HandleBoundaryVertices = HandleVertices;
	ModifiedVertices       = HandleVertices;

	TArray<int> HandleGroups;
	TArray<int> NbrGroups;

	Topology->FindCornerNbrGroups(CornerIDs, NbrGroups);


	CalculateROI(HandleGroups, NbrGroups);

	UpdateSelection(Mesh, NbrGroups, bLocalize);

	// Save the positions of the selected region.
	SaveInitialPositions();
}

void FGroupTopologyLaplacianDeformer::SetActiveHandle(const TArray<int>& FaceGroupIDs, const TArray<int>& TopologyEdgeIDs,
	const TArray<int>& TopologyCornerIDs)
{
	// Currently mixing different types of handles is not yet supported.
	
	if (!FaceGroupIDs.IsEmpty())
	{
		check(TopologyEdgeIDs.IsEmpty() && TopologyCornerIDs.IsEmpty());
		SetActiveHandleFaces(FaceGroupIDs);
		return;
	}
	
	if (!TopologyEdgeIDs.IsEmpty())
	{
		check(FaceGroupIDs.IsEmpty() && TopologyCornerIDs.IsEmpty());
		SetActiveHandleEdges(TopologyEdgeIDs);
		return;
	}

	if (!TopologyCornerIDs.IsEmpty())
	{
		check(FaceGroupIDs.IsEmpty() && TopologyEdgeIDs.IsEmpty());
		SetActiveHandleCorners(TopologyCornerIDs);
		return;
	}
}


void FGroupTopologyLaplacianDeformer::UpdateSelection(const FDynamicMesh3* TargetMesh, const TArray<int>& Groups,
                                                      bool bLocalizeDeformation)
{
	// Build an index buffer (SubsetIdBuffer) and a vertexId buffer (ModifidedVertices) for the region we want to change

	if (bLocalizeDeformation)
	{
		//For each group ID, retrieve the array of all TriangleIDs associated with that GroupID and append that array to the end of the TriSet to remove duplicates
		TSet<int> TriSet;
		for (const int32& GroupID : Groups)
		{
			TriSet.Append(Topology->GetGroupFaces(GroupID));
		} //Now we have every triangle ID involved in the transaction

		//Since we are flattening the Face to a set of 3 indices, we do 3 * number of triangles though it is too many.
		SubsetIDBuffer.Reset(3 * TriSet.Num());
		//Add each triangle's A,B, and C indices to the subset triangle array.
		for (const int& Tri : TriSet)
		{
			FIndex3i Triple = TargetMesh->GetTriangle(Tri);
			SubsetIDBuffer.Add(Triple.A);
			SubsetIDBuffer.Add(Triple.B);
			SubsetIDBuffer.Add(Triple.C);
		}
	}
	else
	{
		// the entire mesh.
		const int32 NumTris = TargetMesh->TriangleCount();
		SubsetIDBuffer.Reset(3 * NumTris);
		for (int TriId : TargetMesh->TriangleIndicesItr())
		{
			FIndex3i Triple = TargetMesh->GetTriangle(TriId);
			SubsetIDBuffer.Add(Triple.A);
			SubsetIDBuffer.Add(Triple.B);
			SubsetIDBuffer.Add(Triple.C);
		}
	}

	// Add the vertices to the set (eliminates duplicates.)  Todo: don't use a set.
	ResetModifiedVertices();
	for (int32 VertexID : SubsetIDBuffer)
	{
		RecordModifiedVertex(VertexID);
	}
}

// This actually updates constraints that correspond to the handle vertices.
void FGroupTopologyLaplacianDeformer::UpdateSolution(
    FDynamicMesh3* TargetMesh, const TFunction<FVector3d(FDynamicMesh3* Mesh, int)>& HandleVertexDeformFunc)
{
	// copy the current positions.
	FVertexPositionCache CurrentPositions;
	for (int VertexID : InitialPositions.Vertices)
	{
		CurrentPositions.AddVertex(TargetMesh, VertexID);
	}

	// Set the target mesh to the initial positions.
	// Note: this only updates the vertices in the selected region.
	InitialPositions.SetPositions(TargetMesh);

	//Reset the constraints
	for (int32 VertexID : ModifiedVertices)
	{
		//Get the vertex's data from the constraint buffer
		FDeformerVertexConstraintData& CData = SrcMeshConstraintBuffer[VertexID];

		CData.Position = TargetMesh->GetVertex(VertexID);
		CData.Weight   = 0.0; //A weight of zero is used to allow this point to move freely when moving the handles
		CData.bPostFix = false;
	}

	//Actually deform the handles and add a constraint.
	for (int VertexID : HandleVertices)
	{
		const FVector3d DeformPos = HandleVertexDeformFunc(TargetMesh, VertexID);

		//Get the vertex's data from the constraint buffer
		FDeformerVertexConstraintData& CData = SrcMeshConstraintBuffer[VertexID];

		//Set the new vertex data
		CData.Position = DeformPos;
		CData.Weight   = HandleWeights;
		CData.bPostFix = bPostfixHandles;
	}

	// Restore Current Positions.  This is done because the target mesh is being used to define the highlight region.
	// if we don't reset the positions the highlight mesh will appear to reset momentarily until the first laplacian solver result is available
	CurrentPositions.SetPositions(TargetMesh);

	bDeformerNeedsToRun = true;
}

void FGroupTopologyLaplacianDeformer::ExportDeformedPositions(FDynamicMesh3* TargetMesh)
{
	bool bIsWorking = IsTaskInFlight();
	if (AsyncMeshDeformTask != nullptr && !bIsWorking)
	{
		const FConstrainedMeshDeformerTask& Task = AsyncMeshDeformTask->GetTask();
		Task.ExportResults(*TargetMesh);
	}
}

inline FGroupTopologyLaplacianDeformer::~FGroupTopologyLaplacianDeformer()
{
	Shutdown();
}

inline void FGroupTopologyLaplacianDeformer::Shutdown()
{
	if (AsyncMeshDeformTask != nullptr)
	{
		if (AsyncMeshDeformTask->IsDone())
		{
			delete AsyncMeshDeformTask;
		}
		else
		{
			AsyncMeshDeformTask->CancelAndDelete();
		}

		AsyncMeshDeformTask = nullptr;
	}
}

/*
* Tool methods
*/

UDeformMeshPolygonsTool::UDeformMeshPolygonsTool()
{
	UInteractiveTool::SetToolDisplayName(LOCTEXT("DeformPolygroupsToolName", "PolyGroup Deform"));
}

void UDeformMeshPolygonsTool::Setup()
{
	UMeshSurfacePointTool::Setup();

	LaplacianDeformer = MakePimpl<FGroupTopologyLaplacianDeformer>();

	// create dynamic mesh component to use for live preview
	check(TargetWorld.IsValid());
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);

	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);
	DynamicMeshComponent->SetupAttachment(PreviewMeshActor->GetRootComponent());
	DynamicMeshComponent->RegisterComponent();
	WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	DynamicMeshComponent->SetWorldTransform((FTransform)WorldTransform);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(DynamicMeshComponent, Target);

	// set materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	// dynamic mesh configuration settings
	DynamicMeshComponent->SetTangentsType(EDynamicMeshComponentTangentsMode::AutoCalculated);
	DynamicMeshComponent->SetMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));
	OnDynamicMeshComponentChangedHandle =
	    DynamicMeshComponent->OnMeshChanged.Add(FSimpleMulticastDelegate::FDelegate::CreateUObject(
	        this, &UDeformMeshPolygonsTool::OnDynamicMeshComponentChanged));


	// add properties
	TransformProps = NewObject<UDeformMeshPolygonsTransformProperties>(this);
	AddToolPropertySource(TransformProps);

	// initialize AABBTree
	MeshSpatial.SetMesh(DynamicMeshComponent->GetMesh());
	PrecomputeTopology();

	//initialize topology selector
	TopoSelector.Initialize(DynamicMeshComponent->GetMesh(), &Topology);
	TopoSelector.SetSpatialSource([this]() { return &GetSpatial(); });
	TopoSelector.PointsWithinToleranceTest = [this](const FVector3d& Position1, const FVector3d& Position2, double TolScale) {
		return ToolSceneQueriesUtil::PointSnapQuery(CameraState, WorldTransform.TransformPosition(Position1), WorldTransform.TransformPosition(Position2),
			ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * TolScale);
	};

	// hide input StaticMeshComponent
	UE::ToolTarget::HideSourceObject(Target);

	// init state flags flags
	bInDrag = false;

	// initialize snap solver
	QuickAxisTranslater.Initialize();
	QuickAxisRotator.Initialize();

	// set up visualizers
	PolyEdgesRenderer.LineColor     = FLinearColor::Red;
	PolyEdgesRenderer.LineThickness = 2.0;
	HilightRenderer.LineColor       = FLinearColor::Green;
	HilightRenderer.LineThickness   = 4.0f;

	// Allocates buffers, sets up the asynchronous task
	// Copies the source mesh positions.
	const ELaplacianWeightScheme LaplacianWeightScheme =
	    ConvertToLaplacianWeightScheme(TransformProps->SelectedWeightScheme);
	LaplacianDeformer->InitBackgroundWorker(LaplacianWeightScheme);


	/**
	// How to add a curve for the weights.
	//Add a default curve for falloff
	FKeyHandle Keys[5];
	Keys[0] = TransformProps->DefaultFalloffCurve.UpdateOrAddKey(0.f, 1.f);
	Keys[1] = TransformProps->DefaultFalloffCurve.UpdateOrAddKey(0.25f, 0.25f);
	Keys[2] = TransformProps->DefaultFalloffCurve.UpdateOrAddKey(0.3333333f, 0.25f);
	Keys[3] = TransformProps->DefaultFalloffCurve.UpdateOrAddKey(0.6666667f, 1.25f);
	Keys[4] = TransformProps->DefaultFalloffCurve.UpdateOrAddKey(1.f, 1.4f);
	for (uint8 i = 0; i < 5; ++i)
	{
		TransformProps->DefaultFalloffCurve.SetKeyInterpMode(Keys[i], ERichCurveInterpMode::RCIM_Cubic);
	}
	TransformProps->WeightAttenuationCurve.EditorCurveData = TransformProps->DefaultFalloffCurve;
	*/

	if (Topology.Groups.Num() < 2)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoGroupsWarning",
			        "This object has only a single PolyGroup. Use the GrpGen, GrpPnt or TriSel (Create PolyGroup) tools to modify PolyGroups."),
			EToolMessageLevel::UserWarning);
	}

	GetToolManager()->DisplayMessage(
		LOCTEXT("DeformMeshPolygonsToolDescription", "Deform the mesh by directly manipulating (i.e. click-and-drag) the PolyGroup edges, faces, and vertices."),
		EToolMessageLevel::UserNotification);
}

void UDeformMeshPolygonsTool::Shutdown(EToolShutdownType ShutdownType)
{
	LongTransactions.CloseAll(GetToolManager());

	//Tell the background thread to cancel the rest of its jobs before we close;
	LaplacianDeformer.Reset();

	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);

		UE::ToolTarget::ShowSourceObject(Target);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
			GetToolManager()->BeginUndoTransaction(LOCTEXT("DeformMeshPolygonsToolTransactionName", "PolyGroup Deform"));
			DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				FConversionToMeshDescriptionOptions ConversionOptions;
				ConversionOptions.bSetPolyGroups =
					false; // don't save polygroups, as we may change these temporarily in this tool just to get a different edit effect
				UE::ToolTarget::CommitDynamicMeshUpdate(Target, ReadMesh, false, ConversionOptions);
			});
			GetToolManager()->EndUndoTransaction();
		}

		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->DestroyComponent();
		DynamicMeshComponent = nullptr;
	}

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

}


void UDeformMeshPolygonsTool::NextTransformTypeAction()
{
	if (bInDrag == false)
	{
		if (TransformProps->TransformMode == EQuickTransformerMode::AxisRotation)
		{
			TransformProps->TransformMode = EQuickTransformerMode::AxisTranslation;
		}
		else
		{
			TransformProps->TransformMode = EQuickTransformerMode::AxisRotation;
		}
		UpdateQuickTransformer();
	}
}


void UDeformMeshPolygonsTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
	                         TEXT("DeformNextTransformType"), LOCTEXT("DeformNextTransformType", "Next Transform Type"),
	                         LOCTEXT("DeformNextTransformTypeTooltip", "Cycle to next transform type"),
	                         EModifierKey::None, EKeys::Q, [this]() { NextTransformTypeAction(); });
}


void UDeformMeshPolygonsTool::OnDynamicMeshComponentChanged()
{
	bSpatialDirty = true;
	TopoSelector.Invalidate(true, false);


	//Makes sure the constraint buffer and position buffers reflect Undo/Redo changes
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();


	//Apply Undo/redo
	for (int VertexID : Mesh->VertexIndicesItr())
	{
		const FVector3d Position                                      = Mesh->GetVertex(VertexID);
		LaplacianDeformer->SrcMeshConstraintBuffer[VertexID].Position = Position;
	}

	// a deform task could still be in flight.
	if (LaplacianDeformer->AsyncMeshDeformTask != nullptr)
	{
		LaplacianDeformer->AsyncMeshDeformTask->CancelAndDelete();
		LaplacianDeformer->AsyncMeshDeformTask = nullptr;
		LaplacianDeformer->bTaskSubmeshIsDirty = true;
	}
}

FDynamicMeshAABBTree3& UDeformMeshPolygonsTool::GetSpatial()
{
	if (bSpatialDirty)
	{
		MeshSpatial.Build();
		bSpatialDirty = false;
	}
	return MeshSpatial;
}


bool UDeformMeshPolygonsTool::HitTest(const FRay& WorldRay, FHitResult& OutHit)
{
	FRay3d LocalRay(WorldTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
					WorldTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	FGroupTopologySelection Selection;
	FVector3d LocalPosition, LocalNormal;
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings();
	if (TopoSelector.FindSelectedElement(TopoSelectorSettings, LocalRay, Selection, LocalPosition, LocalNormal) == false)
	{
		return false;
	}

	if (Selection.SelectedCornerIDs.Num() > 0)
	{
		OutHit.FaceIndex   = Selection.GetASelectedCornerID();
		OutHit.Distance    = LocalRay.GetParameter(LocalPosition);
		OutHit.ImpactPoint = (FVector)WorldTransform.TransformPosition(LocalRay.PointAt(OutHit.Distance));
	}
	else if (Selection.SelectedEdgeIDs.Num() > 0)
	{
		OutHit.FaceIndex   = Selection.GetASelectedEdgeID();
		OutHit.Distance    = LocalRay.GetParameter(LocalPosition);
		OutHit.ImpactPoint = (FVector)WorldTransform.TransformPosition(LocalRay.PointAt(OutHit.Distance));
	}
	else
	{
		int HitTID = GetSpatial().FindNearestHitTriangle(LocalRay);
		if (HitTID != IndexConstants::InvalidID)
		{
			FTriangle3d Triangle;
			GetSpatial().GetMesh()->GetTriVertices(HitTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
			FIntrRay3Triangle3d Query(LocalRay, Triangle);
			Query.Find();
			OutHit.FaceIndex = HitTID;
			OutHit.Distance  = Query.RayParameter;
			OutHit.Normal    = (FVector)WorldTransform.TransformVectorNoScale(GetSpatial().GetMesh()->GetTriNormal(HitTID));
			OutHit.ImpactPoint = (FVector)WorldTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		}
	}
	return true;
}


void UDeformMeshPolygonsTool::OnBeginDrag(const FRay& WorldRay)
{
	FRay3d LocalRay(WorldTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
					WorldTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	HilightSelection.Clear();

	FGroupTopologySelection Selection;
	FVector3d LocalPosition, LocalNormal;
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings();
	bool bHit = TopoSelector.FindSelectedElement(TopoSelectorSettings, LocalRay, Selection, LocalPosition, LocalNormal);

	if (bHit == false)
	{
		bInDrag = false;
		return;
	}

	HilightSelection = Selection;

	FVector3d WorldHitPos    = WorldTransform.TransformPosition(LocalPosition);
	FVector3d WorldHitNormal = WorldTransform.TransformVector(LocalNormal);

	bInDrag             = true;
	StartHitPosWorld    = (FVector)WorldHitPos;
	LastHitPosWorld     = StartHitPosWorld;
	StartHitNormalWorld = (FVector)WorldHitNormal;

	QuickAxisRotator.ClearAxisLock();
	UpdateActiveSurfaceFrame(HilightSelection);
	UpdateQuickTransformer();

	LastBrushPosLocal  = (FVector)WorldTransform.InverseTransformPosition((FVector3d)LastHitPosWorld);
	StartBrushPosLocal = LastBrushPosLocal;

	// Record the requested deformation strategy - NB: will be forced to linear if there aren't any free points to solve.

	DeformationStrategy = TransformProps->DeformationStrategy;

	// Capture the part of the mesh that will deform

	if (DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian)
	{
		LaplacianDeformer->bLocalize = true; // TransformProps->bLocalizeDeformation;

		//Determine which of the following (corners, edges or faces) has been selected by counting the associated feature's IDs
		if (Selection.SelectedCornerIDs.Num() > 0)
		{
			//Add all the the Corner's adjacent poly-groups (NbrGroups) to the ongoing array of groups.
			LaplacianDeformer->SetActiveHandleCorners(Selection.SelectedCornerIDs.Array());
		}
		else if (Selection.SelectedEdgeIDs.Num() > 0)
		{
			//Add all the the edge's adjacent poly-groups (NbrGroups) to the ongoing array of groups.
			LaplacianDeformer->SetActiveHandleEdges(Selection.SelectedEdgeIDs.Array());
		}
		else if (Selection.SelectedGroupIDs.Num() > 0)
		{
			LaplacianDeformer->SetActiveHandleFaces(Selection.SelectedGroupIDs.Array());
		}


		// If there are actually no interior points, then we can't actually use the laplacian deformer. Need to fall back to the linear.
		bool bHasInteriorVerts = false;
		const auto& ROIFaces   = LaplacianDeformer->GetROIFaces();
		for (const auto& Face : ROIFaces)
		{
			bHasInteriorVerts = bHasInteriorVerts || (Face.InteriorVerts.Num() != 0);
		}

		if (!bHasInteriorVerts)
		{
			// Change to the linear strategy for this case.

			DeformationStrategy = EGroupTopologyDeformationStrategy::Linear;
		}
		else
		{
			// finalize the laplacian deformer : the task will need a new mesh that corresponds to the selected region.

			LaplacianDeformer->bTaskSubmeshIsDirty = true;
		}
	}

	if (DeformationStrategy == EGroupTopologyDeformationStrategy::Linear)
	{
		//Determine which of the following (corners, edges or faces) has been selected by counting the associated feature's IDs
		if (Selection.SelectedCornerIDs.Num() > 0)
		{
			//Add all the the Corner's adjacent poly-groups (NbrGroups) to the ongoing array of groups.
			LinearDeformer.SetActiveHandleCorners(Selection.SelectedCornerIDs.Array());
		}
		else if (Selection.SelectedEdgeIDs.Num() > 0)
		{
			//Add all the the edge's adjacent poly-groups (NbrGroups) to the ongoing array of groups.
			LinearDeformer.SetActiveHandleEdges(Selection.SelectedEdgeIDs.Array());
		}
		else if (Selection.SelectedGroupIDs.Num() > 0)
		{
			LinearDeformer.SetActiveHandleFaces(Selection.SelectedGroupIDs.Array());
		}
	}

	BeginChange();
}


void UDeformMeshPolygonsTool::UpdateActiveSurfaceFrame(FGroupTopologySelection& Selection)
{
	// update surface frame
	ActiveSurfaceFrame.Origin = (FVector3d)StartHitPosWorld;
	if (HilightSelection.SelectedCornerIDs.Num() == 1)
	{
		// just keeping existing axes...we don't have enough info to do something smarter
	}
	else
	{
		ActiveSurfaceFrame.AlignAxis(2, (FVector3d)StartHitNormalWorld);
		if (HilightSelection.SelectedEdgeIDs.Num() == 1)
		{
			FVector3d Tangent;
			if (Topology.GetGroupEdgeTangent(HilightSelection.GetASelectedEdgeID(), Tangent))
			{
				Tangent = WorldTransform.TransformVector(Tangent);
				ActiveSurfaceFrame.ConstrainedAlignAxis(0, Tangent, ActiveSurfaceFrame.Z());
			}
		}
	}
}


FQuickTransformer* UDeformMeshPolygonsTool::GetActiveQuickTransformer()
{
	if (TransformProps->TransformMode == EQuickTransformerMode::AxisRotation)
	{
		return &QuickAxisRotator;
	}
	else
	{
		return &QuickAxisTranslater;
	}
}


void UDeformMeshPolygonsTool::UpdateQuickTransformer()
{
	bool bUseLocalAxes =
	    (GetToolManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::Local);
	if (bUseLocalAxes)
	{
		GetActiveQuickTransformer()->SetActiveWorldFrame(ActiveSurfaceFrame);
	}
	else
	{
		GetActiveQuickTransformer()->SetActiveFrameFromWorldAxes((FVector3d)StartHitPosWorld);
	}
}


void UDeformMeshPolygonsTool::UpdateChangeFromROI(bool bFinal)
{
	if (ActiveVertexChange == nullptr)
	{
		return;
	}
	const bool bIsLaplacian = (DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	const TSet<int>& ModifiedVertices =
	    (bIsLaplacian) ? LaplacianDeformer->GetModifiedVertices() : LinearDeformer.GetModifiedVertices();
	ActiveVertexChange->SaveVertices(Mesh, ModifiedVertices, !bFinal);
	const TSet<int>& ModifiedNormals =
	    (bIsLaplacian) ? LaplacianDeformer->GetModifiedOverlayNormals() : LinearDeformer.GetModifiedOverlayNormals();
	ActiveVertexChange->SaveOverlayNormals(Mesh, ModifiedNormals, !bFinal);
}


void UDeformMeshPolygonsTool::OnUpdateDrag(const FRay& Ray)
{
	if (bInDrag)
	{
		bUpdatePending = true;
		UpdateRay      = Ray;
	}
}

void UDeformMeshPolygonsTool::OnEndDrag(const FRay& Ray)
{
	bInDrag        = false;
	bUpdatePending = false;

	// update spatial
	bSpatialDirty = true;

	HilightSelection.Clear();
	TopoSelector.Invalidate(true, false);
	QuickAxisRotator.Reset();
	QuickAxisTranslater.Reset();

	//If it's linear, it's computed real time with no delay. This may need to be restructured for clarity by using the background task for this as well.
	if (DeformationStrategy == EGroupTopologyDeformationStrategy::Linear)
	{
		// close change record
		EndChange();
	}
}

void UDeformMeshPolygonsTool::OnCancelDrag()
{
	bInDrag = false;
	bUpdatePending = false;

	HilightSelection.Clear();
	TopoSelector.Invalidate(true, false);
	QuickAxisRotator.Reset();
	QuickAxisTranslater.Reset();

	if (ActiveVertexChange)
	{
		LongTransactions.Close(GetToolManager());
		delete ActiveVertexChange;
		ActiveVertexChange = nullptr;
	}
}


bool UDeformMeshPolygonsTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	//if (!bNeedEmitEndChange)
	if (ActiveVertexChange == nullptr)
	{
		FRay3d LocalRay(WorldTransform.InverseTransformPosition((FVector3d)DevicePos.WorldRay.Origin),
						WorldTransform.InverseTransformVector((FVector3d)DevicePos.WorldRay.Direction));
		UE::Geometry::Normalize(LocalRay.Direction);

		HilightSelection.Clear();
		FVector3d LocalPosition, LocalNormal;
		FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings();
		bool bHit = TopoSelector.FindSelectedElement(TopoSelectorSettings, LocalRay, HilightSelection, LocalPosition, LocalNormal);

		if (bHit)
		{
			StartHitPosWorld    = (FVector)WorldTransform.TransformPosition(LocalPosition);
			StartHitNormalWorld = (FVector)WorldTransform.TransformVector(LocalNormal);

			UpdateActiveSurfaceFrame(HilightSelection);
			UpdateQuickTransformer();
		}
	}
	return true;
}


void UDeformMeshPolygonsTool::ComputeUpdate()
{
	if (bUpdatePending == true)
	{
		// Linear Deformer : Update the solution
		// Laplacain Deformer : Update the constraints (positions and weights) - the region was identified in onBeginDrag


		if (TransformProps->TransformMode == EQuickTransformerMode::AxisRotation)
		{
			ComputeUpdate_Rotate();
		}
		else
		{
			ComputeUpdate_Translate();
		}
	}

	if (DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian)
	{
		bool bIsWorking = LaplacianDeformer->IsTaskInFlight();

		if (!bIsWorking)
		{
			// Sync update if we have new results.
			if (LaplacianDeformer->bVertexPositionsNeedSync)
			{
				//Update the mesh with the provided solutions.
				LaplacianDeformer->ExportDeformedPositions(DynamicMeshComponent->GetMesh());

				LaplacianDeformer->bVertexPositionsNeedSync = false;

				//Re-sync mesh, and flag the spatial data struct & topology for re-evaluation
				DynamicMeshComponent->FastNotifyPositionsUpdated(true, false, false);

				GetToolManager()->PostInvalidation();
				bSpatialDirty = true;
				TopoSelector.Invalidate(true, false);
			}

			// emit end change if we are done with the drag
			if (!LaplacianDeformer->bDeformerNeedsToRun && !bInDrag)
			{
				EndChange();
			}

			// Not working but we have more work for it to do..

			if (LaplacianDeformer->bDeformerNeedsToRun)
			{
				FRichCurve* Curve = NULL;

				/**
				// How to add a deformation curve
				const bool bApplyAttenuationCurve = TransformProps->bApplyAttenuationCurve
				if (bApplyAttenuationCurve)
				{
					Curve = TransformProps->WeightAttenuationCurve.GetRichCurve();
				}
				*/
				const ELaplacianWeightScheme LaplacianWeightScheme =
				    ConvertToLaplacianWeightScheme(TransformProps->SelectedWeightScheme);
				LaplacianDeformer->UpdateAndLaunchdWorker(LaplacianWeightScheme, Curve);
			}
		}
	}
}


void UDeformMeshPolygonsTool::ComputeUpdate_Rotate()
{
	const bool bIsLaplacian                  = (DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian);
	FGroupTopologyDeformer& SelectedDeformer = (bIsLaplacian) ? *LaplacianDeformer : LinearDeformer;

	FDynamicMesh3* Mesh    = DynamicMeshComponent->GetMesh();
	FVector NewHitPosWorld;

	FVector3d SnappedPoint;
	if (QuickAxisRotator.UpdateSnap(FRay3d(UpdateRay), SnappedPoint))
	{
		NewHitPosWorld = (FVector)SnappedPoint;
	}
	else
	{
		return;
	}

	// check if we are on back-facing part of rotation in which case we ignore...
	FVector3d SphereCenter = QuickAxisRotator.GetActiveWorldFrame().Origin;
	if (QuickAxisRotator.HaveActiveSnapRotation() && QuickAxisRotator.GetHaveLockedToAxis() == false)
	{
		FVector3d ToSnapPointVec = (SnappedPoint - SphereCenter);
		FVector3d ToEyeVec       = (SnappedPoint - (FVector3d)CameraState.Position);
		if (ToSnapPointVec.Dot(ToEyeVec) > 0)
		{
			return;
		}
	}


	// if we haven't snapped to a rotation we can exit
	if (QuickAxisRotator.HaveActiveSnapRotation() == false)
	{
		QuickAxisRotator.ClearAxisLock();

		SelectedDeformer.ClearSolution(Mesh);

		//TODO: This is unseemly here, need to potentially defer this so that it's handled the same way as laplacian. Placeholder for now.
		if (DeformationStrategy == EGroupTopologyDeformationStrategy::Linear)
		{
			DynamicMeshComponent->FastNotifyPositionsUpdated(true);
			GetToolManager()->PostInvalidation();
		}
		bUpdatePending = false;
		return;
	}

	// ok we have an axis...
	if (QuickAxisRotator.GetHaveLockedToAxis() == false)
	{
		QuickAxisRotator.SetAxisLock();
		RotationStartPointWorld = SnappedPoint;
		RotationStartFrame      = QuickAxisRotator.GetActiveRotationFrame();
	}

	FVector2d RotateStartVec = RotationStartFrame.ToPlaneUV(RotationStartPointWorld, 2);
	UE::Geometry::Normalize(RotateStartVec);
	FVector2d RotateToVec = RotationStartFrame.ToPlaneUV((FVector3d)NewHitPosWorld, 2);
	UE::Geometry::Normalize(RotateToVec);
	double AngleRad = UE::Geometry::SignedAngleR(RotateStartVec, RotateToVec);
	FQuaterniond Rotation(WorldTransform.InverseTransformVectorNoScale(RotationStartFrame.Z()), AngleRad, false);
	FVector3d LocalOrigin = WorldTransform.InverseTransformPosition(RotationStartFrame.Origin);

	// Linear Deformer: Update Mesh the rotation,
	// Laplacian Deformer:  Update handles constraints with the rotation and set bDeformerNeedsToRun = true;.
	SelectedDeformer.UpdateSolution(Mesh, [this, LocalOrigin, Rotation](FDynamicMesh3* TargetMesh, int VertIdx) {
		FVector3d V = TargetMesh->GetVertex(VertIdx);
		V -= LocalOrigin;
		V = Rotation * V;
		V += LocalOrigin;
		return V;
	});

	//TODO: This is unseemly here, need to potentially defer this so that it's handled the same way as laplacian. Placeholder for now.
	if (!bIsLaplacian)
	{
		DynamicMeshComponent->FastNotifyPositionsUpdated(true);
		GetToolManager()->PostInvalidation();
	}
	bUpdatePending = false;
}


void UDeformMeshPolygonsTool::ComputeUpdate_Translate()
{
	const bool bIsLaplacian                  = (DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian);
	FGroupTopologyDeformer& SelectedDeformer = (bIsLaplacian) ? *LaplacianDeformer : LinearDeformer;

	TFunction<FVector3d(const FVector3d&)> PointConstraintFunc = nullptr;
	if (GetToolManager()->GetContextQueriesAPI()->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::World)
	{
		// We currently don't support grid snapping in local mode
		PointConstraintFunc = [&](const FVector3d& Pos) {
			FVector3d GridSnapPos;
			return ToolSceneQueriesUtil::FindWorldGridSnapPoint(this, Pos, GridSnapPos) ? GridSnapPos : Pos;
		};
	}

	FVector NewHitPosWorld;
	FVector3d SnappedPoint;
	if (QuickAxisTranslater.UpdateSnap(FRay3d(UpdateRay), SnappedPoint, PointConstraintFunc))
	{
		NewHitPosWorld = (FVector)SnappedPoint;
	}
	else
	{
		return;
	}

	FVector3d NewBrushPosLocal = WorldTransform.InverseTransformPosition(NewHitPosWorld);
	FVector3d NewMoveDelta   = NewBrushPosLocal - (FVector3d)StartBrushPosLocal;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	if (LastMoveDelta.SquaredLength() > 0.)
	{
		if (NewMoveDelta.SquaredLength() > 0.)
		{
			// Linear Deformer: Update Mesh with the translation,
			// Laplacian Deformer:  Update handles constraints and set bDeformerNeedsToRun = true;.

			SelectedDeformer.UpdateSolution(Mesh, [this, NewMoveDelta](FDynamicMesh3* TargetMesh, int VertIdx) {
				return TargetMesh->GetVertex(VertIdx) + NewMoveDelta;
			});
		}
		else
		{
			// Reset mesh to initial positions.

			SelectedDeformer.ClearSolution(Mesh);
		}
		//TODO: This is unseemly here, need to potentially defer this so that it's handled the same way as laplacian. Placeholder for now.
		if (!bIsLaplacian)
		{
			DynamicMeshComponent->FastNotifyPositionsUpdated(true);
			GetToolManager()->PostInvalidation();
		}
	}

	LastMoveDelta     = NewMoveDelta;
	LastBrushPosLocal = NewBrushPosLocal;

	bUpdatePending = false;
}


void UDeformMeshPolygonsTool::OnTick(float DeltaTime)
{
	LaplacianDeformer->HandleWeights   = TransformProps->HandleWeight;
	LaplacianDeformer->bPostfixHandles = TransformProps->bPostFixHandles;
}


void UDeformMeshPolygonsTool::PrecomputeTopology()
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	Topology            = FGroupTopology(Mesh, true);

	LinearDeformer.Initialize(Mesh, &Topology);
	LaplacianDeformer->Initialize(Mesh, &Topology);

	// Make the Constraint Buffer, zero weights, but current pos
	LaplacianDeformer->InitializeConstraintBuffer();
}


void UDeformMeshPolygonsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	ComputeUpdate();

	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
	GetActiveQuickTransformer()->UpdateCameraState(CameraState);

	DynamicMeshComponent->bExplicitShowWireframe = TransformProps->bShowWireframe;
	FDynamicMesh3* TargetMesh                    = DynamicMeshComponent->GetMesh();

	PolyEdgesRenderer.BeginFrame(RenderAPI, CameraState);
	PolyEdgesRenderer.SetTransform((FTransform)WorldTransform);


	for (FGroupTopology::FGroupEdge& Edge : Topology.Edges)
	{
		FVector3d A, B;
		for (int eid : Edge.Span.Edges)
		{
			TargetMesh->GetEdgeV(eid, A, B);
			PolyEdgesRenderer.DrawLine(A, B);
		}
	}

	PolyEdgesRenderer.EndFrame();


	HilightRenderer.BeginFrame(RenderAPI, CameraState);
	HilightRenderer.SetTransform((FTransform)WorldTransform);

#ifdef DEBUG_ROI_WEIGHTS
	FDynamicMesh3* MeshPtr = DynamicMeshComponent->GetMesh();
	for (int32 VertexID : DynamicMeshComponent->GetMesh()->VertexIndicesItr())
	{
		float Color = 1.f - SrcMeshConstraintBuffer[VertexID].Weight;
		HilightRenderer.DrawPoint(MeshPtr->GetVertex(VertexID), FLinearColor(Color, Color, Color, 1.f), 8, true);
	}

#endif

#ifdef DEBUG_ROI_HANDLES
	const FLinearColor FOOF{1.f, 0.f, 1.f, 1.f};
	for (int VertIdx : HandleVertices)
	{
		HilightRenderer.DrawViewFacingCircle(TargetMesh->GetVertex(VertIdx), 0.8f, 8, FOOF, 3, false);
	}
#endif

#ifdef DEBUG_ROI_TRIANGLES
	const FLinearColor Whiteish{0.67f, 0.67f, 0.67f, 1.f};
	for (int32 i = 0; i < SubsetIDBuffer.Num(); i += 3)
	{
		FVector3d A = TargetMesh->GetVertex(SubsetIDBuffer[i]);
		FVector3d B = TargetMesh->GetVertex(SubsetIDBuffer[i + 1]);
		FVector3d C = TargetMesh->GetVertex(SubsetIDBuffer[i + 2]);
		HilightRenderer.DrawLine(A, B, Whiteish, 2.7f, true);
		HilightRenderer.DrawLine(B, C, Whiteish, 2.7f, true);
		HilightRenderer.DrawLine(C, A, Whiteish, 2.7f, true);
	}
#endif

	TopoSelector.VisualAngleSnapThreshold = this->VisualAngleSnapThreshold;
	TopoSelector.DrawSelection(HilightSelection, &HilightRenderer, &CameraState);
	HilightRenderer.EndFrame();


	if (bInDrag)
	{
		GetActiveQuickTransformer()->Render(RenderAPI);
	}
	else
	{
		GetActiveQuickTransformer()->PreviewRender(RenderAPI);
	}
}


void UDeformMeshPolygonsTool::OnPropertyModified(UObject* PropertySet, FProperty* Property) {}

FGroupTopologySelector::FSelectionSettings UDeformMeshPolygonsTool::GetTopoSelectorSettings()
{
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings;
	TopoSelectorSettings.bEnableFaceHits = TransformProps->bSelectFaces;
	TopoSelectorSettings.bEnableEdgeHits = TransformProps->bSelectEdges;
	TopoSelectorSettings.bEnableCornerHits = TransformProps->bSelectVertices;

	return TopoSelectorSettings;
}

//
// Change Tracking
//


void UDeformMeshPolygonsTool::BeginChange()
{
	const bool bIsLaplacian = (DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian);
	if (!bIsLaplacian || LaplacianDeformer->IsDone())
	{
		if (ActiveVertexChange == nullptr)
		{
			ActiveVertexChange = new FMeshVertexChangeBuilder(EMeshVertexChangeComponents::VertexPositions |
			                                                  EMeshVertexChangeComponents::OverlayNormals);
			UpdateChangeFromROI(false);
			LongTransactions.Open(LOCTEXT("PolyMeshDeformationChange", "PolyMesh Edit"), GetToolManager());
		}
	}
}


void UDeformMeshPolygonsTool::EndChange()
{
	if (ActiveVertexChange != nullptr)
	{
		UpdateChangeFromROI(true);
		GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(ActiveVertexChange->Change),
		                                   LOCTEXT("PolyMeshDeformationChange", "PolyMesh Edit"));
		LongTransactions.Close(GetToolManager());
	}

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
}


#undef LOCTEXT_NAMESPACE

