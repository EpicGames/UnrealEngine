// Copyright Epic Games, Inc. All Rights Reserved.

#include "DisplaceMeshTool.h"

#include "AssetUtils/Texture2DUtil.h"
#include "Curves/CurveFloat.h"
#include "Curves/RichCurve.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshNormals.h"
#include "ModelingOperators.h"
#include "Async/ParallelFor.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "MeshDescription.h"
#include "ModelingToolTargetUtil.h"
#include "Operations/PNTriangles.h"
#include "Operations/UniformTessellate.h"
#include "Operations/SelectiveTessellate.h"
#include "Materials/MaterialInterface.h"
#include "Properties/MeshStatisticsProperties.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

// needed to disable normals recalculation on the underlying asset
#include "AssetUtils/MeshDescriptionUtil.h"
#include "Components/StaticMeshComponent.h"
#include "DynamicSubmesh3.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Engine/StaticMesh.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDisplaceMeshTool"

namespace DisplaceMeshToolLocals{

	namespace ComputeDisplacement 
	{
		/// Directional Filter: Scale displacement for a given vertex based on how well 
		/// the vertex normal agrees with the specified direction.
		struct FDirectionalFilter
		{
			bool bEnableFilter = false;
			FVector3d FilterDirection = {1,0,0};
			double FilterWidth = 0.1;
			const double RampSlope = 5.0;

			double FilterValue(const FVector3d& EvalNormal) const
			{
				if (!bEnableFilter)	{ return 1.0;}

				double DotWithFilterDirection = EvalNormal.Dot(FilterDirection);
				double Offset = 1.0 / RampSlope;
				double MinX = 1.0 - (2.0 + Offset) * FilterWidth;			// Start increasing here
				double MaxX = FMathd::Min(1.0, MinX + Offset);				// Stop increasing here
				
				if (FMathd::Abs(MaxX - MinX) < FMathd::ZeroTolerance) { return 0.0; }
				
				double Y = (DotWithFilterDirection - MinX) / (MaxX - MinX); // Clamped linear interpolation for the ramp region
				return FMathd::Clamp(Y, 0.0, 1.0);
			}
		};

		template<typename DisplaceFunc>
		void ParallelDisplace(const FDynamicMesh3& Mesh,
			const TArray<FVector3d>& Positions,
			const FMeshNormals& Normals,
			TArray<FVector3d>& DisplacedPositions,
			TArray<int>* VerticesToDisplace,
			DisplaceFunc Displace,
			bool bUseParallel = true)
		{
			ensure(Positions.Num() == Normals.GetNormals().Num());
			ensure(Positions.Num() == DisplacedPositions.Num());
			ensure(Mesh.VertexCount() == Positions.Num());

			if (VerticesToDisplace) 
			{	
				// Copy over the original positions first since most likely some of the vertices won't be displaced
				for (int VID : Mesh.VertexIndicesItr()) 
				{
					DisplacedPositions[VID] = Positions[VID];
				}

				ParallelFor(VerticesToDisplace->Num(), [&](int32 Idx)
				{	
					const int VID = (*VerticesToDisplace)[Idx];
					if (ensure(Mesh.IsVertex(VID))) // if this test fails then VerticesToDisplace is invalid
					{
						DisplacedPositions[VID] = Displace(VID, Positions[VID], Normals[VID]);
					}
				 }, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
			}
			else 
			{	
				// Displace all vertices
				ParallelFor(Mesh.MaxVertexID(), [&](int32 VID)
				{
					if (Mesh.IsVertex(VID))
					{
						DisplacedPositions[VID] = Displace(VID, Positions[VID], Normals[VID]);
					}
				}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
			}
		}


		void Constant(const FDynamicMesh3& Mesh,
			const TArray<FVector3d>& Positions, 
			const FMeshNormals& Normals, 
			TFunctionRef<float(int32, const FVector3d&, const FVector3d&)> IntensityFunc,
			TArray<FVector3d>& DisplacedPositions,
			TArray<int>* VerticesToDisplace = nullptr)
		{
			ParallelDisplace(Mesh, Positions, Normals, DisplacedPositions, VerticesToDisplace,
				[&](int32 vid, const FVector3d& Position, const FVector3d& Normal)
			{
				double Intensity = IntensityFunc(vid, Position, Normal);
				return Position + (Intensity * Normal);
			});
		}


		void RandomNoise(const FDynamicMesh3& Mesh,
			const TArray<FVector3d>& Positions, 
			const FMeshNormals& Normals,
			TFunctionRef<float(int32, const FVector3d&, const FVector3d&)> IntensityFunc,
			int RandomSeed, 
			TArray<FVector3d>& DisplacedPositions,
			TArray<int>* VerticesToDisplace = nullptr)
		{
			FMath::SRandInit(RandomSeed);
			ParallelDisplace(Mesh, Positions, Normals, DisplacedPositions, VerticesToDisplace,
			[&](int32 vid, const FVector3d& Position, const FVector3d& Normal)
			{
				// FMath::SRand() is not thread safe, hence we pass bUseParallel = false to ParallelDisplace to force 
				// displacement to be single threaded
				double RandVal = 2.0 * (FMath::SRand() - 0.5); 
				double Intensity = IntensityFunc(vid, Position, Normal);
				return Position + (Normal * RandVal * Intensity);
			}, false);
	
		}

		void PerlinNoise(const FDynamicMesh3& Mesh,
			const TArray<FVector3d>& Positions,
			const FMeshNormals& Normals,
			TFunctionRef<float(int32, const FVector3d&, const FVector3d&)> IntensityFunc,
			const TArray<FPerlinLayerProperties>& PerlinLayerProperties,
			int RandomSeed,
			TArray<FVector3d>& DisplacedPositions,
			TArray<int>* VerticesToDisplace = nullptr)
		{
			FMath::SRandInit(RandomSeed);
			const float RandomOffset = 10000.0f * FMath::SRand();

			ParallelDisplace(Mesh, Positions, Normals, DisplacedPositions, VerticesToDisplace,
				[&](int32 vid, const FVector3d& Position, const FVector3d& Normal)
			{
				// Compute the sum of Perlin noise evaluations for this point
				FVector EvalLocation(Position + RandomOffset);
				double TotalNoiseValue = 0.0;
				for (int32 Layer = 0; Layer < PerlinLayerProperties.Num(); ++Layer)
				{
					TotalNoiseValue += PerlinLayerProperties[Layer].Intensity * FMath::PerlinNoise3D(PerlinLayerProperties[Layer].Frequency * EvalLocation);
				}
				double Intensity = IntensityFunc(vid, Position, Normal);
				return Position + (TotalNoiseValue * Intensity * Normal);
			});
		}

		void Map(const FDynamicMesh3& Mesh,
			const TArray<FVector3d>& Positions, 
			const FMeshNormals& Normals,
			TFunctionRef<float(int32, const FVector3d&, const FVector3d&)> IntensityFunc,
			const FSampledScalarField2f& DisplaceField,
			TArray<FVector3d>& DisplacedPositions,
			float DisplaceFieldBaseValue = 128.0/255, // value that corresponds to zero displacement
			FVector2f UVScale = FVector2f(1, 1),
			FVector2f UVOffset = FVector2f(0,0),
			TArray<int>* VerticesToDisplace = nullptr,
			FRichCurve* AdjustmentCurve = nullptr,
			bool bUseParallel = true)
		{
			const FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(0);

			// We set things up such that DisplaceField goes from 0 to 1 in the U direction,
			// but the V direction may be shorter or longer if the texture is not square
			// (it will be 1/AspectRatio)
			float VHeight = DisplaceField.Height() * DisplaceField.CellDimensions.Y;
		
			// Stores average offset for all elements sharing a vertex
			TArray<double> OffsetArray;
			OffsetArray.SetNumZeroed(Mesh.MaxVertexID());
			
			auto ComputeOffsetForElement = [&](int32 ElementID) 
			{
				if (!UVOverlay->IsElement(ElementID))
				{
					return 0.0;
				}
				
				FVector2f UV = UVOverlay->GetElement(ElementID);

				// Adjust UV value and tile it. 
				// Note that we're effectively stretching the texture to be square before tiling, since this
				// seems to be what non square textures do by default in UE. If we decide to tile without 
				// stretching by default someday, we'd do UV - FVector2f(FMath::Floor(UV.X), FMath:Floor(UV.Y/VHeight)*VHeight)
				// without multiplying by VHeight afterward.
				UV = UV * UVScale + UVOffset;
				UV = UV - FVector2f(FMath::Floor(UV.X), FMath::Floor(UV.Y));
				UV.Y *= VHeight;

				double Offset = DisplaceField.BilinearSampleClamped(UV);
				if (AdjustmentCurve) 
				{
					Offset = AdjustmentCurve->Eval(Offset);
				}

				Offset -= DisplaceFieldBaseValue;

				return Offset;
			};
			
			// Copy over the original positions if we are only displacing some of them
			if (VerticesToDisplace) 
			{
				for (int VID : Mesh.VertexIndicesItr()) 
				{
					DisplacedPositions[VID] = Positions[VID];
				}
			}

			const int NumVertices = VerticesToDisplace ? VerticesToDisplace->Num() : Mesh.MaxVertexID();

			ParallelFor(NumVertices, [&](int32 Idx)
			{
				const int VID = VerticesToDisplace ? (*VerticesToDisplace)[Idx] : Idx;

				if (Mesh.IsVertex(VID) == false) 
				{
					return;
				}	

				
				if (UVOverlay->IsSeamVertex(VID)) 
				{
					// If the vertex is on a UV seam edge then average out offsets for all elements sharing the vertex
					FDynamicMesh3::FLocalIntArray TrianglesOut;
					Mesh.GetVtxTriangles(VID, TrianglesOut);

					for (const int TID : TrianglesOut) 
					{
						int ElementID = UVOverlay->GetElementIDAtVertex(TID, VID);
						double Offset = ComputeOffsetForElement(ElementID);
						OffsetArray[VID] += Offset;
					}

					checkSlow(TrianglesOut.Num() > 0);
					OffsetArray[VID] /= TrianglesOut.Num();
				}
				else 
				{
					// Grab any triangle if it's not a uv seam vertex
					int TID = Mesh.GetVtxSingleTriangle(VID);
					if (Mesh.IsTriangle(TID))
					{
						int ElementID = UVOverlay->GetElementIDAtVertex(TID, VID);
						double Offset = ComputeOffsetForElement(ElementID);
						OffsetArray[VID] = Offset;
					}
				}
				
				double Intensity = IntensityFunc(VID, Positions[VID], Normals[VID]);
				DisplacedPositions[VID] = Positions[VID] + (OffsetArray[VID] * Intensity * Normals[VID]);

			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}
		
		void Sine(const FDynamicMesh3& Mesh,
			const TArray<FVector3d>& Positions,
			const FMeshNormals& Normals,
			TFunctionRef<float(int32, const FVector3d&, const FVector3d&)> IntensityFunc,
			double Frequency,
			double PhaseShift,
			const FVector3d& Direction,
			TArray<FVector3d>& DisplacedPositions,
			TArray<int>* VerticesToDisplace = nullptr)
		{
			FQuaterniond RotateToDirection(Direction, { 0.0, 0.0, 1.0 });

			ParallelDisplace(Mesh, Positions, Normals, DisplacedPositions, VerticesToDisplace,
				[&](int32 vid, const FVector3d& Position, const FVector3d& Normal)
			{
				FVector3d RotatedPosition = RotateToDirection * Position;
				double DistXY = FMath::Sqrt(RotatedPosition.X * RotatedPosition.X + RotatedPosition.Y * RotatedPosition.Y);
				double Intensity = IntensityFunc(vid, Position, Normal);
				FVector3d Offset = Intensity * FMath::Sin(Frequency * DistXY + PhaseShift) * Direction;
				return Position + Offset;

			});
		}

	}

	/** A collection of subdivision parameters. */
	struct FSubdivideParameters
	{
		int SubdivisionsCount = -1;
		TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> WeightMap = nullptr;
		TArray<int> SelectedTriangles;

		// Optional selection parameters
		TOptional<EDisplaceMeshToolTriangleSelectionType> SelectionType;
		
		// Material selection variables (SelectionType == EDisplaceMeshToolTriangleSelectionType::Material)
		TOptional<int> ActiveMaterialID;
	};

	class FSubdivideMeshOp : public FDynamicMeshOperator
	{
	public:
		FSubdivideMeshOp(const FDynamicMesh3& SourceMesh, const FSubdivideParameters& InParameters, EDisplaceMeshToolSubdivisionType InSubdivisionType);
		void CalculateResult(FProgressCancel* Progress) final;
		
		TUniquePtr<TArray<int>> ExtractVerticesToDisplace() 
		{
			return MoveTemp(VerticesToDisplace);
		}
		
	protected: 
		TUniquePtr<TArray<int>> VerticesToDisplace = nullptr;
	private:
		FSubdivideParameters Parameters;
		EDisplaceMeshToolSubdivisionType SubdivisionType;
	};

	FSubdivideMeshOp::FSubdivideMeshOp(const FDynamicMesh3& SourceMesh, const FSubdivideParameters& InParameters, EDisplaceMeshToolSubdivisionType InSubdivisionType)
	: 
	Parameters(InParameters), SubdivisionType(InSubdivisionType)
	{
		ResultMesh->Copy(SourceMesh);

		// If we have a WeightMap, initialize VertexUV.X with weightmap value. Note that we are going to process .Y anyway,
		// we could (for exmaple) speculatively compute another weightmap, or store previous weightmap values there, to support
		// fast switching between two...
		ResultMesh->EnableVertexUVs(FVector2f::Zero());
		if (Parameters.WeightMap != nullptr)
		{
			for (int32 vid : ResultMesh->VertexIndicesItr())
			{
				ResultMesh->SetVertexUV(vid, FVector2f(Parameters.WeightMap->GetValue(vid), 0));
			}
		}
		else
		{
			for (int32 vid : ResultMesh->VertexIndicesItr())
			{
				ResultMesh->SetVertexUV(vid, FVector2f::One());
			}
		}
	}

	void FSubdivideMeshOp::CalculateResult(FProgressCancel* ProgressCancel)
	{
		const int SubdivisionsCount = Parameters.SubdivisionsCount;
		bool bUsingSelection = Parameters.SelectedTriangles.Num() != ResultMesh->TriangleCount();
		if (SubdivisionType == EDisplaceMeshToolSubdivisionType::Flat) 
		{
			if (Parameters.SelectionType.IsSet()) // user wants to only tessellate a subset of the triangles
			{	
				EDisplaceMeshToolTriangleSelectionType SelectionType = Parameters.SelectionType.GetValue();
				if (SelectionType == EDisplaceMeshToolTriangleSelectionType::Material && 
					Parameters.ActiveMaterialID.IsSet() && 
					Parameters.ActiveMaterialID.GetValue() != INDEX_NONE)
				{
					TUniquePtr<FTessellationPattern> Pattern;
					if (bUsingSelection)
					{
						Pattern = FSelectiveTessellate::CreateConcentricRingsPatternFromSelectionAndMaterial(ResultMesh.Get(), SubdivisionsCount, Parameters.ActiveMaterialID.GetValue(), Parameters.SelectedTriangles);
					}
					else
					{
						Pattern = FSelectiveTessellate::CreateConcentricRingsPatternFromMaterial(ResultMesh.Get(), SubdivisionsCount, Parameters.ActiveMaterialID.GetValue());
					}
					
					FDynamicMesh3 OutMesh;
					FSelectiveTessellate Tessellator(ResultMesh.Get(), &OutMesh);
					Tessellator.Progress = ProgressCancel;
					Tessellator.SetPattern(Pattern.Get());

					VerticesToDisplace = MakeUnique<TArray<int>>();
					Tessellator.TessInfo.SelectedVertices = VerticesToDisplace.Get();
					
					if (ensureMsgf(Tessellator.Validate() == EOperationValidationResult::Ok, TEXT("The tessellator parameters are invalid.")))
					{
						if (Tessellator.Compute())
						{
							*ResultMesh = MoveTemp(OutMesh);
						}
					}
				} // EDisplaceMeshToolTriangleSelectionType::None but still tessellating only selected geometry
				else if (bUsingSelection)
				{
					TUniquePtr<FTessellationPattern> Pattern =FSelectiveTessellate::CreateConcentricRingsTessellationPattern(ResultMesh.Get(), SubdivisionsCount, Parameters.SelectedTriangles);
					
					FDynamicMesh3 OutMesh;
					FSelectiveTessellate Tessellator(ResultMesh.Get(), &OutMesh);
					Tessellator.Progress = ProgressCancel;
					Tessellator.SetPattern(Pattern.Get());

					VerticesToDisplace = MakeUnique<TArray<int>>();
					Tessellator.TessInfo.SelectedVertices = VerticesToDisplace.Get();
					
					if (ensureMsgf(Tessellator.Validate() == EOperationValidationResult::Ok, TEXT("The tessellator parameters are invalid.")))
					{
						if (Tessellator.Compute())
						{
							*ResultMesh = MoveTemp(OutMesh);
						}
					}
				}
				else
				{
					// Fall back to the uniform tessellation if either SelectionType is EDisplaceMeshToolTriangleSelectionType::None 
					// or not all the parameters were set for the other selection types. 
					FUniformTessellate Tessellator(ResultMesh.Get());
					Tessellator.Progress = ProgressCancel;
					Tessellator.TessellationNum = SubdivisionsCount;

					if (ensureMsgf(Tessellator.Validate() == EOperationValidationResult::Ok, TEXT("The tessellator parameters are invalid.")))
					{
						Tessellator.Compute();
					}
				}
			}
			else
			{
				FUniformTessellate Tessellator(ResultMesh.Get());
				Tessellator.Progress = ProgressCancel;
				Tessellator.TessellationNum = SubdivisionsCount;
						
				if (ensureMsgf(Tessellator.Validate() == EOperationValidationResult::Ok, TEXT("The tessellator parameters are invalid.")))
				{
					Tessellator.Compute();
				}
			}
		}
		else if (SubdivisionType == EDisplaceMeshToolSubdivisionType::PNTriangles) 
		{
			FPNTriangles PNTriangles(ResultMesh.Get());
			PNTriangles.Progress = ProgressCancel;
			PNTriangles.TessellationLevel = SubdivisionsCount;

			if (ensureMsgf(PNTriangles.Validate() == EOperationValidationResult::Ok, TEXT("The tessellator parameters are invalid.")))
			{
				PNTriangles.Compute();
			}
		}
		else 
		{
			// Unsupported subdivision type
			checkNoEntry();
		}
	}

	class FSubdivideMeshOpFactory : public IDynamicMeshOperatorFactory
	{
	public:
		FSubdivideMeshOpFactory(FDynamicMesh3& SourceMeshIn,
								const FSubdivideParameters& InParameters,
								EDisplaceMeshToolSubdivisionType SubdivisionTypeIn)
		: 
		SourceMesh(SourceMeshIn), Parameters(InParameters), SubdivisionType(SubdivisionTypeIn)
		{
		}

		void SetSubdivisionType(EDisplaceMeshToolSubdivisionType SubdivisionTypeIn);
		EDisplaceMeshToolSubdivisionType GetSubdivisionType() const;

		void SetSubdivisionsCount(int SubdivisionsCountIn);
		int  GetSubdivisionsCount() const;

		void SetWeightMap(TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> WeightMapIn);

		void SetSelectionType(EDisplaceMeshToolTriangleSelectionType SelectionTypeIn) 
		{
			Parameters.SelectionType = SelectionTypeIn;
		}

		// Materials
		void SetActiveMaterialID(int ActiveMaterialID) 
		{
			Parameters.ActiveMaterialID = ActiveMaterialID;
		}
		
		TUniquePtr<FDynamicMeshOperator> MakeNewOperator() final
		{
			return MakeUnique<FSubdivideMeshOp>(SourceMesh, Parameters, SubdivisionType);
		}
	private:
		const FDynamicMesh3& SourceMesh;
		FSubdivideParameters Parameters;
		EDisplaceMeshToolSubdivisionType SubdivisionType;
	};

	void FSubdivideMeshOpFactory::SetSubdivisionType(EDisplaceMeshToolSubdivisionType SubdivisionTypeIn) 
	{
		SubdivisionType = SubdivisionTypeIn;
	}

	EDisplaceMeshToolSubdivisionType FSubdivideMeshOpFactory::GetSubdivisionType() const
	{
		return SubdivisionType;
	}

	void FSubdivideMeshOpFactory::SetSubdivisionsCount(int SubdivisionsCountIn)
	{
		Parameters.SubdivisionsCount = SubdivisionsCountIn;
	}

	int FSubdivideMeshOpFactory::GetSubdivisionsCount() const
	{
		return Parameters.SubdivisionsCount;
	}

	void FSubdivideMeshOpFactory::SetWeightMap(TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> WeightMapIn)
	{
		Parameters.WeightMap = WeightMapIn;
	}

	// A collection of parameters to avoid having excess function parameters
	struct DisplaceMeshParameters
	{
		float DisplaceIntensity = 0.0f;
		int RandomSeed = 0;
		UTexture2D* DisplacementMap = nullptr;
		float SineWaveFrequency = 0.0f;
		float SineWavePhaseShift = 0.0f;
		FVector SineWaveDirection = { 0.0f, 0.0f, 0.0f };
		bool bEnableFilter = false;
		FVector FilterDirection = { 0.0f, 0.0f, 0.0f };
		float FilterWidth = 0.0f;
		FSampledScalarField2f DisplaceField;
		TArray<FPerlinLayerProperties> PerlinLayerProperties;
		bool bRecalculateNormals = true;

		// Used in texture map displacement
		int32 DisplacementMapChannel = 0;
		float DisplacementMapBaseValue = 128.0f/255; // i.e., what constitutes no displacement
		FVector2f UVScale = FVector2f(1,1);
		FVector2f UVOffset = FVector2f(0, 0);
		// This gets used by worker threads, so do not try to change an existing curve- make
		// a new one each time.
		TSharedPtr<FRichCurve, ESPMode::ThreadSafe> AdjustmentCurve;

		TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> WeightMap;
		TFunction<float(const FVector3d&, const FIndexedWeightMap)> WeightMapQueryFunc;

		TSharedPtr<TArray<int>, ESPMode::ThreadSafe> VerticesToDisplace; // if set, only displace vertices whose ids are part of the array
	};

	class FDisplaceMeshOp : public FDynamicMeshOperator
	{
	public:
		FDisplaceMeshOp(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> SourceMeshIn,
						const DisplaceMeshParameters& DisplaceParametersIn,
						EDisplaceMeshToolDisplaceType DisplacementTypeIn);
		void CalculateResult(FProgressCancel* Progress) final;

	private:
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> SourceMesh;
		DisplaceMeshParameters Parameters;
		EDisplaceMeshToolDisplaceType DisplacementType;
		TArray<FVector3d> SourcePositions;
		FMeshNormals SourceNormals;
		TArray<FVector3d> DisplacedPositions;
	};

	FDisplaceMeshOp::FDisplaceMeshOp(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> SourceMeshIn,
									 const DisplaceMeshParameters& DisplaceParametersIn,
									 EDisplaceMeshToolDisplaceType DisplacementTypeIn)
	: 
	SourceMesh(MoveTemp(SourceMeshIn)), 
	Parameters(DisplaceParametersIn), 
	DisplacementType(DisplacementTypeIn)
	{
	}

	void FDisplaceMeshOp::CalculateResult(FProgressCancel* Progress)
	{
		if (Progress && Progress->Cancelled()) return;
		ResultMesh->Copy(*SourceMesh);

		if (Progress && Progress->Cancelled()) return;

		if (DisplacementType == EDisplaceMeshToolDisplaceType::DisplacementMap && !Parameters.DisplacementMap)
		{
			return;
		}

		SourceNormals = FMeshNormals(SourceMesh.Get());
		SourceNormals.ComputeVertexNormals();

		if (Progress && Progress->Cancelled()) return;
		// cache initial positions
		SourcePositions.SetNum(SourceMesh->MaxVertexID());
		for (int vid : SourceMesh->VertexIndicesItr())
		{
			SourcePositions[vid] = SourceMesh->GetVertex(vid);
		}

		if (Progress && Progress->Cancelled()) return;
		DisplacedPositions.SetNum(SourceMesh->MaxVertexID());

		if (Progress && Progress->Cancelled()) return;

		ComputeDisplacement::FDirectionalFilter DirectionalFilter{ Parameters.bEnableFilter,
			FVector3d(Parameters.FilterDirection),
			Parameters.FilterWidth };
		double Intensity = Parameters.DisplaceIntensity;

		TUniqueFunction<float(int32 vid, const FVector3d&)> WeightMapQueryFunc = [&](int32, const FVector3d&) { return 1.0f; };
		if (Parameters.WeightMap.IsValid())
		{
			if (SourceMesh->IsCompactV() && SourceMesh->VertexCount() == Parameters.WeightMap->Num())
			{
				WeightMapQueryFunc = [&](int32 vid, const FVector3d& Pos) { return Parameters.WeightMap->GetValue(vid); };
			}
			else
			{
				// disable input query function as it uses expensive AABBTree lookup
				//WeightMapQueryFunc = [&](int32 vid, const FVector3d& Pos) { return Parameters.WeightMapQueryFunc(Pos, *Parameters.WeightMap); };
				WeightMapQueryFunc = [&](int32 vid, const FVector3d& Pos) { return SourceMesh->GetVertexUV(vid).X; };
			}
		}
		auto IntensityFunc = [&](int32 vid, const FVector3d& Position, const FVector3d& Normal) 
		{
			return Intensity * DirectionalFilter.FilterValue(Normal) * WeightMapQueryFunc(vid, Position);
		};



		// compute Displaced positions in PositionBuffer
		switch (DisplacementType)
		{
		default:
		case EDisplaceMeshToolDisplaceType::Constant:
			ComputeDisplacement::Constant(*SourceMesh, 
				SourcePositions, 
				SourceNormals,
				IntensityFunc,
				DisplacedPositions,
				Parameters.VerticesToDisplace.Get());
			break;

		case EDisplaceMeshToolDisplaceType::RandomNoise:
			ComputeDisplacement::RandomNoise(*SourceMesh, 
				SourcePositions, 
				SourceNormals,
				IntensityFunc,
				Parameters.RandomSeed,
				DisplacedPositions,
				Parameters.VerticesToDisplace.Get());
			break;
			
		case EDisplaceMeshToolDisplaceType::PerlinNoise:
			ComputeDisplacement::PerlinNoise(*SourceMesh,
				SourcePositions,
				SourceNormals,
				IntensityFunc,
				Parameters.PerlinLayerProperties,	
				Parameters.RandomSeed,
				DisplacedPositions,
				Parameters.VerticesToDisplace.Get());
			break;

		case EDisplaceMeshToolDisplaceType::DisplacementMap:
			ComputeDisplacement::Map(*SourceMesh, 
				SourcePositions, 
				SourceNormals,
				IntensityFunc,
				Parameters.DisplaceField, 
				DisplacedPositions,
				Parameters.DisplacementMapBaseValue,
				Parameters.UVScale,
				Parameters.UVOffset,
				Parameters.VerticesToDisplace.Get(),
				Parameters.AdjustmentCurve.Get());
			break;

		case EDisplaceMeshToolDisplaceType::SineWave:
			ComputeDisplacement::Sine(*SourceMesh, 
				SourcePositions, 
				SourceNormals,
				IntensityFunc,
				Parameters.SineWaveFrequency,
				Parameters.SineWavePhaseShift,
				(FVector3d)Parameters.SineWaveDirection,
				DisplacedPositions,
				Parameters.VerticesToDisplace.Get());
			break;
		}

		// update preview vertex positions
		for (int vid : ResultMesh->VertexIndicesItr())
		{
			ResultMesh->SetVertex(vid, DisplacedPositions[vid]);
		}

		// recalculate normals
		if (Parameters.bRecalculateNormals)
		{
			if (ResultMesh->HasAttributes())
			{
				FMeshNormals Normals(ResultMesh.Get());
				FDynamicMeshNormalOverlay* NormalOverlay = ResultMesh->Attributes()->PrimaryNormals();
				Normals.RecomputeOverlayNormals(NormalOverlay);
				Normals.CopyToOverlay(NormalOverlay);
			}
			else
			{
				FMeshNormals::QuickComputeVertexNormals(*ResultMesh);
			}
		}
	}

	class FDisplaceMeshOpFactory : public IDynamicMeshOperatorFactory
	{
	public:
		FDisplaceMeshOpFactory(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>& SourceMeshIn,
							   const DisplaceMeshParameters& DisplaceParametersIn,
							   EDisplaceMeshToolDisplaceType DisplacementTypeIn )
		: 
		SourceMesh(SourceMeshIn)
		{
			SetIntensity(DisplaceParametersIn.DisplaceIntensity);
			SetRandomSeed(DisplaceParametersIn.RandomSeed);
			SetDisplacementMap(DisplaceParametersIn.DisplacementMap, DisplaceParametersIn.DisplacementMapChannel); // Calls UpdateMap
			SetFrequency(DisplaceParametersIn.SineWaveFrequency);
			SetPhaseShift(DisplaceParametersIn.SineWavePhaseShift);
			SetSineWaveDirection(DisplaceParametersIn.SineWaveDirection);
			SetEnableDirectionalFilter(DisplaceParametersIn.bEnableFilter);
			SetFilterDirection(DisplaceParametersIn.FilterDirection);
			SetFilterFalloffWidth(DisplaceParametersIn.FilterWidth);
			SetPerlinNoiseLayerProperties(DisplaceParametersIn.PerlinLayerProperties);
			SetDisplacementType(DisplacementTypeIn);
			SetVerticesToDisplace(DisplaceParametersIn.VerticesToDisplace);

			Parameters.WeightMap = DisplaceParametersIn.WeightMap;
			Parameters.WeightMapQueryFunc = DisplaceParametersIn.WeightMapQueryFunc;

			Parameters.DisplacementMapBaseValue = DisplaceParametersIn.DisplacementMapBaseValue;
			Parameters.UVScale = DisplaceParametersIn.UVScale;
			Parameters.UVOffset = DisplaceParametersIn.UVOffset;

			Parameters.AdjustmentCurve = DisplaceParametersIn.AdjustmentCurve;
		}
		
		void SetIntensity(float IntensityIn);
		void SetRandomSeed(int RandomSeedIn);
		void SetDisplacementMap(UTexture2D* DisplacementMapIn, int32 ChannelIn);
		void SetDisplacementMapUVAdjustment(const FVector2f& UVScale, const FVector2f& UVOffset);
		void SetDisplacementMapBaseValue(float DisplacementMapBaseValue);
		void SetAdjustmentCurve(UCurveFloat* CurveFloat);
		void SetFrequency(float FrequencyIn);
		void SetPhaseShift(float PhaseShiftIn);
		void SetSineWaveDirection(const FVector& Direction);
		void SetDisplacementType(EDisplaceMeshToolDisplaceType TypeIn);
		void SetEnableDirectionalFilter(bool EnableDirectionalFilter);
		void SetFilterDirection(const FVector& Direction);
		void SetFilterFalloffWidth(float FalloffWidth);
		void SetPerlinNoiseLayerProperties(const TArray<FPerlinLayerProperties>& PerlinLayerProperties);
		void SetWeightMap(TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> WeightMap);
		void SetRecalculateNormals(bool bRecalculateNormals);
		void SetVerticesToDisplace(TSharedPtr<TArray<int>, ESPMode::ThreadSafe> VerticesToDisplace);

		TUniquePtr<FDynamicMeshOperator> MakeNewOperator() final
		{
			return MakeUnique<FDisplaceMeshOp>(SourceMesh, Parameters, DisplacementType);
		}
	private:
		void UpdateMap();

		DisplaceMeshParameters Parameters;
		EDisplaceMeshToolDisplaceType DisplacementType;

		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>& SourceMesh;
	};

	void FDisplaceMeshOpFactory::SetIntensity(float IntensityIn)
	{
		Parameters.DisplaceIntensity = IntensityIn;
	}

	void FDisplaceMeshOpFactory::SetRandomSeed(int RandomSeedIn)
	{
		Parameters.RandomSeed = RandomSeedIn;
	}

	void FDisplaceMeshOpFactory::SetDisplacementMap(UTexture2D* DisplacementMapIn, int32 ChannelIn)
	{
		Parameters.DisplacementMap = DisplacementMapIn;
		Parameters.DisplacementMapChannel = ChannelIn;

		// Note that we do the update even if we got the same pointer, because the texture
		// may have been changed in the editor.
		UpdateMap();
	}

	void FDisplaceMeshOpFactory::SetDisplacementMapUVAdjustment(const FVector2f& UVScale, const FVector2f& UVOffset)
	{
		Parameters.UVScale = UVScale;
		Parameters.UVOffset = UVOffset;
	}

	void FDisplaceMeshOpFactory::SetDisplacementMapBaseValue(float DisplacementMapBaseValue)
	{
		// We could bake this into the displacement field, but that would require calling UpdateMap with
		// every slider change, which is slow. So we'll just pass this down to the calculation.
		Parameters.DisplacementMapBaseValue = DisplacementMapBaseValue;
	}

	void FDisplaceMeshOpFactory::SetAdjustmentCurve(UCurveFloat* CurveFloat)
	{
		Parameters.AdjustmentCurve = CurveFloat ? TSharedPtr<FRichCurve, ESPMode::ThreadSafe>(
			static_cast<FRichCurve*>(CurveFloat->FloatCurve.Duplicate()))
			: nullptr;
	}

	void FDisplaceMeshOpFactory::SetFrequency(float FrequencyIn)
	{
		Parameters.SineWaveFrequency = FrequencyIn;
	}

	void FDisplaceMeshOpFactory::SetPhaseShift(float PhaseShiftIn)
	{
		Parameters.SineWavePhaseShift = PhaseShiftIn;
	}

	void FDisplaceMeshOpFactory::SetSineWaveDirection(const FVector& Direction)
	{
		Parameters.SineWaveDirection = Direction.GetSafeNormal();
	}

	void FDisplaceMeshOpFactory::SetDisplacementType(EDisplaceMeshToolDisplaceType TypeIn)
	{
		DisplacementType = TypeIn;
	}

	void FDisplaceMeshOpFactory::UpdateMap()
	{
		if (Parameters.DisplacementMap == nullptr ||
			Parameters.DisplacementMap->GetPlatformData() == nullptr ||
			Parameters.DisplacementMap->GetPlatformData()->Mips.Num() < 1)
		{
			Parameters.DisplaceField = FSampledScalarField2f();
			Parameters.DisplaceField.GridValues.Assign(0);
			return;
		}

		TImageBuilder<FVector4f> DisplacementMapValues;
		if (!UE::AssetUtils::ReadTexture(Parameters.DisplacementMap, DisplacementMapValues, false)) 
		{
			Parameters.DisplaceField = FSampledScalarField2f();
			Parameters.DisplaceField.GridValues.Assign(0);
		}
		else
		{
			const FImageDimensions DisplacementMapDimensions = DisplacementMapValues.GetDimensions();
			int64 TextureWidth = DisplacementMapDimensions.GetWidth();
			int64 TextureHeight = DisplacementMapDimensions.GetHeight();
			Parameters.DisplaceField.Resize(TextureWidth, TextureHeight, 0.0f);

			// Note that the height of the texture will not be 1.0 if it was not square. This should be kept in mind when sampling it later.
			Parameters.DisplaceField.SetCellSize(1.0f / (float)TextureWidth);

			for (int64 y = 0; y < TextureHeight; ++y)
			{
				for (int64 x = 0; x < TextureWidth; ++x)
				{
					Parameters.DisplaceField.GridValues[y * TextureWidth + x] = 
						DisplacementMapValues.GetPixel(y * TextureWidth + x)[Parameters.DisplacementMapChannel];
				}
			}
		}
	}

	void FDisplaceMeshOpFactory::SetEnableDirectionalFilter(bool EnableDirectionalFilter)
	{
		Parameters.bEnableFilter = EnableDirectionalFilter;
	}

	void FDisplaceMeshOpFactory::SetFilterDirection(const FVector& Direction)
	{
		Parameters.FilterDirection = Direction.GetSafeNormal();
	}

	void FDisplaceMeshOpFactory::SetFilterFalloffWidth(float FalloffWidth)
	{
		Parameters.FilterWidth = FalloffWidth;
	}

	void FDisplaceMeshOpFactory::SetPerlinNoiseLayerProperties(const TArray<FPerlinLayerProperties>& LayerProperties )
	{
		Parameters.PerlinLayerProperties = LayerProperties;
	}

	void FDisplaceMeshOpFactory::SetWeightMap(TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> WeightMap)
	{
		Parameters.WeightMap = WeightMap;
	}

	void FDisplaceMeshOpFactory::SetRecalculateNormals(bool RecalcNormalsIn)
	{
		Parameters.bRecalculateNormals = RecalcNormalsIn;
	}

	void FDisplaceMeshOpFactory::SetVerticesToDisplace(TSharedPtr<TArray<int>, ESPMode::ThreadSafe> VerticesToDisplace)
	{
		Parameters.VerticesToDisplace = VerticesToDisplace;
	}

} // namespace

/*
 * ToolBuilder
 */
USingleTargetWithSelectionTool* UDisplaceMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UDisplaceMeshTool>(SceneState.ToolManager);
}

const FToolTargetTypeRequirements& UDisplaceMeshToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


/*
 * Tool
 */
TArray<FString> UDisplaceMeshCommonProperties::GetWeightMapsFunc()
{
	return WeightMapsList;
}

void UDisplaceMeshTool::Setup()
{
	using namespace DisplaceMeshToolLocals;

	UInteractiveTool::Setup();

	// UInteractiveToolPropertySets
	NoiseProperties = NewObject<UDisplaceMeshPerlinNoiseProperties>();
	NoiseProperties->RestoreProperties(this);
	CommonProperties = NewObject<UDisplaceMeshCommonProperties>();
	CommonProperties->RestoreProperties(this);
	DirectionalFilterProperties = NewObject<UDisplaceMeshDirectionalFilterProperties>();
	DirectionalFilterProperties->RestoreProperties(this);
	TextureMapProperties = NewObject<UDisplaceMeshTextureMapProperties>();
	TextureMapProperties->RestoreProperties(this);
	SineWaveProperties = NewObject<UDisplaceMeshSineWaveProperties>();
	SineWaveProperties->RestoreProperties(this);
	SelectiveTessellationProperties = NewObject<USelectiveTessellationProperties>();
	SelectiveTessellationProperties->RestoreProperties(this);

	if (TextureMapProperties->DisplacementMap != nullptr && TextureMapProperties->DisplacementMap->IsValidLowLevel() == false)
	{
		TextureMapProperties->DisplacementMap = nullptr;
	}
	if (TextureMapProperties->AdjustmentCurve == nullptr)
	{
		// if curve is null, create from default
		TextureMapProperties->AdjustmentCurve = ToolSetupUtil::GetContrastAdjustmentCurve(GetToolManager());
	}
	
	// populate weight maps list
	const FMeshDescription* MeshDescription = UE::ToolTarget::GetMeshDescription(Target);
	TArray<FName> WeightMaps;
	UE::WeightMaps::FindVertexWeightMaps(MeshDescription, WeightMaps);
	CommonProperties->WeightMapsList.Add(TEXT("None"));
	for (FName Name : WeightMaps)
	{
		CommonProperties->WeightMapsList.Add(Name.ToString());
	}
	if (WeightMaps.Contains(CommonProperties->WeightMap) == false)		// discard restored value if it doesn't apply
	{
		CommonProperties->WeightMap = FName(CommonProperties->WeightMapsList[0]);
	}
	UpdateActiveWeightMap();


	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);

	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);
	DynamicMeshComponent->SetupAttachment(PreviewMeshActor->GetRootComponent());
	DynamicMeshComponent->RegisterComponent();
	DynamicMeshComponent->SetWorldTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	DynamicMeshComponent->bExplicitShowWireframe = CommonProperties->bShowWireframe;
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(DynamicMeshComponent, Target);

	// transfer materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	DynamicMeshComponent->SetTangentsType(EDynamicMeshComponentTangentsMode::AutoCalculated);
	DynamicMeshComponent->SetMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));

	// retrieve the selected triangles
	TArray<int> TrianglesSelectedInMesh;
	if (HasGeometrySelection())
	{
		const FGeometrySelection& InputSelection = GetGeometrySelection();
		UE::Geometry::EnumerateSelectionTriangles(InputSelection, *DynamicMeshComponent->GetMesh(),
			[&](int32 TriangleID){ TrianglesSelectedInMesh.Add(TriangleID); });
	}
	else //if the whole object is selected, SelectedTriangles will be all the triangles in the Object
	{
		for (int32 TriangleID : DynamicMeshComponent->GetMesh()->TriangleIndicesItr())
		{
			TrianglesSelectedInMesh.Add(TriangleID);
		}
	}
	NumTrianglesInSelection = TrianglesSelectedInMesh.Num();

	OriginalMesh.Copy(*DynamicMeshComponent->GetMesh());
	OriginalMeshSpatial.SetMesh(&OriginalMesh, true);

	DisplaceMeshParameters Parameters;
	Parameters.DisplaceIntensity = CommonProperties->DisplaceIntensity;
	Parameters.RandomSeed = CommonProperties->RandomSeed;
	Parameters.DisplacementMap = TextureMapProperties->DisplacementMap;
	Parameters.bRecalculateNormals = TextureMapProperties->bRecalcNormals;
	Parameters.SineWaveFrequency = SineWaveProperties->SineWaveFrequency;
	Parameters.SineWavePhaseShift = SineWaveProperties->SineWavePhaseShift;
	Parameters.SineWaveDirection = SineWaveProperties->SineWaveDirection.GetSafeNormal();
	Parameters.bEnableFilter = DirectionalFilterProperties->bEnableFilter;
	Parameters.FilterDirection = DirectionalFilterProperties->FilterDirection.GetSafeNormal();
	Parameters.FilterWidth = DirectionalFilterProperties->FilterWidth;
	Parameters.PerlinLayerProperties = NoiseProperties->PerlinLayerProperties;
	Parameters.WeightMap = ActiveWeightMap;
	Parameters.WeightMapQueryFunc = [this](const FVector3d& Position, const FIndexedWeightMap& WeightMap) { return WeightMapQuery(Position, WeightMap);	};

	Displacer = MakeUnique<FDisplaceMeshOpFactory>(SubdividedMesh, Parameters, CommonProperties->DisplacementType);
		
	// hide input StaticMeshComponent
	UE::ToolTarget::HideSourceObject(Target);
	
	// Fetch the list of the materials if available
	SelectiveTessellationProperties->MaterialIDList.Reset();
	for (int32 Idx = 0; Idx < MaterialSet.Materials.Num(); ++Idx)
	{
		UMaterialInterface* Mat = MaterialSet.Materials[Idx];
		FString MatName = (Mat != nullptr) ? Mat->GetName() : "(none)";
		FString UseName = FString::Printf(TEXT("[%d] %s"), Idx, *MatName);
		SelectiveTessellationProperties->MaterialIDList.Add(UseName);
	}

	if (SelectiveTessellationProperties->MaterialIDList.IsEmpty())
	{
		SelectiveTessellationProperties->ActiveMaterial = NAME_None;
	}
	else if (SelectiveTessellationProperties->MaterialIDList.Contains(SelectiveTessellationProperties->ActiveMaterial.ToString()) == false) 
	{
		SelectiveTessellationProperties->ActiveMaterial = NAME_None;
	}

	// initialize our properties
	ToolPropertyObjects.Add(this);

	AddToolPropertySource(CommonProperties);
	SetToolPropertySourceEnabled(CommonProperties, true);

	AddToolPropertySource(DirectionalFilterProperties);
	SetToolPropertySourceEnabled(DirectionalFilterProperties, true);

	AddToolPropertySource(TextureMapProperties);
	SetToolPropertySourceEnabled(TextureMapProperties, CommonProperties->DisplacementType == EDisplaceMeshToolDisplaceType::DisplacementMap);

	AddToolPropertySource(SineWaveProperties);
	SetToolPropertySourceEnabled(SineWaveProperties, CommonProperties->DisplacementType == EDisplaceMeshToolDisplaceType::SineWave);

	AddToolPropertySource(SelectiveTessellationProperties);
	SetToolPropertySourceEnabled(SelectiveTessellationProperties, CommonProperties->SubdivisionType == EDisplaceMeshToolSubdivisionType::Flat);

	AddToolPropertySource(NoiseProperties);
	SetToolPropertySourceEnabled(NoiseProperties, CommonProperties->DisplacementType == EDisplaceMeshToolDisplaceType::PerlinNoise);

	// Set up a callback for when the type of displacement changes
	CommonProperties->WatchProperty(CommonProperties->DisplacementType,
									[this](EDisplaceMeshToolDisplaceType NewType)
									{
										SetToolPropertySourceEnabled(NoiseProperties, (NewType == EDisplaceMeshToolDisplaceType::PerlinNoise));
										SetToolPropertySourceEnabled(SineWaveProperties, (NewType == EDisplaceMeshToolDisplaceType::SineWave));
										SetToolPropertySourceEnabled(TextureMapProperties, (NewType == EDisplaceMeshToolDisplaceType::DisplacementMap));
									} );

	CommonProperties->WatchProperty(CommonProperties->SubdivisionType,
									[this](EDisplaceMeshToolSubdivisionType NewType)
									{
										SetToolPropertySourceEnabled(SelectiveTessellationProperties, (NewType == EDisplaceMeshToolSubdivisionType::Flat));
									} );


	ValidateSubdivisions();

	FSubdivideParameters SubParameters;
	SubParameters.SubdivisionsCount = CommonProperties->Subdivisions;
	SubParameters.WeightMap = ActiveWeightMap;
	SubParameters.SelectionType = SelectiveTessellationProperties->SelectionType;
	SubParameters.SelectedTriangles = MoveTemp(TrianglesSelectedInMesh);
	
	if (SelectiveTessellationProperties->ActiveMaterial.IsNone() == false) 
	{
		int32 Index = SelectiveTessellationProperties->MaterialIDList.Find(SelectiveTessellationProperties->ActiveMaterial.ToString());
		SubParameters.ActiveMaterialID = Index;
	}

	Subdivider = MakeUnique<FSubdivideMeshOpFactory>(OriginalMesh, SubParameters, CommonProperties->SubdivisionType);

	MeshStatistics = NewObject<UMeshStatisticsProperties>(this);
	MeshStatistics->Update(OriginalMesh);
	AddToolPropertySource(MeshStatistics);

	StartComputation();

	SetToolDisplayName(LOCTEXT("ToolName", "Displace"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartDisplaceMesh", "Subdivide and Displace the input mesh using different noise functions and maps"),
		EToolMessageLevel::UserNotification);
}

void UDisplaceMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	// unhook any active listener for contrast curve
	DisconnectActiveContrastCurveTarget();

	CommonProperties->SaveProperties(this);
	NoiseProperties->SaveProperties(this);
	DirectionalFilterProperties->SaveProperties(this);
	SineWaveProperties->SaveProperties(this);
	TextureMapProperties->SaveProperties(this);
	SelectiveTessellationProperties->SaveProperties(this);
	

	if (DynamicMeshComponent != nullptr)
	{
		UE::ToolTarget::ShowSourceObject(Target);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
			GetToolManager()->BeginUndoTransaction(LOCTEXT("DisplaceMeshToolTransactionName", "Displace Mesh"));

			// if we are applying a map and not recalculating normals, we need to make sure normals recalculation is disabled
			// on the underlying StaticMesh Asset, or it will run on the Bake() below and the output result will not be the same as the preview
			if (CommonProperties->DisplacementType == EDisplaceMeshToolDisplaceType::DisplacementMap && TextureMapProperties->bRecalcNormals == false)
			{
				UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Target));
				if (StaticMeshComponent)
				{
					if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
					{
						StaticMesh->Modify();

						// disable auto-generated normals and tangents build settings
						UE::MeshDescription::FStaticMeshBuildSettingChange SettingsChange;
						SettingsChange.AutoGeneratedNormals = UE::MeshDescription::EBuildSettingBoolChange::Disable;
						UE::MeshDescription::ConfigureBuildSettings(StaticMesh, 0, SettingsChange);
					}
				}
			}

			DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				UE::ToolTarget::CommitDynamicMeshUpdate(Target, ReadMesh, CommonProperties->Subdivisions > 0);
			});
			GetToolManager()->EndUndoTransaction();
		}

		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->DestroyComponent();
		DynamicMeshComponent = nullptr;
	}

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

}


void UDisplaceMeshTool::ValidateSubdivisions()
{
	if (CommonProperties->bDisableSizeWarning)
	{
		GetToolManager()->DisplayMessage(FText::GetEmpty(), EToolMessageLevel::UserWarning);
		return;
	}

	bool bIsInitialized = (Subdivider != nullptr);

	constexpr int MaxTriangles = 3000000;
	double NumTriangles = NumTrianglesInSelection;
	int MaxSubdivisions = (int)(FMath::Sqrt(MaxTriangles/NumTriangles) - 1);
	if (CommonProperties->Subdivisions > MaxSubdivisions)
	{
		if (bIsInitialized)		// only show warning after initial tool startup
		{
			FText WarningText = FText::Format(LOCTEXT("SubdivisionsTooHigh", "Desired number of Subdivisions ({0}) exceeds maximum number ({1}) for a mesh of this number of triangles."),
				FText::AsNumber(CommonProperties->Subdivisions),
				FText::AsNumber(MaxSubdivisions));
			GetToolManager()->DisplayMessage(WarningText, EToolMessageLevel::UserWarning);
		}
		CommonProperties->Subdivisions = MaxSubdivisions;
	}
	else
	{
		FText ClearWarningText;
		GetToolManager()->DisplayMessage(ClearWarningText, EToolMessageLevel::UserWarning);
	}
	if (CommonProperties->Subdivisions < 0)
	{
		CommonProperties->Subdivisions = 0;
	}
}

#if WITH_EDITOR

void UDisplaceMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	using namespace DisplaceMeshToolLocals;
	
	if (AreAllTargetsValid() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidTargets", "Target mesh is no longer valid"), EToolMessageLevel::UserWarning);
		return;
	}

	if (PropertySet && Property)
	{
		FDisplaceMeshOpFactory* DisplacerDownCast = static_cast<FDisplaceMeshOpFactory*>(Displacer.Get());
		FSubdivideMeshOpFactory* SubdividerDownCast = static_cast<FSubdivideMeshOpFactory*>(Subdivider.Get());

		const FString PropertySetName = PropertySet->GetFName().GetPlainNameString();
		const FName PropName = Property->GetFName();
	
		bNeedsDisplaced = true;

		if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, SubdivisionType))
		{
			if (CommonProperties->SubdivisionType != SubdividerDownCast->GetSubdivisionType())
			{
				SubdividerDownCast->SetSubdivisionType(CommonProperties->SubdivisionType);
				bNeedsSubdivided = true;
			}
			else
			{
				return;
			}
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, Subdivisions))
		{
			ValidateSubdivisions();
			if (CommonProperties->Subdivisions != SubdividerDownCast->GetSubdivisionsCount())
			{
				SubdividerDownCast->SetSubdivisionsCount(CommonProperties->Subdivisions);
				bNeedsSubdivided = true;
			}
			else
			{
				return;
			}
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, RandomSeed))
		{
			DisplacerDownCast->SetRandomSeed(CommonProperties->RandomSeed);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, DisplacementType))
		{
			DisplacerDownCast->SetDisplacementType(CommonProperties->DisplacementType);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, DisplaceIntensity))
		{
			DisplacerDownCast->SetIntensity(CommonProperties->DisplaceIntensity);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, bShowWireframe))
		{
			DynamicMeshComponent->bExplicitShowWireframe = CommonProperties->bShowWireframe;
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshSineWaveProperties, SineWaveFrequency))
		{
			DisplacerDownCast->SetFrequency(SineWaveProperties->SineWaveFrequency);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshSineWaveProperties, SineWavePhaseShift))
		{
			DisplacerDownCast->SetPhaseShift(SineWaveProperties->SineWavePhaseShift);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, DisplacementMap))
		{
			if (TextureMapProperties->DisplacementMap != nullptr
				&& TextureMapProperties->DisplacementMap->VirtualTextureStreaming)
			{
				GetToolManager()->DisplayMessage(
					LOCTEXT("DisplaceToolVirtualTextureMessage", "Virtual Texture must be disabled on the selected Texture2D to use it as a Displacement Map in this Tool"),
					EToolMessageLevel::UserWarning);
			}
			else
			{
				GetToolManager()->DisplayMessage(FText::GetEmpty(), EToolMessageLevel::UserWarning);
			}

			DisplacerDownCast->SetDisplacementMap(TextureMapProperties->DisplacementMap, static_cast<int32>(TextureMapProperties->Channel));
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, Channel))
		{
			DisplacerDownCast->SetDisplacementMap(TextureMapProperties->DisplacementMap, static_cast<int32>(TextureMapProperties->Channel));
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, DisplacementMapBaseValue))
		{
			DisplacerDownCast->SetDisplacementMapBaseValue(TextureMapProperties->DisplacementMapBaseValue);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, bRecalcNormals))
		{
			DisplacerDownCast->SetRecalculateNormals(TextureMapProperties->bRecalcNormals);
		}

		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, bApplyAdjustmentCurve)
			|| PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, AdjustmentCurve))
		{
			DisplacerDownCast->SetAdjustmentCurve(TextureMapProperties->bApplyAdjustmentCurve ? TextureMapProperties->AdjustmentCurve : nullptr);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, WeightMap) 
				 || PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshCommonProperties, bInvertWeightMap))
		{
			UpdateActiveWeightMap();
			SubdividerDownCast->SetWeightMap(ActiveWeightMap);
			DisplacerDownCast->SetWeightMap(ActiveWeightMap);
			bNeedsSubdivided = true;
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshDirectionalFilterProperties, bEnableFilter))
		{
			DisplacerDownCast->SetEnableDirectionalFilter(DirectionalFilterProperties->bEnableFilter);
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshDirectionalFilterProperties, FilterWidth))
		{
			DisplacerDownCast->SetFilterFalloffWidth(DirectionalFilterProperties->FilterWidth);
		}
		else if (PropertySet == NoiseProperties)
		{
			DisplacerDownCast->SetPerlinNoiseLayerProperties(NoiseProperties->PerlinLayerProperties);
		}
		// The FName we get for the individual vector elements are all the same, whereas resetting with the "revert
		// to default" arrow gets us the name of the vector itself. We'll just update all of them if any of them
		// change.
		else if (PropName == "X" || PropName == "Y" || PropName == "Z"
			|| PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshDirectionalFilterProperties, FilterDirection)
			|| PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshSineWaveProperties, SineWaveDirection)
			|| PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, UVScale)
			|| PropName == GET_MEMBER_NAME_CHECKED(UDisplaceMeshTextureMapProperties, UVOffset))
		{
			DisplacerDownCast->SetFilterDirection(DirectionalFilterProperties->FilterDirection);
			DisplacerDownCast->SetSineWaveDirection(SineWaveProperties->SineWaveDirection);
			DisplacerDownCast->SetDisplacementMapUVAdjustment(FVector2f(TextureMapProperties->UVScale), FVector2f(TextureMapProperties->UVOffset));	// LWC_TODO: Precision loss
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(USelectiveTessellationProperties, SelectionType))
		{
			SubdividerDownCast->SetSelectionType(SelectiveTessellationProperties->SelectionType);
			bNeedsSubdivided = true;
		}
		else if (PropName == GET_MEMBER_NAME_CHECKED(USelectiveTessellationProperties, ActiveMaterial)) 
		{
			int32 Index = SelectiveTessellationProperties->MaterialIDList.Find(SelectiveTessellationProperties->ActiveMaterial.ToString());
			SubdividerDownCast->SetActiveMaterialID(Index);
			bNeedsSubdivided = true;
		}
		
		StartComputation();
	}
}
#endif


void UDisplaceMeshTool::UpdateActiveContrastCurveTarget()
{
	using namespace DisplaceMeshToolLocals;

	// if user resets the AdjustmentCurve field, it will go to nullptr, in this case we will force it 
	// back to a new default curve
	if (TextureMapProperties->AdjustmentCurve == nullptr)
	{
		using namespace DisplaceMeshToolLocals;
		TextureMapProperties->AdjustmentCurve = ToolSetupUtil::GetContrastAdjustmentCurve(GetToolManager());
		bNeedsDisplaced = true;
	}

#if WITH_EDITORONLY_DATA
	// if the AdjustmentCurve changes, then we need to change which one we are listening to for CurveUpdate events
	if (TextureMapProperties->AdjustmentCurve != ActiveContrastCurveTarget)
	{
		DisconnectActiveContrastCurveTarget();

		if (TextureMapProperties->AdjustmentCurve != nullptr)
		{
			ActiveContrastCurveTarget = TextureMapProperties->AdjustmentCurve;
			ActiveContrastCurveListenerHandle = ActiveContrastCurveTarget->OnUpdateCurve.AddWeakLambda(this,
				[this](UCurveBase* Curve, EPropertyChangeType::Type ChangeType) {
				if (TextureMapProperties->bApplyAdjustmentCurve)
				{
					FDisplaceMeshOpFactory* DisplacerDownCast = static_cast<FDisplaceMeshOpFactory*>(Displacer.Get());
					DisplacerDownCast->SetAdjustmentCurve(TextureMapProperties->AdjustmentCurve);
					bNeedsDisplaced = true;
					StartComputation();
				}
			});
		}
	}
#endif
}

void UDisplaceMeshTool::DisconnectActiveContrastCurveTarget()
{
	using namespace DisplaceMeshToolLocals;

#if WITH_EDITORONLY_DATA
	if (ActiveContrastCurveTarget != nullptr)
	{
		ActiveContrastCurveTarget->OnUpdateCurve.Remove(ActiveContrastCurveListenerHandle);
		ActiveContrastCurveListenerHandle = FDelegateHandle();
		ActiveContrastCurveTarget = nullptr;

		FDisplaceMeshOpFactory* DisplacerDownCast = static_cast<FDisplaceMeshOpFactory*>(Displacer.Get());
		DisplacerDownCast->SetAdjustmentCurve(nullptr);
	}
#endif
}


void UDisplaceMeshTool::OnTick(float DeltaTime)
{
	UpdateActiveContrastCurveTarget();

	AdvanceComputation();
}

void UDisplaceMeshTool::StartComputation()
{
	if ( bNeedsSubdivided )
	{
		if (SubdivideTask)
		{
			SubdivideTask->CancelAndDelete();
		}
		SubdividedMesh = nullptr;
		SubdivideTask = new FAsyncTaskExecuterWithAbort<TModelingOpTask<FDynamicMeshOperator>>(Subdivider->MakeNewOperator());
		SubdivideTask->StartBackgroundTask();
		bNeedsSubdivided = false;
	}
	if (bNeedsDisplaced && DisplaceTask)
	{
		DisplaceTask->CancelAndDelete();
		DisplaceTask = nullptr;
	}
	ComputationStartTime = FPlatformTime::Seconds();
	AdvanceComputation();
}

void UDisplaceMeshTool::AdvanceComputation()
{
	using namespace DisplaceMeshToolLocals;

	const float ElapsedTime = (float)(FPlatformTime::Seconds() - ComputationStartTime);
	if (ElapsedTime > 0.75f && DisplaceTask && !DisplaceTask->IsDone())
	{
		DynamicMeshComponent->SetOverrideRenderMaterial(ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	}

	if (SubdivideTask && SubdivideTask->IsDone())
	{
		TUniquePtr<FDynamicMeshOperator> SubOp = SubdivideTask->GetTask().ExtractOperator();
		FSubdivideMeshOp* SubOpDownCast = static_cast<FSubdivideMeshOp*>(SubOp.Get());
		
		SubdividedMesh = TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>(SubOp->ExtractResult().Release());
		VerticesToDisplace = TSharedPtr<TArray<int>, ESPMode::ThreadSafe>(SubOpDownCast->ExtractVerticesToDisplace().Release());

		MeshStatistics->Update(*SubdividedMesh);

		delete SubdivideTask;
		SubdivideTask = nullptr;
	}
	if (SubdividedMesh && bNeedsDisplaced)
	{
		// force update of contrast curve
		FDisplaceMeshOpFactory* DisplacerDownCast = static_cast<FDisplaceMeshOpFactory*>(Displacer.Get());
		DisplacerDownCast->SetAdjustmentCurve(TextureMapProperties->bApplyAdjustmentCurve ? TextureMapProperties->AdjustmentCurve : nullptr);
		DisplacerDownCast->SetVerticesToDisplace(VerticesToDisplace);

		DisplaceTask = new FAsyncTaskExecuterWithAbort<TModelingOpTask<FDynamicMeshOperator>>(Displacer->MakeNewOperator());
		DisplaceTask->StartBackgroundTask();
		bNeedsDisplaced = false;
	}
	if (DisplaceTask && DisplaceTask->IsDone())
	{
		TUniquePtr<FDynamicMesh3> DisplacedMesh = DisplaceTask->GetTask().ExtractOperator()->ExtractResult();
		MeshStatistics->Update(*SubdividedMesh);

		delete DisplaceTask;
		DisplaceTask = nullptr;
		DynamicMeshComponent->ClearOverrideRenderMaterial();
		DynamicMeshComponent->GetMesh()->Copy(*DisplacedMesh);
		DynamicMeshComponent->NotifyMeshUpdated();
		GetToolManager()->PostInvalidation();
	}
}



void UDisplaceMeshTool::UpdateActiveWeightMap()
{
	if (CommonProperties->WeightMap == FName(TEXT("None")))
	{
		ActiveWeightMap = nullptr;
	}
	else
	{
		TSharedPtr<FIndexedWeightMap, ESPMode::ThreadSafe> NewWeightMap = MakeShared<FIndexedWeightMap, ESPMode::ThreadSafe>();
		const FMeshDescription* MeshDescription = UE::ToolTarget::GetMeshDescription(Target);
		bool bFound = UE::WeightMaps::GetVertexWeightMap(MeshDescription, CommonProperties->WeightMap, *NewWeightMap, 1.0f);
		if (CommonProperties->bInvertWeightMap)
		{
			NewWeightMap->InvertWeightMap();
		}
		// We'll check for invalid weightmaps here and reset to None if they aren't valid. This helps in cases where values are set externally,
		// for example from preset loading, where the weightmap values incoming might not have any relation to the current target mesh.
		if (bFound)
		{
			ActiveWeightMap = NewWeightMap;
		}
		else
		{
			CommonProperties->WeightMap = FName(CommonProperties->WeightMapsList[0]);
			ActiveWeightMap = nullptr;
		}
	}
}


float UDisplaceMeshTool::WeightMapQuery(const FVector3d& Position, const FIndexedWeightMap& WeightMap) const
{
	double NearDistSqr;
	int32 NearTID = OriginalMeshSpatial.FindNearestTriangle(Position, NearDistSqr);
	if (NearTID < 0)
	{
		return 1.0f;
	}
	FDistPoint3Triangle3d Distance = TMeshQueries<FDynamicMesh3>::TriangleDistance(OriginalMesh, NearTID, Position);
	FIndex3i Tri = OriginalMesh.GetTriangle(NearTID);
	return WeightMap.GetInterpValue(Tri, Distance.TriangleBaryCoords);
}

#undef LOCTEXT_NAMESPACE

#include "Tests/DisplaceMeshTool_Tests.inl"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DisplaceMeshTool)

