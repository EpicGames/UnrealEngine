// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawAndRevolveTool.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "CompositionOps/CurveSweepOp.h"
#include "Generators/SweepGenerator.h"
#include "InteractiveToolManager.h" // To use SceneState.ToolManager
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "Mechanics/CurveControlPointsMechanic.h"
#include "Properties/MeshMaterialProperties.h"
#include "ModelingObjectsCreationAPI.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface
#include "Selection/ToolSelectionUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawAndRevolveTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDrawAndRevolveTool"

const FText InitializationModeMessage = LOCTEXT("CurveInitialization", "Draw a path and revolve it around the purple axis. Left-click to place path vertices, and click on the last or first vertex to complete the path. Hold Shift to invert snapping behavior.");
const FText EditModeMessage = LOCTEXT("CurveEditing", "Click points to select them, Shift+click to modify selection. Ctrl+click a segment to add a point, or select an endpoint and Ctrl+click to add new endpoint. Backspace deletes selected points. Hold Shift to invert snapping behavior.");


// Tool builder

bool UDrawAndRevolveToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UDrawAndRevolveToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UDrawAndRevolveTool* NewTool = NewObject<UDrawAndRevolveTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	FQuaterniond DefaultOrientation = (FQuaterniond)FRotator(90, 0, 0);
	NewTool->SetInitialDrawFrame(ToolSetupUtil::GetDefaultWorldReferenceFrame(SceneState.ToolManager, DefaultOrientation));
	return NewTool;
}


// Operator factory

TUniquePtr<FDynamicMeshOperator> URevolveOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FCurveSweepOp> CurveSweepOp = MakeUnique<FCurveSweepOp>();

	// Assemble profile curve
	CurveSweepOp->ProfileCurve.Reserve(RevolveTool->ControlPointsMechanic->GetNumPoints() + 2); // extra space for top/bottom caps
	RevolveTool->ControlPointsMechanic->ExtractPointPositions(CurveSweepOp->ProfileCurve);
	CurveSweepOp->bProfileCurveIsClosed = RevolveTool->ControlPointsMechanic->GetIsLoop();

	// If we are capping the top and bottom, we just add a couple extra vertices and mark the curve as being closed
	if (!CurveSweepOp->bProfileCurveIsClosed && RevolveTool->Settings->bClosePathToAxis)
	{
		// Project first and last points onto the revolution axis.
		FVector3d FirstPoint = CurveSweepOp->ProfileCurve[0];
		FVector3d LastPoint = CurveSweepOp->ProfileCurve.Last();
		double DistanceAlongAxis = RevolveTool->RevolutionAxisDirection.Dot(LastPoint - RevolveTool->RevolutionAxisOrigin);
		FVector3d ProjectedPoint = RevolveTool->RevolutionAxisOrigin + (RevolveTool->RevolutionAxisDirection * DistanceAlongAxis);

		CurveSweepOp->ProfileCurve.Add(ProjectedPoint);

		DistanceAlongAxis = RevolveTool->RevolutionAxisDirection.Dot(FirstPoint - RevolveTool->RevolutionAxisOrigin);
		ProjectedPoint = RevolveTool->RevolutionAxisOrigin + (RevolveTool->RevolutionAxisDirection * DistanceAlongAxis);

		CurveSweepOp->ProfileCurve.Add(ProjectedPoint);
		CurveSweepOp->bProfileCurveIsClosed = true;
	}

	RevolveTool->Settings->ApplyToCurveSweepOp(*RevolveTool->MaterialProperties,
		RevolveTool->RevolutionAxisOrigin, RevolveTool->RevolutionAxisDirection, *CurveSweepOp);

	return CurveSweepOp;
}


// Tool itself

void UDrawAndRevolveTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 1,
		TEXT("DeletePointBackspaceKey"),
		LOCTEXT("DeletePointUIName", "Delete Point"),
		LOCTEXT("DeletePointTooltip", "Delete currently selected point(s)"),
		EModifierKey::None, EKeys::BackSpace,
		[this]() { OnPointDeletionKeyPress(); });
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
		TEXT("DeletePointDeleteKey"),
		LOCTEXT("DeletePointUIName", "Delete Point"),
		LOCTEXT("DeletePointTooltip", "Delete currently selected point(s)"),
		EModifierKey::None, EKeys::Delete,
		[this]() { OnPointDeletionKeyPress(); });
}

void UDrawAndRevolveTool::OnPointDeletionKeyPress()
{
	if (ControlPointsMechanic)
	{
		ControlPointsMechanic->DeleteSelectedPoints();
	}
}

bool UDrawAndRevolveTool::CanAccept() const
{
	return Preview != nullptr && Preview->HaveValidNonEmptyResult();
}

void UDrawAndRevolveTool::Setup()
{
	UInteractiveTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Revolve PolyPath"));
	GetToolManager()->DisplayMessage(InitializationModeMessage, EToolMessageLevel::UserNotification);

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	Settings = NewObject<URevolveToolProperties>(this, TEXT("Revolve Tool Settings"));
	Settings->RestoreProperties(this);
	Settings->bAllowedToEditDrawPlane = true;
	AddToolPropertySource(Settings);

	Settings->DrawPlaneOrigin = InitialDrawFrame.Origin;
	Settings->DrawPlaneOrientation = (FRotator)InitialDrawFrame.Rotation;

	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	AddToolPropertySource(MaterialProperties);
	MaterialProperties->RestoreProperties(this);

	ControlPointsMechanic = NewObject<UCurveControlPointsMechanic>(this);
	ControlPointsMechanic->Setup(this);
	ControlPointsMechanic->SetWorld(TargetWorld);
	ControlPointsMechanic->OnPointsChanged.AddLambda([this]() {
		if (Preview)
		{
			Preview->InvalidateResult();
		}
		bool bAllowedToEditDrawPlane = (ControlPointsMechanic->GetNumPoints() == 0);
		if (Settings->bAllowedToEditDrawPlane != bAllowedToEditDrawPlane)
		{
			Settings->bAllowedToEditDrawPlane = bAllowedToEditDrawPlane;
			NotifyOfPropertyChangeByTool(Settings);
		}
		});
	// This gets called when we enter/leave curve initialization mode
	ControlPointsMechanic->OnModeChanged.AddLambda([this]() {
		if (ControlPointsMechanic->IsInInteractiveIntialization())
		{
			// We're back to initializing so hide the preview
			if (Preview)
			{
				Preview->Cancel();
				Preview = nullptr;
			}
			GetToolManager()->DisplayMessage(InitializationModeMessage, EToolMessageLevel::UserNotification);
		}
		else
		{
			StartPreview();
			GetToolManager()->DisplayMessage(EditModeMessage, EToolMessageLevel::UserNotification);
		}
		});
	ControlPointsMechanic->SetSnappingEnabled(Settings->bEnableSnapping);

	UpdateRevolutionAxis();

	// The plane mechanic lets us update the plane in which we draw the profile curve, as long as we haven't
	// started adding points to it already.
	FFrame3d ProfileDrawPlane(Settings->DrawPlaneOrigin, Settings->DrawPlaneOrientation.Quaternion());
	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(TargetWorld, ProfileDrawPlane);
	PlaneMechanic->UpdateClickPriority(ControlPointsMechanic->ClickBehavior->GetPriority().MakeHigher());
	PlaneMechanic->CanUpdatePlaneFunc = [this]() 
	{ 
		return ControlPointsMechanic->GetNumPoints() == 0;
	};
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		Settings->DrawPlaneOrigin = (FVector)PlaneMechanic->Plane.Origin;
		Settings->DrawPlaneOrientation = ((FQuat)PlaneMechanic->Plane.Rotation).Rotator();
		NotifyOfPropertyChangeByTool(Settings);
		if (ControlPointsMechanic)
		{
			ControlPointsMechanic->SetPlane(PlaneMechanic->Plane);
		}
		UpdateRevolutionAxis();
		});

	ControlPointsMechanic->SetPlane(PlaneMechanic->Plane);
	ControlPointsMechanic->SetInteractiveInitialization(true);
	ControlPointsMechanic->SetAutoRevertToInteractiveInitialization(true);

	// For things to behave nicely, we expect to revolve at least a two point
	// segment if it's not a loop, and a three point segment if it is. Revolving
	// a two-point loop to make a double sided thing is a pain to support because
	// it forces us to deal with manifoldness issues that we would really rather
	// not worry about (we'd have to duplicate vertices to stay manifold)
	ControlPointsMechanic->SetMinPointsToLeaveInteractiveInitialization(3, 2);
}

/** Uses the settings currently stored in the properties object to update the revolution axis. */
void UDrawAndRevolveTool::UpdateRevolutionAxis()
{
	RevolutionAxisOrigin = (FVector3d)Settings->DrawPlaneOrigin;
	RevolutionAxisDirection = (FVector3d)Settings->DrawPlaneOrientation.RotateVector(FVector(1,0,0));

	const int32 AXIS_SNAP_TARGET_ID = 1;
	ControlPointsMechanic->RemoveSnapLine(AXIS_SNAP_TARGET_ID);
	ControlPointsMechanic->AddSnapLine(AXIS_SNAP_TARGET_ID, FLine3d(RevolutionAxisOrigin, RevolutionAxisDirection));
}

void UDrawAndRevolveTool::Shutdown(EToolShutdownType ShutdownType)
{
	OutputTypeProperties->SaveProperties(this);
	Settings->SaveProperties(this);
	MaterialProperties->SaveProperties(this);

	PlaneMechanic->Shutdown();
	ControlPointsMechanic->Shutdown();

	if (Preview)
	{
		if (ShutdownType == EToolShutdownType::Accept)
		{
			Preview->PreviewMesh->CalculateTangents(); // Copy tangents from the PreviewMesh
			GenerateAsset(Preview->Shutdown());
		}
		else
		{
			Preview->Cancel();
		}
	}
}

void UDrawAndRevolveTool::GenerateAsset(const FDynamicMeshOpResult& OpResult)
{
	if (OpResult.Mesh->TriangleCount() <= 0)
	{
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("RevolveToolTransactionName", "Path Revolve Tool"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = (FTransform)OpResult.Transform;
	NewMeshObjectParams.BaseName = TEXT("Revolve");
	NewMeshObjectParams.Materials.Add(MaterialProperties->Material.Get());
	NewMeshObjectParams.SetMesh(OpResult.Mesh.Get());
	OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();
}

void UDrawAndRevolveTool::StartPreview()
{
	URevolveOperatorFactory* RevolveOpCreator = NewObject<URevolveOperatorFactory>();
	RevolveOpCreator->RevolveTool = this;

	// Normally we wouldn't give the object a name, but since we may destroy the preview using undo,
	// the ability to reuse the non-cleaned up memory is useful. Careful if copy-pasting this!
	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(RevolveOpCreator, "RevolveToolPreview");

	Preview->Setup(TargetWorld, RevolveOpCreator);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr); 
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

	Preview->ConfigureMaterials(MaterialProperties->Material.Get(),
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);

	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute* UpdatedPreview)
		{
			UpdateAcceptWarnings(UpdatedPreview->HaveEmptyResult() ? EAcceptWarning::EmptyForbidden : EAcceptWarning::NoWarning);
		}
	);

	Preview->SetVisibility(true);
	Preview->InvalidateResult();
}

void UDrawAndRevolveTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	FFrame3d ProfileDrawPlane(Settings->DrawPlaneOrigin, Settings->DrawPlaneOrientation.Quaternion());
	ControlPointsMechanic->SetPlane(ProfileDrawPlane);
	PlaneMechanic->SetPlaneWithoutBroadcast(ProfileDrawPlane);
	UpdateRevolutionAxis();

	ControlPointsMechanic->SetSnappingEnabled(Settings->bEnableSnapping);

	if (Preview)
	{
		if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UNewMeshMaterialProperties, Material)))
		{
			Preview->ConfigureMaterials(MaterialProperties->Material.Get(),
				ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
		}

		Preview->PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
		Preview->InvalidateResult();
	}
}


void UDrawAndRevolveTool::OnTick(float DeltaTime)
{
	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Tick(DeltaTime);
	}

	if (Preview)
	{
		Preview->Tick(DeltaTime);
	}
}


void UDrawAndRevolveTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (PlaneMechanic != nullptr)
	{
		PlaneMechanic->Render(RenderAPI);

		// Draw the axis of rotation
		float PdiScale = CameraState.GetPDIScalingFactor();
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

		FColor AxisColor(240, 16, 240);
		double AxisThickness = 1.0 * PdiScale;
		double AxisHalfLength = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, RevolutionAxisOrigin, 90);

		FVector3d StartPoint = RevolutionAxisOrigin - (RevolutionAxisDirection * (AxisHalfLength * PdiScale));
		FVector3d EndPoint = RevolutionAxisOrigin + (RevolutionAxisDirection * (AxisHalfLength * PdiScale));

		PDI->DrawLine((FVector)StartPoint, (FVector)EndPoint, AxisColor, SDPG_Foreground, 
			AxisThickness, 0.0f, true);
	}

	if (ControlPointsMechanic != nullptr)
	{
		ControlPointsMechanic->Render(RenderAPI);
	}
}

bool UDrawAndRevolveTool::CanGenerateAsset() const
{
	if (Preview)
	{
		return 0 < Preview->PreviewMesh->GetPreviewDynamicMesh()->TriangleCount();
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

