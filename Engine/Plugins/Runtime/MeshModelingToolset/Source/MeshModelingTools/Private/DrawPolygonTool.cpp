// Copyright Epic Games, Inc. All Rights Reserved.

#include "DrawPolygonTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "BaseBehaviors/MultiClickSequenceInputBehavior.h"
#include "BaseBehaviors/KeyAsModifierInputBehavior.h"

#include "Polygon2.h"
#include "Curve/GeneralPolygon2.h"
#include "FrameTypes.h"
#include "MatrixTypes.h"

#include "Generators/FlatTriangulationMeshGenerator.h"
#include "Generators/DiscMeshGenerator.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Operations/ExtrudeMesh.h"
#include "Distance/DistLine3Ray3.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "ToolSceneQueriesUtil.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface
#include "SceneQueries/SceneSnappingManager.h"
#include "ConstrainedDelaunay2.h"
#include "Arrangement2d.h"
#include "DynamicMesh/MeshTangents.h"

#include "DynamicMeshEditor.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Drawing/MeshDebugDrawing.h"

#include "Selection/SelectClickedAction.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ModelingObjectsCreationAPI.h"
#include "Mechanics/ConstructionPlaneMechanic.h"

#include "Mechanics/DragAlignmentMechanic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DrawPolygonTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDrawPolygonTool"

/*
 * ToolBuilder
 */
constexpr int StartPointSnapID = FPointPlanarSnapSolver::BaseExternalPointID + 1;
constexpr int CurrentSceneSnapID = FPointPlanarSnapSolver::BaseExternalPointID + 2;
constexpr int CurrentGridSnapID = FPointPlanarSnapSolver::BaseExternalPointID + 3;

bool UDrawPolygonToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UDrawPolygonToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UDrawPolygonTool* NewTool = NewObject<UDrawPolygonTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);
	NewTool->SetInitialDrawFrame(ToolSetupUtil::GetDefaultWorldReferenceFrame(SceneState.ToolManager));
	return NewTool;
}

/*
 * Properties
 */
UDrawPolygonToolStandardProperties::UDrawPolygonToolStandardProperties()
{
}

/*
 * Tool
 */
UDrawPolygonTool::UDrawPolygonTool()
{
	bInInteractiveExtrude = false;
	UInteractiveTool::SetToolDisplayName(LOCTEXT("ToolName", "Polygon Extrude"));
}

void UDrawPolygonTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UDrawPolygonTool::Setup()
{
	UInteractiveTool::Setup();

	// add default button input behaviors for devices
	UMultiClickSequenceInputBehavior* MouseBehavior = NewObject<UMultiClickSequenceInputBehavior>();
	MouseBehavior->Initialize(this);
	MouseBehavior->Modifiers.RegisterModifier(IgnoreSnappingModifier, FInputDeviceState::IsShiftKeyDown);
	AddInputBehavior(MouseBehavior);

	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->RestoreProperties(this);
	OutputTypeProperties->InitializeDefault();
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });

	PolygonProperties = NewObject<UDrawPolygonToolStandardProperties>(this);
	PolygonProperties->RestoreProperties(this);
	PolygonProperties->WatchProperty(PolygonProperties->bShowGridGizmo,
	                                 [this](bool bNewValue) { PlaneMechanic->PlaneTransformGizmo->SetVisibility(bNewValue); });

	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->CanUpdatePlaneFunc = [this]() { return AllowDrawPlaneUpdates(); };
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() { SnapEngine.Plane = PlaneMechanic->Plane; });	// Keep SnapEngine plane up to date with PlaneMechanic's plane
	PlaneMechanic->Initialize(TargetWorld, InitialDrawFrame);
	
	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->AddToGizmo(PlaneMechanic->PlaneTransformGizmo);
	
	// initialize material properties for new objects
	MaterialProperties = NewObject<UNewMeshMaterialProperties>(this);
	MaterialProperties->RestoreProperties(this);
	MaterialProperties->bShowExtendedOptions = true;

	// create preview mesh object
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(this->TargetWorld, FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr); 
	PreviewMesh->SetVisible(false);
	{
		UMaterialInterface* Material = nullptr;
		if ( MaterialProperties->Material.IsValid() )
		{
			Material = MaterialProperties->Material.Get();
		}
		PreviewMesh->SetMaterial(Material);
	}
	bPreviewUpdatePending = false;

	// initialize snapping engine and properties
	SnapEngine.SnapMetricTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	SnapEngine.SnapMetricFunc = [this](const FVector3d& Position1, const FVector3d& Position2) {
		return ToolSceneQueriesUtil::CalculateNormalizedViewVisualAngleD(this->CameraState, Position1, Position2);
	};
	SnapEngine.Plane = PlaneMechanic->Plane;

	SnapProperties = NewObject<UDrawPolygonToolSnapProperties>(this);
	SnapProperties->RestoreProperties(this);

	const FToolContextSnappingConfiguration SnappingSettings = GetToolManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
	SnapProperties->bSnapToWorldGrid = SnappingSettings.IsPositionGridSnappingActive();

	// register tool properties
	if (OutputTypeProperties->ShouldShowPropertySet())
	{
		AddToolPropertySource(OutputTypeProperties);
	}
	AddToolPropertySource(PolygonProperties);
	AddToolPropertySource(SnapProperties);
	AddToolPropertySource(MaterialProperties);

	ShowStartupMessage();
}


void UDrawPolygonTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (bHasSavedExtrudeHeight)
	{
		PolygonProperties->ExtrudeHeight = SavedExtrudeHeight;
		bHasSavedExtrudeHeight = false;
	}

	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	DragAlignmentMechanic->Shutdown();

	PlaneMechanic->Shutdown();
	PlaneMechanic = nullptr;
	
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	OutputTypeProperties->SaveProperties(this);
	PolygonProperties->SaveProperties(this);
	SnapProperties->SaveProperties(this);
	MaterialProperties->SaveProperties(this);
}

void UDrawPolygonTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	// This is an uncommon hotkey to want, and can make tool seem broken if accidentally pressed. The only
	// reason a user might want it is if the gizmo is in the way of their drawing, in which case they could
	// presumably translate it in the plane... Still, allow it to be settable if the user wants.
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
		TEXT("ToggleGizmoVisibility"),
		LOCTEXT("ToggleGizmoVisibilityUIName", "Toggle Gizmo"),
		LOCTEXT("ToggleGizmoVisibilityTooltip", "Toggle visibility of the transformation gizmo"),
		EModifierKey::None, EKeys::Invalid,
		[this]() { PolygonProperties->bShowGridGizmo = !PolygonProperties->bShowGridGizmo; });
}



void UDrawPolygonTool::ApplyUndoPoints(const TArray<FVector3d>& ClickPointsIn, const TArray<FVector3d>& PolygonVerticesIn)
{
	if (bInInteractiveExtrude || PolygonVertices.Num() == 0)
	{
		return;
	}

	bHaveSelfIntersection = false;

	if (bInFixedPolygonMode == false)
	{
		PolygonVertices = PolygonVerticesIn;
		if ( PolygonVertices.Num() == 0 )
		{
			bAbortActivePolygonDraw = true;
			CurrentCurveTimestamp++;
		}
	}
	else
	{
		FixedPolygonClickPoints = ClickPointsIn;
		if ( FixedPolygonClickPoints.Num() == 0 )
		{
			bAbortActivePolygonDraw = true;
			CurrentCurveTimestamp++;
		}
	}
}

bool UDrawPolygonTool::IsInputSelectionValidOnOutput()
{
	return bRestoreInputSelection;
}


void UDrawPolygonTool::OnTick(float DeltaTime)
{
	if (SnapProperties)
	{
		const FToolContextSnappingConfiguration SnappingSettings = GetToolManager()->GetContextQueriesAPI()->GetCurrentSnappingSettings();
		const bool bSnappingEnabledInViewport = SnappingSettings.IsPositionGridSnappingActive();
		if (SnapProperties->bSnapToWorldGrid != bSnappingEnabledInViewport)
		{
			SnapProperties->bSnapToWorldGrid = bSnappingEnabledInViewport;
			NotifyOfPropertyChangeByTool(SnapProperties);
		}
	}
}



void DrawEdgeTicks(FPrimitiveDrawInterface* PDI, 
	const FSegment3d& Segment, float Height,
	const FVector3d& PlaneNormal, 
	const FLinearColor& Color, uint8 DepthPriorityGroup, float LineThickness, bool bIsScreenSpace)
{
	FVector3d Center = Segment.Center;
	FVector3d X = Segment.Direction;
	FVector3d Y = X.Cross(PlaneNormal);
	UE::Geometry::Normalize(Y);
	FVector3d A = Center - Height * 0.25*X - Height * Y;
	FVector3d B = Center + Height * 0.25*X + Height * Y;
	PDI->DrawLine((FVector)A, (FVector)B, Color, DepthPriorityGroup, LineThickness, 0.0f, bIsScreenSpace);
	A += Height * 0.5*X;
	B += Height * 0.5*X;
	PDI->DrawLine((FVector)A, (FVector)B, Color, DepthPriorityGroup, LineThickness, 0.0f, bIsScreenSpace);
}

void UDrawPolygonTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	// Cache here for usage during interaction, should probably happen in ::Tick() or elsewhere
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
	
	FViewCameraState RenderCameraState = RenderAPI->GetCameraState();
	float PDIScale = RenderCameraState.GetPDIScalingFactor();

	if (bPreviewUpdatePending)
	{
		UpdateLivePreview();
		bPreviewUpdatePending = false;
	}

	double CurViewSizeFactor = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(RenderCameraState, PreviewVertex, 1.0);

	FColor PreviewColor = FColor::Green;
	FColor OpenPolygonColor = FColor::Orange;
	FColor ClosedPolygonColor = FColor::Yellow;
	FColor ErrorColor = FColor::Magenta;
	float HiddenLineThickness = 1.0f*PDIScale;
	float LineThickness = 4.0f*PDIScale;
	FColor SnapLineColor = FColor::Yellow;
	FColor SnapHighlightColor = SnapLineColor;
	float ElementSize = CurViewSizeFactor;

	bool bIsClosed = bInInteractiveExtrude 
		|| (SnapEngine.HaveActiveSnap() && SnapEngine.GetActiveSnapTargetID() == StartPointSnapID);

	//
	// Render the plane mechanic after correctly setting bShowGrid
	//
	PlaneMechanic->bShowGrid = !bInInteractiveExtrude;
	PlaneMechanic->Render(RenderAPI);

	//
	// Generate the fixed polygon contour
	//
	if ((bInFixedPolygonMode) && (FixedPolygonClickPoints.Num() > 0))
	{
		TArray<FVector3d> PreviewClickPoints = FixedPolygonClickPoints;
		if ( !bInInteractiveExtrude ){ PreviewClickPoints.Add(PreviewVertex); }
		GenerateFixedPolygon(PreviewClickPoints, PolygonVertices, PolygonHolesVertices);
	}
	bIsClosed |= bInFixedPolygonMode;

	int NumVerts = PolygonVertices.Num();

	//
	// Render snap indicators
	//
	if (SnapEngine.HaveActiveSnap())
	{
		PDI->DrawPoint((FVector)SnapEngine.GetActiveSnapToPoint(), ClosedPolygonColor, 10.0f*PDIScale, SDPG_Foreground);

		PDI->DrawPoint((FVector)SnapEngine.GetActiveSnapFromPoint(), SnapHighlightColor, 15.0f*PDIScale, SDPG_Foreground);
		PDI->DrawLine((FVector)SnapEngine.GetActiveSnapToPoint(), (FVector)SnapEngine.GetActiveSnapFromPoint(),
			ClosedPolygonColor, SDPG_Foreground, 0.5f*PDIScale, 0.0f, true);
		if (SnapEngine.GetActiveSnapTargetID() == CurrentSceneSnapID)
		{
			if (LastSnapGeometry.PointCount == 1) {
				DrawCircle(PDI, (FVector)LastSnapGeometry.Points[0], RenderCameraState.Right(), RenderCameraState.Up(),
					SnapHighlightColor, ElementSize, 32, SDPG_Foreground, 1.0f*PDIScale, 0.0f, true);
			}
			else
			{
				PDI->DrawLine((FVector)LastSnapGeometry.Points[0], (FVector)LastSnapGeometry.Points[1],
					SnapHighlightColor, SDPG_Foreground, 1.0f*PDIScale, 0.0f, true);
			}
		}
		else if (SnapEngine.GetActiveSnapTargetID() == CurrentGridSnapID)
		{
			DrawCircle(PDI, (FVector)LastGridSnapPoint, RenderCameraState.Right(), RenderCameraState.Up(),
				SnapHighlightColor, ElementSize, 4, SDPG_Foreground, 1.0f*PDIScale, 0.0f, true);
		}

		if (SnapEngine.HaveActiveSnapLine())
		{
			// clip this line to the view plane because if it goes through the view plane the pixel-line-thickness
			// calculation appears to fail
			FLine3d DrawSnapLine = SnapEngine.GetActiveSnapLine();
			FVector3d P0(DrawSnapLine.PointAt(-99999.0)), P1(DrawSnapLine.PointAt(99999.0));		// should be smarter here...
			if (RenderCameraState.bIsOrthographic == false)
			{
				UE::Geometry::FPlane3d CameraPlane((FVector3d)RenderCameraState.Forward(), (FVector3d)RenderCameraState.Position + 1.0*(FVector3d)RenderCameraState.Forward());
				CameraPlane.ClipSegment(P0, P1);
			}
			PDI->DrawLine((FVector)P0, (FVector)P1, SnapLineColor, SDPG_Foreground, 0.5 * PDIScale, 0.0f, true);

			if (SnapEngine.HaveActiveSnapDistance())
			{
				int iSegment = SnapEngine.GetActiveSnapDistanceID();
				TArray<FVector3d>& HistoryPoints = (bInFixedPolygonMode) ? FixedPolygonClickPoints : PolygonVertices;
				FVector3d UseNormal = PlaneMechanic->Plane.Rotation.AxisZ();
				DrawEdgeTicks(PDI, FSegment3d(HistoryPoints[iSegment], HistoryPoints[iSegment+1]),
					0.75f*ElementSize, UseNormal, SnapHighlightColor, SDPG_Foreground, 1.0f*PDIScale, true);
				DrawEdgeTicks(PDI, FSegment3d(HistoryPoints[HistoryPoints.Num()-1], PreviewVertex),
					0.75f*ElementSize, UseNormal, SnapHighlightColor, SDPG_Foreground, 1.0f*PDIScale, true);
				// Drawing a highlight
				PDI->DrawLine((FVector)HistoryPoints[iSegment], (FVector)HistoryPoints[iSegment + 1],
					SnapHighlightColor, SDPG_Foreground, 2.0f*PDIScale, 1.0f, true);
			}
		}
	}

	//
	// Draw Surface Hit Indicator
	//
	if (bHaveSurfaceHit)
	{
		PDI->DrawPoint((FVector)SurfaceHitPoint, ClosedPolygonColor, 10*PDIScale, SDPG_Foreground);
		if (SnapProperties->SnapToSurfacesOffset != 0)
		{
			PDI->DrawPoint((FVector)SurfaceOffsetPoint, OpenPolygonColor, 15*PDIScale, SDPG_Foreground);
			PDI->DrawLine((FVector)SurfaceOffsetPoint, (FVector)SurfaceHitPoint,
				ClosedPolygonColor, SDPG_Foreground, 0.5f*PDIScale, 0.0f, true);
		}
		PDI->DrawLine((FVector)SurfaceOffsetPoint, (FVector)PreviewVertex,
			ClosedPolygonColor, SDPG_Foreground, 0.5f*PDIScale, 0.0f, true);
	}


	//
	// Draw the polygon contour preview
	//
	if (PolygonVertices.Num() > 0)
	{
		FColor UseColor = bIsClosed ? ClosedPolygonColor 
			: bHaveSelfIntersection ? ErrorColor : OpenPolygonColor;
		FColor LastSegmentColor = bIsClosed ? ClosedPolygonColor 
			: bHaveSelfIntersection ? ErrorColor : PreviewColor;
		FVector3d UseLastVertex = bIsClosed ? PolygonVertices[0] : PreviewVertex;

		auto DrawVertices = [&PDI, &UseColor](const TArray<FVector3d>& Vertices, ESceneDepthPriorityGroup Group, float Thickness)
		{
			for (int lasti = Vertices.Num() - 1, i = 0, NumVertices = Vertices.Num(); i < NumVertices; lasti = i++)

			{
				PDI->DrawLine((FVector)Vertices[lasti], (FVector)Vertices[i], UseColor, Group, Thickness, 0.0f, true);
			}
		};

		// draw thin no-depth (x-ray draw)
		//DrawVertices(PolygonVertices, SDPG_Foreground, HiddenLineThickness);
		for (int i = 0; i < NumVerts - 1; ++i)
		{
			PDI->DrawLine((FVector)PolygonVertices[i], (FVector)PolygonVertices[i + 1],
				UseColor, SDPG_Foreground, HiddenLineThickness, 0.0f, true);
		}
		PDI->DrawLine((FVector)PolygonVertices[NumVerts - 1], (FVector)UseLastVertex,
			LastSegmentColor, SDPG_Foreground, HiddenLineThickness, 0.0f, true);
		for (int HoleIdx = 0; HoleIdx < PolygonHolesVertices.Num(); HoleIdx++)
		{
			DrawVertices(PolygonHolesVertices[HoleIdx], SDPG_Foreground, HiddenLineThickness);
		}

		// draw thick depth-tested
		//DrawVertices(PolygonVertices, SDPG_World, LineThickness);
		for (int i = 0; i < NumVerts - 1; ++i)
		{
			PDI->DrawLine((FVector)PolygonVertices[i], (FVector)PolygonVertices[i + 1],
				UseColor, SDPG_World, LineThickness, 0.0f, true);
		}
		PDI->DrawLine((FVector)PolygonVertices[NumVerts - 1], (FVector)UseLastVertex,
			LastSegmentColor, SDPG_World, LineThickness, 0.0f, true);
		for (int HoleIdx = 0; HoleIdx < PolygonHolesVertices.Num(); HoleIdx++)
		{
			DrawVertices(PolygonHolesVertices[HoleIdx], SDPG_World, LineThickness);
		}

		// Intersection point
		if (bHaveSelfIntersection && !bInInteractiveExtrude)
		{
			PDI->DrawPoint((FVector)SelfIntersectionPoint, SnapHighlightColor, 12*PDIScale, SDPG_Foreground);
		}
	}

	// draw preview vertex
	if (!bInInteractiveExtrude)
	{
		PDI->DrawPoint((FVector)PreviewVertex, PreviewColor, 10 * PDIScale, SDPG_Foreground);
	}

	// draw height preview stuff
	if (bInInteractiveExtrude)
	{
		HeightMechanic->Render(RenderAPI);
	}
}


void UDrawPolygonTool::ResetPolygon()
{
	PolygonVertices.Reset();
	PolygonHolesVertices.Reset();
	SnapEngine.Reset();
	bHaveSurfaceHit = false;
	bInFixedPolygonMode = false;
	bHaveSelfIntersection = false;
	CurrentCurveTimestamp++;
}

void UDrawPolygonTool::UpdatePreviewVertex(const FVector3d& PreviewVertexIn)
{
	PreviewVertex = PreviewVertexIn;

	// update length and angle
	if (PolygonVertices.Num() > 0)
	{
		const FVector3d LastVertex = PolygonVertices[PolygonVertices.Num() - 1];
		if (bInFixedPolygonMode)
		{
			double FixedDistance = 0; // get a representative distance for the shape, to show to the user
			if (FixedPolygonClickPoints.Num() > 0)
			{
				// Build standard polygon parameters
				TArray<FVector3d> PreviewClickPoints = FixedPolygonClickPoints;
				PreviewClickPoints.Add(PreviewVertex);
				FVector2d FirstReferencePt, BoxSize;
				double YSign, AngleRad;
				GetPolygonParametersFromFixedPoints(PreviewClickPoints, FirstReferencePt, BoxSize, YSign, AngleRad);
				double Width = BoxSize.X, Height = BoxSize.Y;
				// For rectangles, interface uses width for earlier click, height for later
				if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Rectangle || PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle)
				{
					if (PreviewClickPoints.Num() == 2)
					{
						FixedDistance = Width;
					}
					else
					{
						FixedDistance = Height;
					}
				}
				else // For all else (circles, discs and squares), only width is used
				{
					FixedDistance = Width;
				}
			}
			PolygonProperties->Distance = (float)FixedDistance;
		}
		else
		{
			PolygonProperties->Distance = Distance(LastVertex, PreviewVertex);
		}
	}
}

void UDrawPolygonTool::AppendVertex(const FVector3d& Vertex)
{
	PolygonVertices.Add(Vertex);
}

bool UDrawPolygonTool::FindDrawPlaneHitPoint(const FInputDeviceRay& ClickPos, FVector3d& HitPosOut)
{
	bHaveSurfaceHit = false;

	const FFrame3d& Frame = PlaneMechanic->Plane;
	FVector3d HitPos;
	FVector3d SceneSnapPos;
	bool bHitPlane = Frame.RayPlaneIntersection((FVector3d)ClickPos.WorldRay.Origin, (FVector3d)ClickPos.WorldRay.Direction, 2, HitPos);
	if (!bHitPlane && (!SnapProperties->bEnableSnapping || bIgnoreSnappingToggle))
	{
		return false;
	}

	// if we found a scene snap point, add to snap set
	if (bIgnoreSnappingToggle || SnapProperties->bEnableSnapping == false)
	{
		// if snapping is disabled, still snap to the first vertex (so the polygon can be closed)
		SnapEngine.Reset();
		if (bInFixedPolygonMode == false && PolygonVertices.Num() > 0)
		{
			SnapEngine.AddPointTarget(PolygonVertices[0], StartPointSnapID, 1);
		}
	}
	else 
	{
		// Since we do not early out if we do not hit the tool plane when snapping is enabled,
		// HitPos is not guaranteed to be meaningful here unless bHitPlane is true
		FVector3d WorldGridSnapPos;
		if (bHitPlane && ToolSceneQueriesUtil::FindWorldGridSnapPoint(this, HitPos, WorldGridSnapPos))
		{
			WorldGridSnapPos = Frame.ToPlane(WorldGridSnapPos, 2);
			SnapEngine.AddPointTarget(WorldGridSnapPos, CurrentGridSnapID, 
				FBasePositionSnapSolver3::FCustomMetric::Replace(999), SnapEngine.MinInternalPriority() - 5);
			LastGridSnapPoint = WorldGridSnapPos;
		}

		if (SnapProperties->bSnapToVertices || SnapProperties->bSnapToEdges)
		{
			const FVector3d PointOnWorldRay = ClickPos.WorldRay.PointAt(1);
			if (ToolSceneQueriesUtil::FindSceneSnapPoint(this, bHitPlane ? HitPos : PointOnWorldRay, SceneSnapPos, SnapProperties->bSnapToVertices, SnapProperties->bSnapToEdges, 0, &LastSnapGeometry))
			{
				SnapEngine.AddPointTarget(SceneSnapPos, CurrentSceneSnapID, SnapEngine.MinInternalPriority() - 10);
			}
		}

		const TArray<FVector3d>& HistoryPoints = (bInFixedPolygonMode) ? FixedPolygonClickPoints : PolygonVertices;
		SnapEngine.UpdatePointHistory(HistoryPoints);
		if (SnapProperties->bSnapToAxes)
		{
			SnapEngine.RegenerateTargetLines(true, true);
		}
		SnapEngine.bEnableSnapToKnownLengths = SnapProperties->bSnapToLengths;
	}
	
	// ignore snapping to start point unless we have at least 3 vertices
	if (bInFixedPolygonMode == false && PolygonVertices.Num() > 0)
	{
		if (PolygonVertices.Num() < 3)
		{
			SnapEngine.AddIgnoreTarget(StartPointSnapID);
		}
		else
		{
			SnapEngine.RemoveIgnoreTarget(StartPointSnapID);
		}
	}
	
	SnapEngine.UpdateSnappedPoint(bHitPlane ? HitPos : SceneSnapPos);

	// remove scene snap point
	SnapEngine.RemovePointTargetsByID(CurrentSceneSnapID);
	SnapEngine.RemovePointTargetsByID(CurrentGridSnapID);

	// Success case 1: HitPosOut is set to a snap point.
	if (SnapEngine.HaveActiveSnap())
	{
		HitPosOut = SnapEngine.GetActiveSnapToPoint();
		return true;
	}
	
	// if not yet snapped and we want to hit objects, do that
	if (SnapProperties->bEnableSnapping && SnapProperties->bSnapToSurfaces && !bIgnoreSnappingToggle)
	{
		FHitResult Result;
		bool bWorldHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, ClickPos.WorldRay);

		// Success case 2: HitPosOut is set to a point on the tool plane by projecting a found point in the world onto the plane.
		if (bWorldHit)
		{
			bHaveSurfaceHit = true;
			SurfaceHitPoint = (FVector3d)Result.ImpactPoint;
			const FVector3d UseHitPos = Result.ImpactPoint + static_cast<double>(SnapProperties->SnapToSurfacesOffset) * Result.Normal;
			HitPos = Frame.ToPlane(UseHitPos, 2);
			SurfaceOffsetPoint = UseHitPos;

			HitPosOut = HitPos;
			return true;
		}
	}

	// Success case 3: HitPosOut is set to a point on the tool plane based on raycast intersection with the plane.
	if (bHitPlane)
	{
		HitPosOut = HitPos;
		return true;
	}
	
	return false;
}

void UDrawPolygonTool::OnBeginSequencePreview(const FInputDeviceRay& DevicePos)
{
	// just update snapped point preview
	FVector3d HitPos;
	if (FindDrawPlaneHitPoint(DevicePos, HitPos))
	{
		PreviewVertex = HitPos;
	}
	
}

bool UDrawPolygonTool::CanBeginClickSequence(const FInputDeviceRay& ClickPos)
{
	return true;
}

void UDrawPolygonTool::OnBeginClickSequence(const FInputDeviceRay& ClickPos)
{
	ResetPolygon();
	
	FVector3d HitPos;
	bool bHit = FindDrawPlaneHitPoint(ClickPos, HitPos);
	if (bHit == false)
	{
		bAbortActivePolygonDraw = true;
		return;
	}
	if (ToolSceneQueriesUtil::IsPointVisible(CameraState, HitPos) == false)
	{
		bAbortActivePolygonDraw = true;
		return;		// cannot start a poly an a point that is not visible, this is almost certainly an error due to draw plane
	}

	UpdatePreviewVertex(HitPos);

	bInFixedPolygonMode = (PolygonProperties->PolygonDrawMode != EDrawPolygonDrawMode::Freehand);
	FixedPolygonClickPoints.Reset();

	// Actually process the click.
	// TODO: This slightly awkward organization is a reflection of an earlier time when
	// MultiClickSequenceInputBehavior issued a duplicate OnNextSequenceClick() call 
	// immediately after OnBeginClickSequence(). The code could be cleaned up.
	OnNextSequenceClick(ClickPos);
}

void UDrawPolygonTool::OnNextSequencePreview(const FInputDeviceRay& ClickPos)
{
	if (bInInteractiveExtrude)
	{
		HeightMechanic->UpdateCurrentDistance(ClickPos.WorldRay);
		PolygonProperties->ExtrudeHeight = HeightMechanic->CurrentHeight;
		bPreviewUpdatePending = true;
		return;
	}

	FVector3d HitPos;
	bool bHit = FindDrawPlaneHitPoint(ClickPos, HitPos);
	if (bHit == false)
	{
		return;
	}

	if (bInFixedPolygonMode)
	{
		UpdatePreviewVertex(HitPos);
		bPreviewUpdatePending = true;
		return;
	}

	UpdatePreviewVertex(HitPos);
	UpdateSelfIntersection();
	if (PolygonVertices.Num() > 2)
	{
		bPreviewUpdatePending = true;
	}
}

bool UDrawPolygonTool::OnNextSequenceClick(const FInputDeviceRay& ClickPos)
{
	if (bInInteractiveExtrude)
	{
		EndInteractiveExtrude();
		return false;
	}

	FVector3d HitPos;
	bool bHit = FindDrawPlaneHitPoint(ClickPos, HitPos);
	if (bHit == false)
	{
		return true;  // ignore click but continue accepting clicks
	}

	// Construct the change now for the undo queue so it reflects the current state.  We might not do anything, in which
	// case we will not emit the change
	TUniquePtr<FDrawPolygonStateChange> Change =
		MakeUnique<FDrawPolygonStateChange>(CurrentCurveTimestamp, FixedPolygonClickPoints, PolygonVertices);

	bool bDonePolygon;
	if (bInFixedPolygonMode)
	{
		// ignore very close click points
		if (FixedPolygonClickPoints.Num() > 0 && ToolSceneQueriesUtil::PointSnapQuery(this, FixedPolygonClickPoints[FixedPolygonClickPoints.Num()-1], HitPos) )
		{
			return true;
		}

		FixedPolygonClickPoints.Add(HitPos);
		int NumTargetPoints = (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Rectangle || PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle) ? 3 : 2;
		bDonePolygon = (FixedPolygonClickPoints.Num() == NumTargetPoints);
		if (bDonePolygon)
		{
			GenerateFixedPolygon(FixedPolygonClickPoints, PolygonVertices, PolygonHolesVertices);
		}
	}
	else
	{
		// ignore very close click points
		if (PolygonVertices.Num() > 0 && ToolSceneQueriesUtil::PointSnapQuery(this, PolygonVertices[PolygonVertices.Num()-1], HitPos))
		{
			return true;
		}

		if (bHaveSelfIntersection)
		{
			// If the self-intersection point is coincident with a polygon vertex, don't add that point twice (it would produce a degenerate polygon edge)
			if (SelfIntersectSegmentIdx < PolygonVertices.Num()-1 && FVector3d::PointsAreSame(SelfIntersectionPoint, PolygonVertices[SelfIntersectSegmentIdx+1]))
			{
				++SelfIntersectSegmentIdx;
			}

			// discard vertex in segments before intersection (this is redundant if idx is 0)
			for (int j = SelfIntersectSegmentIdx; j < PolygonVertices.Num(); ++j)
			{
				PolygonVertices[j-SelfIntersectSegmentIdx] = PolygonVertices[j];
			}
			PolygonVertices.SetNum(PolygonVertices.Num() - SelfIntersectSegmentIdx);
			PolygonVertices[0] = PreviewVertex = SelfIntersectionPoint;
			bDonePolygon = true;
		}
		else
		{
			// close polygon if we clicked on start point
			bDonePolygon = SnapEngine.HaveActiveSnap() && SnapEngine.GetActiveSnapTargetID() == StartPointSnapID;
		}
	}

	// emit change event
	GetToolManager()->EmitObjectChange(this, MoveTemp(Change), LOCTEXT("DrawPolyAddPoint", "Add Point"));

	if (bDonePolygon)
	{
		//SnapEngine.Reset();
		bHaveSurfaceHit = false;
		if (PolygonProperties->ExtrudeMode == EDrawPolygonExtrudeMode::Interactive)
		{
			BeginInteractiveExtrude();

			PreviewMesh->ClearPreview();
			PreviewMesh->SetVisible(true);

			return true;
		}
		else 
		{
			EmitCurrentPolygon();

			PreviewMesh->ClearPreview();
			PreviewMesh->SetVisible(false);

			return false;
		}
	}

	AppendVertex(HitPos);

	// if we are starting a freehand poly, add start point as snap target.
	// Note that logic in FindDrawPlaneHitPoint will ignore it until we get 3 verts
	if (bInFixedPolygonMode == false && PolygonVertices.Num() == 1)
	{
		SnapEngine.AddPointTarget(PolygonVertices[0], StartPointSnapID, 1);
	}

	UpdatePreviewVertex(HitPos);
	return true;
}

void UDrawPolygonTool::OnTerminateClickSequence()
{
	ResetPolygon();
}

bool UDrawPolygonTool::RequestAbortClickSequence()
{
	if (bAbortActivePolygonDraw)
	{
		bAbortActivePolygonDraw = false;
		return true;
	}
	return false;
}

void UDrawPolygonTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == IgnoreSnappingModifier)
	{
		bIgnoreSnappingToggle = bIsOn;
	}
	else if (ModifierID == AngleSnapModifier)
	{

	}
}

bool UDrawPolygonTool::UpdateSelfIntersection()
{
	bHaveSelfIntersection = false;
	if (bInFixedPolygonMode || PolygonProperties->bAllowSelfIntersections == true)
	{
		return false;
	}

	int NumVertices = PolygonVertices.Num();
	if (NumVertices < 3)
	{
		return false;
	}

	const FFrame3d& DrawFrame = PlaneMechanic->Plane;
	FSegment2d PreviewSegment(DrawFrame.ToPlaneUV(PolygonVertices[NumVertices - 1],2), DrawFrame.ToPlaneUV(PreviewVertex,2));

	double BestIntersectionParameter = FMathd::MaxReal;
	for (int k = 0; k < NumVertices - 2; ++k) 
	{
		FSegment2d Segment(DrawFrame.ToPlaneUV(PolygonVertices[k],2), DrawFrame.ToPlaneUV(PolygonVertices[k + 1],2));
		FIntrSegment2Segment2d Intersection(PreviewSegment, Segment);
		if (Intersection.Find()) 
		{
			bHaveSelfIntersection = true;
			if (Intersection.Parameter0 < BestIntersectionParameter)
			{
				BestIntersectionParameter = Intersection.Parameter0;
				SelfIntersectSegmentIdx = k;
				SelfIntersectionPoint = DrawFrame.FromPlaneUV(Intersection.Point0, 2);
			}
		}
	}
	return bHaveSelfIntersection;
}

void UDrawPolygonTool::GetPolygonParametersFromFixedPoints(const TArray<FVector3d>& FixedPoints, FVector2d& FirstReferencePt, FVector2d& BoxSize, double& YSign, double& AngleRad)
{
	if (FixedPoints.Num() < 2)
	{
		return;
	}

	const FFrame3d& DrawFrame = PlaneMechanic->Plane;
	FirstReferencePt = DrawFrame.ToPlaneUV(FixedPoints[0], 2);

	FVector2d EdgePt = DrawFrame.ToPlaneUV(FixedPoints[1], 2);
	FVector2d Delta = EdgePt - FirstReferencePt;
	AngleRad = FMathd::Atan2(Delta.Y, Delta.X);

	double Radius = Delta.Length();
	FVector2d AxisX = Radius != 0 ? Delta / Radius
		: FVector2d(1,0); // arbitrary if delta was 0 vector
	FVector2d AxisY = -UE::Geometry::PerpCW(AxisX);
	FVector2d HeightPt = DrawFrame.ToPlaneUV((FixedPoints.Num() == 3) ? FixedPoints[2] : FixedPoints[1], 2);
	FVector2d HeightDelta = HeightPt - FirstReferencePt;
	YSign = FMathd::Sign(HeightDelta.Dot(AxisY));
	BoxSize.X = Radius;
	BoxSize.Y = FMathd::Abs(HeightDelta.Dot(AxisY));
}

void UDrawPolygonTool::GenerateFixedPolygon(const TArray<FVector3d>& FixedPoints, TArray<FVector3d>& VerticesOut, TArray<TArray<FVector3d>>& HolesVerticesOut)
{
	FVector2d FirstReferencePt, BoxSize;
	double YSign, AngleRad;
	GetPolygonParametersFromFixedPoints(FixedPoints, FirstReferencePt, BoxSize, YSign, AngleRad);
	double Width = BoxSize.X, Height = BoxSize.Y;
	FMatrix2d RotationMat = FMatrix2d::RotationRad(AngleRad);

	FPolygon2d Polygon;
	TArray<FPolygon2d> PolygonHoles;
	if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Square)
	{
		Polygon = FPolygon2d::MakeRectangle(FVector2d::Zero(), 2*Width, 2*Width);
	}
	else if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Rectangle || PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle)
	{
		if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Rectangle)
		{
			Polygon = FPolygon2d::MakeRectangle(FVector2d(Width / 2, YSign*Height / 2), Width, Height);
		}
		else // PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle
		{
			Polygon = FPolygon2d::MakeRoundedRectangle(FVector2d(Width / 2, YSign*Height / 2), Width, Height, FMathd::Min(Width,Height) * FMathd::Clamp(PolygonProperties->FeatureSizeRatio, .01, .99) * .5, PolygonProperties->RadialSlices);
		}
	}
	else // Circle or Ring
	{
		Polygon = FPolygon2d::MakeCircle(Width, PolygonProperties->RadialSlices, 0);
		if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Ring)
		{
			PolygonHoles.Add(FPolygon2d::MakeCircle(Width * FMathd::Clamp(PolygonProperties->FeatureSizeRatio, .01, .99), PolygonProperties->RadialSlices, 0));
		}
	}
	Polygon.Transform([RotationMat](const FVector2d& Pt) { return RotationMat * Pt; });
	for (FPolygon2d& Hole : PolygonHoles)
	{
		Hole.Transform([RotationMat](const FVector2d& Pt) { return RotationMat * Pt; });
	}

	const FFrame3d& DrawFrame = PlaneMechanic->Plane;
	VerticesOut.SetNum(Polygon.VertexCount());
	for (int k = 0; k < Polygon.VertexCount(); ++k)
	{
		FVector2d NewPt = FirstReferencePt + Polygon[k];
		VerticesOut[k] = DrawFrame.FromPlaneUV(NewPt, 2);
	}

	HolesVerticesOut.SetNum(PolygonHoles.Num());
	for (int HoleIdx = 0; HoleIdx < PolygonHoles.Num(); HoleIdx++)
	{
		int NumHoleVerts = PolygonHoles[HoleIdx].VertexCount();
		HolesVerticesOut[HoleIdx].SetNum(NumHoleVerts);
		for (int k = 0; k < NumHoleVerts; ++k)
		{
			FVector2d NewPt = FirstReferencePt + PolygonHoles[HoleIdx][k];
			HolesVerticesOut[HoleIdx][k] = DrawFrame.FromPlaneUV(NewPt, 2);
		}
	}
}


void UDrawPolygonTool::BeginInteractiveExtrude()
{
	bInInteractiveExtrude = true;

	bHasSavedExtrudeHeight = true;
	SavedExtrudeHeight = PolygonProperties->ExtrudeHeight;

	SnapEngine.ResetActiveSnap();

	HeightMechanic = NewObject<UPlaneDistanceFromHitMechanic>(this);
	HeightMechanic->Setup(this);

	HeightMechanic->WorldHitQueryFunc = [this](const FRay& WorldRay, FHitResult& HitResult)
	{
		if (this->bIgnoreSnappingToggle == false)
		{
			return ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, HitResult, WorldRay);
		}
		return false;
	};
	HeightMechanic->WorldPointSnapFunc = [this](const FVector3d& WorldPos, FVector3d& SnapPos)
	{
		if (bIgnoreSnappingToggle == false && SnapProperties->bEnableSnapping)
		{
			return ToolSceneQueriesUtil::FindWorldGridSnapPoint(this, WorldPos, SnapPos);
		}
		return false;
	};
	HeightMechanic->CurrentHeight = 1.0f;  // initialize to something non-zero...prob should be based on polygon bounds maybe?

	FDynamicMesh3 HeightMesh;
	FFrame3d WorldMeshFrame;
	GeneratePolygonMesh(PolygonVertices, PolygonHolesVertices, &HeightMesh, WorldMeshFrame, false, 99999, true);
	HeightMechanic->Initialize( MoveTemp(HeightMesh), WorldMeshFrame, false);

	ShowExtrudeMessage();
}

void UDrawPolygonTool::EndInteractiveExtrude()
{
	EmitCurrentPolygon();

	PreviewMesh->ClearPreview();
	PreviewMesh->SetVisible(false);

	bInInteractiveExtrude = false;
	HeightMechanic = nullptr;

	ShowStartupMessage();
}


bool UDrawPolygonTool::AllowDrawPlaneUpdates()
{
	if (bInInteractiveExtrude)
	{
		return false;
	}
	if (bInFixedPolygonMode)
	{
		return FixedPolygonClickPoints.IsEmpty();
	}
	else
	{
		return PolygonVertices.IsEmpty();
	}
}


void UDrawPolygonTool::EmitCurrentPolygon()
{
	FString BaseName = (PolygonProperties->ExtrudeMode == EDrawPolygonExtrudeMode::Flat) ?
		TEXT("Polygon") : TEXT("Extrude");

	// generate new mesh
	FFrame3d PlaneFrameOut;
	FDynamicMesh3 Mesh;
	const double ExtrudeDist = (PolygonProperties->ExtrudeMode == EDrawPolygonExtrudeMode::Flat) ?
		0 : PolygonProperties->ExtrudeHeight;
	bool bSucceeded = GeneratePolygonMesh(PolygonVertices, PolygonHolesVertices, &Mesh, PlaneFrameOut, false, ExtrudeDist, false);
	if (!bSucceeded) // somehow made a polygon with no valid triangulation; just throw it away ...
	{
		ResetPolygon();
		return;
	}
	UE::Geometry::FMeshTangentsf::ComputeDefaultOverlayTangents(Mesh);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("CreatePolygon", "Create Polygon"));

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = TargetWorld;
	NewMeshObjectParams.Transform = PlaneFrameOut.ToFTransform();
	NewMeshObjectParams.BaseName = BaseName;
	NewMeshObjectParams.Materials.Add(MaterialProperties->Material.Get());
	NewMeshObjectParams.SetMesh(&Mesh);
	OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		bRestoreInputSelection = false;
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}

	GetToolManager()->EndUndoTransaction();

	if (bHasSavedExtrudeHeight)
	{
		PolygonProperties->ExtrudeHeight = SavedExtrudeHeight;
		bHasSavedExtrudeHeight = false;
	}

	ResetPolygon();
}

void UDrawPolygonTool::UpdateLivePreview()
{
	int NumVerts = PolygonVertices.Num();
	if (NumVerts < 2 || PreviewMesh == nullptr || PreviewMesh->IsVisible() == false)
	{
		return;
	}

	FFrame3d PlaneFrame;
	FDynamicMesh3 Mesh;
	const double ExtrudeDist = (PolygonProperties->ExtrudeMode == EDrawPolygonExtrudeMode::Flat) ?
		0 : PolygonProperties->ExtrudeHeight;
	if (GeneratePolygonMesh(PolygonVertices, PolygonHolesVertices, &Mesh, PlaneFrame, false, ExtrudeDist, false))
	{
		PreviewMesh->SetTransform(PlaneFrame.ToFTransform());
		PreviewMesh->SetMaterial(MaterialProperties->Material.Get());
		PreviewMesh->EnableWireframe(MaterialProperties->bShowWireframe);
		PreviewMesh->UpdatePreview(&Mesh);
	}
}

bool UDrawPolygonTool::GeneratePolygonMesh(const TArray<FVector3d>& Polygon, const TArray<TArray<FVector3d>>& PolygonHoles, FDynamicMesh3* ResultMeshOut, FFrame3d& WorldFrameOut, bool bIncludePreviewVtx, double ExtrudeDistance, bool bExtrudeSymmetric)
{
	// construct centered frame for polygon
	WorldFrameOut = PlaneMechanic->Plane;

	int NumVerts = Polygon.Num();
	FVector3d Centroid3d(0, 0, 0);
	for (int k = 0; k < NumVerts; ++k)
	{
		Centroid3d += Polygon[k];
	}
	Centroid3d /= (double)NumVerts;
	FVector2d CentroidInDrawPlane = WorldFrameOut.ToPlaneUV(Centroid3d);
	WorldFrameOut.Origin = Centroid3d;

	// Compute outer polygon & bounds
	auto VertexArrayToPolygon = [&WorldFrameOut](const TArray<FVector3d>& Vertices)
	{
		FPolygon2d OutPolygon;
		for (int k = 0, N = Vertices.Num(); k < N; ++k)
		{
			OutPolygon.AppendVertex(WorldFrameOut.ToPlaneUV(Vertices[k], 2));
		}
		return OutPolygon;
	};
	FPolygon2d OuterPolygon = VertexArrayToPolygon(Polygon);
	// add preview vertex
	if (bIncludePreviewVtx)
	{
		if (Distance(PreviewVertex, Polygon[NumVerts-1]) > 0.1)
		{
			OuterPolygon.AppendVertex(WorldFrameOut.ToPlaneUV(PreviewVertex, 2));
		}
	}
	FAxisAlignedBox2d Bounds(OuterPolygon.Bounds());

	// special case paths
	if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Ring || PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Circle || PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle)
	{
		// get polygon parameters
		FVector2d FirstReferencePt, BoxSize;
		double YSign, AngleRad;
		GetPolygonParametersFromFixedPoints(FixedPolygonClickPoints, FirstReferencePt, BoxSize, YSign, AngleRad);
		FirstReferencePt -= CentroidInDrawPlane;
		FMatrix2d RotationMat = FMatrix2d::RotationRad(AngleRad);

		// translate general polygon parameters to specific mesh generator parameters, and generate mesh
		if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Ring)
		{
			FPuncturedDiscMeshGenerator HCGen;
			HCGen.AngleSamples = PolygonProperties->RadialSlices;
			HCGen.RadialSamples = 1;
			HCGen.Radius = BoxSize.X;
			HCGen.HoleRadius = BoxSize.X * FMathd::Clamp(PolygonProperties->FeatureSizeRatio, .01f, .99f);
			ResultMeshOut->Copy(&HCGen.Generate());
		}
		else if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::Circle)
		{
			FDiscMeshGenerator CGen;
			CGen.AngleSamples = PolygonProperties->RadialSlices;
			CGen.RadialSamples = 1;
			CGen.Radius = BoxSize.X;
			ResultMeshOut->Copy(&CGen.Generate());
		}
		else if (PolygonProperties->PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle)
		{
			FRoundedRectangleMeshGenerator RRGen;
			FirstReferencePt += RotationMat * (FVector2d(BoxSize.X, BoxSize.Y * YSign)*.5f);
			RRGen.AngleSamples = PolygonProperties->RadialSlices;
			RRGen.Radius = .5 * FMathd::Min(BoxSize.X, BoxSize.Y) * FMathd::Clamp(PolygonProperties->FeatureSizeRatio, .01f, .99f);
			RRGen.Height = BoxSize.Y - RRGen.Radius * 2.;
			RRGen.Width = BoxSize.X - RRGen.Radius * 2.;
			RRGen.WidthVertexCount = 1;
			RRGen.HeightVertexCount = 1;
			ResultMeshOut->Copy(&RRGen.Generate());
		}

		// transform generated mesh
		for (int VertIdx : ResultMeshOut->VertexIndicesItr())
		{
			FVector3d V = ResultMeshOut->GetVertex(VertIdx);
			FVector2d VTransformed = RotationMat * FVector2d(V.X, V.Y) + FirstReferencePt;
			ResultMeshOut->SetVertex(VertIdx, FVector3d(VTransformed.X, VTransformed.Y, 0));
		}
	}
	else // generic path: triangulate using polygon vertices
	{
		// triangulate polygon into the MeshDescription
		FGeneralPolygon2d GeneralPolygon;
		FFlatTriangulationMeshGenerator TriangulationMeshGen;

		if (OuterPolygon.IsClockwise() == false)
		{
			OuterPolygon.Reverse();
		}

		GeneralPolygon.SetOuter(OuterPolygon);

		for (int HoleIdx = 0; HoleIdx < PolygonHoles.Num(); HoleIdx++)
		{
			// attempt to add holes (skipping if safety checks fail)
			GeneralPolygon.AddHole(VertexArrayToPolygon(PolygonHoles[HoleIdx]), true, false /*currently don't care about hole orientation; we'll just set the triangulation algo not to care*/);
		}

		FConstrainedDelaunay2d Triangulator;
		if (PolygonProperties->bAllowSelfIntersections)
		{
			FArrangement2d Arrangement(OuterPolygon.Bounds());
			// arrangement2d builds a general 2d graph that discards orientation info ...
			Triangulator.FillRule = FConstrainedDelaunay2d::EFillRule::Odd;
			Triangulator.bOrientedEdges = false;
			Triangulator.bSplitBowties = true;
			for (FSegment2d Seg : GeneralPolygon.GetOuter().Segments())
			{
				Arrangement.Insert(Seg);
			}
			Triangulator.Add(Arrangement.Graph);
			for (const FPolygon2d& Hole : GeneralPolygon.GetHoles())
			{
				Triangulator.Add(Hole, true);
			}
		}
		else
		{
			Triangulator.Add(GeneralPolygon);
		}


		Triangulator.Triangulate([&GeneralPolygon](const TArray<FVector2d>& Vertices, FIndex3i Tri)
		{
			// keep triangles based on the input polygon's winding
			return GeneralPolygon.Contains((Vertices[Tri.A] + Vertices[Tri.B] + Vertices[Tri.C]) / 3.0);
		});
		// only truly fail if we got zero triangles back from the triangulator; if it just returned false it may still have managed to partially generate something
		if (Triangulator.Triangles.Num() == 0)
		{
			return false;
		}

		TriangulationMeshGen.Vertices2D = Triangulator.Vertices;
		TriangulationMeshGen.Triangles2D = Triangulator.Triangles;

		ResultMeshOut->Copy(&TriangulationMeshGen.Generate());
	}

	// for symmetric extrude we translate the first poly by -dist along axis
	if (bExtrudeSymmetric)
	{
		FVector3d ShiftNormal = FVector3d::UnitZ();
		for (int vid : ResultMeshOut->VertexIndicesItr())
		{
			FVector3d Pos = ResultMeshOut->GetVertex(vid);
			ResultMeshOut->SetVertex(vid, Pos - ExtrudeDistance * ShiftNormal);
		}
		// double extrude dist
		ExtrudeDistance *= 2.0;
	}

	if (ExtrudeDistance != 0)
	{
		FExtrudeMesh Extruder(ResultMeshOut);
		Extruder.DefaultExtrudeDistance = ExtrudeDistance;
		
		Extruder.UVScaleFactor = 1.0 / Bounds.MaxDim();
		if (ExtrudeDistance < 0)
		{
			Extruder.IsPositiveOffset = false;
		}

		FVector3d ExtrudeNormal = FVector3d::UnitZ();
		Extruder.ExtrudedPositionFunc = [&ExtrudeDistance, &ExtrudeNormal](const FVector3d& Position, const FVector3f& Normal, int VertexID)
		{
			return Position + ExtrudeDistance * (FVector3d)ExtrudeNormal;
		};

		Extruder.Apply();
	}

	FDynamicMeshEditor Editor(ResultMeshOut);
	float InitialUVScale = 1.0 / Bounds.MaxDim(); // this is the UV scale used by both the polymeshgen and the extruder above
	// default global rescale -- initial scale doesn't factor in extrude distance; rescale so UVScale of 1.0 fits in the unit square texture
	float GlobalUVRescale = MaterialProperties->UVScale / FMathf::Max(1.0f, FMathd::Abs(ExtrudeDistance) * InitialUVScale);
	if (MaterialProperties->bWorldSpaceUVScale)
	{
		// since we know the initial uv scale, directly compute the global scale (relative to 1 meter as a standard scale)
		GlobalUVRescale = MaterialProperties->UVScale * .01 / InitialUVScale;
	}
	Editor.RescaleAttributeUVs(GlobalUVRescale, false);

	return true;
}




void UDrawPolygonTool::ShowStartupMessage()
{
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartDraw", "Draw a polygon on the drawing plane, and extrude it. Left-click to place polygon vertices. Hold Shift to ignore snapping while drawing."),
		EToolMessageLevel::UserNotification);
}

void UDrawPolygonTool::ShowExtrudeMessage()
{
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartExtrude", "Set the height of the extrusion by positioning the mouse over the extrusion volume, or over objects to snap to their heights. Hold Shift to ignore snapping."),
		EToolMessageLevel::UserNotification);
}




void UDrawPolygonTool::UndoCurrentOperation(const TArray<FVector3d>& ClickPointsIn, const TArray<FVector3d>& PolygonVerticesIn)
{
	if (bInInteractiveExtrude)
	{
		PreviewMesh->ClearPreview();
		PreviewMesh->SetVisible(false);
		bInInteractiveExtrude = false;
	}
	ApplyUndoPoints(ClickPointsIn, PolygonVerticesIn);
}


void FDrawPolygonStateChange::Revert(UObject* Object)
{
	Cast<UDrawPolygonTool>(Object)->UndoCurrentOperation( FixedVertexPoints, PolyPoints );
	bHaveDoneUndo = true;
}
bool FDrawPolygonStateChange::HasExpired(UObject* Object) const
{
	return bHaveDoneUndo || (Cast<UDrawPolygonTool>(Object)->CheckInCurve(CurveTimestamp) == false);
}
FString FDrawPolygonStateChange::ToString() const
{
	return TEXT("FDrawPolygonStateChange");
}



#undef LOCTEXT_NAMESPACE

