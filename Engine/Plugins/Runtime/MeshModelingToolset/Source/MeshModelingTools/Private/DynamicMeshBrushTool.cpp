// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshBrushTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshBrushTool)

using namespace UE::Geometry;

// localization namespace
#define LOCTEXT_NAMESPACE "UDynamicMeshBrushTool"

/*
 * Tool
 */

UDynamicMeshBrushTool::UDynamicMeshBrushTool()
{
}


void UDynamicMeshBrushTool::Setup()
{
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = true;
	PreviewMesh->CreateInWorld(GetTargetWorld(), FTransform::Identity);
	BakedTargetTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	CurrentTargetTransform = BakedTargetTransform;
	PreviewMesh->SetTransform((FTransform)BakedTargetTransform);

	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);

	PreviewMesh->ReplaceMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));

	if (bBakeRotateScale)
	{
		if (UDynamicMeshComponent* MeshComponent = Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent()))
		{
			// clamp scaling because if we allow zero-scale we cannot invert this transform on Accept
			BakedTargetTransform.ClampMinimumScale(0.01);
			FVector3d Translation = BakedTargetTransform.GetTranslation();
			BakedTargetTransform.SetTranslation(FVector3d::Zero());
			MeshComponent->ApplyTransform(BakedTargetTransform, false);
			CurrentTargetTransform = FTransformSRT3d(Translation);
			PreviewMesh->SetTransform(CurrentTargetTransform);
		}
	}

	OnBaseMeshComponentChangedHandle = PreviewMesh->GetOnMeshChanged().Add(
		FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UDynamicMeshBrushTool::OnBaseMeshComponentChanged));

	// call this here so that base tool can estimate target dimension
	InputMeshBoundsLocal = PreviewMesh->GetPreviewDynamicMesh()->GetBounds();
	double ScaledDim = CurrentTargetTransform.TransformVector(FVector::OneVector).Size();
	this->WorldToLocalScale = FMathd::Sqrt3 / FMathd::Max(FMathf::ZeroTolerance, ScaledDim);
	UBaseBrushTool::Setup();

	UE::ToolTarget::HideSourceObject(Target);
}



double UDynamicMeshBrushTool::EstimateMaximumTargetDimension()
{
	return InputMeshBoundsLocal.MaxDim();
}


void UDynamicMeshBrushTool::Shutdown(EToolShutdownType ShutdownType)
{
	UBaseBrushTool::Shutdown(ShutdownType);

	LongTransactions.CloseAll(GetToolManager());

	UE::ToolTarget::ShowSourceObject(Target);

	if (PreviewMesh != nullptr)
	{
		PreviewMesh->GetOnMeshChanged().Remove(OnBaseMeshComponentChangedHandle);

		OnShutdown(ShutdownType);

		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

}




bool UDynamicMeshBrushTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	return PreviewMesh->FindRayIntersection(FRay3d(Ray), OutHit);
}






#undef LOCTEXT_NAMESPACE

