// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshSculptTool.h"
#include "Containers/Map.h"
#include "Async/Async.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "ToolBuilderUtil.h"

#include "SubRegionRemesher.h"
#include "ProjectionTargets.h"
#include "MeshConstraints.h"
#include "MeshConstraintsUtil.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "Drawing/MeshDebugDrawing.h"
#include "PreviewMesh.h"
#include "ToolSetupUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "ModelingToolTargetUtil.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"

#include "ToolDataVisualizer.h"
#include "Components/PrimitiveComponent.h"
#include "Generators/SphereGenerator.h"

#include "Sculpting/KelvinletBrushOp.h"

#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "UObject/ObjectMacros.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshSculptTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDynamicMeshSculptTool"

namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution DynamicSculptToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution DynamicSculptToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}



/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UDynamicMeshSculptToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UDynamicMeshSculptTool* SculptTool = NewObject<UDynamicMeshSculptTool>(SceneState.ToolManager);
	SculptTool->SetEnableRemeshing(this->bEnableRemeshing);
	SculptTool->SetWorld(SceneState.World);
	return SculptTool;
}

const FToolTargetTypeRequirements& UDynamicMeshSculptToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
	});
	return TypeRequirements;
}


void UDynamicSculptToolActions::DiscardAttributes()
{
	ParentTool->DiscardAttributes();
}



/*
 * Tool
 */
UDynamicMeshSculptTool::UDynamicMeshSculptTool()
{
	// initialize parameters
	bEnableRemeshing = true;
}

void UDynamicMeshSculptTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

namespace
{
const FString BrushIndicatorGizmoType = TEXT("BrushIndicatorGizmoType");
}

void UDynamicMeshSculptTool::Setup()
{
	UMeshSurfacePointTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "DynaSculpt"));

	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
	DynamicMeshComponent = NewObject<UOctreeDynamicMeshComponent>(PreviewMeshActor);
	DynamicMeshComponent->SetShadowsEnabled(false);
	DynamicMeshComponent->SetupAttachment(PreviewMeshActor->GetRootComponent());
	DynamicMeshComponent->SetWorldTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	DynamicMeshComponent->RegisterComponent();
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(DynamicMeshComponent, Target); 

	// initialize from LOD-0 MeshDescription
	DynamicMeshComponent->SetMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));

	// transform mesh to world space because handling scaling inside brush is a mess
	// Note: this transform does not include translation ( so only the 3x3 transform)
	InitialTargetTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	// clamp scaling because if we allow zero-scale we cannot invert this transform on Accept
	InitialTargetTransform.ClampMinimumScale(0.01);
	FVector3d Translation = InitialTargetTransform.GetTranslation();
	InitialTargetTransform.SetTranslation(FVector3d::Zero());
	DynamicMeshComponent->ApplyTransform(InitialTargetTransform, false);
	// since we moved to World coords there is not a current transform anymore.
	CurTargetTransform = FTransformSRT3d(Translation);
	DynamicMeshComponent->SetWorldTransform((FTransform)CurTargetTransform);

	// copy material if there is one
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	if (MaterialSet.Materials.Num() > 0)
	{
		DynamicMeshComponent->SetMaterial(0, MaterialSet.Materials[0]);
	}

	OnDynamicMeshComponentChangedHandle = DynamicMeshComponent->OnMeshChanged.Add(
		FSimpleMulticastDelegate::FDelegate::CreateUObject(this, &UDynamicMeshSculptTool::OnDynamicMeshComponentChanged));

	// do we always want to keep vertex normals updated? Perhaps should discard vertex normals before baking?
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FMeshNormals::QuickComputeVertexNormals(*Mesh);

	// switch to vertex normals for testing
	//DynamicMeshComponent->GetMesh()->DiscardAttributes();

	// initialize target mesh
	UpdateTarget();
	bTargetDirty = false;
	PendingTargetUpdate.Wait();

	// initialize brush radius range interval, brush properties
	FAxisAlignedBox3d Bounds = DynamicMeshComponent->GetMesh()->GetBounds(true);
	double MaxDimension = Bounds.MaxDim();
	BrushRelativeSizeRange = FInterval1d(MaxDimension*0.01, MaxDimension);
	BrushProperties = NewObject<UDynamicMeshBrushProperties>(this);
	BrushProperties->BrushSize.InitializeWorldSizeRange(
		TInterval<float>((float)BrushRelativeSizeRange.Min, (float)BrushRelativeSizeRange.Max));
	CalculateBrushRadius();

	// initialize other properties
	SculptProperties = NewObject<UDynamicMeshBrushSculptProperties>(this);
	KelvinBrushProperties = NewObject<UKelvinBrushProperties>(this);

	RemeshProperties = NewObject<UBrushRemeshProperties>(this);
	RemeshProperties->RestoreProperties(this);

	InitialEdgeLength = EstimateIntialSafeTargetLength(*Mesh, 5000);

	// hide input Component
	UE::ToolTarget::HideSourceObject(Target);

	// init state flags flags
	bInDrag = false;
	bHaveRemeshed = false;
	bRemeshPending = false;
	bStampPending = false;
	ActiveVertexChange = nullptr;

	// register and spawn brush indicator gizmo
	GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(BrushIndicatorGizmoType, NewObject<UBrushStampIndicatorBuilder>());
	BrushIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UBrushStampIndicator>(BrushIndicatorGizmoType, FString(), this);
	BrushIndicatorMesh = MakeDefaultSphereMesh(this, TargetWorld);
	BrushIndicator->AttachedComponent = BrushIndicatorMesh->GetRootComponent();
	BrushIndicator->LineThickness = 1.0;
	BrushIndicator->bDrawIndicatorLines = true;
	BrushIndicator->bDrawRadiusCircle = false;
	BrushIndicator->LineColor = FLinearColor(0.9f, 0.4f, 0.4f);

	// initialize our properties
	AddToolPropertySource(BrushProperties);
	AddToolPropertySource(SculptProperties);

	// add brush-specific properties 
	SculptMaxBrushProperties = NewObject<USculptMaxBrushProperties>();
	SculptMaxBrushProperties->RestoreProperties(this);
	AddToolPropertySource(SculptMaxBrushProperties);

	AddToolPropertySource(KelvinBrushProperties);
	KelvinBrushProperties->RestoreProperties(this);

	GizmoProperties = NewObject<UFixedPlaneBrushProperties>();
	GizmoProperties->RestoreProperties(this);
	AddToolPropertySource(GizmoProperties);
	GizmoProperties->RecenterGizmoIfFar(CurTargetTransform.TransformPosition(Bounds.Center()), Bounds.MaxDim());

	if (this->bEnableRemeshing)
	{
		SculptProperties->bIsRemeshingEnabled = true;
		AddToolPropertySource(RemeshProperties);

		SculptToolActions = NewObject<UDynamicSculptToolActions>();
		SculptToolActions->Initialize(this);
		AddToolPropertySource(SculptToolActions);
	}

	BrushProperties->RestoreProperties(this);
	CalculateBrushRadius();
	SculptProperties->RestoreProperties(this);

	// disable tool-specific properties
	SetToolPropertySourceEnabled(GizmoProperties, false);
	SetToolPropertySourceEnabled(SculptMaxBrushProperties, false);
	SetToolPropertySourceEnabled(KelvinBrushProperties, false);

	ViewProperties = NewObject<UMeshEditingViewProperties>();
	ViewProperties->RestoreProperties(this);
	AddToolPropertySource(ViewProperties);

	// register watchers
	ShowWireframeWatcher.Initialize(
		[this]() { return ViewProperties->bShowWireframe; },
		[this](bool bNewValue) { DynamicMeshComponent->bExplicitShowWireframe = bNewValue; }, ViewProperties->bShowWireframe);
	MaterialModeWatcher.Initialize(
		[this]() { return ViewProperties->MaterialMode; },
		[this](EMeshEditingMaterialModes NewMode) { UpdateMaterialMode(NewMode); }, EMeshEditingMaterialModes::ExistingMaterial);
	CustomMaterialWatcher.Initialize( 
		[this]() { return ViewProperties->CustomMaterial; },
		[this](TWeakObjectPtr<UMaterialInterface> NewMaterial) { UpdateCustomMaterial(NewMaterial); }, ViewProperties->CustomMaterial);
	FlatShadingWatcher.Initialize(
		[this]() { return ViewProperties->bFlatShading; },
		[this](bool bNewValue) { UpdateFlatShadingSetting(bNewValue); }, ViewProperties->bFlatShading);
	ColorWatcher.Initialize(
		[this]() { return ViewProperties->Color; },
		[this](FLinearColor NewColor) { UpdateColorSetting(NewColor); }, ViewProperties->Color);
	ImageWatcher.Initialize(
		[this]() { return ViewProperties->Image; },
		[this](UTexture2D* NewImage) { UpdateImageSetting(NewImage); }, ViewProperties->Image);
	TransparentColorWatcher.Initialize(
		[this]() { return ViewProperties->TransparentMaterialColor; },
		[this](FLinearColor NewColor) { UpdateColorSetting(NewColor); }, ViewProperties->TransparentMaterialColor);
	OpacityWatcher.Initialize(
		[this]() { return ViewProperties->Opacity; },
		[this](double Opacity) { UpdateOpacitySetting(Opacity); }, ViewProperties->Opacity);
	TwoSidedWatcher.Initialize(
		[this]() { return ViewProperties->bTwoSided; },
		[this](bool bOn) { UpdateTwoSidedSetting(bOn); }, ViewProperties->bTwoSided);
	BrushTypeWatcher.Initialize(
		[this]() { return SculptProperties->PrimaryBrushType; },
		[this](EDynamicMeshSculptBrushType NewBrushType) { UpdateBrushType(NewBrushType); }, SculptProperties->PrimaryBrushType);
	GizmoPositionWatcher.Initialize(
		[this]() { return GizmoProperties->Position; },
		[this](FVector NewPosition) { UpdateGizmoFromProperties(); }, GizmoProperties->Position);
	GizmoRotationWatcher.Initialize(
		[this]() { return GizmoProperties->Rotation; },
		[this](FQuat NewRotation) { UpdateGizmoFromProperties(); }, GizmoProperties->Rotation);

	DynamicMeshComponent->bExplicitShowWireframe = ViewProperties->bShowWireframe;

	// create proxy for plane gizmo, but not gizmo itself, as it only appears in FixedPlane brush mode
	// listen for changes to the proxy and update the plane when that happens
	PlaneTransformProxy = NewObject<UTransformProxy>(this);
	PlaneTransformProxy->OnTransformChanged.AddUObject(this, &UDynamicMeshSculptTool::PlaneTransformChanged);

	if (bEnableRemeshing)
	{
		PrecomputeRemeshInfo();
		if (bHaveUVSeams)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("UVSeamWarning", "This mesh has UV seams which may limit remeshing. Consider clearing the UV layers using \"Discard Attributes\" or the Remesh Tool."),
				EToolMessageLevel::UserWarning);
		}
		else if (bHaveNormalSeams)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("NormalSeamWarning", "This mesh has Hard Normal seams which may limit remeshing. Consider clearing Hard Normals using \"Discard Attributes,\" or the Remesh or Normals Tool."),
				EToolMessageLevel::UserWarning);
		}
	}

	UpdateBrushType(SculptProperties->PrimaryBrushType);

	// add input behavior to click-drag while holding hotkey to adjust brush size and strength
    {
    	BrushEditBehavior = NewObject<ULocalTwoAxisPropertyEditInputBehavior>(this);
		
    	BrushEditBehavior->HorizontalProperty.GetValueFunc = [this]()
    	{
    		if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
    		{
    			return BrushProperties->BrushSize.AdaptiveSize;
    		}
            else
            {
	            return BrushProperties->BrushSize.WorldRadius;
            }
    	};
    	BrushEditBehavior->HorizontalProperty.SetValueFunc = [this](float NewValue)
    	{
    		if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
    		{
    			BrushProperties->BrushSize.AdaptiveSize = FMath::Clamp(NewValue, 0.f, 1.f);
    		}
    		else
    		{
    			BrushProperties->BrushSize.WorldRadius = FMath::Max(NewValue, 0.01f);
    		}
    		
#if WITH_EDITOR
    		FPropertyChangedEvent PropertyChangedEvent(UDynamicMeshBrushProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDynamicMeshBrushProperties, BrushSize)));
    		BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
    	};
		BrushEditBehavior->HorizontalProperty.MutateDeltaFunc = [this](float Delta)
		{
			// Scale delta if brush size is in world units.
			return Delta * (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::World ? (CameraState.Position - LastBrushPosWorld).Length() : 1.f);
		};
		BrushEditBehavior->HorizontalProperty.Name = LOCTEXT("BrushRadius", "Radius");
		BrushEditBehavior->HorizontalProperty.bEnabled = true;
		
    	BrushEditBehavior->VerticalProperty.GetValueFunc = [this]()
    	{
    		return SculptProperties->PrimaryBrushSpeed;
    	};
    	BrushEditBehavior->VerticalProperty.SetValueFunc = [this](float NewValue)
    	{
    		SculptProperties->PrimaryBrushSpeed = FMath::Clamp(NewValue, 0.f, 1.f);

#if WITH_EDITOR
    		FPropertyChangedEvent PropertyChangedEvent(UDynamicMeshBrushSculptProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDynamicMeshBrushSculptProperties, PrimaryBrushSpeed)));
    		BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
    	};
		BrushEditBehavior->VerticalProperty.Name = LOCTEXT("BrushStrength", "Strength");
		BrushEditBehavior->VerticalProperty.bEnabled = true;


		
    	BrushEditBehavior->OnDragUpdated.AddWeakLambda(this, [this]()
    	{
    		CalculateBrushRadius();
    		NotifyOfPropertyChangeByTool(BrushProperties);
    	});
    	
    	BrushEditBehavior->Initialize();
    	AddInputBehavior(BrushEditBehavior.Get());
    }
}

void UDynamicMeshSculptTool::Shutdown(EToolShutdownType ShutdownType)
{
	LongTransactions.CloseAll(GetToolManager());

	if (ShutdownType == EToolShutdownType::Accept && AreAllTargetsValid() == false)
	{
		UE_LOGF(LogTemp, Error, "Tool Target has become Invalid (possibly it has been Force Deleted). Aborting Tool.");
		ShutdownType = EToolShutdownType::Cancel;
	}

	BrushIndicatorMesh->Disconnect();
	BrushIndicatorMesh = nullptr;

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	BrushIndicator = nullptr;
	GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(BrushIndicatorGizmoType);

	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);

		UE::ToolTarget::ShowSourceObject(Target);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			// safe to do this here because we are about to destroy componeont
			DynamicMeshComponent->ApplyTransform(InitialTargetTransform, true);

			// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
			GetToolManager()->BeginUndoTransaction(LOCTEXT("SculptMeshToolTransactionName", "Sculpt Mesh"));
			DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
			{
				UE::ToolTarget::CommitDynamicMeshUpdate(Target, ReadMesh, bHaveRemeshed);
			});
			GetToolManager()->EndUndoTransaction();
		}

		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->DestroyComponent();
		DynamicMeshComponent = nullptr;
	}

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

	BrushProperties->SaveProperties(this);
	SculptProperties->SaveProperties(this);
	KelvinBrushProperties->SaveProperties(this);
	ViewProperties->SaveProperties(this);
	GizmoProperties->SaveProperties(this);
	SculptMaxBrushProperties->SaveProperties(this);
	RemeshProperties->SaveProperties(this);
}

void UDynamicMeshSculptTool::OnDynamicMeshComponentChanged()
{
	bNormalUpdatePending = true;
	bTargetDirty = true;
}

void UDynamicMeshSculptTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	CalculateBrushRadius();
}


FBox UDynamicMeshSculptTool::GetWorldSpaceFocusBox()
{
	if (LastBrushTriangleID == INDEX_NONE)
	{
		return Super::GetWorldSpaceFocusBox();
	}
	FVector Center = LastBrushPosWorld;
	double Size = CurrentBrushRadius;
	return FBox(Center - FVector(Size), Center + FVector(Size));
}


bool UDynamicMeshSculptTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	FRay3d LocalRay(CurTargetTransform.InverseTransformPosition((FVector3d)Ray.Origin),
		CurTargetTransform.InverseTransformVector((FVector3d)Ray.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();

	int HitTID = FindHitSculptMeshTriangle(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		FTriangle3d Triangle;
		Mesh->GetTriVertices(HitTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();

		OutHit.FaceIndex = HitTID;
		OutHit.Distance = Query.RayParameter;
		OutHit.Normal = (FVector)CurTargetTransform.TransformNormal(Mesh->GetTriNormal(HitTID));
		OutHit.ImpactPoint = (FVector)CurTargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		return true;
	}
	return false;
}

void UDynamicMeshSculptTool::OnBeginDrag(const FRay& Ray)
{
	bSmoothing = GetShiftToggle();
	bInvert = GetCtrlToggle();

	FHitResult OutHit;
	if (HitTest(Ray, OutHit))
	{
		BrushStartCenterWorld = (FVector3d)Ray.PointAt(OutHit.Distance) + (double)BrushProperties->Depth*CurrentBrushRadius*(FVector3d)Ray.Direction;

		bInDrag = true;

		ActiveDragPlane = FFrame3d(BrushStartCenterWorld, -(FVector3d)Ray.Direction);
		ActiveDragPlane.RayPlaneIntersection((FVector3d)Ray.Origin, (FVector3d)Ray.Direction, 2, LastHitPosWorld);

		LastBrushPosWorld = LastHitPosWorld;
		LastBrushPosNormalWorld = ActiveDragPlane.Z();
		LastBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastHitPosWorld);
		LastSmoothBrushPosLocal = LastBrushPosLocal;

		BeginChange(bEnableRemeshing == false);

		UpdateROI(LastBrushPosLocal);

		if (SculptProperties->PrimaryBrushType == EDynamicMeshSculptBrushType::Plane)
		{
			ActiveFixedBrushPlane = ComputeROIBrushPlane(LastBrushPosLocal, false, false);
		}
		else if (SculptProperties->PrimaryBrushType == EDynamicMeshSculptBrushType::PlaneViewAligned)
		{
			AlignBrushToView();
			ActiveFixedBrushPlane = ComputeROIBrushPlane(LastBrushPosLocal, false, true);
		}

		// apply initial stamp
		PendingStampRay = Ray;
		bStampPending = true;
	}
}


void UDynamicMeshSculptTool::UpdateROI(const FVector3d& BrushPos)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateROI);

	float RadiusSqr = CurrentBrushRadius * CurrentBrushRadius;

	FAxisAlignedBox3d BrushBox(
		BrushPos - CurrentBrushRadius * FVector3d::One(),
		BrushPos + CurrentBrushRadius * FVector3d::One());

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshOctree3* Octree = DynamicMeshComponent->GetOctree();

	// find set of triangles in brush bounding box
	UpdateROITriBuffer.Reset();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateROI_1RangeQuery);
		Octree->ParallelRangeQuery(BrushBox, UpdateROITriBuffer);
	}

	// collect set of vertices inside brush sphere, from that box
	VertexROIBuilder.Initialize(Mesh->MaxVertexID());
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateROI_2Collect);
		for (int32 TriIdx : UpdateROITriBuffer)
		{
			FIndex3i TriV = Mesh->GetTriangle(TriIdx);
			for (int j = 0; j < 3; ++j)
			{
				if (VertexROIBuilder.Contains(TriV[j]) == false)
				{
					//const FVector3d& Position = Mesh->GetVertexRef(TriV[j]);
					FVector3d Position = Mesh->GetVertex(TriV[j]);
					if (DistanceSquared(BrushPos, Position) < RadiusSqr)
					{
						VertexROIBuilder.Add(TriV[j]);
					}
				}
			}
		}
		VertexROI.Reset();
		VertexROIBuilder.SwapValuesWith(VertexROI);
	}

	// find triangle ROI as full one-rings of all vertices (this is surprisingly expensive...)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateROI_3TriangleROI);
		TriangleROIBuilder.Initialize(Mesh->MaxTriangleID());
		for (int32 vid : VertexROI)
		{
			Mesh->EnumerateVertexEdges(vid, [&](int32 eid)
			{
				FDynamicMesh3::FEdge Edge = Mesh->GetEdge(eid);
				TriangleROIBuilder.Add(Edge.Tri.A);
				if (Edge.Tri.B != IndexConstants::InvalidID) TriangleROIBuilder.Add(Edge.Tri.B);
			});
		}
		TriangleROI.Reset();
		TriangleROIBuilder.Collect(TriangleROI);
	}
}

void UDynamicMeshSculptTool::OnUpdateDrag(const FRay& WorldRay)
{
	if (bInDrag)
	{
		PendingStampRay = WorldRay;
		bStampPending = true;
	}
}

void UDynamicMeshSculptTool::CalculateBrushRadius()
{
	CurrentBrushRadius = BrushProperties->BrushSize.GetWorldRadius();
}

void UDynamicMeshSculptTool::ApplyStamp(const FRay& WorldRay)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_ApplyStamp);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshOctree3* Octree = DynamicMeshComponent->GetOctree();

	// update brush type history. apologies for convoluted logic.
	StampTimestamp++;
	if (LastStampType != PendingStampType)
	{
		if (BrushTypeHistoryIndex != BrushTypeHistory.Num() - 1)
		{
			if (LastStampType != EDynamicMeshSculptBrushType::LastValue)
			{
				BrushTypeHistory.Add(LastStampType);
			}
			BrushTypeHistoryIndex = BrushTypeHistory.Num()-1;
		}
		LastStampType = PendingStampType;
		if (BrushTypeHistory.Num() == 0 || BrushTypeHistory[BrushTypeHistory.Num()-1] != PendingStampType)
		{
			BrushTypeHistory.Add(PendingStampType);
			BrushTypeHistoryIndex = BrushTypeHistory.Num() - 1;
		}
	}

	CalculateBrushRadius();

	TFuture<void> DirtyOctreeFuture = Async(DynamicSculptToolAsyncExecTarget, [&]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_ApplyStamp_DirtyOctree);
		Octree->NotifyPendingModification(TriangleROI);
	});

	TFuture<void> SaveROIFuture = Async(DynamicSculptToolAsyncExecTarget, [&]() 
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_ApplyStamp_SaveActiveROI);
		SaveActiveROI();
	});

	// TODO: 
	//   - we can begin Octree->RemoveTriangles below as soon as we have (1) finished
	//     marking dirty box in DirtyOctreeFuture and (2) updated brush position. Unfortunately
	//     right now that happens inside each brush func :(

	EDynamicMeshSculptBrushType ApplyBrushType = (bSmoothing) ?
		EDynamicMeshSculptBrushType::Smooth : SculptProperties->PrimaryBrushType;

	bool bBrushApplied = false;
	switch (ApplyBrushType)
	{
		case EDynamicMeshSculptBrushType::Offset:
			bBrushApplied = ApplyOffsetBrush(WorldRay, false);
			break;
		case EDynamicMeshSculptBrushType::SculptView:
			bBrushApplied = ApplyOffsetBrush(WorldRay, true);
			break;
		case EDynamicMeshSculptBrushType::SculptMax:
			bBrushApplied = ApplySculptMaxBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Move:
			bBrushApplied = ApplyMoveBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::PullKelvin:
			bBrushApplied = ApplyPullKelvinBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::PullSharpKelvin:
			bBrushApplied = ApplyPullSharpKelvinBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Smooth:
			bBrushApplied = ApplySmoothBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Pinch:
			bBrushApplied = ApplyPinchBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::TwistKelvin:
			bBrushApplied = ApplyTwistKelvinBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Inflate:
			bBrushApplied = ApplyInflateBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::ScaleKelvin:
			bBrushApplied = ApplyScaleKelvinBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Flatten:
			bBrushApplied = ApplyFlattenBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Plane:
			bBrushApplied = ApplyPlaneBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::PlaneViewAligned:
			bBrushApplied = ApplyPlaneBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::FixedPlane:
			bBrushApplied = ApplyFixedPlaneBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::Resample:
			bBrushApplied = ApplyResampleBrush(WorldRay);
			break;
		case EDynamicMeshSculptBrushType::LastValue:
			break;
	}

	// wait for ROI to finish saving before we update positions
	SaveROIFuture.Wait();
	DirtyOctreeFuture.Wait();

	// we are going to reinsert these later
	TFuture<void> OctreeRemoveFuture = Async(DynamicSculptToolAsyncExecTarget, [&]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_ApplyStamp_OctreeRemove);
		Octree->RemoveTriangles(TriangleROI, false);	// already marked dirty above
	});

	// Update the mesh positions to match those in the position buffer
	if (bBrushApplied)
	{
		SyncMeshWithPositionBuffer(Mesh);
	}

	// we don't stricty have to wait here, we could return this future
	OctreeRemoveFuture.Wait();
}

double UDynamicMeshSculptTool::CalculateBrushFalloff(double Distance)
{
	double f = FMathd::Clamp(1.0 - BrushProperties->BrushFalloffAmount, 0.0, 1.0);
	double d = Distance / CurrentBrushRadius;
	double w = 1;
	if (d > f)
	{
		d = FMathd::Clamp((d - f) / (1.0 - f), 0.0, 1.0);
		w = (1.0 - d * d);
		w = w * w * w;
	}
	return w;
}



void UDynamicMeshSculptTool::SyncMeshWithPositionBuffer(FDynamicMesh3* Mesh)
{
	const int NumV = ROIPositionBuffer.Num();
	checkSlow(VertexROI.Num() <= NumV);

	// only if remeshing is disabled?
	if (ActiveVertexChange != nullptr)
	{
		for (int k = 0; k < NumV; ++k)
		{
			int VertIdx = VertexROI[k];
			ActiveVertexChange->UpdateVertex(VertIdx, Mesh->GetVertex(VertIdx), ROIPositionBuffer[k]);
		}
	}

	ParallelFor(NumV, [&](int32 k)
	{
		int VertIdx = VertexROI[k];
		const FVector3d& NewPos = ROIPositionBuffer[k];
		Mesh->SetVertex(VertIdx, NewPos, false);
	});
	Mesh->UpdateChangeStamps(true, false);
}

bool UDynamicMeshSculptTool::ApplySmoothBrush(const FRay& WorldRay)
{
	bool bHit = UpdateBrushPositionOnSculptMesh(WorldRay, true);
	if (bHit == false)
	{
		return false;
	}

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [this, Mesh, NewBrushPosLocal](int k)
	{
		int VertIdx = VertexROI[k];

		FVector3d OrigPos = Mesh->GetVertex(VertIdx);

		double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));

		FVector3d SmoothedPos = (SculptProperties->bPreserveUVFlow) ?
			FMeshWeights::CotanCentroidSafe(*Mesh, VertIdx, 10.0) : FMeshWeights::UniformCentroid(*Mesh, VertIdx);

		FVector3d NewPos = UE::Geometry::Lerp(OrigPos, SmoothedPos, Falloff*SculptProperties->SmoothBrushSpeed);

		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyMoveBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnActivePlane(WorldRay);

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d MoveVec = NewBrushPosLocal - LastBrushPosLocal;

	if (MoveVec.SquaredLength() <= 0)
	{
		LastBrushPosLocal = NewBrushPosLocal;
		return false;
	}

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [this, Mesh, NewBrushPosLocal, MoveVec](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);

		double PrevDist = (OrigPos - LastBrushPosLocal).Length();
		double NewDist = (OrigPos - NewBrushPosLocal).Length();
		double UseDist = FMath::Min(PrevDist, NewDist);

		double Falloff = CalculateBrushFalloff(UseDist) * ActivePressure;

		FVector3d NewPos = OrigPos + Falloff * MoveVec;
		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyOffsetBrush(const FRay& WorldRay, bool bUseViewDirection)
{
	UpdateBrushPositionOnTargetMesh(WorldRay, bUseViewDirection);
	if (bUseViewDirection)
	{
		AlignBrushToView();
	}

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d LocalNormal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);

	double Direction = (bInvert) ? -1.0 : 1.0;
	double UseSpeed = 0.5 * Direction * FMathd::Sqrt(CurrentBrushRadius) * (SculptProperties->PrimaryBrushSpeed) * ActivePressure;
	double MaxOffset = CurrentBrushRadius;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [&](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);

		FVector3d BasePos, BaseNormal;
		if (GetTargetMeshNearest(OrigPos, (double)(4 * CurrentBrushRadius), BasePos, BaseNormal) == false)
		{
			ROIPositionBuffer[k] = OrigPos;
		}
		else
		{
			FVector3d MoveVec = (bUseViewDirection) ?  (UseSpeed*LocalNormal) : (UseSpeed*BaseNormal);
			double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));
			FVector3d NewPos = OrigPos + Falloff * MoveVec;
			ROIPositionBuffer[k] = NewPos;
		}
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplySculptMaxBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnTargetMesh(WorldRay, true);
	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);

	double Direction = (bInvert) ? -1.0 : 1.0;
	double UseSpeed = Direction * FMathd::Sqrt(CurrentBrushRadius) * (SculptProperties->PrimaryBrushSpeed) * ActivePressure;

	double MaxOffset = CurrentBrushRadius * SculptMaxBrushProperties->MaxHeight;
	if (SculptMaxBrushProperties->bFreezeCurrentHeight && SculptMaxFixedHeight >= 0)
	{
		MaxOffset = SculptMaxFixedHeight;
	}
	SculptMaxFixedHeight = MaxOffset;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [this, Mesh, NewBrushPosLocal, UseSpeed, MaxOffset](int k)
	{
		int VertIdx = VertexROI[k];

		FVector3d OrigPos = Mesh->GetVertex(VertIdx);

		FVector3d BasePos, BaseNormal;
		if (GetTargetMeshNearest(OrigPos, (double)(2 * CurrentBrushRadius), BasePos, BaseNormal) == false)
		{
			ROIPositionBuffer[k] = OrigPos;
		}
		else
		{
			FVector3d MoveVec = UseSpeed * BaseNormal;
			double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));
			FVector3d NewPos = OrigPos + Falloff * MoveVec;

			FVector3d DeltaPos = NewPos - BasePos;
			if (DeltaPos.SquaredLength() > MaxOffset*MaxOffset)
			{
				UE::Geometry::Normalize(DeltaPos);
				NewPos = BasePos + MaxOffset * DeltaPos;
			}
			ROIPositionBuffer[k] = NewPos;
		}
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyPinchBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnTargetMesh(WorldRay, true);
	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	FVector3d OffsetBrushPosLocal = NewBrushPosLocal - BrushProperties->Depth * CurrentBrushRadius * BrushNormalLocal;

	// hardcoded lazybrush...
	FVector3d NewSmoothBrushPosLocal = (0.75f)*LastSmoothBrushPosLocal + (0.25f)*NewBrushPosLocal;

	double Direction = (bInvert) ? -1.0 : 1.0;
	double UseSpeed = Direction * FMathd::Sqrt(CurrentBrushRadius) * (SculptProperties->PrimaryBrushSpeed*0.05) * ActivePressure;

	FVector3d MotionVec = NewSmoothBrushPosLocal - LastSmoothBrushPosLocal;
	bool bHaveMotion = (MotionVec.Length() > FMathf::ZeroTolerance);
	UE::Geometry::Normalize(MotionVec);
	FLine3d MoveLine(LastSmoothBrushPosLocal, MotionVec);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [this, Mesh, NewBrushPosLocal, OffsetBrushPosLocal, bHaveMotion, MotionVec, UseSpeed](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);
		FVector3d Delta = OffsetBrushPosLocal - OrigPos;

		FVector3d MoveVec = UseSpeed * Delta;

		// pinch uses 1/x falloff, shifted so that
		double Dist = Distance(OrigPos, NewBrushPosLocal);
		double NormalizedDistance = Dist / CurrentBrushRadius + FMathf::ZeroTolerance;
		double Falloff = (1.0/NormalizedDistance) - 1.0;
		Falloff = FMathd::Clamp(Falloff, 0.0, 1.0);

		if (bHaveMotion && Falloff < 0.8f)
		{
			double AnglePower = 1.0 - FMathd::Abs(UE::Geometry::Normalized(MoveVec).Dot(MotionVec));
			Falloff *= AnglePower;
		}

		FVector3d NewPos = OrigPos + Falloff * MoveVec;
		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	LastSmoothBrushPosLocal = NewSmoothBrushPosLocal;
	return true;
}

FFrame3d UDynamicMeshSculptTool::ComputeROIBrushPlane(const FVector3d& BrushCenter, bool bIgnoreDepth, bool bViewAligned)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FVector3d AverageNormal(0, 0, 0);
	FVector3d AveragePos(0, 0, 0);
	double WeightSum = 0;
	for (int TriID : TriangleROI)
	{
		FVector3d Centroid = Mesh->GetTriCentroid(TriID);
		double Weight = CalculateBrushFalloff(Distance(BrushCenter, Centroid));

		AverageNormal += Weight * Mesh->GetTriNormal(TriID);
		AveragePos += Weight * Centroid;
		WeightSum += Weight;
	}
	UE::Geometry::Normalize(AverageNormal);
	AveragePos /= WeightSum;

	if (bViewAligned)
	{
		AverageNormal = -(FVector3d)CameraState.Forward();
	}

	FFrame3d Result = FFrame3d(AveragePos, AverageNormal);
	if (bIgnoreDepth == false)
	{
		Result.Origin -= BrushProperties->Depth * CurrentBrushRadius * Result.Z();
	}

	return Result;
}

bool UDynamicMeshSculptTool::ApplyPlaneBrush(const FRay& WorldRay)
{
	bool bHit = UpdateBrushPositionOnSculptMesh(WorldRay, true);
	if (bHit == false)
	{
		return false;
	}

	static const double PlaneSigns[3] = { 0, -1, 1 };
	double PlaneSign = PlaneSigns[0];

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	double UseSpeed = FMathd::Sqrt(CurrentBrushRadius) * FMathd::Sqrt(SculptProperties->PrimaryBrushSpeed) * 0.05 * ActivePressure;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [&](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);
		FVector3d PlanePos = ActiveFixedBrushPlane.ToPlane(OrigPos, 2);
		FVector3d Delta = PlanePos - OrigPos;
		double Dot = Delta.Dot(ActiveFixedBrushPlane.Z());
		FVector3d NewPos = OrigPos;
		if (Dot * PlaneSign >= 0)
		{
			FVector3d MoveVec = UseSpeed * Delta;
			double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));
			NewPos = OrigPos + Falloff * MoveVec;
		}
		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyFixedPlaneBrush(const FRay& WorldRay)
{
	bool bHit = UpdateBrushPositionOnSculptMesh(WorldRay, true);
	if (bHit == false)
	{
		return false;
	}

	static const double PlaneSigns[3] = { 0, -1, 1 };
	double PlaneSign = PlaneSigns[0];

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	double UseSpeed = CurrentBrushRadius * FMathd::Sqrt(SculptProperties->PrimaryBrushSpeed) * 0.1 * ActivePressure;

	FFrame3d FixedPlaneLocal(
		CurTargetTransform.InverseTransformPosition((FVector3d)GizmoProperties->Position),
		CurTargetTransform.GetRotation().Inverse() * (FQuaterniond)GizmoProperties->Rotation);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [&](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);
		FVector3d PlanePos = FixedPlaneLocal.ToPlane(OrigPos, 2);
		FVector3d Delta = PlanePos - OrigPos;
		double Dot = Delta.Dot(FixedPlaneLocal.Z());
		FVector3d NewPos = OrigPos;
		if (Dot * PlaneSign >= 0)
		{
			double MaxDist = UE::Geometry::Normalize(Delta);
			double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));
			FVector3d MoveVec = Falloff * UseSpeed * Delta;
			NewPos = (MoveVec.SquaredLength() > MaxDist* MaxDist) ?
				PlanePos : OrigPos + Falloff * MoveVec;
		}
		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyFlattenBrush(const FRay& WorldRay)
{
	bool bHit = UpdateBrushPositionOnSculptMesh(WorldRay, true);
	if (bHit == false)
	{
		return false;
	}

	static const double PlaneSigns[3] = { 0, -1, 1 };
	double PlaneSign = PlaneSigns[0];

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);

	double UseSpeed = FMathd::Sqrt(CurrentBrushRadius) * FMathd::Sqrt(SculptProperties->PrimaryBrushSpeed) * 0.05 * ActivePressure;
	FFrame3d StampFlattenPlane = ComputeROIBrushPlane(NewBrushPosLocal, true, false);
	//StampFlattenPlane.Origin -= BrushProperties->Depth * CurrentBrushRadius * StampFlattenPlane.Z();

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	ParallelFor(NumV, [&](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);
		FVector3d PlanePos = StampFlattenPlane.ToPlane(OrigPos, 2);
		FVector3d Delta = PlanePos - OrigPos;

		double Dot = Delta.Dot(StampFlattenPlane.Z());
		FVector3d NewPos = OrigPos;
		if (Dot * PlaneSign >= 0)
		{
			double MaxDist = UE::Geometry::Normalize(Delta);
			double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));
			FVector3d MoveVec = Falloff * UseSpeed * Delta;
			NewPos = (MoveVec.SquaredLength() > MaxDist*MaxDist) ?
				PlanePos : OrigPos + Falloff * MoveVec;
		}

		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyInflateBrush(const FRay& WorldRay)
{
	bool bHit = UpdateBrushPositionOnSculptMesh(WorldRay, true);
	if (bHit == false)
	{
		return false;
	}

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);

	double Direction = (bInvert) ? -1.0 : 1.0;
	double UseSpeed = Direction * CurrentBrushRadius * SculptProperties->PrimaryBrushSpeed * 0.05 * ActivePressure;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

	// calculate vertex normals
	ParallelFor(VertexROI.Num(), [this, Mesh](int Index) {
		int VertIdx = VertexROI[Index];
		FVector3d Normal = FMeshNormals::ComputeVertexNormal(*Mesh, VertIdx);
		Mesh->SetVertexNormal(VertIdx, (FVector3f)Normal);
	});


	ParallelFor(VertexROI.Num(), [this, Mesh, UseSpeed, NewBrushPosLocal](int k)
	{
		int VertIdx = VertexROI[k];
		FVector3d OrigPos = Mesh->GetVertex(VertIdx);
		FVector3d Normal = (FVector3d)Mesh->GetVertexNormal(VertIdx);

		FVector3d MoveVec = UseSpeed * Normal;

		double Falloff = CalculateBrushFalloff(Distance(OrigPos, NewBrushPosLocal));

		FVector3d NewPos = OrigPos + Falloff*MoveVec;
		ROIPositionBuffer[k] = NewPos;
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}


bool UDynamicMeshSculptTool::ApplyResampleBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnTargetMesh(WorldRay, true);
	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d LocalNormal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);

	double UseSpeed = FMathd::Sqrt(CurrentBrushRadius) * (SculptProperties->PrimaryBrushSpeed) * ActivePressure;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	int NumV = VertexROI.Num();
	ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);
	ParallelFor(NumV, [&](int k)
	{
		ROIPositionBuffer[k] = Mesh->GetVertex(VertexROI[k]);
	});

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}


bool UDynamicMeshSculptTool::ApplyPullKelvinBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnActivePlane(WorldRay);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d MoveVec = NewBrushPosLocal - LastBrushPosLocal;

	if (MoveVec.SquaredLength() <= 0)
	{
		LastBrushPosLocal = NewBrushPosLocal;
		return false;
	}
	
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FKelvinletBrushOp KelvinBrushOp(*Mesh);

	const EKelvinletBrushMode KelvinMode = EKelvinletBrushMode::PullKelvinlet;

	FKelvinletBrushOp::FKelvinletBrushOpProperties  KelvinletBrushOpProperties(KelvinMode, *KelvinBrushProperties, CurrentBrushRadius, BrushProperties->BrushFalloffAmount);
	KelvinletBrushOpProperties.Direction = FVector(MoveVec.X, MoveVec.Y, MoveVec.Z);  //FVector(BrushNormalLocal.X, BrushNormalLocal.Y, BrushNormalLocal.Z);
	KelvinletBrushOpProperties.Size *= 0.6;

	FMatrix ToBrush; ToBrush.SetIdentity();  ToBrush.SetOrigin(-FVector(NewBrushPosLocal.X, NewBrushPosLocal.Y, NewBrushPosLocal.Z)); //  ToBrush.

	KelvinBrushOp.ApplyBrush(KelvinletBrushOpProperties, ToBrush, VertexROI, ROIPositionBuffer);

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}


bool UDynamicMeshSculptTool::ApplyPullSharpKelvinBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnActivePlane(WorldRay);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d MoveVec = NewBrushPosLocal - LastBrushPosLocal;

	if (MoveVec.SquaredLength() <= 0)
	{
		LastBrushPosLocal = NewBrushPosLocal;
		return false;
	}

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FKelvinletBrushOp KelvinBrushOp(*Mesh);

	const EKelvinletBrushMode KelvinMode = EKelvinletBrushMode::SharpPullKelvinlet;

	FKelvinletBrushOp::FKelvinletBrushOpProperties  KelvinletBrushOpProperties(KelvinMode, *KelvinBrushProperties, CurrentBrushRadius, BrushProperties->BrushFalloffAmount);
	KelvinletBrushOpProperties.Direction = FVector(MoveVec.X, MoveVec.Y, MoveVec.Z);  //FVector(BrushNormalLocal.X, BrushNormalLocal.Y, BrushNormalLocal.Z);
	KelvinletBrushOpProperties.Size *= 0.6;

	FMatrix ToBrush; ToBrush.SetIdentity();  ToBrush.SetOrigin(-FVector(NewBrushPosLocal.X, NewBrushPosLocal.Y, NewBrushPosLocal.Z)); //  ToBrush.

	KelvinBrushOp.ApplyBrush(KelvinletBrushOpProperties, ToBrush, VertexROI, ROIPositionBuffer);

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyTwistKelvinBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnTargetMesh(WorldRay, true);
	
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	
	double Direction = (bInvert) ? -1.0 : 1.0;
	double UseSpeed  = Direction * FMathd::Sqrt(CurrentBrushRadius) * (SculptProperties->PrimaryBrushSpeed) * ActivePressure;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FKelvinletBrushOp KelvinBrushOp(*Mesh);

	FKelvinletBrushOp::FKelvinletBrushOpProperties  KelvinletBrushOpProperties(EKelvinletBrushMode::TwistKelvinlet, *KelvinBrushProperties, CurrentBrushRadius, BrushProperties->BrushFalloffAmount);
	KelvinletBrushOpProperties.Direction = UseSpeed * FVector(BrushNormalLocal.X, BrushNormalLocal.Y, BrushNormalLocal.Z); // twist about local normal
	KelvinletBrushOpProperties.Size *= 0.35; // reduce the core size of this brush.
	
	FMatrix ToBrush; ToBrush.SetIdentity();  ToBrush.SetOrigin(-FVector(NewBrushPosLocal.X, NewBrushPosLocal.Y, NewBrushPosLocal.Z)); //  ToBrush.

	KelvinBrushOp.ApplyBrush(KelvinletBrushOpProperties, ToBrush, VertexROI, ROIPositionBuffer);

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::ApplyScaleKelvinBrush(const FRay& WorldRay)
{
	UpdateBrushPositionOnSculptMesh(WorldRay, true);

	FVector3d NewBrushPosLocal = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
	FVector3d BrushNormalLocal = CurTargetTransform.InverseTransformNormal(LastBrushPosNormalWorld);
	FVector3d OffsetBrushPosLocal = NewBrushPosLocal - BrushProperties->Depth * CurrentBrushRadius * BrushNormalLocal;

	double Direction = (bInvert) ? -1.0 : 1.0;
	double UseSpeed = Direction * FMath::Sqrt(CurrentBrushRadius) * SculptProperties->PrimaryBrushSpeed * 0.025 * ActivePressure; ; 

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FKelvinletBrushOp KelvinBrushOp(*Mesh);

	FKelvinletBrushOp::FKelvinletBrushOpProperties  KelvinletBrushOpProperties(EKelvinletBrushMode::ScaleKelvinlet, *KelvinBrushProperties, CurrentBrushRadius, BrushProperties->BrushFalloffAmount);
	KelvinletBrushOpProperties.Direction = FVector(UseSpeed, 0., 0.); // it is a bit iffy, but we only use the first component for the scale
	KelvinletBrushOpProperties.Size *= 0.35;

	FMatrix ToBrush; ToBrush.SetIdentity();  ToBrush.SetOrigin(-FVector(OffsetBrushPosLocal.X, OffsetBrushPosLocal.Y, OffsetBrushPosLocal.Z)); //  ToBrush.

	KelvinBrushOp.ApplyBrush(KelvinletBrushOpProperties, ToBrush, VertexROI, ROIPositionBuffer);

	ScheduleRemeshPass();
	LastBrushPosLocal = NewBrushPosLocal;
	return true;
}

bool UDynamicMeshSculptTool::IsHitTriangleBackFacing(int32 TriangleID, const FDynamicMesh3* QueryMesh)
{
	if (TriangleID != IndexConstants::InvalidID)
	{
		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));

		FVector3d Normal, Centroid;
		double Area;
		QueryMesh->GetTriInfo(TriangleID, Normal, Area, Centroid);

		return (Normal.Dot((Centroid - LocalEyePosition)) >= 0);
	}
	return false;
}

int UDynamicMeshSculptTool::FindHitSculptMeshTriangle(const FRay3d& LocalRay)
{
	int32 HitTID = DynamicMeshComponent->GetOctree()->FindNearestHitObject(LocalRay);
	if (BrushProperties->bHitBackFaces == false && IsHitTriangleBackFacing(HitTID, DynamicMeshComponent->GetMesh()) )
	{
		HitTID = IndexConstants::InvalidID;
	}
	return HitTID;
}

int UDynamicMeshSculptTool::FindHitTargetMeshTriangle(const FRay3d& LocalRay)
{
	PendingTargetUpdate.Wait();

	int32 HitTID = BrushTargetMeshSpatial.FindNearestHitTriangle(LocalRay);
	if (BrushProperties->bHitBackFaces == false && IsHitTriangleBackFacing(HitTID, &BrushTargetMesh))
	{
		HitTID = IndexConstants::InvalidID;
	}
	return HitTID;
}

bool UDynamicMeshSculptTool::UpdateBrushPositionOnActivePlane(const FRay& WorldRay)
{
	FVector3d NewHitPosWorld;
	ActiveDragPlane.RayPlaneIntersection((FVector3d)WorldRay.Origin, (FVector3d)WorldRay.Direction, 2, NewHitPosWorld);
	LastBrushPosWorld = NewHitPosWorld;
	LastBrushPosNormalWorld = ActiveDragPlane.Z();
	return true;
}

bool UDynamicMeshSculptTool::UpdateBrushPositionOnTargetMesh(const FRay& WorldRay, bool bFallbackToViewPlane)
{
	PendingTargetUpdate.Wait();

	FRay3d LocalRay(CurTargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		CurTargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	const FDynamicMesh3* TargetMesh = BrushTargetMeshSpatial.GetMesh();

	int HitTID = FindHitTargetMeshTriangle(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		FTriangle3d Triangle;
		TargetMesh->GetTriVertices(HitTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();

		LastBrushPosNormalWorld = CurTargetTransform.TransformNormal(TargetMesh->GetTriNormal(HitTID));
		LastBrushPosWorld = CurTargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		return true;
	}

	if (bFallbackToViewPlane)
	{
		FFrame3d BrushPlane(LastBrushPosWorld, (FVector3d)CameraState.Forward());
		FVector3d NewHitPosWorld;
		BrushPlane.RayPlaneIntersection((FVector3d)WorldRay.Origin, (FVector3d)WorldRay.Direction, 2, NewHitPosWorld);
		LastBrushPosWorld = NewHitPosWorld;
		LastBrushPosNormalWorld = ActiveDragPlane.Z();
		return true;
	}

	return false;
}

bool UDynamicMeshSculptTool::UpdateBrushPositionOnSculptMesh(const FRay& WorldRay, bool bFallbackToViewPlane)
{
	FRay3d LocalRay(CurTargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		CurTargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	int HitTID = FindHitSculptMeshTriangle(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		const FDynamicMesh3* SculptMesh = DynamicMeshComponent->GetMesh();

		FTriangle3d Triangle;
		SculptMesh->GetTriVertices(HitTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();

		LastBrushPosNormalWorld = CurTargetTransform.TransformNormal(SculptMesh->GetTriNormal(HitTID));
		LastBrushPosWorld = CurTargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		LastBrushTriangleID = HitTID;
		return true;
	}

	if (bFallbackToViewPlane)
	{
		FFrame3d BrushPlane(LastBrushPosWorld, (FVector3d)CameraState.Forward());
		FVector3d NewHitPosWorld;
		BrushPlane.RayPlaneIntersection((FVector3d)WorldRay.Origin, (FVector3d)WorldRay.Direction, 2, NewHitPosWorld);
		LastBrushPosWorld = NewHitPosWorld;
		LastBrushPosNormalWorld = ActiveDragPlane.Z();
		LastBrushTriangleID = -1;
		return true;
	}

	return false;
}

void UDynamicMeshSculptTool::AlignBrushToView()
{
	LastBrushPosNormalWorld = -(FVector3d)CameraState.Forward();
}


bool UDynamicMeshSculptTool::UpdateBrushPosition(const FRay& WorldRay)
{
	// This is an unfortunate hack necessary because we haven't refactored brushes properly yet

	if (bSmoothing)
	{
		return UpdateBrushPositionOnSculptMesh(WorldRay, false);
	}

	bool bHit = false;
	switch (SculptProperties->PrimaryBrushType)
	{
	case EDynamicMeshSculptBrushType::Offset:
	case EDynamicMeshSculptBrushType::SculptMax:
	case EDynamicMeshSculptBrushType::Pinch:
	case EDynamicMeshSculptBrushType::Resample:
		bHit = UpdateBrushPositionOnTargetMesh(WorldRay, false);
		break;

	case EDynamicMeshSculptBrushType::SculptView:
	case EDynamicMeshSculptBrushType::PlaneViewAligned:
		bHit = UpdateBrushPositionOnTargetMesh(WorldRay, false);
		AlignBrushToView();
		break;

	case EDynamicMeshSculptBrushType::Move:
		//return UpdateBrushPositionOnActivePlane(WorldRay);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;

	case EDynamicMeshSculptBrushType::Smooth:
	case EDynamicMeshSculptBrushType::Inflate:
	case EDynamicMeshSculptBrushType::Flatten:
	case EDynamicMeshSculptBrushType::Plane:
	case EDynamicMeshSculptBrushType::FixedPlane:
	case EDynamicMeshSculptBrushType::PullSharpKelvin:
	case EDynamicMeshSculptBrushType::PullKelvin:
	case EDynamicMeshSculptBrushType::TwistKelvin:
	case EDynamicMeshSculptBrushType::ScaleKelvin:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;

	default:
		UE_LOGF(LogTemp, Warning, "UDynamicMeshSculptTool: unknown brush type in UpdateBrushPosition");
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	return bHit;
}



void UDynamicMeshSculptTool::OnEndDrag(const FRay& Ray)
{
	bInDrag = false;

	// cancel these! otherwise change record could become invalid
	bStampPending = false;
	bRemeshPending = false;

	// update spatial
	bTargetDirty = true;

	// close change record
	EndChange();

	// destroy active remesher. Should we do this every stroke?? need to do it on undo/redo...
	if (ActiveRemesher)
	{
		ActiveRemesher = nullptr;
	}
}

void UDynamicMeshSculptTool::OnCancelDrag()
{
	bInDrag = false;
	bStampPending = false;
	bRemeshPending = false;

	CancelChange();

	// destroy active remesher
	ActiveRemesher = nullptr;
}


FInputRayHit UDynamicMeshSculptTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	return UMeshSurfacePointTool::BeginHoverSequenceHitTest(PressPos);
}

bool UDynamicMeshSculptTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// 4.26 HOTFIX: update LastWorldRay position so that we have it for updating WorkPlane position
	UMeshSurfacePointTool::LastWorldRay = DevicePos.WorldRay;
	
	PendingStampType = SculptProperties->PrimaryBrushType;

	if (bInDrag)
	{
		FVector3d NewHitPosWorld;
		ActiveDragPlane.RayPlaneIntersection((FVector3d)DevicePos.WorldRay.Origin, (FVector3d)DevicePos.WorldRay.Direction, 2, NewHitPosWorld);
		LastBrushPosWorld = NewHitPosWorld;
		LastBrushPosNormalWorld = ActiveDragPlane.Z();
	}
	else
	{
		UpdateBrushPosition(DevicePos.WorldRay);

		//FHitResult OutHit;
		//if (HitTest(DevicePos.WorldRay, OutHit))
		//{
		//	LastBrushPosWorld = DevicePos.WorldRay.PointAt(OutHit.Distance + BrushProperties->Depth*CurrentBrushRadius);
		//	LastBrushPosNormalWorld = OutHit.Normal;
		//}
	}
	return true;
}


void UDynamicMeshSculptTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSurfacePointTool::Render(RenderAPI);
	// Cache here for usage during interaction, should probably happen in ::Tick() or elsewhere
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	FViewCameraState RenderCameraState = RenderAPI->GetCameraState();

	constexpr float Falloff = 0.5f;
	constexpr float Strength = 1.0f;
	BrushIndicator->Update(
		(float)this->CurrentBrushRadius,
		(FVector)this->LastBrushPosWorld,
		(FVector)this->LastBrushPosNormalWorld,
		Falloff,
		Strength);
	
	if (BrushIndicatorMaterial)
	{
		double FixedDimScale = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(RenderCameraState, LastBrushPosWorld, 1.5f);
		BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffWidth"), FixedDimScale);

		if (BrushEditBehavior.IsValid())
		{
			FLinearColor IndicatorColor;
		
			if (BrushEditBehavior->IsEditing())
			{
				IndicatorColor = FLinearColor::White;
			}
			else
			{
				BrushIndicatorMaterial->GetVectorParameterDefaultValue(TEXT("IndicatorColor"), IndicatorColor);
			}
			
			BrushIndicatorMaterial->SetVectorParameterValue(TEXT("IndicatorColor"), IndicatorColor);
		}
	}

	if (SculptProperties->PrimaryBrushType == EDynamicMeshSculptBrushType::FixedPlane)
	{
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		FColor GridColor(128, 128, 128, 32);
		float GridThickness = 0.5f*RenderCameraState.GetPDIScalingFactor();
		int NumGridLines = 10;
		FFrame3d DrawFrame(GizmoProperties->Position, GizmoProperties->Rotation);
		MeshDebugDraw::DrawSimpleFixedScreenAreaGrid(RenderCameraState, DrawFrame, NumGridLines, 45.0, GridThickness, GridColor, false, PDI, FTransform::Identity);
	}
}

void UDynamicMeshSculptTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	Super::DrawHUD(Canvas, RenderAPI);
	if (BrushEditBehavior.IsValid())
	{
		BrushEditBehavior->DrawHUD(Canvas, RenderAPI);	
	}
}

void UDynamicMeshSculptTool::OnTick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_OnTick);

	ActivePressure = GetCurrentDevicePressure();

	// Allow a tick to pass between application of brush stamps. If we do not do this then on large stamps
	// that take a significant fraction of a second to compute, frames will be skipped and the editor will appear
	// frozen, but when the user releases the mouse, sculpting has clearly happened
	static int TICK_SKIP_HACK = 0;
	if (TICK_SKIP_HACK++ % 2 == 0)
	{
		return;
	}

	ShowWireframeWatcher.CheckAndUpdate();
	MaterialModeWatcher.CheckAndUpdate();
	CustomMaterialWatcher.CheckAndUpdate();
	FlatShadingWatcher.CheckAndUpdate();
	ColorWatcher.CheckAndUpdate();
	TransparentColorWatcher.CheckAndUpdate();
	OpacityWatcher.CheckAndUpdate();
	TwoSidedWatcher.CheckAndUpdate();
	ImageWatcher.CheckAndUpdate();
	BrushTypeWatcher.CheckAndUpdate();
	GizmoPositionWatcher.CheckAndUpdate();
	GizmoRotationWatcher.CheckAndUpdate();

	bool bGizmoVisible = (SculptProperties->PrimaryBrushType == EDynamicMeshSculptBrushType::FixedPlane)
		&& (GizmoProperties->bShowGizmo);
	UpdateFixedPlaneGizmoVisibility(bGizmoVisible);
	GizmoProperties->bPropertySetEnabled = (SculptProperties->PrimaryBrushType == EDynamicMeshSculptBrushType::FixedPlane);

	if (PendingWorkPlaneUpdate != EPendingWorkPlaneUpdate::NoUpdatePending)
	{
		// raycast into scene and current sculpt and place plane at closest hit point
		FRay CursorWorldRay = UMeshSurfacePointTool::LastWorldRay;
		FHitResult Result;
		bool bWorldHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, CursorWorldRay);
		FRay3d LocalRay(CurTargetTransform.InverseTransformPosition((FVector3d)CursorWorldRay.Origin),
			CurTargetTransform.InverseTransformVector((FVector3d)CursorWorldRay.Direction));
		UE::Geometry::Normalize(LocalRay.Direction);
		bool bObjectHit = (FindHitSculptMeshTriangle(LocalRay) != IndexConstants::InvalidID);
		if (bWorldHit &&
			(bObjectHit == false || (CursorWorldRay.GetParameter(Result.ImpactPoint) < CursorWorldRay.GetParameter((FVector)LastBrushPosWorld))))
		{
			SetFixedSculptPlaneFromWorldPos(Result.ImpactPoint, Result.ImpactNormal, PendingWorkPlaneUpdate);
		}
		else
		{
			SetFixedSculptPlaneFromWorldPos((FVector)LastBrushPosWorld, (FVector)LastBrushPosNormalWorld, PendingWorkPlaneUpdate);
		}

		PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::NoUpdatePending;
	}

	// if user changed to not-frozen, we need to update the target
	if (bCachedFreezeTarget != SculptProperties->bFreezeTarget)
	{
		UpdateTarget();
		PendingTargetUpdate.Wait();
	}

	bool bMeshModified = false;
	bool bMeshShapeModified = false;

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshOctree3* Octree = DynamicMeshComponent->GetOctree();

	//Octree->CheckValidity(EValidityCheckFailMode::Check, false, true);

	//
	// Apply stamp
	//

	bool bROIUpdatePending = false;
	bool bOctreeUpdatePending = false;

	TFuture<void> InitializeRemesher;
	TFuture<void> PrecomputeRemeshROI;

	if (bStampPending)
	{
		// if we don't have an active remesher for this brush stroke, create one
		if (ActiveRemesher == nullptr)
		{
			InitializeRemesher = Async(DynamicSculptToolAsyncExecTarget, [&]()
			{
				InitializeActiveRemesher();
			});
		}

		// initialize ROI for current brush position
		// TODO: does this break move brush??
		UpdateBrushPosition(PendingStampRay);
		FVector3d BrushPos = CurTargetTransform.InverseTransformPosition(LastBrushPosWorld);
		UpdateROI(BrushPos);


		// once we know ROI we can speculatively start initializing remesh ROI
		PrecomputeRemeshROI = Async(DynamicSculptToolAsyncExecTarget, [&]()
		{
			// make sure our remesher is initialized
			InitializeRemesher.Wait();
			// initialize the ROI
			PrecomputeRemesherROI();
		});

		// apply brush stamp to ROI
		ApplyStamp(PendingStampRay);
		bStampPending = (bInDrag) ? true : false;

		bNormalUpdatePending = true;
		bROIUpdatePending = true;
		bMeshModified = true;
		bMeshShapeModified = true;
		bOctreeUpdatePending = true;

		if (bRemeshPending)
		{
			check(bInDrag == true);    // this would break undo otherwise!

			// make sure our remesher is initialized
			InitializeRemesher.Wait();
			// make sure our ROI is computed
			PrecomputeRemeshROI.Wait();

			// remesh the ROI (this removes all ROI triangles from octree)
			if (ActiveRemesher)
			{
				RemeshROIPass_ActiveRemesher(true);
			}
			else
			{
				check(false);		// broken now
				RemeshROIPass();
			}

			// accumulate new triangles into TriangleROI
			for (int32 tid : RemeshFinalTriangleROI)
			{
				TriangleROI.Add(tid);
			}

			bMeshModified = true;
			bMeshShapeModified = true;
			bRemeshPending = false;
			bNormalUpdatePending = true;
			bROIUpdatePending = true;
			bOctreeUpdatePending = true;
			bHaveRemeshed = true;
		}
	}
	check(bRemeshPending == false);		// should never happen...

	// launch octree update that inserts/reinserts all triangles in TriangleROI
	TFuture<void> UpdateOctreeFuture;
	if (bOctreeUpdatePending)
	{
		// reinsert new ROI into octree
		UpdateOctreeFuture = Async(DynamicSculptToolAsyncExecTarget, [&]() {
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_OctreeReinsert);
			Octree->ReinsertTriangles(TriangleROI);
			//Octree->CheckValidity(EValidityCheckFailMode::Check, false, true);
		});
		bOctreeUpdatePending = false;
	}

	//Octree->CheckValidity(EValidityCheckFailMode::Check, false, true);

	if (bNormalUpdatePending)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateNormals);

		if (Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals() != nullptr)
		{
			RecalculateNormals_Overlay(TriangleROI);
		}
		else
		{
			RecalculateNormals_PerVertex(TriangleROI);
		}
		bNormalUpdatePending = false;
		bMeshModified = true;
	}

	// next steps need to wait for octree update to finish
	UpdateOctreeFuture.Wait();

	// launch async target update task
	if (bTargetDirty)
	{
		UpdateTarget();
		bTargetDirty = false;
	}

	// update render data
	if (bMeshModified)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateRenderMesh);
		DynamicMeshComponent->NotifyMeshUpdated();
		GetToolManager()->PostInvalidation();

		bMeshModified = false;
	}

	// Allow futures to finish in case bRemeshPending == false
	InitializeRemesher.Wait();
	PrecomputeRemeshROI.Wait();
}

void UDynamicMeshSculptTool::PrecomputeRemeshInfo()
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();

	// check if we have any open boundary edges
	bHaveMeshBoundaries = false;
	for (int eid : Mesh->EdgeIndicesItr())
	{
		if (Mesh->IsBoundaryEdge(eid))
		{
			bHaveMeshBoundaries = true;
			break;
		}
	}

	// check if we have any UV seams
	bHaveUVSeams = false;
	bHaveNormalSeams = false;
	if (Mesh->HasAttributes())
	{
		FDynamicMeshAttributeSet* Attribs = Mesh->Attributes();
		for (int k = 0; k < Attribs->NumUVLayers(); ++k)
		{
			bHaveUVSeams = bHaveUVSeams || Attribs->GetUVLayer(k)->HasInteriorSeamEdges();
		}

		bHaveNormalSeams = Attribs->PrimaryNormals()->HasInteriorSeamEdges();
	}
}

void UDynamicMeshSculptTool::ScheduleRemeshPass()
{
	if (bEnableRemeshing && RemeshProperties != nullptr && RemeshProperties->bEnableRemeshing)
	{
		bRemeshPending = true;
	}
}






/**
 * This is an internal class we will use to just hold onto a FSubRegionRemesher instance
 * between brush stamps. Perhaps does not need to exist.
 */
class FPersistentStampRemesher
{
public:
	FDynamicMesh3* Mesh;
	TUniquePtr<FSubRegionRemesher> Remesher;

	FPersistentStampRemesher(FDynamicMesh3* MeshIn)
	{
		this->Mesh = MeshIn;
		Remesher = MakeUnique<FSubRegionRemesher>(MeshIn);
	}
};




/*
        Split	Collapse	Vertices Pinned	Flip
Fixed	FALSE	FALSE	TRUE	FALSE
Refine	TRUE	FALSE	TRUE	FALSE
Free	TRUE	TRUE	FALSE	FALSE
Ignore	TRUE	TRUE	FALSE	TRUE
*/

void UDynamicMeshSculptTool::ConfigureRemesher(FSubRegionRemesher& Remesher)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();

	const double SizeRange = 5;
	double LengthMultiplier = (RemeshProperties->TriangleSize >= 0) ?
		FMathd::Lerp(1.0, 5.0, FMathd::Pow( (double)RemeshProperties->TriangleSize / SizeRange, 2.0) )
		: FMathd::Lerp(0.25, 1.0, 1.0 - FMathd::Pow(FMathd::Abs((double)RemeshProperties->TriangleSize) / SizeRange, 2.0) );
	double TargetEdgeLength = LengthMultiplier * InitialEdgeLength;

	Remesher.SetTargetEdgeLength(TargetEdgeLength);

	double DetailT = (double)(RemeshProperties->PreserveDetail) / 5.0;
	double UseSmoothing = RemeshProperties->SmoothingStrength * 0.25;
	UseSmoothing *= FMathd::Lerp(1.0, 0.25, DetailT);
	Remesher.SmoothSpeedT = UseSmoothing;

	// this is a temporary tweak for Pinch brush. Remesh params should be per-brush!
	if (SculptProperties->PrimaryBrushType == EDynamicMeshSculptBrushType::Pinch && bSmoothing == false)
	{
		Remesher.MinEdgeLength = TargetEdgeLength * 0.1;

		Remesher.CustomSmoothSpeedF = [this, UseSmoothing](const FDynamicMesh3& Mesh, int vID)
		{
			FVector3d Pos = Mesh.GetVertex(vID);
			double Falloff = CalculateBrushFalloff(Distance(Pos, (FVector3d)LastBrushPosLocal));
			return (1.0f - Falloff) * UseSmoothing;
		};
	}
	else if (bSmoothing && SculptProperties->bDetailPreservingSmooth)
	{
		// this is the case where we don't want remeshing in smoothing
		Remesher.MaxEdgeLength = 3 * InitialEdgeLength;
		Remesher.MinEdgeLength = InitialEdgeLength * 0.05;
	}
	else
	{
		if (RemeshProperties->PreserveDetail > 0)
		{
			Remesher.MinEdgeLength *= FMathd::Lerp(1.0, 0.1, DetailT);
			Remesher.CustomSmoothSpeedF = [this, UseSmoothing, DetailT](const FDynamicMesh3& Mesh, int vID)
			{
				FVector3d Pos = Mesh.GetVertex(vID);
				double FalloffT = 1.0 - CalculateBrushFalloff(Distance(Pos, (FVector3d)LastBrushPosLocal));
				FalloffT = FMathd::Lerp(1.0, FalloffT, DetailT);
				return FalloffT * UseSmoothing;
			};
		}
	}


	if (SculptProperties->bPreserveUVFlow)
	{
		Remesher.SmoothType = FRemesher::ESmoothTypes::MeanValue;
		Remesher.FlipMetric = FRemesher::EFlipMetric::MinEdgeLength;
	}
	else
	{
		Remesher.SmoothType = FRemesher::ESmoothTypes::Uniform;
		Remesher.FlipMetric = FRemesher::EFlipMetric::OptimalValence;
	}
	Remesher.bEnableCollapses = RemeshProperties->bCollapses;
	Remesher.bEnableFlips = RemeshProperties->bFlips;
	Remesher.bEnableSplits = RemeshProperties->bSplits;
	Remesher.bPreventNormalFlips = RemeshProperties->bPreventNormalFlips;

	FMeshConstraints Constraints;
	bool bConstraintAllowSplits = true;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_Configure_Constraints);

		FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, *Mesh,
															 (EEdgeRefineFlags)RemeshProperties->MeshBoundaryConstraint,
															 (EEdgeRefineFlags)RemeshProperties->GroupBoundaryConstraint,
															 (EEdgeRefineFlags)RemeshProperties->MaterialBoundaryConstraint,
															 bConstraintAllowSplits, !RemeshProperties->bPreserveSharpEdges);
			
		Remesher.SetExternalConstraints(MoveTemp(Constraints));
	}
}


void UDynamicMeshSculptTool::InitializeRemesherROI(FSubRegionRemesher& Remesher)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_Configure_InitializeROI);
	Remesher.SetInitialVertexROI(this->VertexROI);
	Remesher.InitializeFromVertexROI();
}



void UDynamicMeshSculptTool::InitializeActiveRemesher()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_InitializeActiveRemesher);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	ActiveRemesher = MakeShared<FPersistentStampRemesher>(Mesh);
	ConfigureRemesher(* ActiveRemesher->Remesher);
}

void UDynamicMeshSculptTool::PrecomputeRemesherROI()
{
	FSubRegionRemesher& Remesher = *ActiveRemesher->Remesher;
	Remesher.Reset();
	InitializeRemesherROI(Remesher);
}


void UDynamicMeshSculptTool::RemeshROIPass_ActiveRemesher(bool bHasPrecomputedROI)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROIActive);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshOctree3* Octree = DynamicMeshComponent->GetOctree();

	FSubRegionRemesher& Remesher = *ActiveRemesher->Remesher;
	
	if (bHasPrecomputedROI == false)
	{
		PrecomputeRemesherROI();
	}

	// remove initial triangles
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_CopyROIStep);
		RemeshRemovedTriangles = Remesher.GetCurrentTriangleROI();
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_OctreeRemoveStep);
		Octree->RemoveTriangles(RemeshRemovedTriangles, true);
	}

	if (ActiveMeshChange != nullptr)
	{
		Remesher.SetMeshChangeTracker(ActiveMeshChange);
	}

	bool bIsUniformSmooth = (Remesher.SmoothType == FRemesher::ESmoothTypes::Uniform);
	for (int k = 0; k < RemeshProperties->Iterations; ++k)
	{
		if ((bIsUniformSmooth == false) && (k > 1))
		{
			Remesher.bEnableFlips = false;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_UpdateStep);

			Remesher.UpdateROI();

			if (ActiveMeshChange != nullptr)
			{
				// [TODO] would like to only save vertices here, as triangles will be saved by Remesher as necessary.
				// However currently FDynamicMeshChangeTracker cannot independently save vertices, only vertices 
				// that are part of saved triangles will be included in the output FDynamicMeshChange
				Remesher.SaveActiveROI(ActiveMeshChange);
				//ActiveMeshChange->VerifySaveState();    // useful for debugging
			}

			Remesher.BeginTrackRemovedTrisInPass();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_RemeshStep);
			Remesher.BasicRemeshPass();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_OctreeStep);
			const TSet<int32>& TrisRemovedInPass = Remesher.EndTrackRemovedTrisInPass();
			Octree->RemoveTriangles(TrisRemovedInPass);
			for (int32 tid : TrisRemovedInPass)
			{
				RemeshRemovedTriangles.Add(tid);
			}
		}
	}
	//UE_LOGF(LogTemp, Warning, "Triangle Count %d after update", Mesh->TriangleCount());

	RemeshFinalTriangleROI = Remesher.ExtractFinalTriangleROI();
}



void UDynamicMeshSculptTool::RemeshROIPass()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI);

	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshOctree3* Octree = DynamicMeshComponent->GetOctree();

	FSubRegionRemesher Remesher(Mesh);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_Configure);
		ConfigureRemesher(Remesher);
		InitializeRemesherROI(Remesher);
	}

	// remove initial triangles
	RemeshRemovedTriangles = Remesher.GetCurrentTriangleROI();
	Octree->RemoveTriangles(RemeshRemovedTriangles);

	if (ActiveMeshChange != nullptr)
	{
		Remesher.SetMeshChangeTracker(ActiveMeshChange);
	}

	bool bIsUniformSmooth = (Remesher.SmoothType == FRemesher::ESmoothTypes::Uniform);
	for (int k = 0; k < 5; ++k)
	{
		if ( ( bIsUniformSmooth == false ) && ( k > 1 ) )
		{
			Remesher.bEnableFlips = false;
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_UpdateStep);

			Remesher.UpdateROI();

			if (ActiveMeshChange != nullptr)
			{
				// [TODO] would like to only save vertices here, as triangles will be saved by Remesher as necessary.
				// However currently FDynamicMeshChangeTracker cannot independently save vertices, only vertices 
				// that are part of saved triangles will be included in the output FDynamicMeshChange
				Remesher.SaveActiveROI(ActiveMeshChange);
				//ActiveMeshChange->VerifySaveState();    // useful for debugging
			}

			Remesher.BeginTrackRemovedTrisInPass();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_RemeshROI_RemeshStep);
			Remesher.BasicRemeshPass();
		}

		{
			const TSet<int32>& TrisRemovedInPass = Remesher.EndTrackRemovedTrisInPass();
			Octree->RemoveTriangles(TrisRemovedInPass);
			for (int32 tid : TrisRemovedInPass)
			{
				RemeshRemovedTriangles.Add(tid);
			}
		}
	}
	//UE_LOGF(LogTemp, Warning, "Triangle Count %d after update", Mesh->TriangleCount());

	RemeshFinalTriangleROI = Remesher.ExtractFinalTriangleROI();
}

void UDynamicMeshSculptTool::RecalculateNormals_PerVertex(const TSet<int32>& Triangles)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();

	int MaxVertexID = Mesh->MaxVertexID();
	if (NormalsVertexFlags.Num() < MaxVertexID)
	{
		NormalsVertexFlags.Init(false, MaxVertexID * 2);
	}

	{
		NormalsBuffer.Reset();
		for (int TriangleID : Triangles)
		{
			if (Mesh->IsTriangle(TriangleID))
			{
				FIndex3i TriV = Mesh->GetTriangle(TriangleID);
				for (int j = 0; j < 3; ++j)
				{
					int vid = TriV[j];
					if (NormalsVertexFlags[vid] == false)
					{
						NormalsBuffer.Add(vid);
						NormalsVertexFlags[vid] = true;
					}
				}
			}
		}
	}

	{
		ParallelFor(NormalsBuffer.Num(), [&](int k) {
			int vid = NormalsBuffer[k];
			FVector3d NewNormal = FMeshNormals::ComputeVertexNormal(*Mesh, vid);
			Mesh->SetVertexNormal(vid, (FVector3f)NewNormal);
			NormalsVertexFlags[vid] = false;
		});
	}
}

void UDynamicMeshSculptTool::RecalculateNormals_Overlay(const TSet<int32>& Triangles)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
	check(Normals != nullptr);

	int MaxElementID = Normals->MaxElementID();
	if (NormalsVertexFlags.Num() < MaxElementID)
	{
		NormalsVertexFlags.Init(false, MaxElementID * 2);
	}

	{
		NormalsBuffer.Reset();
		for (int TriangleID : Triangles)
		{
			if (Mesh->IsTriangle(TriangleID))
			{
				FIndex3i TriElems = Normals->GetTriangle(TriangleID);
				if (TriElems.A == FDynamicMesh3::InvalidID)
				{
					continue;
				}
				for (int j = 0; j < 3; ++j)
				{
					int elemid = TriElems[j];
					if (NormalsVertexFlags[elemid] == false)
					{
						NormalsBuffer.Add(elemid);
						NormalsVertexFlags[elemid] = true;
					}
				}
			}
		}
	}

	{
		ParallelFor(NormalsBuffer.Num(), [&](int k) {
			int elemid = NormalsBuffer[k];
			FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
			Normals->SetElement(elemid, (FVector3f)NewNormal);
			NormalsVertexFlags[elemid] = false;
		});
	}
}


void UDynamicMeshSculptTool::UpdateTarget()
{
	if (SculptProperties != nullptr )
	{
		bCachedFreezeTarget = SculptProperties->bFreezeTarget;
		if (SculptProperties->bFreezeTarget)
		{
			return;   // do not update frozen target
		}
	}

	// Allow any in-progress update to finish before starting a new one
	PendingTargetUpdate.Wait();

	// TODO: could have a second set of target meshes, and swap between them while we update the other one
	PendingTargetUpdate = Async(DynamicSculptToolAsyncExecTarget, [this]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateTarget);
		BrushTargetMesh.Copy(*DynamicMeshComponent->GetMesh(), false, false, false, false);

		TFuture<void> TargetSpatialUpdate = Async(DynamicSculptToolAsyncExecTarget, [this]() {
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateTarget_Spatial);
			BrushTargetMeshSpatial.SetMesh(&BrushTargetMesh, true);
		});
		TFuture<void> TargetNormalsUpdate = Async(DynamicSculptToolAsyncExecTarget, [this]() {
			TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateTarget_Normals);
			BrushTargetNormals.SetMesh(&BrushTargetMesh);
			BrushTargetNormals.ComputeVertexNormals();
		});

		TargetSpatialUpdate.Wait();
		TargetNormalsUpdate.Wait();
	});

}

bool UDynamicMeshSculptTool::GetTargetMeshNearest(const FVector3d& Position, double SearchRadius, FVector3d& TargetPosOut, FVector3d& TargetNormalOut)
{
	PendingTargetUpdate.Wait();

	double fDistSqr;
	int NearTID = BrushTargetMeshSpatial.FindNearestTriangle(Position, fDistSqr, SearchRadius);
	if (NearTID <= 0)
	{
		return false;
	}
	FTriangle3d Triangle;
	BrushTargetMesh.GetTriVertices(NearTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
	FDistPoint3Triangle3d Query(Position, Triangle);
	Query.Get();
	FIndex3i Tri = BrushTargetMesh.GetTriangle(NearTID);
	TargetNormalOut =
		Query.TriangleBaryCoords.X*BrushTargetNormals[Tri.A]
		+ Query.TriangleBaryCoords.Y*BrushTargetNormals[Tri.B]
		+ Query.TriangleBaryCoords.Z*BrushTargetNormals[Tri.C];
	UE::Geometry::Normalize(TargetNormalOut);
	TargetPosOut = Query.ClosestTrianglePoint;
	return true;
}

double UDynamicMeshSculptTool::EstimateIntialSafeTargetLength(const FDynamicMesh3& Mesh, int MinTargetTriCount)
{
	double AreaSum = 0;
	for (int tid : Mesh.TriangleIndicesItr())
	{
		AreaSum += Mesh.GetTriArea(tid);
	}

	int TriCount = Mesh.TriangleCount();
	double TargetTriArea = 1.0;
	if (TriCount < MinTargetTriCount)
	{
		TargetTriArea = AreaSum / (double)MinTargetTriCount;
	}
	else
	{
		TargetTriArea = AreaSum / (double)TriCount;
	}

	double EdgeLen = TriangleUtil::EquilateralEdgeLengthForArea(TargetTriArea);
	return (double)FMath::RoundToInt(EdgeLen*100.0) / 100.0;
}

UPreviewMesh* UDynamicMeshSculptTool::MakeDefaultSphereMesh(UObject* Parent, UWorld* World, int Resolution /*= 32*/)
{
	UPreviewMesh* SphereMesh = NewObject<UPreviewMesh>(Parent);
	SphereMesh->CreateInWorld(World, FTransform::Identity);
	FSphereGenerator SphereGen;
	SphereGen.NumPhi = SphereGen.NumTheta = Resolution;
	SphereGen.Generate();
	FDynamicMesh3 Mesh(&SphereGen);
	SphereMesh->UpdatePreview(&Mesh);

	BrushIndicatorMaterial = ToolSetupUtil::GetDefaultBrushVolumeMaterial(GetToolManager());
	if (BrushIndicatorMaterial)
	{
		SphereMesh->SetMaterial(BrushIndicatorMaterial);
	}

	// make sure raytracing is disabled on the brush indicator
	Cast<UDynamicMeshComponent>(SphereMesh->GetRootComponent())->SetEnableRaytracing(false);
	SphereMesh->SetShadowsEnabled(false);

	return SphereMesh;
}

void UDynamicMeshSculptTool::IncreaseBrushRadiusAction()
{	
	BrushProperties->BrushSize.IncreaseRadius(false);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UDynamicMeshSculptTool::DecreaseBrushRadiusAction()
{
	BrushProperties->BrushSize.DecreaseRadius(false);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UDynamicMeshSculptTool::IncreaseBrushRadiusSmallStepAction()
{
	BrushProperties->BrushSize.IncreaseRadius(true);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UDynamicMeshSculptTool::DecreaseBrushRadiusSmallStepAction()
{
	BrushProperties->BrushSize.DecreaseRadius(true);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UDynamicMeshSculptTool::IncreaseBrushSpeedAction()
{
	SculptProperties->PrimaryBrushSpeed = FMath::Clamp(SculptProperties->PrimaryBrushSpeed + 0.05f, 0.0f, 1.0f);
	NotifyOfPropertyChangeByTool(SculptProperties);
}

void UDynamicMeshSculptTool::DecreaseBrushSpeedAction()
{
	SculptProperties->PrimaryBrushSpeed = FMath::Clamp(SculptProperties->PrimaryBrushSpeed - 0.05f, 0.0f, 1.0f);
	NotifyOfPropertyChangeByTool(SculptProperties);
}


void UDynamicMeshSculptTool::NextHistoryBrushModeAction()
{
	int MaxHistory = BrushTypeHistory.Num() - 1;
	if (BrushTypeHistoryIndex < MaxHistory)
	{
		BrushTypeHistoryIndex++;
		SculptProperties->PrimaryBrushType = BrushTypeHistory[BrushTypeHistoryIndex];
		LastStampType = SculptProperties->PrimaryBrushType;
	}
}

void UDynamicMeshSculptTool::PreviousHistoryBrushModeAction()
{
	if (BrushTypeHistoryIndex > 0)
	{
		BrushTypeHistoryIndex--;
		SculptProperties->PrimaryBrushType = BrushTypeHistory[BrushTypeHistoryIndex];
		LastStampType = SculptProperties->PrimaryBrushType;
	}
}

void UDynamicMeshSculptTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	constexpr EToolActionKeyRepeatMode KeyRepeatOnBrushAdjust = EToolActionKeyRepeatMode::Enabled;
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::IncreaseBrushSize,
		TEXT("SculptIncreaseRadius"),
		LOCTEXT("SculptIncreaseRadius", "Increase Sculpt Radius"),
		LOCTEXT("SculptIncreaseRadiusTooltip", "Increase radius of sculpting brush"),
		EModifierKey::None, EKeys::RightBracket,
		[this]() { IncreaseBrushRadiusAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::DecreaseBrushSize,
		TEXT("SculptDecreaseRadius"),
		LOCTEXT("SculptDecreaseRadius", "Decrease Sculpt Radius"),
		LOCTEXT("SculptDecreaseRadiusTooltip", "Decrease radius of sculpting brush"),
		EModifierKey::None, EKeys::LeftBracket,
		[this]() { DecreaseBrushRadiusAction(); },
		KeyRepeatOnBrushAdjust);


	//ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 10,
	//	TEXT("NextBrushHistoryState"),
	//	LOCTEXT("SculptNextBrushHistoryState", "Next Brush History State"),
	//	LOCTEXT("SculptSculptNextBrushHistoryStateTooltip", "Cycle to next Brush History state"),
	//	EModifierKey::Shift, EKeys::Q,
	//	[this]() { NextHistoryBrushModeAction(); });

	//ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 11,
	//	TEXT("PreviousBrushHistoryState"),
	//	LOCTEXT("SculptPreviousBrushHistoryState", "Previous Brush History State"),
	//	LOCTEXT("SculptPreviousBrushHistoryStateTooltip", "Cycle to previous Brush History state"),
	//	EModifierKey::Shift, EKeys::A,
	//	[this]() { PreviousHistoryBrushModeAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 50,
		TEXT("SculptIncreaseSize"),
		LOCTEXT("SculptIncreaseSize", "Increase Size"),
		LOCTEXT("SculptIncreaseSizeTooltip", "Increase Brush Size"),
		EModifierKey::None, EKeys::D,
		[this]() { IncreaseBrushRadiusAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 51,
		TEXT("SculptDecreaseSize"),
		LOCTEXT("SculptDecreaseSize", "Decrease Size"),
		LOCTEXT("SculptDecreaseSizeTooltip", "Decrease Brush Size"),
		EModifierKey::None, EKeys::S,
		[this]() { DecreaseBrushRadiusAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 52,
		TEXT("SculptIncreaseSizeSmallStep"),
		LOCTEXT("SculptIncreaseSize", "Increase Size"),
		LOCTEXT("SculptIncreaseSizeTooltip", "Increase Brush Size"),
		EModifierKey::Shift, EKeys::D,
		[this]() { IncreaseBrushRadiusSmallStepAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 53,
		TEXT("SculptDecreaseSizeSmallStemp"),
		LOCTEXT("SculptDecreaseSize", "Decrease Size"),
		LOCTEXT("SculptDecreaseSizeTooltip", "Decrease Brush Size"),
		EModifierKey::Shift, EKeys::S,
		[this]() { DecreaseBrushRadiusSmallStepAction(); });




	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 60,
		TEXT("SculptIncreaseSpeed"),
		LOCTEXT("SculptIncreaseSpeed", "Increase Speed"),
		LOCTEXT("SculptIncreaseSpeedTooltip", "Increase Brush Speed"),
		EModifierKey::None, EKeys::E,
		[this]() { IncreaseBrushSpeedAction(); });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 61,
		TEXT("SculptDecreaseSpeed"),
		LOCTEXT("SculptDecreaseSpeed", "Decrease Speed"),
		LOCTEXT("SculptDecreaseSpeedTooltip", "Decrease Brush Speed"),
		EModifierKey::None, EKeys::W,
		[this]() { DecreaseBrushSpeedAction(); });



	ActionSet.RegisterAction(this, (int32)EStandardToolActions::ToggleWireframe,
		TEXT("ToggleWireframe"),
		LOCTEXT("ToggleWireframe", "Toggle Wireframe"),
		LOCTEXT("ToggleWireframeTooltip", "Toggle visibility of wireframe overlay"),
		EModifierKey::Alt, EKeys::W,
		[this]() { ViewProperties->bShowWireframe = !ViewProperties->bShowWireframe; });


	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 100,
		TEXT("SetSculptWorkSurfacePosNormal"),
		LOCTEXT("SetSculptWorkSurfacePosNormal", "Reorient Work Surface"),
		LOCTEXT("SetSculptWorkSurfacePosNormalTooltip", "Move the Sculpting Work Plane/Surface to Position and Normal of World hit point under cursor"),
		EModifierKey::Shift, EKeys::T,
		[this]() { PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::MoveToHitPositionNormal; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 101,
		TEXT("SetSculptWorkSurfacePos"),
		LOCTEXT("SetSculptWorkSurfacePos", "Reposition Work Surface"),
		LOCTEXT("SetSculptWorkSurfacePosTooltip", "Move the Sculpting Work Plane/Surface to World hit point under cursor (keep current Orientation)"),
		EModifierKey::None, EKeys::T,
		[this]() { PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::MoveToHitPosition; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 102,
		TEXT("SetSculptWorkSurfaceView"),
		LOCTEXT("SetSculptWorkSurfaceView", "View-Align Work Surface"),
		LOCTEXT("SetSculptWorkSurfaceViewTooltip", "Move the Sculpting Work Plane/Surface to World hit point under cursor and align to View"),
		EModifierKey::Control | EModifierKey::Shift, EKeys::T,
		[this]() { PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::MoveToHitPositionViewAligned; });

}

//
// Change Tracking
//
void UDynamicMeshSculptTool::BeginChange(bool bIsVertexChange)
{
	check(ActiveVertexChange == nullptr);
	check(ActiveMeshChange == nullptr);
	LongTransactions.Open(LOCTEXT("MeshSculptChange", "Brush Stroke"), GetToolManager()); // Open brush stroke transaction to prevent undo during the operation
	if (bIsVertexChange)
	{
		ActiveVertexChange = new FMeshVertexChangeBuilder();
	}
	else
	{
		ActiveMeshChange = new FDynamicMeshChangeTracker(DynamicMeshComponent->GetMesh());
		ActiveMeshChange->BeginChange();
	}
}

void UDynamicMeshSculptTool::EndChange()
{
	if (ActiveVertexChange != nullptr)
	{
		GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(ActiveVertexChange->Change), LOCTEXT("MeshSculptChange", "Brush Stroke"));

		delete ActiveVertexChange;
		ActiveVertexChange = nullptr;
	}

	if (ActiveMeshChange != nullptr)
	{
		FMeshChange* NewMeshChange = new FMeshChange();
		NewMeshChange->DynamicMeshChange = ActiveMeshChange->EndChange();
		//NewMeshChange->DynamicMeshChange->CheckValidity();
		TUniquePtr<FMeshChange> NewChange(NewMeshChange);
		GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(NewChange), LOCTEXT("MeshSculptChange", "Brush Stroke"));
		delete ActiveMeshChange;
		ActiveMeshChange = nullptr;
	}

	LongTransactions.Close(GetToolManager());
}

void UDynamicMeshSculptTool::CancelChange()
{
	LongTransactions.Close(GetToolManager());
	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
	
	delete ActiveMeshChange;
	ActiveMeshChange = nullptr;
}

void UDynamicMeshSculptTool::SaveActiveROI()
{
	if (ActiveMeshChange != nullptr)
	{
		// must save triangles containing vertex ROI or they will not be included in emitted mesh change (due to limitations of change tracker)
		ActiveMeshChange->SaveVertexOneRingTriangles(VertexROI, true);
	}
}


void UDynamicMeshSculptTool::UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode)
{
	if (MaterialMode == EMeshEditingMaterialModes::ExistingMaterial)
	{
		DynamicMeshComponent->ClearOverrideRenderMaterial();
		DynamicMeshComponent->SetShadowsEnabled(UE::ToolTarget::GetTargetComponent(Target)->bCastDynamicShadow);
		ActiveOverrideMaterial = nullptr; 
	}
	else 
	{
		if (MaterialMode == EMeshEditingMaterialModes::Custom)
		{
			if (ViewProperties->CustomMaterial.IsValid())
			{
				ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(ViewProperties->CustomMaterial.Get(), this);
			}
			else
			{
				DynamicMeshComponent->ClearOverrideRenderMaterial();
				ActiveOverrideMaterial = nullptr;
			}
		}
		else if (MaterialMode == EMeshEditingMaterialModes::CustomImage)
		{
			ActiveOverrideMaterial = ToolSetupUtil::GetCustomImageBasedSculptMaterial(GetToolManager(), ViewProperties->Image);
			if (ViewProperties->Image != nullptr)
			{
				ActiveOverrideMaterial->SetTextureParameterValue(TEXT("ImageTexture"), ViewProperties->Image);
			}
		}
		else if (MaterialMode == EMeshEditingMaterialModes::Transparent)
		{
			ActiveOverrideMaterial = ToolSetupUtil::GetTransparentSculptMaterial(GetToolManager(), 
				ViewProperties->TransparentMaterialColor, ViewProperties->Opacity, ViewProperties->bTwoSided);
		}
		else
		{
			UMaterialInterface* SculptMaterial = nullptr;
			switch (MaterialMode)
			{
			case EMeshEditingMaterialModes::Diffuse:
				SculptMaterial = ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager());
				break;
			case EMeshEditingMaterialModes::Grey:
				SculptMaterial = ToolSetupUtil::GetImageBasedSculptMaterial(GetToolManager(), ToolSetupUtil::ImageMaterialType::DefaultBasic);
				break;
			case EMeshEditingMaterialModes::Soft:
				SculptMaterial = ToolSetupUtil::GetImageBasedSculptMaterial(GetToolManager(), ToolSetupUtil::ImageMaterialType::DefaultSoft);
				break;
			case EMeshEditingMaterialModes::TangentNormal:
				SculptMaterial = ToolSetupUtil::GetImageBasedSculptMaterial(GetToolManager(), ToolSetupUtil::ImageMaterialType::TangentNormalFromView);
				break;
			case EMeshEditingMaterialModes::VertexColor:
				SculptMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager());
				break;
			}
			if (SculptMaterial != nullptr)
			{
				ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(SculptMaterial, this);
			}
		}

		if (ActiveOverrideMaterial != nullptr)
		{
			DynamicMeshComponent->SetOverrideRenderMaterial(ActiveOverrideMaterial);
			ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (ViewProperties->bFlatShading) ? 1.0f : 0.0f);
		}

		DynamicMeshComponent->SetShadowsEnabled(false);
	}
}


void UDynamicMeshSculptTool::UpdateFlatShadingSetting(bool bNewValue)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (bNewValue) ? 1.0f : 0.0f);
	}
}


void UDynamicMeshSculptTool::UpdateColorSetting(FLinearColor NewColor)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetVectorParameterValue(TEXT("Color"), NewColor);
	}
}

void UDynamicMeshSculptTool::UpdateOpacitySetting(double Opacity)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetScalarParameterValue(TEXT("Opacity"), Opacity);
	}
}

void UDynamicMeshSculptTool::UpdateTwoSidedSetting(bool bOn)
{
	ActiveOverrideMaterial = ToolSetupUtil::GetTransparentSculptMaterial(GetToolManager(),
		ViewProperties->TransparentMaterialColor, ViewProperties->Opacity, bOn);
	if (ActiveOverrideMaterial)
	{
		DynamicMeshComponent->SetOverrideRenderMaterial(ActiveOverrideMaterial);
	}
}

void UDynamicMeshSculptTool::UpdateCustomMaterial(TWeakObjectPtr<UMaterialInterface> NewMaterial)
{
	if (ViewProperties->MaterialMode == EMeshEditingMaterialModes::Custom)
	{
		if (NewMaterial.IsValid())
		{
			ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(NewMaterial.Get(), this);
			DynamicMeshComponent->SetOverrideRenderMaterial(ActiveOverrideMaterial);
		}
		else
		{
			DynamicMeshComponent->ClearOverrideRenderMaterial();
			ActiveOverrideMaterial = nullptr;
		}
	}
}

void UDynamicMeshSculptTool::UpdateImageSetting(UTexture2D* NewImage)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetTextureParameterValue(TEXT("ImageTexture"), NewImage);
	}
}

void UDynamicMeshSculptTool::UpdateBrushType(EDynamicMeshSculptBrushType BrushType)
{
	static const FText BaseMessage = LOCTEXT("OnStartSculptTool", "Hold Shift to Smooth, Ctrl to Invert (where applicable). [/] and S/D change Size (+Shift to small-step), W/E changes Strength.");
	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	SetToolPropertySourceEnabled(GizmoProperties, false);
	SetToolPropertySourceEnabled(SculptMaxBrushProperties, false);

	if (BrushType == EDynamicMeshSculptBrushType::FixedPlane)
	{
		Builder.AppendLine(LOCTEXT("FixedPlaneTip", "Use T to reposition Work Plane at cursor, Shift+T to align to Normal, Ctrl+Shift+T to align to View"));
		SetToolPropertySourceEnabled(GizmoProperties, true);
	}
	if (BrushType == EDynamicMeshSculptBrushType::SculptMax)
	{
		SetToolPropertySourceEnabled(SculptMaxBrushProperties, true);
	}

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}


void UDynamicMeshSculptTool::SetFixedSculptPlaneFromWorldPos(const FVector& Position, const FVector& Normal, EPendingWorkPlaneUpdate UpdateType)
{
	if (UpdateType == EPendingWorkPlaneUpdate::MoveToHitPositionNormal)
	{
		UpdateFixedSculptPlanePosition(Position);
		FFrame3d CurFrame(FVector::ZeroVector, GizmoProperties->Rotation);
		CurFrame.AlignAxis(2, (FVector3d)Normal );
		UpdateFixedSculptPlaneRotation( (FQuat)CurFrame.Rotation );
	}
	else if (UpdateType == EPendingWorkPlaneUpdate::MoveToHitPositionViewAligned)
	{
		UpdateFixedSculptPlanePosition(Position);
		FFrame3d CurFrame(FVector::ZeroVector, GizmoProperties->Rotation);
		CurFrame.AlignAxis(2, -(FVector3d)CameraState.Forward());
		UpdateFixedSculptPlaneRotation((FQuat)CurFrame.Rotation);
	}
	else
	{
		UpdateFixedSculptPlanePosition(Position);
	}

	if (PlaneTransformGizmo != nullptr)
	{
		PlaneTransformGizmo->SetNewGizmoTransform(FTransform(GizmoProperties->Rotation, GizmoProperties->Position));
	}
}

void UDynamicMeshSculptTool::PlaneTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	UpdateFixedSculptPlaneRotation(Transform.GetRotation());
	UpdateFixedSculptPlanePosition(Transform.GetLocation());
}

void UDynamicMeshSculptTool::UpdateFixedSculptPlanePosition(const FVector& Position)
{
	GizmoProperties->Position = Position;
	GizmoPositionWatcher.SilentUpdate();
}

void UDynamicMeshSculptTool::UpdateFixedSculptPlaneRotation(const FQuat& Rotation)
{
	GizmoProperties->Rotation = Rotation;
	GizmoRotationWatcher.SilentUpdate();
}

void UDynamicMeshSculptTool::UpdateGizmoFromProperties()
{
	if (PlaneTransformGizmo != nullptr)
	{
		PlaneTransformGizmo->SetNewGizmoTransform(FTransform(GizmoProperties->Rotation, GizmoProperties->Position));
	}
}

void UDynamicMeshSculptTool::UpdateFixedPlaneGizmoVisibility(bool bVisible)
{
	if (bVisible == false)
	{
		if (PlaneTransformGizmo != nullptr)
		{
			GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(PlaneTransformGizmo);
			PlaneTransformGizmo = nullptr;
		}
	}
	else
	{
		if (PlaneTransformGizmo == nullptr)
		{
			PlaneTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(),
				ETransformGizmoSubElements::StandardTranslateRotate, this);
			PlaneTransformGizmo->bUseContextCoordinateSystem = false;
			PlaneTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
			PlaneTransformGizmo->SetActiveTarget(PlaneTransformProxy, GetToolManager());
			PlaneTransformGizmo->ReinitializeGizmoTransform(FTransform(GizmoProperties->Rotation, GizmoProperties->Position));
		}
	}
}



void UDynamicMeshSculptTool::DiscardAttributes()
{
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> BeforeMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*DynamicMeshComponent->GetMesh());
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> AfterMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*BeforeMesh);
	// Reset attributes and compute per-vertex normals
	// (we need to clear rather than permanently discard the attributes because UDynamicMesh::EditMeshInternal would automatically add them back anyway)
	AfterMesh->DiscardAttributes();
	AfterMesh->EnableAttributes();
	FMeshNormals::InitializeOverlayToPerVertexNormals(AfterMesh->Attributes()->PrimaryNormals(), false);

	TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(BeforeMesh, AfterMesh);
	DynamicMeshComponent->ApplyChange(ReplaceChange.Get(), false);

	// other internals are still valid?

	GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(ReplaceChange), LOCTEXT("SculptDiscardAttribsChange", "Discard Attributes"));

	// The tool's warning messages about seams are no longer relevant after discarding attributes, so clear warning messages
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
}

#undef LOCTEXT_NAMESPACE

