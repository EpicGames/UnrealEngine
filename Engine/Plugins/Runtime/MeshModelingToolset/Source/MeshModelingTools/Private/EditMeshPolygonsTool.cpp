// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditMeshPolygonsTool.h"

#include "Algo/ForEach.h"
#include "Algo/Reverse.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "CompGeom/PolygonTriangulation.h"
#include "ConstrainedDelaunay2.h"
#include "Components/BrushComponent.h"
#include "ContextObjectStore.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshIndexUtil.h" // TriangleToVertexIDs
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "FaceGroupUtil.h"
#include "GroupTopology.h"
#include "Input/Reply.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "MeshBoundaryLoops.h"
#include "MeshOpPreviewHelpers.h" // UMeshOpPreviewWithBackgroundCompute
#include "MeshRegionBoundaryLoops.h"
#include "ModelingToolTargetUtil.h" // UE::ToolTarget:: functions
#include "Operations/SimpleHoleFiller.h"
#include "Operations/MinimalHoleFiller.h"
#include "Operations/PolygroupRemesh.h"
#include "Operations/WeldEdgeSequence.h"
#include "Operations/LocalPlanarSimplify.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Selections/MeshConnectedComponents.h"
#include "Styling/AppStyle.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolActivities/PolyEditExtrudeActivity.h"
#include "ToolActivities/PolyEditExtrudeEdgeActivity.h"
#include "ToolActivities/PolyEditInsertEdgeActivity.h"
#include "ToolActivities/PolyEditInsertEdgeLoopActivity.h"
#include "ToolActivities/PolyEditInsetOutsetActivity.h"
#include "ToolActivities/PolyEditCutFacesActivity.h"
#include "ToolActivities/PolyEditPlanarProjectionUVActivity.h"
#include "ToolActivities/PolyEditBevelEdgeActivity.h"
#include "ToolContextInterfaces.h" // FToolBuilderState
#include "ToolHostCustomizationAPI.h"
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"
#include "TransformTypes.h"
#include "Util/CompactMaps.h"
#include "Selections/GeometrySelection.h"
#include "Selection/GeometrySelectionManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditMeshPolygonsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UEditMeshPolygonsTool"

namespace EditMeshPolygonsToolLocals
{
	FText PolyEditDefaultMessage = LOCTEXT("OnStartEditMeshPolygonsTool_TriangleMode", "Select triangles to edit mesh. Use middle mouse on gizmo to "
		"reposition it. Hold Ctrl while translating or (in local mode) rotating to align to scene. Shift and Ctrl "
		"change marquee select behavior. Ctrl+R toggles Gizmo Orientation Lock.");

	FText TriEditDefaultMessage = LOCTEXT("OnStartEditMeshPolygonsTool", "Select PolyGroups to edit mesh. Use middle mouse on gizmo to reposition it. "
		"Hold Ctrl while translating or (in local mode) rotating to align to scene. Shift and Ctrl change marquee select "
		"behavior. Ctrl+R toggles Gizmo Orientation Lock.");

	FText WeldIncompleteMessage = LOCTEXT("OnWeldEdgesCompletedSeamsRemain", "Warning: welding incomplete because it would create "
		"invalid geometry (attached non manifold edge or duplicate triangle). Seam still exists at weld "
		"location. Modify attached triangles and retry, or undo.");

	FText PartialCollapseFailureMessage = LOCTEXT("OnCollapseFailures", "Some edges could not be collapsed, "
		"likely because adjoining edges would then have non manifold geometry (more than two faces), or "
		"the mesh would end up empty.");
	FText CollapseEdgeTransactionLabel = LOCTEXT("PolyMeshCollapseChange", "Collapse Edges");

	FString GetPropertyCacheIdentifier(bool bTriangleMode)
	{
		return bTriangleMode ? TEXT("TriEditTool") : TEXT("PolyEditTool");
	}

	TAutoConsoleVariable<int32> CVarEdgeLimit(
		TEXT("modeling.PolyEdit.EdgeLimit"),
		60000,
		TEXT("Maximal number of edges that PolyEd and TriEd support. Meshes that would require "
			"more than this number of edges to be rendered in PolyEd or TriEd force the tools to "
			"be disabled to avoid hanging the editor."));

	bool bAllowBowtieWeldAtInternalVertex = false;
	static FAutoConsoleVariableRef CVarAllowWeldInternalBowtie(
		TEXT("modeling.PolyEdit.AllowWeldInternalBowtie"),
		bAllowBowtieWeldAtInternalVertex,
		TEXT("If true, \"Weld\" and \"Weld To\" operations on a pair of vertices will allow the creation "
			"of non-boundary bowties. If false, then the vertices in these situations will not be welded, "
			"and will instead be moved to the destination."));

	// Allows undo/redo of addition of extra corners in the group topology based on user angle thresholds.
	// Used after user-triggered topology corner changes where the mesh was not actually edited.
	class FExtraCornerChange : public FToolCommandChange
	{
	public:
		FExtraCornerChange(const TSet<int32>& BeforeIn, const TSet<int32>& AfterIn)
			: Before(BeforeIn)
			, After(AfterIn)
		{
		}
		virtual void Apply(UObject* Object) override 
		{
			Cast<UEditMeshPolygonsTool>(Object)->RebuildTopologyWithGivenExtraCorners(After);
		}
		virtual void Revert(UObject* Object) override
		{
			Cast<UEditMeshPolygonsTool>(Object)->RebuildTopologyWithGivenExtraCorners(Before);
		}
		virtual bool HasExpired(UObject* Object) const override
		{
			return false;
		}
		virtual FString ToString() const override
		{
			return TEXT("FExtraCornerChange");
		}

	protected:
		TSet<int32> Before;
		TSet<int32> After;
	};

	/**
	 * Creates a group edge selection out of a group corner selection by selecting 
	 *  those edges whose endpoints are BOTH selected.
	 */
	void ConvertCornerSelectionToGroupEdgeSelection(const FGroupTopology* Topology, const TSet<int32>& CornerIDs, TSet<int32>& GroupEdgeIDs)
	{
		for (int32 CornerID : CornerIDs)
		{
			Topology->ForCornerNbrEdges(CornerID, [CornerID, &CornerIDs, &GroupEdgeIDs, Topology](int32 EdgeID)
			{
				if (CornerIDs.Contains(Topology->Edges[EdgeID].EndpointCorners.A)
					&& CornerIDs.Contains(Topology->Edges[EdgeID].EndpointCorners.B))
				{
					GroupEdgeIDs.Add(EdgeID);					
				}
				return true;
			});
		}
	}

	// TODO: Note that so far, simply converting our selection sets to arrays via set.Array() has
	//  been sufficient to get a selection order, but that only works due to TSet storing its
	//  elements in a sparse array, and seems likely to break depending on how the set is updated.
	//  For now this is good enough, but we may need to store selection order some other way someday.
	/** 
	 * Attempts to link boundary edges together to create either two separate boundaries, or
	 *   one boundary loop.
	 * 
	 * @param GroupEdgesIn Input edges. The order of the array affects which component ends up in 
	 *  GroupEdgesAOut, and the output of bShouldReverseAForIteration. 
	 * @param GroupEdgesAOut This will always have the first edge in GroupEdgesIn. If the result was
	 *  a loop, it will be the only one with edges.
	 * @param GroupEdgesBOut The other boundary, if result was not a loop.
	 * @param bShouldReverseAForIteration If iterating pairwise across the two groups, one of the
	 *  arrays needs reversing, since the boundary orientation will orient the sequences in opposite
	 *  directions. bShouldReverseAForIteration says that this array should be GroupEdgesAOut rather
	 *  than GroupEdgesBOut based on the selection order of the longer sequence.
	 * @return true if the edges were able to be partitioned either into one boundary loop, 
	 *  or two separate boundaries.
	 */
	bool LinkBoundaryGroupEdges(const FGroupTopology* Topology, const FDynamicMesh3* Mesh,
		const TArray<int32>& GroupEdgesIn, TArray<int32>& GroupEdgesAOut, TArray<int32>& GroupEdgesBOut, 
		bool& bShouldReverseAForIteration)
	{
		if (GroupEdgesIn.IsEmpty() || !Topology || !Mesh)
		{
			return false;
		}

		bShouldReverseAForIteration = false;
		if (GroupEdgesIn.Num() == 1)
		{
			GroupEdgesAOut.Add(GroupEdgesIn[0]);
			return Topology->IsIsolatedLoop(GroupEdgesAOut[0]);
		}

		for (int32 GroupEdgeID : GroupEdgesIn)
		{
			if (Topology->IsIsolatedLoop(GroupEdgeID) || !Topology->IsBoundaryEdge(GroupEdgeID))
			{
				return false;
			}
		}

		if (GroupEdgesIn.Num() == 2)
		{
			GroupEdgesAOut.Add(GroupEdgesIn[0]);
			GroupEdgesBOut.Add(GroupEdgesIn[1]);
			return true;
		}

		// Build a graph through start/end vids of the edges.

		// GroupID to start vid and end vid pair
		TMap<int32, FIndex2i> EdgeToStartEnd;
		// start/end vid to edge id. The bool is true if start.
		TMap<TPair<int32, bool>, int32> StartEndToEdge;
		for (int32 GroupEdgeID : GroupEdgesIn)
		{
			const FEdgeSpan& Span = Topology->Edges[GroupEdgeID].Span;
			FIndex2i OrientedEdgeVids = Mesh->GetOrientedBoundaryEdgeV(Span.Edges[0]);
			bool bReversed = OrientedEdgeVids.A != Span.Vertices[0];

			TPair<int32, bool> KeyForFirst(Span.Vertices[0], !bReversed);
			TPair<int32, bool> KeyForLast(Span.Vertices.Last(), bReversed);
			if (StartEndToEdge.Contains(KeyForFirst) || StartEndToEdge.Contains(KeyForLast))
			{
				// This means that a vertex was the end or start point for more than one edge,
				//  i.e. there was a branch. So, there is ambiguity in how to partition.
				return false;
			}
			StartEndToEdge.Add(KeyForFirst, GroupEdgeID);
			StartEndToEdge.Add(KeyForLast, GroupEdgeID);
			EdgeToStartEnd.Add(GroupEdgeID, bReversed ? FIndex2i(Span.Vertices.Last(), Span.Vertices[0])
				: FIndex2i(Span.Vertices[0], Span.Vertices.Last()));
		}

		TSet<int32> PartitionedEdges;
		// Helper that gets all the connected edges from a given edge, in order.
		auto GetEdgeSequence = [&PartitionedEdges, &EdgeToStartEnd, &StartEndToEdge](int32 StartEdge, TArray<int32>& EdgeSequenceOut)
		{
			bool bAlreadyProcessed = false;
			PartitionedEdges.Add(StartEdge, &bAlreadyProcessed);
			if (bAlreadyProcessed) return;

			// Go backwards and forwards through the graph to get our adjoining edges.
			// We'll start by going backwards (we'll reverse this output in a bit so that it is in the correct order)
			FIndex2i StartEnd = EdgeToStartEnd[StartEdge];
			int32 CurrentEndpoint = StartEnd.A;
			while (int32* CurrentEdge = StartEndToEdge.Find(TPair<int32, bool>(CurrentEndpoint, false)))
			{
				PartitionedEdges.Add(*CurrentEdge, &bAlreadyProcessed);
				if (bAlreadyProcessed) break;

				EdgeSequenceOut.Add(*CurrentEdge);
				CurrentEndpoint = EdgeToStartEnd[*CurrentEdge].A;
			}
			Algo::Reverse(EdgeSequenceOut);

			// Now that we have the preceding edges, add this one and search forwards
			EdgeSequenceOut.Add(StartEdge);
			CurrentEndpoint = StartEnd.B;
			while (int32* CurrentEdge = StartEndToEdge.Find(TPair<int32, bool>(CurrentEndpoint, true)))
			{
				PartitionedEdges.Add(*CurrentEdge, &bAlreadyProcessed);
				if (bAlreadyProcessed) break;

				EdgeSequenceOut.Add(*CurrentEdge);
				CurrentEndpoint = EdgeToStartEnd[*CurrentEdge].B;
			}
		};

		for (int32 GroupEdgeID : GroupEdgesIn)
		{
			if (PartitionedEdges.Contains(GroupEdgeID))
			{
				continue;
			}
			if (GroupEdgesAOut.IsEmpty())
			{
				GetEdgeSequence(GroupEdgeID, GroupEdgesAOut);
			}
			else if (GroupEdgesBOut.IsEmpty())
			{
				GetEdgeSequence(GroupEdgeID, GroupEdgesBOut);
			}
			else
			{
				// Had a third connected component
				return false;
			}
		}

		if (GroupEdgesBOut.IsEmpty())
		{
			// Make sure that the output result is a loop
			return !GroupEdgesAOut.IsEmpty()
				&& EdgeToStartEnd[GroupEdgesAOut[0]].A == EdgeToStartEnd[GroupEdgesAOut.Last()].B;
		}

		// Figure out the preferred iteration order based on the longer subsequence.
		// The issue is this: if we have edges A, B, and C in EdgesA and corresponding edges 1, 2, and 3 on the other
		//  side, then the latter will be ordered 3, 2, 1 in EdgesB according to triangle orientations, and as long
		//  as we reverse one group or the other, pairwise iteration will give us the correct pairings (through which
		//  we will iterate either as A1, B2, C3, or as C3, B2, A1, depending on whether we reverse EdgesB or EdgesA,
		//  respectively). However, what happens if we have a mismatched number of edges, e.g. edge D after C? We will 
		//  group the extra edge(s) with the last edge in the shorter sequence, so iteration order matters: either we 
		//  end up grouping CD with 3 if we reverse GroupB, or we end up grouping AB with 1 if we reverse GroupA.
		// We choose to decide based on which of the ends of the longer sequence was selected last- this is the direction
		//  that we interpret the user wanting to iterate in. E.g., if user selected D after they selected A, then we decide
		//  that the iteration order should be A1, B2, CD3.
		if (ensure(!GroupEdgesAOut.IsEmpty()))
		{
			if (GroupEdgesAOut.Num() > GroupEdgesBOut.Num())
			{
				// Reverse if the later edge in the sequence was selected earlier the first
				bShouldReverseAForIteration = GroupEdgesIn.IndexOfByKey(GroupEdgesAOut.Last()) < GroupEdgesIn.IndexOfByKey(GroupEdgesAOut[0]);
			}
			else if (GroupEdgesBOut.Num() > GroupEdgesAOut.Num())
			{
				// The comparison is backwards here because bShouldReverseAForIteration needs to be the opposite of 
				//  whether we should reverse EdgesB.
				bShouldReverseAForIteration = GroupEdgesIn.IndexOfByKey(GroupEdgesBOut[0]) < GroupEdgesIn.IndexOfByKey(GroupEdgesBOut.Last());
			}
		}
		
		// Make sure that both partitions are not loops
		return !GroupEdgesAOut.IsEmpty()
			&& EdgeToStartEnd[GroupEdgesAOut[0]].A != EdgeToStartEnd[GroupEdgesAOut.Last()].B
			&& EdgeToStartEnd[GroupEdgesBOut[0]].A != EdgeToStartEnd[GroupEdgesBOut.Last()].B;
	}

	// Helper to share the retriangulation code
	int32 RetriangulateGroups(FDynamicMesh3* Mesh, FGroupTopology* Topology, TSet<int32> GroupIDs, FDynamicMeshChangeTracker& ChangeTracker)
	{
		FDynamicMeshEditor Editor(Mesh);
		int32 NumCompleted = 0;
		for (int32 GroupID : GroupIDs)
		{
			const TArray<int32>& Triangles = Topology->GetGroupTriangles(GroupID);
			ChangeTracker.SaveTriangles(Triangles, true);
			FMeshRegionBoundaryLoops RegionLoops(Mesh, Triangles, true);
			if (!RegionLoops.bFailed && RegionLoops.Loops.Num() == 1 && Triangles.Num() > 1)
			{
				TArray<FMeshRegionBoundaryLoops::VidOverlayMap<FVector2f>> VidUVMaps;
				if (Mesh->HasAttributes())
				{
					const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
					for (int i = 0; i < Attributes->NumUVLayers(); ++i)
					{
						VidUVMaps.Emplace();
						RegionLoops.GetLoopOverlayMap(RegionLoops.Loops[0], *Attributes->GetUVLayer(i), VidUVMaps.Last());
					}
				}

				// We don't want to remove isolated vertices while removing triangles because we don't
				// want to throw away boundary verts. However, this means that we'll have to go back
				// through these vertices later to throw away isolated internal verts.
				TArray<int32> OldVertices;
				UE::Geometry::TriangleToVertexIDs(Mesh, Triangles, OldVertices);
				Editor.RemoveTriangles(Topology->GetGroupTriangles(GroupID), false);

				RegionLoops.Loops[0].Reverse();
				FSimpleHoleFiller Filler(Mesh, RegionLoops.Loops[0]);
				Filler.FillType = FSimpleHoleFiller::EFillType::PolygonEarClipping;
				Filler.Fill(GroupID);

				// Throw away any of the old verts that are still isolated (they were in the interior of the group)
				Algo::ForEachIf(OldVertices,
					[Mesh](int32 Vid)
					{
						return !Mesh->IsReferencedVertex(Vid);
					},
					[Mesh](int32 Vid)
					{
						checkSlow(!Mesh->IsReferencedVertex(Vid));
						constexpr bool bPreserveManifold = false;
						Mesh->RemoveVertex(Vid, bPreserveManifold);
					});

				if (Mesh->HasAttributes())
				{
					const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
					for (int i = 0; i < Attributes->NumUVLayers(); ++i)
					{
						RegionLoops.UpdateLoopOverlayMapValidity(VidUVMaps[i], *Attributes->GetUVLayer(i));
					}
					Filler.UpdateAttributes(VidUVMaps);
				}

				NumCompleted++;
			}
		}
		return NumCompleted;
	}//end RetriangulateGroups

	// Helper that removes the triangles around an edge as long as they are not the last
	//  ones in the mesh. Used to allow collapses of isolated triangles and quads, which
	//  are not currently permitted by CollapseEdge.
	// TODO: We should probably have a permissiveness option that does allow this in
	//  CollapseEdge, though it should be noted that the kept vert may end up deleted
	//  in that case.
	bool RemoveEdgeTrisIfNotLast(FDynamicMesh3& Mesh, int32 Eid)
	{
		if (!Mesh.IsEdge(Eid))
		{
			return false;
		}

		FIndex2i EdgeTids = Mesh.GetEdgeT(Eid);

		if (Mesh.TriangleCount() > 2
			|| (Mesh.TriangleCount() > 1 && EdgeTids.B == IndexConstants::InvalidID))
		{
			Mesh.RemoveTriangle(EdgeTids.A);
			if (EdgeTids.B != IndexConstants::InvalidID)
			{
				Mesh.RemoveTriangle(EdgeTids.B);
			}
			return true;
		}
		return false;
	}
}

/*
 * ToolBuilder
 */


USingleTargetWithSelectionTool* UEditMeshPolygonsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UEditMeshPolygonsTool>(SceneState.ToolManager);
}

void UEditMeshPolygonsToolBuilder::InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const
{
	USingleTargetWithSelectionToolBuilder::InitializeNewTool(Tool, SceneState);
	UEditMeshPolygonsTool* EditPolygonsTool = CastChecked<UEditMeshPolygonsTool>(Tool);
	if (bTriangleMode)
	{
		EditPolygonsTool->EnableTriangleMode();
	}
}


bool UEditMeshPolygonsActionModeToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (UEditMeshPolygonsToolBuilder::CanBuildTool(SceneState))
	{
		if ( UGeometrySelectionManager* SelectionManager = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>() )
		{
			EGeometryTopologyType TopologyType = EGeometryTopologyType::Triangle;
			EGeometryElementType ElementType = EGeometryElementType::Face;
			int NumTargets;
			bool bIsEmpty = false;
			SelectionManager->GetActiveSelectionInfo(TopologyType, ElementType, NumTargets, bIsEmpty);

			// Default to Polygroup topology type if no topology mode selected. GetActiveSelectionInfo will return Triangle in this case.
			if (SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None)
			{
				TopologyType = EGeometryTopologyType::Polygroup;
			}

			bool bCanBuild = false;
			if (StartupAction == EEditMeshPolygonsToolActions::Extrude
				|| StartupAction == EEditMeshPolygonsToolActions::PushPull
				|| StartupAction == EEditMeshPolygonsToolActions::Offset
				|| StartupAction == EEditMeshPolygonsToolActions::Inset
				|| StartupAction == EEditMeshPolygonsToolActions::Outset
				|| StartupAction == EEditMeshPolygonsToolActions::CutFaces)
			{
				bCanBuild = (TopologyType == EGeometryTopologyType::Polygroup && ElementType == EGeometryElementType::Face && bIsEmpty == false);
			}
			else if (StartupAction == EEditMeshPolygonsToolActions::BevelAuto)
			{
				bCanBuild = (TopologyType == EGeometryTopologyType::Polygroup && ElementType != EGeometryElementType::Vertex && bIsEmpty == false);
			}
			else if (StartupAction == EEditMeshPolygonsToolActions::ExtrudeEdges)
			{
				bCanBuild = (ElementType == EGeometryElementType::Edge && !bIsEmpty);
			}
			else if ( StartupAction == EEditMeshPolygonsToolActions::SimplifyByGroups
			 || StartupAction == EEditMeshPolygonsToolActions::InsertEdge
			 || StartupAction == EEditMeshPolygonsToolActions::InsertEdgeLoop )
			{
				bCanBuild = (TopologyType == EGeometryTopologyType::Polygroup);
			}
			return bCanBuild;
		}
	}
	return false;
}


void UEditMeshPolygonsActionModeToolBuilder::InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const
{
	UEditMeshPolygonsToolBuilder::InitializeNewTool(Tool, SceneState);

	// Need to enable triangle mode on the tool if our selection was a triangle (not group) selection.
	// This normally gets done in the base class if bTriangleMode is true, but we can't change that in a
	// const method.
	if (UGeometrySelectionManager* SelectionManager = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>())
	{
		// Note that we don't use GetActiveSelectionInfo here because that defaults to Triangle in the
		// None/Object selection case and we want this tool to default to polygroup.
		if (SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::Triangle)
		{
			if (UEditMeshPolygonsTool* EditPolygonsTool = Cast<UEditMeshPolygonsTool>(Tool))
			{
				EditPolygonsTool->EnableTriangleMode();
			}
		}
	}

	UEditMeshPolygonsTool* EditPolygonsTool = CastChecked<UEditMeshPolygonsTool>(Tool);

	EEditMeshPolygonsToolActions UseAction = StartupAction;
	EditPolygonsTool->PostSetupFunction = [UseAction](UEditMeshPolygonsTool* PolyTool)
	{
		PolyTool->SetToSelectionModeInterface();
		PolyTool->RequestSingleShotAction(UseAction);
	};
}



bool UEditMeshPolygonsSelectionModeToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (UEditMeshPolygonsToolBuilder::CanBuildTool(SceneState))
	{
		if ( UGeometrySelectionManager* SelectionManager = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>() )
		{
			// if not actively selecting mesh components, tool can be started in 'standard' full-PolyEd mode
			if (SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None)
			{
				return true;
			}
			// otherwise can only start tool in sub-modes
			EGeometryTopologyType TopologyType = EGeometryTopologyType::Triangle;
			EGeometryElementType ElementType = EGeometryElementType::Face;
			int NumTargets;
			bool bIsEmpty = false;
			SelectionManager->GetActiveSelectionInfo(TopologyType, ElementType, NumTargets, bIsEmpty);
			if (TopologyType == EGeometryTopologyType::Polygroup)
			{
				return ElementType != EGeometryElementType::Vertex;
			}
		}
	}
	return false;
}

void UEditMeshPolygonsSelectionModeToolBuilder::InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const
{
	UEditMeshPolygonsToolBuilder::InitializeNewTool(Tool, SceneState);
	UEditMeshPolygonsTool* EditPolygonsTool = CastChecked<UEditMeshPolygonsTool>(Tool);

	// if not actively selecting mesh components, start in full-PolyEd mode
	if (UGeometrySelectionManager* SelectionManager = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>())
	{
		if (SelectionManager->GetMeshTopologyMode() == UGeometrySelectionManager::EMeshTopologyMode::None)
		{
			return;
		}

		// otherwise can only start tool in sub-modes
		EGeometryTopologyType TopologyType = EGeometryTopologyType::Triangle;
		EGeometryElementType ElementType = EGeometryElementType::Face;
		int NumTargets;
		bool bIsEmpty = false;
		SelectionManager->GetActiveSelectionInfo(TopologyType, ElementType, NumTargets, bIsEmpty);
		if (TopologyType != EGeometryTopologyType::Polygroup)
		{
			return;		// should not happen...
		}

		EEditMeshPolygonsToolSelectionMode UseMode = EEditMeshPolygonsToolSelectionMode::Faces;
		bool bIsEdgeSelection = false;
		if (ElementType == EGeometryElementType::Edge)
		{
			bIsEdgeSelection = true;
			UseMode = EEditMeshPolygonsToolSelectionMode::Edges;
		}
		else if (ElementType == EGeometryElementType::Vertex)
		{
			UseMode = EEditMeshPolygonsToolSelectionMode::Vertices;
		}

		EditPolygonsTool->PostSetupFunction = [UseMode, bIsEdgeSelection](UEditMeshPolygonsTool* PolyTool)
		{
			PolyTool->SetToolPropertySourceEnabled(PolyTool->EditActions, !bIsEdgeSelection);
			PolyTool->SetToolPropertySourceEnabled(PolyTool->EditEdgeActions, bIsEdgeSelection);
			PolyTool->SetToolPropertySourceEnabled(PolyTool->EditUVActions, !bIsEdgeSelection);
			
			PolyTool->SetToolPropertySourceEnabled(PolyTool->TopologyProperties, false);

			UPolygonSelectionMechanic* SelectionMechanic = PolyTool->SelectionMechanic;
			UMeshTopologySelectionMechanicProperties* SelectionProps = SelectionMechanic->Properties;
			SelectionProps->bSelectFaces = SelectionProps->bSelectEdges = SelectionProps->bSelectVertices = false;
			SelectionProps->bSelectEdgeLoops = SelectionProps->bSelectEdgeRings = false;

			switch (UseMode)
			{
			default:
			case EEditMeshPolygonsToolSelectionMode::Faces:
				SelectionProps->bSelectFaces = true;
				break;
			case EEditMeshPolygonsToolSelectionMode::Edges:
				SelectionProps->bSelectEdges = true;
				break;
			case EEditMeshPolygonsToolSelectionMode::Vertices:
				SelectionProps->bSelectVertices = true;
				break;
			}

			PolyTool->SetToolPropertySourceEnabled(SelectionProps, false);
		};
	}



}


void UEditMeshPolygonsTool::SetToSelectionModeInterface()
{
	if (EditActions) SetToolPropertySourceEnabled(EditActions, false);
	if (EditEdgeActions) SetToolPropertySourceEnabled(EditEdgeActions, false);
	if (EditUVActions) SetToolPropertySourceEnabled(EditUVActions, false);
}



void UEditMeshPolygonsToolActionPropertySet::PostAction(EEditMeshPolygonsToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}

/*
* Tool methods
*/

UEditMeshPolygonsTool::UEditMeshPolygonsTool()
{
	SetToolDisplayName(LOCTEXT("EditMeshPolygonsToolName", "PolyGroup Edit"));
}

void UEditMeshPolygonsTool::EnableTriangleMode()
{
	check(Preview == nullptr);		// must not have been initialized!
	bTriangleMode = true;
}

void UEditMeshPolygonsTool::Setup()
{
	using namespace EditMeshPolygonsToolLocals;

	// TODO: Currently we draw all the edges in the tool with PDI and can lock up the editor on high-res meshes. 
	// As a hack, disable everything if the number of edges is too high, so that user doesn't lose work accidentally
	// if they start the tool on the wrong thing.
	int32 MaxEdges = CVarEdgeLimit.GetValueOnGameThread();

	CurrentMesh = MakeShared<FDynamicMesh3>(UE::ToolTarget::GetDynamicMeshCopy(Target));
	WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector ScaleToBake = WorldTransform.GetScale();
	BakedTransform = FTransformSRT3d(FQuaterniond::Identity(), FVector::Zero(), ScaleToBake);
	WorldTransform.SetScale(FVector::One());
	MeshTransforms::ApplyTransform(*CurrentMesh, BakedTransform, true);

	if (bTriangleMode)
	{
		bToolDisabled = CurrentMesh->EdgeCount() > MaxEdges;
		if (bToolDisabled)
		{
			GetToolManager()->DisplayMessage(FText::Format(
				LOCTEXT("TriEditTooManyEdges", 
					"This tool is currently disallowed from operating on a mesh of this resolution. "
					"Current limit set by \"modeling.PolyEdit.EdgeLimit\" is {0} edges, and mesh has "
					"{1}. Limit can be changed but exists to avoid hanging the editor when trying to "
					"render too many edges using the current system, so make sure to save your work "
					"if you change the upper limit and try to edit a very dense mesh."),
				MaxEdges, CurrentMesh->EdgeCount()), EToolMessageLevel::UserError);
			return;
		}

		Topology = MakeShared<FTriangleGroupTopology, ESPMode::ThreadSafe>(CurrentMesh.Get(), false);
	}
	else
	{
		Topology = MakeShared<FGroupTopology, ESPMode::ThreadSafe>(CurrentMesh.Get(), false);
		
		Topology->ShouldAddExtraCornerAtVert = 
			[this](const FGroupTopology& GroupTopology, int32 Vid, const FIndex2i& AttachedGroupEdgeEids)
		{
			// Note: it's important that we don't use CurrentMesh here. It's possible that an activity might create a copy of 
			// the topology that uses the same corner forcing function but points to a different mesh, so we want to use
			// whatever mesh the passed-in topology uses.
			return TopologyProperties->bAddExtraCorners 
				&& FGroupTopology::IsEdgeAngleSharp(GroupTopology.GetMesh(), Vid, AttachedGroupEdgeEids, ExtraCornerDotProductThreshold);
		};
	}

	TopologyProperties = NewObject<UPolyEditTopologyProperties>(this);
	TopologyProperties->Initialize(this);
	TopologyProperties->RestoreProperties(this, GetPropertyCacheIdentifier(bTriangleMode));

	auto UpdateExtraCornerThreshold = [this]() { ExtraCornerDotProductThreshold = FMathd::Cos(TopologyProperties->ExtraCornerAngleThresholdDegrees * FMathd::DegToRad); };
	UpdateExtraCornerThreshold();
	TopologyProperties->WatchProperty(TopologyProperties->ExtraCornerAngleThresholdDegrees,
		// Note: it may seem tempting to auto-rebuild the topology as the user drags the threshold slider (rather than waiting for
		// the button click or next topology rebuild), but we have to transact corner additions/removals so that selection events in the 
		// undo stack are able to refer to the same edges/corners (this is also why we store the current extra corners in FEditMeshPolygonsToolMeshChange).
		// In an ideal world, we would know the end of a slider drag and only transact at that point, but we don't have that.
		[this, UpdateExtraCornerThreshold](double) { UpdateExtraCornerThreshold(); });

	Topology->RebuildTopology();

	if (!bTriangleMode)
	{
		int32 NumEdgesToRender = 0;
		for (const FGroupTopology::FGroupEdge& Edge : Topology->Edges)
		{
			NumEdgesToRender += Edge.Span.Edges.Num();
		}

		bToolDisabled = NumEdgesToRender > MaxEdges;
		if (bToolDisabled)
		{
			GetToolManager()->DisplayMessage(FText::Format(
				LOCTEXT("PolyEditTooManyEdges",
					"This tool is currently disallowed from operating on a group topology of this resolution. "
					"Current limit set by \"modeling.PolyEdit.EdgeLimit\" is {0} displayed edges, and topology has "
					"{1} edge segments to display. Limit can be changed, but it exists to avoid hanging the editor "
					"when trying to render too many edges using the current system, so make sure to save your work "
					"if you change the upper limit and try to edit a very complicated topology."),
				MaxEdges, NumEdgesToRender), EToolMessageLevel::UserError);
			return;
		}
	}

	// Start by adding the actions, because we want them at the top.
	if (bTriangleMode)
	{
		EditActions_Triangles = NewObject<UEditMeshPolygonsToolActions_Triangles>();
		EditActions_Triangles->Initialize(this);
		AddToolPropertySource(EditActions_Triangles);

		EditEdgeActions_Triangles = NewObject<UEditMeshPolygonsToolEdgeActions_Triangles>();
		EditEdgeActions_Triangles->Initialize(this);
		AddToolPropertySource(EditEdgeActions_Triangles);

		SetToolDisplayName(LOCTEXT("EditMeshTrianglesToolName", "Triangle Edit"));
		DefaultMessage = PolyEditDefaultMessage;
	}
	else
	{
		EditActions = NewObject<UEditMeshPolygonsToolActions>();
		EditActions->Initialize(this);
		AddToolPropertySource(EditActions);

		EditEdgeActions = NewObject<UEditMeshPolygonsToolEdgeActions>();
		EditEdgeActions->Initialize(this);
		AddToolPropertySource(EditEdgeActions);

		EditUVActions = NewObject<UEditMeshPolygonsToolUVActions>();
		EditUVActions->Initialize(this);
		AddToolPropertySource(EditUVActions);

		DefaultMessage = TriEditDefaultMessage;
	}

	GetToolManager()->DisplayMessage(DefaultMessage,
		EToolMessageLevel::UserNotification);

	// We add an empty line for the error message so that things don't jump when we use it.
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

	// Initialize the common properties but don't add them yet, because we want them to be under the activity-specific ones.
	CommonProps = NewObject<UPolyEditCommonProperties>(this);
	CommonProps->RestoreProperties(this, GetPropertyCacheIdentifier(bTriangleMode));

	CommonProps->WatchProperty(CommonProps->LocalFrameMode,
		[this](ELocalFrameMode) { UpdateGizmoFrame(); });
	CommonProps->WatchProperty(CommonProps->bLockRotation,
		[this](bool) { LockedTransfomerFrame = LastTransformerFrame; });
	CommonProps->WatchProperty(CommonProps->bGizmoVisible,
		[this](bool)
		{
			if (!CurrentActivity)
			{
				UpdateGizmoVisibility();
				ResetUserMessage();
			}
		});

	// We are going to SilentUpdate here because otherwise the Watches above will immediately fire
	// and cause UpdateGizmoFrame() to be called emitting a spurious Transform change. 
	CommonProps->SilentUpdateWatched();

	// TODO: Do we need this?
	FMeshNormals::QuickComputeVertexNormals(*CurrentMesh);

	// Create the preview object
	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>();
	Preview->Setup(GetTargetWorld());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target); 
	Preview->PreviewMesh->SetTransform((FTransform)WorldTransform);

	// We'll use the spatial inside preview mesh mainly for the convenience of having it update automatically.
	Preview->PreviewMesh->bBuildSpatialDataStructure = true;

	// set materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	Preview->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

	// configure secondary render material
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor::Yellow, GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		// Note that you have to do it this way rather than reaching into the PreviewMesh because the background compute
		// mesh has to be able to swap in/out a working material and restore the primary/secondary ones.
		Preview->SecondaryMaterial = SelectionMaterial;
	}

	Preview->PreviewMesh->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return SelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, Topology.Get(), TriangleID);
	});

	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get());
	Preview->PreviewMesh->EnableWireframe(CommonProps->bShowWireframe);
	Preview->SetVisibility(true);

	// initialize AABBTree
	MeshSpatial = MakeShared<FDynamicMeshAABBTree3>();
	MeshSpatial->SetMesh(CurrentMesh.Get());

	// set up SelectionMechanic
	SelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false; // We'll do this ourselves later
	SelectionMechanic->Setup(this);
	SelectionMechanic->SetShowSelectableCorners(CommonProps->bShowSelectableCorners);
	SelectionMechanic->Properties->RestoreProperties(this, GetPropertyCacheIdentifier(bTriangleMode));
	SelectionMechanic->Properties->bDisplayPolygroupReliantControls = !bTriangleMode;
	SelectionMechanic->OnSelectionChanged.AddUObject(this, &UEditMeshPolygonsTool::OnSelectionModifiedEvent);
	SelectionMechanic->OnFaceSelectionPreviewChanged.AddWeakLambda(this, [this]() {
		Preview->PreviewMesh->FastNotifySecondaryTrianglesChanged();
	});
	if (bTriangleMode)
	{
		SelectionMechanic->PolyEdgesRenderer.LineThickness = 1.0;
	}
	SelectionMechanic->Initialize(CurrentMesh.Get(),
		(FTransform3d)Preview->PreviewMesh->GetTransform(),
		GetTargetWorld(),
		Topology.Get(),
		[this]() { return &GetSpatial(); }
	);

	LinearDeformer.Initialize(CurrentMesh.Get(), Topology.Get());

	// initialize our selection from input selection, if available 
	if (HasGeometrySelection())
	{
		const FGeometrySelection& CurSelection = GetGeometrySelection();
		// If the topology type doesn't match, we'll need to convert it here
		FGeometrySelection ConvertedSelection;
		const FGeometrySelection* UseSelection = &CurSelection;
		bool bCanUseSelection = true;
		// For polygroup edge selections, if the tool's polygroups have extra corners, need to convert to that
		if (!bTriangleMode && 
			CurSelection.TopologyType == EGeometryTopologyType::Polygroup && CurSelection.ElementType == EGeometryElementType::Edge && 
			TopologyProperties->bAddExtraCorners)
		{
			// Convert default (no corner) group topology -> triangle topology -> tool (w/ corner) group topology
			ConvertedSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Polygroup);
			FGeometrySelection TempTriSelection;
			TempTriSelection.InitializeTypes(EGeometryElementType::Edge, EGeometryTopologyType::Triangle);
			FGroupTopology GroupTopology(CurrentMesh.Get(), true);
			bCanUseSelection = UE::Geometry::ConvertSelection(*CurrentMesh, &GroupTopology, CurSelection, TempTriSelection, EEnumerateSelectionConversionParams::ContainSelection);
			bCanUseSelection = bCanUseSelection && UE::Geometry::ConvertSelection(*CurrentMesh, Topology.Get(), TempTriSelection, ConvertedSelection, EEnumerateSelectionConversionParams::ContainSelection);
			UseSelection = &ConvertedSelection;
		}
		// If topology type is triangle but we want polygroup, or vice versa, convert accordingly
		else if ((CurSelection.TopologyType == EGeometryTopologyType::Triangle) != bTriangleMode)
		{
			ConvertedSelection.InitializeTypes(CurSelection.ElementType, bTriangleMode ? EGeometryTopologyType::Triangle : EGeometryTopologyType::Polygroup);
			const FGroupTopology* UseTopology = Topology.Get();
			// We need a default topology to reference if we're converting from polygroup->triangle, since w/ Triangle Mode the tool's Topology has per-triangle groups
			FGroupTopology DefaultGroupTopology(CurrentMesh.Get(), false);
			if (bTriangleMode)
			{
				DefaultGroupTopology.RebuildTopology();
				UseTopology = &DefaultGroupTopology;
			}
			bCanUseSelection = UE::Geometry::ConvertSelection(*CurrentMesh, UseTopology, CurSelection, ConvertedSelection, EEnumerateSelectionConversionParams::ContainSelection);
			UseSelection = &ConvertedSelection;
		}
		if (bCanUseSelection && UseSelection->TopologyType == EGeometryTopologyType::Triangle && bTriangleMode)
		{
			SelectionMechanic->SetSelection_AsTriangleTopology(*UseSelection);
		}
		else if (bCanUseSelection && UseSelection->TopologyType == EGeometryTopologyType::Polygroup && bTriangleMode == false)
		{
			SelectionMechanic->SetSelection_AsGroupTopology(*UseSelection);
		}
	}

	bSelectionStateDirty = SelectionMechanic->HasSelection();

	// Set UV Scale factor based on initial mesh bounds
	float BoundsMaxDim = CurrentMesh->GetBounds().MaxDim();
	if (BoundsMaxDim > 0)
	{
		UVScaleFactor = 1.0 / BoundsMaxDim;
	}

	// Wrap the data structures into a context that we can give to the activities
	ActivityContext = NewObject<UPolyEditActivityContext>();
	ActivityContext->bTriangleMode = bTriangleMode;
	ActivityContext->CommonProperties = CommonProps;
	ActivityContext->CurrentMesh = CurrentMesh;
	ActivityContext->Preview = Preview;
	ActivityContext->CurrentTopology = Topology;
	ActivityContext->MeshSpatial = MeshSpatial;
	ActivityContext->SelectionMechanic = SelectionMechanic;
	ActivityContext->EmitActivityStart = [this](const FText& TransactionLabel)
	{
		EmitActivityStart(TransactionLabel);
	};
	ActivityContext->EmitCurrentMeshChangeAndUpdate = [this](const FText& TransactionLabel,
		TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChangeIn,
		const UE::Geometry::FGroupTopologySelection& OutputSelection) 
	{
		EmitCurrentMeshChangeAndUpdate(TransactionLabel, MoveTemp(MeshChangeIn), OutputSelection);
	};
	GetToolManager()->GetContextObjectStore()->RemoveContextObjectsOfType(UPolyEditActivityContext::StaticClass());
	GetToolManager()->GetContextObjectStore()->AddContextObject(ActivityContext);

	ExtrudeActivity = NewObject<UPolyEditExtrudeActivity>();
	ExtrudeActivity->Setup(this);
	// The icons/labels differ depending on whether we're doing extrude, offset, or push/pull, so
	// set those when we launch the activity.
	
	InsetOutsetActivity = NewObject<UPolyEditInsetOutsetActivity>();
	InsetOutsetActivity->Setup(this);
	// The icons/labels differ depending on whether we are doing an inset or outset, so we set those 
	// when we launch the activity.

	CutFacesActivity = NewObject<UPolyEditCutFacesActivity>();
	CutFacesActivity->Setup(this);
	ActivityLabels.Add(CutFacesActivity, LOCTEXT("CutFacesActivityLabel", "Cut Faces"));
	ActivityIconNames.Add(CutFacesActivity, "PolyEd.CutFaces");

	PlanarProjectionUVActivity = NewObject<UPolyEditPlanarProjectionUVActivity>();
	PlanarProjectionUVActivity->Setup(this);
	ActivityLabels.Add(PlanarProjectionUVActivity, LOCTEXT("UVProjectActivityLabel", "UV Project"));
	ActivityIconNames.Add(PlanarProjectionUVActivity, "PolyEd.ProjectUVs");

	InsertEdgeLoopActivity = NewObject<UPolyEditInsertEdgeLoopActivity>();
	InsertEdgeLoopActivity->Setup(this);
	ActivityLabels.Add(InsertEdgeLoopActivity, LOCTEXT("InsertEdgeLoopsActivityLabel", "Insert Edge Loops"));
	ActivityIconNames.Add(InsertEdgeLoopActivity, "PolyEd.InsertEdgeLoop");

	InsertEdgeActivity = NewObject<UPolyEditInsertEdgeActivity>();
	InsertEdgeActivity->Setup(this);
	ActivityLabels.Add(InsertEdgeActivity, LOCTEXT("InsertEdgesActivityLabel", "Insert Edges"));
	ActivityIconNames.Add(InsertEdgeActivity, "PolyEd.InsertGroupEdge");

	BevelEdgeActivity = NewObject<UPolyEditBevelEdgeActivity>();
	BevelEdgeActivity->Setup(this);
	ActivityLabels.Add(BevelEdgeActivity, LOCTEXT("BevelActivityLabel", "Bevel"));
	ActivityIconNames.Add(BevelEdgeActivity, "PolyEd.Bevel");

	ExtrudeEdgeActivity = NewObject<UPolyEditExtrudeEdgeActivity>();
	ExtrudeEdgeActivity->Setup(this);
	ActivityLabels.Add(ExtrudeEdgeActivity, LOCTEXT("EdgeExtrudeActivityLabel", "Extrude Edges"));
	ActivityIconNames.Add(ExtrudeEdgeActivity, "PolyEd.ExtrudeEdge");

	// Now that we've initialized the activities, add in the selection settings and 
	// CommonProps so that they are at the bottom.
	AddToolPropertySource(SelectionMechanic->Properties);
	AddToolPropertySource(CommonProps);
	if (!bTriangleMode)
	{
		AddToolPropertySource(TopologyProperties);
	}
	else
	{
		// Not actually necessary since we don't use the forcing function in triangle mode, but might as well turn it off here too.
		TopologyProperties->bAddExtraCorners = false;
	}

	// hide input StaticMeshComponent
	UE::ToolTarget::HideSourceObject(Target);

	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::FullTranslateRotateScale, this);
	if (ensure(TransformGizmo)) // If we don't get a valid gizmo a lot of interactions won't work, but at least we won't crash
	{
		// Stop scaling at 0 rather than going negative
		TransformGizmo->SetDisallowNegativeScaling(true);
		// We allow non uniform scale even when the gizmo mode is set to "world" because we're not scaling components- we're
		// moving vertices, so we don't care which axes we "scale" along.
		TransformGizmo->SetIsNonUniformScaleAllowedFunction([]() {
			return true;
			});

		// Hook up callbacks
		TransformProxy = NewObject<UTransformProxy>(this);
		TransformProxy->OnTransformChanged.AddUObject(this, &UEditMeshPolygonsTool::OnGizmoTransformChanged);
		TransformProxy->OnBeginTransformEdit.AddUObject(this, &UEditMeshPolygonsTool::OnBeginGizmoTransform);
		TransformProxy->OnEndTransformEdit.AddUObject(this, &UEditMeshPolygonsTool::OnEndGizmoTransform);
		TransformProxy->OnEndPivotEdit.AddWeakLambda(this, [this](UTransformProxy* Proxy) {
			LastTransformerFrame = FFrame3d(Proxy->GetTransform());
			if (CommonProps->bLockRotation)
			{
				LockedTransfomerFrame = LastTransformerFrame;
			}
			});
		TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());
		TransformGizmo->SetVisibility(false);
	}

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->InitializeDeformedMeshRayCast(
		[this]() { return &GetSpatial(); },
		WorldTransform, &LinearDeformer); // Should happen after LinearDeformer is initialized

	if (TransformGizmo)
	{
		DragAlignmentMechanic->AddToGizmo(TransformGizmo);
	}

	if (Topology->Groups.Num() < 2)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoGroupsWarning",
			        "This object has only a single Polygroup. Use the GrpGen, GrpPnt or TriSel (Create Polygroup) tools to modify PolyGroups."),
			EToolMessageLevel::UserWarning);
	}

	if (PostSetupFunction)
	{
		PostSetupFunction(this);
	}
}

void UEditMeshPolygonsTool::ResetUserMessage()
{
	// When the gizmo is hidden, notify the user and tell them how to fix it.
	if (!TransformGizmo->IsVisible())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("ToggleTransformGizmoNotify", "Transform gizmo hidden, unhide by toggling \"Gizmo Visible\" (or using hotkey, if set)"),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		GetToolManager()->DisplayMessage(
			DefaultMessage,
			EToolMessageLevel::UserNotification);
	}
}


void UEditMeshPolygonsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	using namespace EditMeshPolygonsToolLocals;

	if (bToolDisabled)
	{
		CurrentMesh.Reset();
		Topology.Reset();
		return;
	}

	if (CurrentActivity)
	{
		if (IToolHostCustomizationAPI* ButtonCustomizer = IToolHostCustomizationAPI::Find(GetToolManager()).GetInterface())
		{
			ButtonCustomizer->ClearButtonOverrides();
		}
		CurrentActivity->End(ShutdownType);
		CurrentActivity = nullptr;
	}
	CommonProps->SaveProperties(this, GetPropertyCacheIdentifier(bTriangleMode));
	SelectionMechanic->Properties->SaveProperties(this, GetPropertyCacheIdentifier(bTriangleMode));
	TopologyProperties->SaveProperties(this, GetPropertyCacheIdentifier(bTriangleMode));

	GetToolManager()->GetContextObjectStore()->RemoveContextObjectsOfType(UPolyEditActivityContext::StaticClass());
	ActivityContext = nullptr;

	ExtrudeActivity->Shutdown(ShutdownType);
	InsetOutsetActivity->Shutdown(ShutdownType);
	CutFacesActivity->Shutdown(ShutdownType);
	PlanarProjectionUVActivity->Shutdown(ShutdownType);
	InsertEdgeActivity->Shutdown(ShutdownType);
	InsertEdgeLoopActivity->Shutdown(ShutdownType);
	BevelEdgeActivity->Shutdown(ShutdownType);
	ExtrudeEdgeActivity->Shutdown(ShutdownType);

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	DragAlignmentMechanic->Shutdown();
	// We wait to shut down the selection mechanic in case we need to do work to store the selection.

	if (Preview != nullptr)
	{
		UE::ToolTarget::ShowSourceObject(Target);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			FGeometrySelection OutputSelection;
			EGeometryElementType CurElemType = EGeometryElementType::Face;
			if (SelectionMechanic->GetActiveSelection().SelectedCornerIDs.Num() > 0)
			{
				CurElemType = EGeometryElementType::Vertex;
			}
			else if (SelectionMechanic->GetActiveSelection().SelectedEdgeIDs.Num() > 0)
			{
				CurElemType = EGeometryElementType::Edge;
			}
			OutputSelection.InitializeTypes( CurElemType, (bTriangleMode) ? EGeometryTopologyType::Triangle : EGeometryTopologyType::Polygroup);

			FCompactMaps CompactMaps;
			//bool bIsBrushComponent = (Cast<UBrushComponent>(UE::ToolTarget::GetTargetComponent(Target)) == nullptr);
			bool bIsBrushComponent = false;		// can we allow this now?
			bool bWantSelection = (SelectionMechanic->GetActiveSelection().IsEmpty() == false);

			// Note: When not in triangle mode, ModifiedTopologyCounter refers to polygroup topology, so does not tell us
			// about the triangle topology.  In this case, we just assume the triangle topology may have been modified.
			bool bModifiedTriangleTopology = bTriangleMode ? ModifiedTopologyCounter > 0 : true;

			// may need to compact the mesh if we did undo on a mesh edit, then vertices will be dense but compact checks will fail...
			if (bModifiedTriangleTopology)
			{
				// Store the compact maps if we have a selection that we need to update
				CurrentMesh->CompactInPlace(bWantSelection ? &CompactMaps : nullptr);
			}

			// Finish prepping the stored selection
			if (bWantSelection)
			{
				if (bTriangleMode)
				{
					SelectionMechanic->GetSelection_AsTriangleTopology(OutputSelection, bModifiedTriangleTopology ? &CompactMaps : nullptr);
				}
				else
				{
					SelectionMechanic->GetSelection_AsGroupTopology(OutputSelection, bModifiedTriangleTopology ? &CompactMaps : nullptr);
				}
			}

			// Bake CurrentMesh back to target inside an undo transaction
			GetToolManager()->BeginUndoTransaction(LOCTEXT("EditMeshPolygonsToolTransactionName", "Deform Mesh"));
			MeshTransforms::ApplyTransformInverse(*CurrentMesh, BakedTransform, true);
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *CurrentMesh, bModifiedTriangleTopology);

			if (OutputSelection.IsEmpty() == false)
			{
				UE::Geometry::SetToolOutputGeometrySelectionForTarget(this, Target, OutputSelection);
			}
		
			GetToolManager()->EndUndoTransaction();
		}

		Preview->Shutdown();
		Preview = nullptr;
	}

	// The seleciton mechanic shutdown has to happen after (potentially) saving selection above
	SelectionMechanic->Shutdown();

	// We null out as many pointers as we can because the tool pointer usually ends up sticking
	// around in the undo stack.
	CommonProps = nullptr;
	EditActions = nullptr;
	EditActions_Triangles = nullptr;
	EditEdgeActions = nullptr;
	EditEdgeActions_Triangles = nullptr;
	EditUVActions = nullptr;

	ExtrudeActivity = nullptr;
	InsetOutsetActivity = nullptr;
	CutFacesActivity = nullptr;
	PlanarProjectionUVActivity = nullptr;
	InsertEdgeActivity = nullptr;
	InsertEdgeLoopActivity = nullptr;
	BevelEdgeActivity = nullptr;
	ExtrudeEdgeActivity = nullptr;

	SelectionMechanic = nullptr;
	DragAlignmentMechanic = nullptr;

	TransformGizmo = nullptr;
	TransformProxy = nullptr;

	CurrentMesh.Reset();
	Topology.Reset();
	MeshSpatial.Reset();

}


void UEditMeshPolygonsTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 2,
		TEXT("ToggleLockRotation"),
		LOCTEXT("ToggleLockRotationUIName", "Lock Rotation"),
		LOCTEXT("ToggleLockRotationTooltip", "Toggle Frame Rotation Lock on and off"),
		EModifierKey::Control, EKeys::R,
		[this]() { CommonProps->bLockRotation = !CommonProps->bLockRotation; });
	
	// Backspace and delete both trigger deletion (as long as the delete button is also enabled)
	auto OnDeletionKeyPress = [this]() 
	{
		if ((EditActions && EditActions->IsPropertySetEnabled())
			|| (EditActions_Triangles && EditActions_Triangles->IsPropertySetEnabled())
			|| (EditEdgeActions && EditEdgeActions->IsPropertySetEnabled())
			)
		{
			RequestAction(EEditMeshPolygonsToolActions::Delete);
		}
	};
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 3,
		TEXT("DeleteSelectionBackSpaceKey"),
		LOCTEXT("DeleteSelectionUIName", "Delete Selection"),
		LOCTEXT("DeleteSelectionTooltip", "Delete Selection"),
		EModifierKey::None, EKeys::BackSpace, OnDeletionKeyPress);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 4,
		TEXT("DeleteSelectionDeleteKey"),
		LOCTEXT("DeleteSelectionUIName", "Delete Selection"),
		LOCTEXT("DeleteSelectionTooltip", "Delete Selection"),
		EModifierKey::None, EKeys::Delete, OnDeletionKeyPress);

	// This hotkey can make the tool seem broken if it is accidentally pressed, so don't set a default.
	// However we still register it because setting a hotkey can be useful in some workflows (when the
	// gizmo gets in the way of shift-selecting multiple things).
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 5,
		TEXT("ToggleGizmoVisibilityKey"),
		LOCTEXT("ToggleGizmoVisibilityUIName", "Toggle Transform Gizmo Visibility"),
		LOCTEXT("ToggleGizmoVisibilityTooltip", "Toggle the visibility of the transform gizmo"),
		EModifierKey::None, EKeys::Invalid,
		[this]() 
		{
			if (!CurrentActivity)
			{
				CommonProps->bGizmoVisible = !CommonProps->bGizmoVisible;
			}
		});

	// TODO: Esc should be made to exit out of current activity if one is active. However this
	// requires a bit of work because we don't seem to be able to register conditional actions,
	// and we don't want to always capture Esc.
}


void UEditMeshPolygonsTool::RequestAction(EEditMeshPolygonsToolActions ActionType)
{
	if (SelectionMechanic && SelectionMechanic->IsCurrentlyMarqueeDragging())
	{
		PendingAction = EEditMeshPolygonsToolActions::NoAction;
		GetToolManager()->DisplayMessage(
			LOCTEXT("CannotActDuringMarquee", "Cannot perform action while marquee selecting"), 
			EToolMessageLevel::UserWarning);
		return;
	}

	if (PendingAction != EEditMeshPolygonsToolActions::NoAction)
	{
		return;
	}

	PendingAction = ActionType;
}


void UEditMeshPolygonsTool::RequestSingleShotAction(EEditMeshPolygonsToolActions ActionType)
{
	bTerminateOnPendingActionComplete = true;

	if ( ActionType == EEditMeshPolygonsToolActions::BevelAuto )
	{
		if (SelectionMechanic->GetActiveSelection().SelectedEdgeIDs.Num() > 0)
		{
			ActionType = EEditMeshPolygonsToolActions::BevelEdges;
		}
		else
		{
			ActionType = EEditMeshPolygonsToolActions::BevelFaces;
		}
	}

	RequestAction(ActionType);
}


FDynamicMeshAABBTree3& UEditMeshPolygonsTool::GetSpatial()
{
	if (bSpatialDirty)
	{
		MeshSpatial->Build();
		bSpatialDirty = false;
	}
	return *MeshSpatial;
}

void UEditMeshPolygonsTool::UpdateGizmoFrame(const FFrame3d* UseFrame)
{
	FFrame3d SetFrame = LastTransformerFrame;
	if (UseFrame == nullptr)
	{
		if (CommonProps->LocalFrameMode == ELocalFrameMode::FromGeometry)
		{
			SetFrame = LastGeometryFrame;
		}
		else
		{
			SetFrame = FFrame3d(LastGeometryFrame.Origin, WorldTransform.GetRotation());
		}
	}
	else
	{
		SetFrame = *UseFrame;
	}

	if (CommonProps->bLockRotation)
	{
		SetFrame.Rotation = LockedTransfomerFrame.Rotation;
	}

	LastTransformerFrame = SetFrame;

	if (TransformGizmo)
	{
		// This resets the scale as well
		TransformGizmo->ReinitializeGizmoTransform(SetFrame.ToFTransform());
	}
}


FBox UEditMeshPolygonsTool::GetWorldSpaceFocusBox()
{
	if (ensure(SelectionMechanic))
	{
		FAxisAlignedBox3d Bounds = SelectionMechanic->GetSelectionBounds(true);
		return (FBox)Bounds;
	}
	return FBox(EForceInit::ForceInit);
}

bool UEditMeshPolygonsTool::GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut)
{
	FRay3d LocalRay(WorldTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		WorldTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	int32 HitTID = GetSpatial().FindNearestHitTriangle(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		FIntrRay3Triangle3d TriHit = TMeshQueries<FDynamicMesh3>::TriangleIntersection(*GetSpatial().GetMesh(), HitTID, LocalRay);
		FVector3d LocalPos = LocalRay.PointAt(TriHit.RayParameter);
		PointOut = (FVector)WorldTransform.TransformPosition(LocalPos);
		return true;
	}
	return false;
}


void UEditMeshPolygonsTool::OnSelectionModifiedEvent()
{
	bSelectionStateDirty = true;
	LastGeometryFrame = FFrame3d();
}


void UEditMeshPolygonsTool::OnBeginGizmoTransform(UTransformProxy* Proxy)
{
	SelectionMechanic->ClearHighlight();
	UpdateDeformerFromSelection( SelectionMechanic->GetActiveSelection() );
	
	FTransform Transform = Proxy->GetTransform();
	InitialGizmoFrame = FFrame3d(Transform);
	InitialGizmoScale = FVector3d(Transform.GetScale3D());

	BeginDeformerChange();

	bInGizmoDrag = true;
}

void UEditMeshPolygonsTool::OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (bInGizmoDrag)
	{
		LastUpdateGizmoFrame = FFrame3d(Transform);
		LastUpdateGizmoScale = FVector3d(Transform.GetScale3D());
		GetToolManager()->PostInvalidation();
		bGizmoUpdatePending = true;
		bLastUpdateUsedWorldFrame = (TransformGizmo ? TransformGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::World : false);
	}
}

void UEditMeshPolygonsTool::OnEndGizmoTransform(UTransformProxy* Proxy)
{
	bInGizmoDrag = false;
	// Sometimes we don't get a tick between OnGizmoTransformChanged and OnEndGizmoTransform. In
	// most drag cases this is not much of a problem, but if we type values into the gizmo numerical
	// UI, it is.
	if (bGizmoUpdatePending)
	{
		ComputeUpdate_Gizmo();
	}
	bGizmoUpdatePending = false;
	bSpatialDirty = true;
	SelectionMechanic->NotifyMeshChanged(false);

	FFrame3d TransformFrame(Proxy->GetTransform());

	if (TransformGizmo)
	{
		if (CommonProps->bLockRotation)
		{
			FFrame3d SetFrame = TransformFrame;
			SetFrame.Rotation = LockedTransfomerFrame.Rotation;
			TransformGizmo->ReinitializeGizmoTransform(SetFrame.ToFTransform());		
		}
		else
		{
			TransformGizmo->SetNewChildScale(FVector::OneVector);
		}
	}

	LastTransformerFrame = TransformFrame;

	// close change record
	EndDeformerChange();
}


void UEditMeshPolygonsTool::UpdateDeformerFromSelection(const FGroupTopologySelection& Selection)
{
	//Determine which of the following (corners, edges or faces) has been selected by counting the associated feature's IDs
	if (Selection.SelectedCornerIDs.Num() > 0)
	{
		//Add all the the Corner's adjacent poly-groups (NbrGroups) to the ongoing array of groups.
		LinearDeformer.SetActiveHandleCorners(Selection.SelectedCornerIDs.Array());
	}
	else if (Selection.SelectedEdgeIDs.Num() > 0)
	{
		//Add all the the edge's adjacent poly-groups (NbrGroups) to the ongoing array of groups.
		LinearDeformer.SetActiveHandleEdges(Selection.SelectedEdgeIDs.Array());
	}
	else if (Selection.SelectedGroupIDs.Num() > 0)
	{
		LinearDeformer.SetActiveHandleFaces(Selection.SelectedGroupIDs.Array());
	}
}

void UEditMeshPolygonsTool::ComputeUpdate_Gizmo()
{
	if (SelectionMechanic->HasSelection() == false || bGizmoUpdatePending == false)
	{
		return;
	}
	bGizmoUpdatePending = false;

	FFrame3d CurFrame = LastUpdateGizmoFrame;
	FVector3d CurScale = LastUpdateGizmoScale;
	FVector3d TranslationDelta = CurFrame.Origin - InitialGizmoFrame.Origin;
	FQuaterniond RotateDelta = CurFrame.Rotation - InitialGizmoFrame.Rotation;
	FVector3d CurScaleDelta = CurScale - InitialGizmoScale;
	FVector3d LocalTranslation = WorldTransform.InverseTransformVector(TranslationDelta);

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	if (TranslationDelta.SquaredLength() > 0.0001 || RotateDelta.SquaredLength() > 0.0001 || CurScaleDelta.SquaredLength() > 0.0001)
	{
		if (bLastUpdateUsedWorldFrame)
		{
			// For a world frame gizmo, the scaling needs to happen in world aligned gizmo space, but the 
			// rotation is still encoded in the local gizmo frame change.
			FQuaterniond RotationToApply = CurFrame.Rotation * InitialGizmoFrame.Rotation.Inverse();
			LinearDeformer.UpdateSolution(Mesh, [&](FDynamicMesh3* TargetMesh, int VertIdx)
			{
				FVector3d PosLocal = TargetMesh->GetVertex(VertIdx);
				FVector3d PosWorld = WorldTransform.TransformPosition(PosLocal);
				FVector3d PosWorldGizmo = PosWorld - InitialGizmoFrame.Origin;

				FVector3d NewPosWorld = RotationToApply * (PosWorldGizmo * CurScale) + CurFrame.Origin;
				FVector3d NewPosLocal = WorldTransform.InverseTransformPosition(NewPosWorld);
				return NewPosLocal;
			});
		}
		else
		{
			LinearDeformer.UpdateSolution(Mesh, [&](FDynamicMesh3* TargetMesh, int VertIdx)
			{
				// For a local gizmo, we just get the coordinates in the original frame, scale in that frame,
				// then interpret them as coordinates in the new frame.
				FVector3d PosLocal = TargetMesh->GetVertex(VertIdx);
				FVector3d PosWorld = WorldTransform.TransformPosition(PosLocal);
				FVector3d PosGizmo = InitialGizmoFrame.ToFramePoint(PosWorld);
				PosGizmo = CurScale * PosGizmo;
				FVector3d NewPosWorld = CurFrame.FromFramePoint(PosGizmo);
				FVector3d NewPosLocal = WorldTransform.InverseTransformPosition(NewPosWorld);
				return NewPosLocal;
			});
		}
	}
	else
	{
		// Reset mesh to initial positions.
		LinearDeformer.ClearSolution(Mesh);
	}

	Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get(),
		// It's important to use the fast update path for the gizmo manipulations that only
		// affect positions.
		UPreviewMesh::ERenderUpdateMode::FastUpdate,
		EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals);

	GetToolManager()->PostInvalidation();
}



void UEditMeshPolygonsTool::OnTick(float DeltaTime)
{
	if (bToolDisabled)
	{
		return;
	}

	Preview->Tick(DeltaTime);

	if (CurrentActivity)
	{
		CurrentActivity->Tick(DeltaTime);
	}

	bool bLocalCoordSystem = GetToolManager()->GetPairedGizmoManager()
		->GetContextQueriesAPI()->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::Local;
	if (CommonProps->bLocalCoordSystem != bLocalCoordSystem)
	{
		CommonProps->bLocalCoordSystem = bLocalCoordSystem;
		NotifyOfPropertyChangeByTool(CommonProps);
	}

	if (bGizmoUpdatePending)
	{
		ComputeUpdate_Gizmo();
	}

	if (bSelectionStateDirty)
	{
		// update color highlights
		Preview->PreviewMesh->FastNotifySecondaryTrianglesChanged();

		UpdateGizmoVisibility();

		bSelectionStateDirty = false;
	}

	if (PendingAction != EEditMeshPolygonsToolActions::NoAction)
	{
		// Clear any existing error messages.
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);

		switch (PendingAction)
		{

		//Interactive operations:
		case EEditMeshPolygonsToolActions::Extrude:
		{
			if (SelectionMechanic->GetActiveSelection().SelectedGroupIDs.IsEmpty()
				&& !SelectionMechanic->GetActiveSelection().SelectedEdgeIDs.IsEmpty())
			{
				// This particular button happens to be under "face edits", but it's very tempting to click it anyway
				// when you have an edge selection and expect it to extrude edges. We'll allow it to avoid frustrating
				// the user. Not relevant for mesh element selection, where we don't use PolyEd and instead extrude
				// the adjacent faces when edges are selected.
				StartActivity(ExtrudeEdgeActivity);
				break;
			}
			ExtrudeActivity->ExtrudeMode = FExtrudeOp::EExtrudeMode::MoveAndStitch;
			ExtrudeActivity->PropertySetToUse = UPolyEditExtrudeActivity::EPropertySetToUse::Extrude;

			ActivityLabels.Add(ExtrudeActivity, LOCTEXT("ExtrudeActivityLabel", "Extrude"));
			ActivityIconNames.Add(ExtrudeActivity, "PolyEd.Extrude");

			StartActivity(ExtrudeActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::PushPull:
		{
			ExtrudeActivity->ExtrudeMode = FExtrudeOp::EExtrudeMode::Boolean;
			ExtrudeActivity->PropertySetToUse = UPolyEditExtrudeActivity::EPropertySetToUse::PushPull;

			ActivityLabels.Add(ExtrudeActivity, LOCTEXT("PushPullActivityLabel", "Push/Pull"));
			ActivityIconNames.Add(ExtrudeActivity, "PolyEd.PushPull");

			StartActivity(ExtrudeActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::Offset:
		{
			ExtrudeActivity->ExtrudeMode = FExtrudeOp::EExtrudeMode::MoveAndStitch;
			ExtrudeActivity->PropertySetToUse = UPolyEditExtrudeActivity::EPropertySetToUse::Offset;

			ActivityLabels.Add(ExtrudeActivity, LOCTEXT("OffsetActivityLabel", "Offset"));
			ActivityIconNames.Add(ExtrudeActivity, "PolyEd.Offset");

			StartActivity(ExtrudeActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::Inset:
		{
			InsetOutsetActivity->Settings->bOutset = false;
			
			ActivityLabels.Add(InsetOutsetActivity, LOCTEXT("InsetActivityLabel", "Inset"));
			ActivityIconNames.Add(InsetOutsetActivity, "PolyEd.Inset");
			
			StartActivity(InsetOutsetActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::Outset:
		{
			InsetOutsetActivity->Settings->bOutset = true;
			
			ActivityLabels.Add(InsetOutsetActivity, LOCTEXT("OutsetActivityLabel", "Outset"));
			ActivityIconNames.Add(InsetOutsetActivity, "PolyEd.Outset");

			StartActivity(InsetOutsetActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::CutFaces:
		{
			StartActivity(CutFacesActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::PlanarProjectionUV:
		{
			StartActivity(PlanarProjectionUVActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::InsertEdge:
		{
			StartActivity(InsertEdgeActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::InsertEdgeLoop:
		{
			StartActivity(InsertEdgeLoopActivity);
			break;
		}
		case EEditMeshPolygonsToolActions::ExtrudeEdges:
		{
			// Hack: We currently don't support extra corners in mesh element selection, and
			// the switch to using them can cause us to lose some of our selected edges. For
			// now we just rebuild topology without the corners in this scenario, but we should
			// fix instead just carry over the selection properly (and carry it back).
			if (bTerminateOnPendingActionComplete && HasGeometrySelection())
			{
				const FGeometrySelection& CurSelection = GetGeometrySelection();
				if (CurSelection.TopologyType == EGeometryTopologyType::Polygroup && bTriangleMode == false)
				{
					TSet<int32> EmptySet;
					RebuildTopologyWithGivenExtraCorners(EmptySet);
					SelectionMechanic->SetSelection_AsGroupTopology(CurSelection); // Have to reinitialize selection
				}
			}

			StartActivity(ExtrudeEdgeActivity);
			break;
		}

		case EEditMeshPolygonsToolActions::BevelFaces:
		case EEditMeshPolygonsToolActions::BevelEdges:
		{
			StartActivity(BevelEdgeActivity);
			break;
		}

		// Single action operations:
		case EEditMeshPolygonsToolActions::Merge:
			ApplyMerge();
			break;
		case  EEditMeshPolygonsToolActions::Delete:
			ApplyDelete();
			break;
		case EEditMeshPolygonsToolActions::RecalculateNormals:
			ApplyRecalcNormals();
			break;
		case EEditMeshPolygonsToolActions::FlipNormals:
			ApplyFlipNormals();
			break;
		case EEditMeshPolygonsToolActions::CollapseEdge:
			ApplyCollapseEdge();
			break;
		case EEditMeshPolygonsToolActions::WeldEdges:
			ApplyWeldEdges();
			break;
		case EEditMeshPolygonsToolActions::WeldEdgesCentered:
			ApplyWeldEdges(0.5);
			break;
		case EEditMeshPolygonsToolActions::StraightenEdge:
			ApplyStraightenEdges();
			break;
		case EEditMeshPolygonsToolActions::FillHole:
			ApplyFillHole();
			break;
		case EEditMeshPolygonsToolActions::BridgeEdges:
			ApplyBridgeEdges();
			break;
		case EEditMeshPolygonsToolActions::SimplifyAlongEdges:
			ApplySimplifyAlongEdges();
			break;
		case EEditMeshPolygonsToolActions::Retriangulate:
			ApplyRetriangulate();
			break;
		case EEditMeshPolygonsToolActions::Decompose:
			ApplyDecompose();
			break;
		case EEditMeshPolygonsToolActions::Disconnect:
			ApplyDisconnect();
			break;
		case EEditMeshPolygonsToolActions::Duplicate:
			ApplyDuplicate();
			break;
		case EEditMeshPolygonsToolActions::PokeSingleFace:
			ApplyPokeSingleFace();
			break;
		case EEditMeshPolygonsToolActions::SplitSingleEdge:
			ApplySplitSingleEdge();
			break;
		case EEditMeshPolygonsToolActions::CollapseSingleEdge:
			ApplyCollapseEdge();
			break;
		case EEditMeshPolygonsToolActions::FlipSingleEdge:
			ApplyFlipSingleEdge();
			break;
		case EEditMeshPolygonsToolActions::SimplifyByGroups:
			SimplifyByGroups();
			break;
		case EEditMeshPolygonsToolActions::RegenerateExtraCorners:
			ApplyRegenerateExtraCorners();
			break;
		}

		PendingAction = EEditMeshPolygonsToolActions::NoAction;
	}
}

void UEditMeshPolygonsTool::StartActivity(TObjectPtr<UInteractiveToolActivity> Activity)
{
	EndCurrentActivity(EToolShutdownType::Accept);

	// Right now we rely on the activity to fail to start or to issue an error message if the
	// conditions are not right. Someday, we are going to disable the buttons based on a CanStart
	// call.
	if (Activity->Start() == EToolActivityStartResult::Running)
	{
		if (TransformGizmo)
		{
			TransformGizmo->SetVisibility(false);
		}
		SelectionMechanic->SetIsEnabled(false);
		SetToolPropertySourceEnabled(SelectionMechanic->Properties, false);
		SetToolPropertySourceEnabled(TopologyProperties, false);
		CurrentActivity = Activity;

		if ( bTerminateOnPendingActionComplete == false )
		{
			// Customize the tool accept/cancel buttons to the current activity.
			if (IToolHostCustomizationAPI* ButtonCustomizer = IToolHostCustomizationAPI::Find(GetToolManager()).GetInterface())
			{
				const FText SubActionFallbackLabel = LOCTEXT("SubActionFallbackLabel", "Current Action");
				if (CurrentActivity->HasAccept())
				{
					IToolHostCustomizationAPI::FAcceptCancelButtonOverrideParams Params;
					Params.Label = ActivityLabels.Contains(CurrentActivity) ? ActivityLabels[CurrentActivity] : SubActionFallbackLabel;
					if (ActivityIconNames.Contains(CurrentActivity))
					{
						Params.IconName = ActivityIconNames[CurrentActivity];
					}
					Params.OverrideAcceptButtonText = LOCTEXT("AcceptSubActionButton", "Accept Action");
					Params.OverrideAcceptButtonTooltip = LOCTEXT("AcceptSubActionTooltip", "Accept the action currently being performed.");
					Params.OverrideCancelButtonText = LOCTEXT("CancelSubActionButton", "Cancel Action");
					Params.OverrideCancelButtonTooltip = LOCTEXT("CancelSubActionTooltip", "Cancel the action currently being performed.");
					Params.CanAccept = [this]() { return CurrentActivity->CanAccept(); };
					Params.OnAcceptCancelTriggered = [this](bool bAccept) 
					{ 
						EndCurrentActivity(bAccept ? EToolShutdownType::Accept : EToolShutdownType::Cancel);
						return FReply::Handled();
					};

					ButtonCustomizer->RequestAcceptCancelButtonOverride(Params);
				}
				else
				{
					IToolHostCustomizationAPI::FCompleteButtonOverrideParams Params;
					Params.Label = ActivityLabels.Contains(CurrentActivity) ? ActivityLabels[CurrentActivity] : SubActionFallbackLabel;
					if (ActivityIconNames.Contains(CurrentActivity))
					{
						Params.IconName = ActivityIconNames[CurrentActivity];
					}
					Params.OverrideCompleteButtonText = LOCTEXT("CompleteSubActionButton", "Done");
					Params.OverrideCompleteButtonTooltip = LOCTEXT("CompleteSubActionTooltip", "Exit the current activity.");
					Params.OnCompleteTriggered = [this]() 
					{
						EndCurrentActivity(EToolShutdownType::Completed);
						return FReply::Handled();
					};

					ButtonCustomizer->RequestCompleteButtonOverride(Params);
				}
			}
		}
		else
		{
			SetToolPropertySourceEnabled(CommonProps, false);			
		}

		SetActionButtonPanelsVisible(false);
	}
	else
	{
		if (bTerminateOnPendingActionComplete)
		{
			GetToolManager()->PostActiveToolShutdownRequest(this, EToolShutdownType::Cancel);
			return;
		}
	}
}

void UEditMeshPolygonsTool::EndCurrentActivity(EToolShutdownType ShutdownType)
{
	if (CurrentActivity)
	{
		if (CurrentActivity->IsRunning())
		{
			CurrentActivity->End(ShutdownType);
		}

		CurrentActivity = nullptr;
		++ActivityTimestamp;

		if (bTerminateOnPendingActionComplete)
		{
			GetToolManager()->PostActiveToolShutdownRequest(this, ShutdownType);
			return;
		}

		if (IToolHostCustomizationAPI* ButtonCustomizer = IToolHostCustomizationAPI::Find(GetToolManager()).GetInterface())
		{
			ButtonCustomizer->ClearButtonOverrides();
		}
		SetActionButtonPanelsVisible(true);
		SelectionMechanic->SetIsEnabled(true);
		SetToolPropertySourceEnabled(TopologyProperties, true);
		SetToolPropertySourceEnabled(SelectionMechanic->Properties, true);
		UpdateGizmoVisibility();
	}

	// If an activity displays a notification, it should be
	// overwritten with an appropriate notification once finished
	ResetUserMessage();
}

void UEditMeshPolygonsTool::NotifyActivitySelfEnded(UInteractiveToolActivity* Activity)
{
	EndCurrentActivity(EToolShutdownType::Accept);
}

void UEditMeshPolygonsTool::UpdateGizmoVisibility()
{
	// Only allow gizmo to become visible if something is selected,
	// the gizmo isn't hidden, and there is no current activity.
	if (SelectionMechanic->HasSelection() && CommonProps->bGizmoVisible && !CurrentActivity)
	{
		if (TransformGizmo)
		{
			TransformGizmo->SetVisibility(true);
		}

		// Update frame because we might be here due to an undo event/etc,
		// rather than an explicit selection change
		LastGeometryFrame = SelectionMechanic->GetSelectionFrame(true, &LastGeometryFrame);
		UpdateGizmoFrame();
	}
	else if(TransformGizmo)
	{
		TransformGizmo->SetVisibility(false);
	}
}

void UEditMeshPolygonsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (bToolDisabled)
	{
		return;
	}

	Preview->PreviewMesh->EnableWireframe(CommonProps->bShowWireframe);
	SelectionMechanic->Render(RenderAPI);
	DragAlignmentMechanic->Render(RenderAPI);

	if (CurrentActivity)
	{
		CurrentActivity->Render(RenderAPI);
	}
}

void UEditMeshPolygonsTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (bToolDisabled)
	{
		return;
	}

	SelectionMechanic->DrawHUD(Canvas, RenderAPI);
}

void UEditMeshPolygonsTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UPolyEditCommonProperties, bShowSelectableCorners)))
	{
		SelectionMechanic->SetShowSelectableCorners(CommonProps->bShowSelectableCorners);
	}
}

//
// Gizmo change tracking
//
void UEditMeshPolygonsTool::UpdateDeformerChangeFromROI(bool bFinal)
{
	if (ActiveVertexChange == nullptr)
	{
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	ActiveVertexChange->SaveVertices(Mesh, LinearDeformer.GetModifiedVertices(), !bFinal);
	ActiveVertexChange->SaveOverlayNormals(Mesh, LinearDeformer.GetModifiedOverlayNormals(), !bFinal);
}

void UEditMeshPolygonsTool::BeginDeformerChange()
{
	if (ActiveVertexChange == nullptr)
	{
		ActiveVertexChange = new FMeshVertexChangeBuilder(EMeshVertexChangeComponents::VertexPositions | EMeshVertexChangeComponents::OverlayNormals);
		UpdateDeformerChangeFromROI(false);
	}
}

void UEditMeshPolygonsTool::EndDeformerChange()
{
	if (ActiveVertexChange != nullptr)
	{
		UpdateDeformerChangeFromROI(true);
		GetToolManager()->EmitObjectChange(this, MoveTemp(ActiveVertexChange->Change), 
			LOCTEXT("PolyMeshDeformationChange", "PolyMesh Edit"));
	}

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
}

// This gets called by vertex change events emitted via gizmo (deformer) interaction
void UEditMeshPolygonsTool::ApplyChange(const FMeshVertexChange* Change, bool bRevert)
{
	Preview->PreviewMesh->ApplyChange(Change, bRevert);
	CurrentMesh->Copy(*Preview->PreviewMesh->GetMesh());
	bSpatialDirty = true;
	SelectionMechanic->NotifyMeshChanged(false);

	// Topology does not need updating
}


void UEditMeshPolygonsTool::UpdateFromCurrentMesh(bool bUpdateTopology)
{
	Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get(), UPreviewMesh::ERenderUpdateMode::FullUpdate);
	bSpatialDirty = true;
	SelectionMechanic->NotifyMeshChanged(bUpdateTopology);

	if (bUpdateTopology)
	{
		Topology->RebuildTopology();
	}
}



void UEditMeshPolygonsTool::ApplyDelete()
{
	if (BeginMeshFaceEditChange())
	{
		ApplyDeleteFaces();
	}
	else if (BeginMeshEdgeEditChange())
	{
		ApplyDeleteEdges();
	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnDeleteFailedMessage", "Cannot Delete Current Selection"),
			EToolMessageLevel::UserWarning);
	}
}



void UEditMeshPolygonsTool::ApplyMerge()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnMergeFailedMessage", "Cannot Merge Current Selection"),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	ChangeTracker.SaveTriangles(ActiveTriangleSelection, true);
	FMeshConnectedComponents Components(Mesh);
	Components.FindConnectedTriangles(ActiveTriangleSelection);
	FGroupTopologySelection NewSelection;
	for (const FMeshConnectedComponents::FComponent& Component : Components)
	{
		int32 NewGroupID = Mesh->AllocateTriangleGroup();
		FaceGroupUtil::SetGroupID(*Mesh, Component.Indices, NewGroupID);
		NewSelection.SelectedGroupIDs.Add(NewGroupID);
	}
	
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshMergeChange", "Merge"),
		ChangeTracker.EndChange(), NewSelection);
}






void UEditMeshPolygonsTool::ApplyDeleteFaces()
{
	FDynamicMesh3* Mesh = CurrentMesh.Get();

	// prevent deleting all triangles
	if (ActiveTriangleSelection.Num() >= Mesh->TriangleCount())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnDeleteAllFailedMessage", "Cannot Delete Entire Mesh"),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	ChangeTracker.SaveTriangles(ActiveTriangleSelection, true);
	FDynamicMeshEditor Editor(Mesh);
	Editor.RemoveTriangles(ActiveTriangleSelection, true);

	FGroupTopologySelection NewSelection;
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshDeleteFacesChange", "Delete Faces"),
		ChangeTracker.EndChange(), NewSelection);
}



void UEditMeshPolygonsTool::ApplyRecalcNormals()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnRecalcNormalsFailedMessage", "Cannot Recalculate Normals for Current Selection"),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FDynamicMeshEditor Editor(Mesh);
	FGroupTopologySelection ActiveSelection = SelectionMechanic->GetActiveSelection();
	for (int32 GroupID : ActiveSelection.SelectedGroupIDs)
	{
		ChangeTracker.SaveTriangles(Topology->GetGroupTriangles(GroupID), true);
		Editor.SetTriangleNormals(Topology->GetGroupTriangles(GroupID));
	}

	// We actually don't even need any of the wrapper around this change since we're not altering
	// positions or topology (so no other structures need updating), but we go ahead and go the
	// same route as everything else. See :HandlePositionOnlyMeshChanges
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshRecalcNormalsChange", "Recalculate Normals"), 
		ChangeTracker.EndChange(), ActiveSelection);
}


void UEditMeshPolygonsTool::ApplyFlipNormals()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnFlipNormalsFailedMessage", "Cannot Flip Normals for Current  Selection"),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FDynamicMeshEditor Editor(Mesh);
	FGroupTopologySelection ActiveSelection = SelectionMechanic->GetActiveSelection();
	for (int32 GroupID : ActiveSelection.SelectedGroupIDs)
	{
		for ( int32 tid : Topology->GetGroupTriangles(GroupID) )
		{ 
			ChangeTracker.SaveTriangle(tid, true);
			Mesh->ReverseTriOrientation(tid);
		}
	}

	// Note the topology can change in that the ordering of edge elements can reverse
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshFlipNormalsChange", "Flip Normals"), 
		ChangeTracker.EndChange(), ActiveSelection);
}


void UEditMeshPolygonsTool::ApplyRetriangulate()
{
	using namespace EditMeshPolygonsToolLocals;
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage( LOCTEXT("OnRetriangulateFailed", "Cannot Retriangulate Current Selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FGroupTopologySelection ActiveSelection = SelectionMechanic->GetActiveSelection();
	
	int32 nCompleted = RetriangulateGroups(Mesh, Topology.Get(), ActiveSelection.SelectedGroupIDs, ChangeTracker);

	if (nCompleted != ActiveSelection.SelectedGroupIDs.Num())
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnRetriangulateFailures", "Some faces could not be retriangulated"), EToolMessageLevel::UserWarning);
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshRetriangulateChange", "Retriangulate"),
		ChangeTracker.EndChange(), ActiveSelection);
}



void UEditMeshPolygonsTool::SimplifyByGroups()
{
	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	ChangeTracker.SaveTriangles(Mesh->TriangleIndicesItr(), true); // We will change the entire mesh
	
	FPolygroupRemesh Remesh(Mesh, Topology.Get(), ConstrainedDelaunayTriangulate<double>);
	bool bSuccess = Remesh.Compute();
	if (!bSuccess)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnSimplifyByGroupFailures", "Some polygroups could not be correctly simplified"), EToolMessageLevel::UserWarning);
	}

	FGroupTopologySelection NewSelection; // Empty the selection

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshSimplifyByGroup", "Simplify by Group"),
		ChangeTracker.EndChange(), NewSelection);
}



void UEditMeshPolygonsTool::ApplyRegenerateExtraCorners()
{
	if (!ensure(!bTriangleMode && Topology))
	{
		return;
	}

	// We need to remember the extra corners that get generated and put them into the undo system so that if we
	// change the settings later, undoing still brings us back to the result we saw at that time.
	TSet<int32> PreviousExtraCorners(Topology->GetCurrentExtraCornerVids());
	Topology->RebuildTopology();
	const TSet<int32>& NewExtraCorners = Topology->GetCurrentExtraCornerVids();

	bool bCornersChanged = PreviousExtraCorners.Num() != NewExtraCorners.Num() || !PreviousExtraCorners.Includes(NewExtraCorners);
	if (bCornersChanged)
	{
		const FText TransactionLabel = LOCTEXT("RegenerateCornersTransactionName", "Regenerate Corners");

		GetToolManager()->BeginUndoTransaction(TransactionLabel);
		if (SelectionMechanic && !SelectionMechanic->GetActiveSelection().IsEmpty())
		{
			SelectionMechanic->BeginChange();
			SelectionMechanic->ClearSelection();
			GetToolManager()->EmitObjectChange(SelectionMechanic, SelectionMechanic->EndChange(), TransactionLabel);
		}

		GetToolManager()->EmitObjectChange(this,
			MakeUnique<EditMeshPolygonsToolLocals::FExtraCornerChange>(PreviousExtraCorners, NewExtraCorners),
			TransactionLabel);

		GetToolManager()->EndUndoTransaction();
	}
	
	if (SelectionMechanic)
	{
		SelectionMechanic->NotifyMeshChanged(true);
	}
}


void UEditMeshPolygonsTool::RebuildTopologyWithGivenExtraCorners(const TSet<int32>& Vids)
{
	Topology->RebuildTopologyWithSpecificExtraCorners(Vids);
	SelectionMechanic->NotifyMeshChanged(true);
}



void UEditMeshPolygonsTool::ApplyDecompose()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnDecomposeFailed", "Cannot Decompose Current Selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FGroupTopologySelection NewSelection;
	for (int32 GroupID : SelectionMechanic->GetActiveSelection().SelectedGroupIDs)
	{
		const TArray<int32>& Triangles = Topology->GetGroupTriangles(GroupID);
		ChangeTracker.SaveTriangles(Triangles, true);
		for (int32 tid : Triangles)
		{
			int32 NewGroupID = Mesh->AllocateTriangleGroup();
			Mesh->SetTriangleGroup(tid, NewGroupID);
			NewSelection.SelectedGroupIDs.Add(NewGroupID);
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshDecomposeChange", "Decompose"),
		ChangeTracker.EndChange(), NewSelection);
}


void UEditMeshPolygonsTool::ApplyDisconnect()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnDisconnectFailed", "Cannot Disconnect Current Selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FGroupTopologySelection ActiveSelection = SelectionMechanic->GetActiveSelection();
	TArray<int32> AllTriangles;
	for (int32 GroupID : ActiveSelection.SelectedGroupIDs)
	{
		AllTriangles.Append(Topology->GetGroupTriangles(GroupID));
	}
	ChangeTracker.SaveTriangles(AllTriangles, true);
	FDynamicMeshEditor Editor(Mesh);
	Editor.DisconnectTriangles(AllTriangles, false);

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshDisconnectChange", "Disconnect"),
		ChangeTracker.EndChange(), ActiveSelection);
}




void UEditMeshPolygonsTool::ApplyDuplicate()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnDuplicateFailed", "Cannot Duplicate Current Selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FGroupTopologySelection ActiveSelection = SelectionMechanic->GetActiveSelection();
	TArray<int32> AllTriangles;
	for (int32 GroupID : ActiveSelection.SelectedGroupIDs)
	{
		AllTriangles.Append(Topology->GetGroupTriangles(GroupID));
	}
	FDynamicMeshEditor Editor(Mesh);
	FMeshIndexMappings Mappings;
	FDynamicMeshEditResult EditResult;
	Editor.DuplicateTriangles(AllTriangles, Mappings, EditResult);

	FGroupTopologySelection NewSelection;
	NewSelection.SelectedGroupIDs.Append(bTriangleMode ? EditResult.NewTriangles : EditResult.NewGroups);

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshDisconnectChange", "Disconnect"),
		ChangeTracker.EndChange(), NewSelection);
}



// Deprecated
void UEditMeshPolygonsTool::ApplyCollapseSingleEdge()
{
	ApplyCollapseEdge();
}



void UEditMeshPolygonsTool::ApplyWeldEdges()
{
	ApplyWeldEdges(0);
}

void UEditMeshPolygonsTool::ApplyWeldEdges(double InterpolationT)
{
	using namespace EditMeshPolygonsToolLocals;

	FGroupTopologySelection CurrentSelection = SelectionMechanic->GetActiveSelection();
	
	TSet<int32> GroupEdges;
	if (!CurrentSelection.SelectedCornerIDs.IsEmpty())
	{
		if (CurrentSelection.SelectedCornerIDs.Num() == 2)
		{
			ApplyWeldVertices(InterpolationT);
			return;
		}
		ConvertCornerSelectionToGroupEdgeSelection(Topology.Get(), CurrentSelection.SelectedCornerIDs, GroupEdges);
		if (GroupEdges.Num() < 2)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldVerticesFailedInvalidCount", "Cannot Weld current selection, "
					"selection must be either 2 vertices or convertible to at least 2 edges."),
				EToolMessageLevel::UserWarning);
			return;
		}
	}
	else
	{
		GroupEdges = CurrentSelection.SelectedEdgeIDs;
	}
	
	if (GroupEdges.Num() < 2)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnWeldEdgesFailedTooFew", "Cannot Weld current selection, selection must be at least 2 edges."),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	TArray<int32> GroupEdgesA;
	TArray<int32> GroupEdgesB;
	bool bShouldReverseA = false;
	if (!LinkBoundaryGroupEdges(Topology.Get(), Mesh, GroupEdges.Array(), GroupEdgesA, GroupEdgesB, bShouldReverseA)
		// We don't allow a single loop
		|| GroupEdgesB.IsEmpty())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnWeldEdgesFailedEdgeCount", "Cannot Weld current selection, selection could not be partitioned into two non-loop open-boundary sequences."),
			EToolMessageLevel::UserWarning);
			return;
	}

	// The two sequences are given in boundary orientation, which will be in the opposite direction.
	//  We reverse one of them so we can do our pairwise welding in the proper order.
	Algo::Reverse(bShouldReverseA ? GroupEdgesA : GroupEdgesB);

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();

	bool bAllSucceeded = true;
	bool bHaveSeam = false;

	// Conceptually, we weld pairwise across group edges until we reach the last group edge
	//  of the shorter sequence, and then weld that edge to remaining concatenated group edges
	//  in the longer sequence. 
	// However there are some pathological cases where welding of one edge could remove an edge
	//  of an adjacent group edge, and these are best handled inside FWeldEdgeSequence if it 
	//  knows all of the edges it needs to weld. So, we want to pass FWeldEdgeSequence the 
	//  concatenated sequences, but we need to do the equalizing splits on a per-group-edge
	//  basis so that we can make sure that group corners still get welded to other group corners.
	
	TArray<int32> ConcatenatedKeptEids;
	TArray<int32> ConcatenatedDiscardEids;

	auto PrepGroupEdgePair = [&ChangeTracker, Mesh, &bAllSucceeded, 
		&ConcatenatedKeptEids, &ConcatenatedDiscardEids, bShouldReverseA](FEdgeSpan& SpanA, FEdgeSpan& SpanB)
	{
		// Save one ring tri's for vertices along both edges. The kept edge is necessary
		//  because we might be splitting its triangles if needed.
		for (int32 Vid : SpanA.Vertices)
		{
			if (!ensure(Mesh->IsVertex(Vid)))
			{
				return false;
			}
			Mesh->EnumerateVertexTriangles(Vid, [&ChangeTracker](int32 Tid)
			{
				ChangeTracker.SaveTriangle(Tid, true);
			});
		}
		for (int32 Vid : SpanB.Vertices)
		{
			if (!ensure(Mesh->IsVertex(Vid)))
			{
				return false;
			}
			Mesh->EnumerateVertexTriangles(Vid, [&ChangeTracker](int32 Tid)
			{
				ChangeTracker.SaveTriangle(Tid, true);
			});
		}

		SpanA.SetCorrectOrientation();
		SpanB.SetCorrectOrientation();

		FWeldEdgeSequence::EWeldResult Result = FWeldEdgeSequence::SplitEdgesToEqualizeSpanLengths(*Mesh, SpanA, SpanB);
		
		if (bShouldReverseA)
		{
			ConcatenatedDiscardEids.Insert(SpanA.Edges, 0);
			ConcatenatedKeptEids.Append(SpanB.Edges);
		}
		else
		{
			ConcatenatedDiscardEids.Append(SpanA.Edges);
			ConcatenatedKeptEids.Insert(SpanB.Edges, 0);
		}

		if (Result != FWeldEdgeSequence::EWeldResult::Ok)
		{
			bAllSucceeded = false;
			return false;
		}
		return true;
	};

	int32 NumMatched = GroupEdgesA.Num() == GroupEdgesB.Num() ? GroupEdgesA.Num()
		: FMath::Min(GroupEdgesA.Num(), GroupEdgesB.Num()) - 1;
	for (int32 i = 0; i < NumMatched; ++i)
	{
		FEdgeSpan& SpanA = Topology->Edges[GroupEdgesA[i]].Span;
		FEdgeSpan& SpanB = Topology->Edges[GroupEdgesB[i]].Span;
		PrepGroupEdgePair(SpanA, SpanB);
	}

	// If there was a mismatched number of edges, we have set NumMatched to be one less than
	//  the shorter sequence so we can weld the last edge to the remainder on the other side.
	if (NumMatched < GroupEdgesA.Num())
	{
		// Assemble our two edge sequences to weld.
		TArray<int32> SpanAEids;
		for (int32 i = 0; i < GroupEdgesA.Num() - NumMatched; ++i)
		{
			int32 Index = bShouldReverseA ? GroupEdgesA.Num() - 1 - i : NumMatched + i;
			FEdgeSpan Span = Topology->Edges[GroupEdgesA[Index]].Span;
			Span.SetCorrectOrientation();
			SpanAEids.Append(Span.Edges);
		}
		TArray<int32> SpanBEids;
		for (int32 i = 0; i < GroupEdgesB.Num() - NumMatched; ++i)
		{
			int32 Index = !bShouldReverseA ? GroupEdgesB.Num() - 1 - i : NumMatched + i;
			FEdgeSpan Span = Topology->Edges[GroupEdgesB[Index]].Span;
			Span.SetCorrectOrientation();
			SpanBEids.Append(Span.Edges);
		}

		FEdgeSpan SpanA;
		SpanA.InitializeFromEdges(Mesh, SpanAEids);
		FEdgeSpan SpanB;
		SpanB.InitializeFromEdges(Mesh, SpanBEids);

		PrepGroupEdgePair(SpanA, SpanB);
	}

	FEdgeSpan ConcatenatedKeptSpan(Mesh), ConcatenatedDiscardSpan(Mesh);
	ConcatenatedKeptSpan.InitializeFromEdges(ConcatenatedKeptEids);
	ConcatenatedDiscardSpan.InitializeFromEdges(ConcatenatedDiscardEids);

	FWeldEdgeSequence EdgeWelder(Mesh, ConcatenatedDiscardSpan, ConcatenatedKeptSpan);
	EdgeWelder.bAllowIntermediateTriangleDeletion = true;
	EdgeWelder.bAllowFailedMerge = true;
	EdgeWelder.InterpolationT = InterpolationT;

	FWeldEdgeSequence::EWeldResult Result = EdgeWelder.Weld();
	if (EdgeWelder.UnmergedEdgePairsOut.Num() != 0)
	{
		bHaveSeam = true;
	}
	if (Result != FWeldEdgeSequence::EWeldResult::Ok)
	{
		bAllSucceeded = false;
	}

	if (CurrentMesh->TriangleCount() == 0)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("WeldEdgesWouldDeleteAll",
			"Could not weld current selection because doing so would discard entire mesh."), 
			EToolMessageLevel::UserWarning);

		// Use our change tracker to undo what we've done
		ChangeTracker.EndChange()->Apply(CurrentMesh.Get(), /*bRevert*/ true);
		// Update so spatial doesn't complain about mismatched changestamps. 
		UpdateFromCurrentMesh(false);
		
		// The topology didn't actually change, but unfortunately the eids it stores in its spans
		//  are now invalid, and we need to update those. We do this update ourselves (rather than
		//  passing true to UpdateFromCurrentMesh above) so that we can keep the same extra corners
		//  and therefore same selection.
		TSet<int32> ExtraCorners = Topology->GetCurrentExtraCornerVids();
		Topology->RebuildTopologyWithSpecificExtraCorners(ExtraCorners);
		return;
	}

	FText TransactionName = LOCTEXT("PolyMeshWeldEdgeChange", "Weld Edges");
	GetToolManager()->BeginUndoTransaction(TransactionName);
	EmitCurrentMeshChangeAndUpdate(TransactionName, ChangeTracker.EndChange(), FGroupTopologySelection());

	// Now that the topology is updated, set the new selection
	FGroupTopologySelection NewSelection;
	TSet<int32> SelectedEids;
	for (int32 Eid : ConcatenatedKeptEids)
	{
		if (Mesh->IsEdge(Eid) && !SelectedEids.Contains(Eid))
		{
			int32 GroupEdgeID = Topology->FindGroupEdgeID(Eid);
			if (GroupEdgeID != IndexConstants::InvalidID)
			{
				NewSelection.SelectedEdgeIDs.Add(GroupEdgeID);
				SelectedEids.Append(Topology->GetGroupEdgeEdges(GroupEdgeID));
			}
		}
	}
	// Seems possible to end up with an empty selection if we welded edges of the same group,
	//  so the new edge is not a group boundary, or if we ended up collapsing things.
	if (!NewSelection.IsEmpty())
	{
		SelectionMechanic->SetSelection(NewSelection);
	}

	if (bHaveSeam)
	{
		GetToolManager()->DisplayMessage(WeldIncompleteMessage, EToolMessageLevel::UserWarning);
	}
	else if (!bAllSucceeded)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnWeldEdgesPartialFailure", 
			"Warning: some edges could not be welded."), EToolMessageLevel::UserWarning);
	}

	GetToolManager()->EndUndoTransaction();
}

void UEditMeshPolygonsTool::ApplyWeldVertices(double InterpolationT)
{
	using namespace EditMeshPolygonsToolLocals;

	FGroupTopologySelection CurrentSelection = SelectionMechanic->GetActiveSelection();

	TArray<int32> CornerIDs = CurrentSelection.SelectedCornerIDs.Array();
	if (CornerIDs.Num() != 2)
	{
		return;
	}

	FDynamicMeshChangeTracker ChangeTracker(CurrentMesh.Get());
	ChangeTracker.BeginChange();

	// See if there's a group edge between the two selected corners. If there is, the
	//  user was probably expecting to collapse the group edge.
	TSet<int32> GroupEdges;
	ConvertCornerSelectionToGroupEdgeSelection(Topology.Get(), CurrentSelection.SelectedCornerIDs, GroupEdges);
	if (GroupEdges.Num() != 0)
	{
		GetToolManager()->BeginUndoTransaction(CollapseEdgeTransactionLabel);
		CollapseGroupEdges(GroupEdges, ChangeTracker);
		GetToolManager()->EndUndoTransaction();
		return;
	}
	// Othewise do the operation
	
	int32 KeptVid = Topology->GetCornerVertexID(CornerIDs[1]);
	int32 DiscardedVid = Topology->GetCornerVertexID(CornerIDs[0]);

	CurrentMesh->EnumerateVertexTriangles(DiscardedVid, [&ChangeTracker](int32 Tid)
	{
		ChangeTracker.SaveTriangle(Tid, true);
	});
	CurrentMesh->EnumerateVertexTriangles(KeptVid, [&ChangeTracker](int32 Tid)
	{
		ChangeTracker.SaveTriangle(Tid, true);
	});

	// Helper used when we can't weld, but choose to move the verts to the destination instead
	auto MoveToDestination = [this, KeptVid, DiscardedVid, InterpolationT]()
	{
		FVector3d Destination = Lerp(CurrentMesh->GetVertex(KeptVid), CurrentMesh->GetVertex(DiscardedVid), InterpolationT);
		CurrentMesh->SetVertex(DiscardedVid, Destination);
		CurrentMesh->SetVertex(KeptVid, Destination);
	};

	FDynamicMesh3::FMergeVerticesInfo MergeInfo;
	FDynamicMesh3::FMergeVerticesOptions Options;
	Options.bAllowNonBoundaryBowtieCreation = bAllowBowtieWeldAtInternalVertex;
	EMeshResult Result = CurrentMesh->MergeVertices(KeptVid, DiscardedVid, InterpolationT, Options, MergeInfo);

	if (Result == EMeshResult::Failed_CollapseTriangle
		|| Result == EMeshResult::Failed_CollapseQuad
		|| Result == EMeshResult::Failed_FoundDuplicateTriangle)
	{
		bool bSuccessful = RemoveEdgeTrisIfNotLast(*CurrentMesh, CurrentMesh->FindEdge(KeptVid, DiscardedVid));
		if (!bSuccessful)
		{
			GetToolManager()->DisplayMessage(LOCTEXT("WeldVerticesCannotDeleteAll",
				"Could not weld vertices because it would delete remainder of mesh."), EToolMessageLevel::UserWarning);
			return;
		}
	}
	// Align with behavior in weld and collapse when we're unable to weld due to topology.
	else if (Result == EMeshResult::Failed_InvalidNeighbourhood)
	{
		if (CurrentMesh->FindEdge(KeptVid, DiscardedVid) != IndexConstants::InvalidID)
		{
			// Collapse case: refuse to collapse
			GetToolManager()->DisplayMessage(LOCTEXT("WeldVerticesCollapseInvalidTopology",
				"Could not weld vertices because the collapse would create an edge with more than "
				"two triangles (non-manifold geometry)."), EToolMessageLevel::UserWarning);
			return;
		}
		else
		{
			// Weld case: move to destination and complain
			MoveToDestination();
			GetToolManager()->DisplayMessage(WeldIncompleteMessage, EToolMessageLevel::UserWarning);
		}
	}
	else if (Result == EMeshResult::Failed_WouldCreateBowtie)
	{
		MoveToDestination();
		GetToolManager()->DisplayMessage(LOCTEXT("WeldVerticesDisallowInternalBowtie",
			"Could not weld vertices because it would create a non-boundary edge bowtie. Vertices "
			"were moved to their destination without actually welding. Set "
			"modeling.PolyEdit.AllowWeldInternalBowtie to true to allow a true weld."), 
			EToolMessageLevel::UserWarning);
	}
	else if (Result == EMeshResult::Failed_NotABoundaryEdge)
	{
		// This happens if a user is trying to weld internal edges by successive internal vertices.
		//  Handle this the same way as the other weld failure.
		MoveToDestination();
		GetToolManager()->DisplayMessage(WeldIncompleteMessage, EToolMessageLevel::UserWarning);
	}
	else if (!ensure(Result == EMeshResult::Ok))
	{
		GetToolManager()->DisplayMessage(LOCTEXT("WeldVerticesGenericFailure",
			"Could not weld vertices."), EToolMessageLevel::UserWarning);
		return;
	}


	FText TransactionName = LOCTEXT("PolyMeshWeldVerticesChange", "Weld Vertices");
	GetToolManager()->BeginUndoTransaction(TransactionName);
	EmitCurrentMeshChangeAndUpdate(TransactionName, ChangeTracker.EndChange(), FGroupTopologySelection());

	// Now that the topology is updated, set the new selection
	int32 RemainingCornerID = Topology->GetCornerIDFromVertexID(KeptVid);
	if (RemainingCornerID != IndexConstants::InvalidID)
	{
		FGroupTopologySelection NewSelection;
		NewSelection.SelectedCornerIDs.Add(RemainingCornerID);
		SelectionMechanic->SetSelection(NewSelection);
	}

	GetToolManager()->EndUndoTransaction();
}

void UEditMeshPolygonsTool::ApplyStraightenEdges()
{
	if (BeginMeshEdgeEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnStraightenEdgesFailed", "Cannot Straighten current selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();

	for (const FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		const TArray<int32>& EdgeVerts = Topology->GetGroupEdgeVertices(Edge.EdgeTopoID);
		int32 NumV = EdgeVerts.Num();
		if ( NumV > 2 )
		{
			ChangeTracker.SaveVertexOneRingTriangles(EdgeVerts, true);
			FVector3d A(Mesh->GetVertex(EdgeVerts[0])), B(Mesh->GetVertex(EdgeVerts[NumV-1]));
			TArray<double> VtxArcLengths;
			double EdgeArcLen = Topology->GetEdgeArcLength(Edge.EdgeTopoID, &VtxArcLengths);
			for (int k = 1; k < NumV-1; ++k)
			{
				double t = VtxArcLengths[k] / EdgeArcLen;
				Mesh->SetVertex(EdgeVerts[k], UE::Geometry::Lerp(A, B, t));
			}
		}
	}

	// TODO :HandlePositionOnlyMeshChanges Due to the group topology storing edge IDs that do not stay the same across
	// undo/redo events even when the mesh topology stays the same after a FDynamicMeshChange, we actually have to treat
	// all FDynamicMeshChange-based transactions as affecting group topology. Here we only changed vertex positions so
	// we could add a separate overload that takes a FMeshVertexChange, and possibly one that takes an attribute change
	// (or unify the three via an interface)
	FGroupTopologySelection NewSelection;
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshStraightenEdgeChange", "Straighten Edges"), 
		ChangeTracker.EndChange(), NewSelection);
}

void UEditMeshPolygonsTool::ApplyDeleteEdges()
{
	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	FGroupTopologySelection NewSelection;
	FMeshConnectedComponents Components(Mesh);

	// Using sets here because we only want unique triangles/edges
	TSet<int32> EdgeIDs;
	TSet<int32> SeedTriangleIDs;
	for (FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		for (int32 Eid : Edge.EdgeIDs)
		{
			FIndex2i AdjacentTriangles = Mesh->GetEdgeT(Eid);
			EdgeIDs.Add(Eid);
			SeedTriangleIDs.Add(AdjacentTriangles.A);
			if (AdjacentTriangles.B != FDynamicMesh3::InvalidID)
			{
				SeedTriangleIDs.Add(AdjacentTriangles.B);
			}
		}
	}
	
	Components.FindTrianglesConnectedToSeeds(SeedTriangleIDs.Array(), [&Mesh, &EdgeIDs](int32 t0, int32 t1)
	{
		return Mesh->GetTriangleGroup(t0) == Mesh->GetTriangleGroup(t1) || EdgeIDs.Contains(Mesh->FindEdgeFromTriPair(t0,t1));
	});
	
	ChangeTracker.BeginChange();
	
	for (FMeshConnectedComponents::FComponent& Component : Components.Components)
	{
		ChangeTracker.SaveTriangles(Component.Indices, true);
		int32 NewGroupID = Mesh->GetTriangleGroup(Component.Indices[0]);
		FaceGroupUtil::SetGroupID(*Mesh, Component.Indices, NewGroupID);
		NewSelection.SelectedGroupIDs.Add(NewGroupID);
	}
	
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshDeleteEdgesChange", "Delete Edges"),
		ChangeTracker.EndChange(), NewSelection);
}

void UEditMeshPolygonsTool::ApplySimplifyAlongEdges()
{
	if (BeginMeshEdgeEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnSimplifyAlongEdgesFailed", "Cannot Simplify current selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();

	// Storage for edge sets is re-used for each selected polygon edge path
	TSet<int32> SimplifyEdgeSet;

	// Pre-save vertices and triangles along selected edges
	for (const FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		if (Edge.EdgeIDs.Num() > 1)
		{
			const TArray<int32>& EdgeVerts = Topology->GetGroupEdgeVertices(Edge.EdgeTopoID);
			ChangeTracker.SaveVertexOneRingTriangles(EdgeVerts, true);
		}
	}
	
	// Attempt simplification along edges
	for (const FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		if (Edge.EdgeIDs.Num() > 1)
		{
			SimplifyEdgeSet.Reset();
			SimplifyEdgeSet.Append(Edge.EdgeIDs);
			FLocalPlanarSimplify LocalSimplify;
			LocalSimplify.bPreserveVertexNormals = false;
			LocalSimplify.SimplifyAlongEdges(*Mesh, SimplifyEdgeSet);
		}
	}

	FGroupTopologySelection NewSelection;
	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshSimplifyAlongEdgesChange", "Simplify Along Edges"),
		ChangeTracker.EndChange(), NewSelection);
}


void UEditMeshPolygonsTool::ApplyFillHole()
{
	if (BeginMeshBoundaryEdgeEditChange(false) == false)
	{
		GetToolManager()->DisplayMessage( LOCTEXT("OnEdgeFillFailed", "Cannot Fill current selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	FGroupTopologySelection NewSelection;
	for (FSelectedEdge& FillEdge : ActiveEdgeSelection)
	{
		if (Mesh->IsBoundaryEdge(FillEdge.EdgeIDs[0]))		// may no longer be boundary due to previous fill
		{
			FMeshBoundaryLoops BoundaryLoops(Mesh);
			int32 LoopID = BoundaryLoops.FindLoopContainingEdge(FillEdge.EdgeIDs[0]);
			if (LoopID >= 0)
			{
				FEdgeLoop& Loop = BoundaryLoops.Loops[LoopID];
				FSimpleHoleFiller Filler(Mesh, Loop);
				Filler.FillType = FSimpleHoleFiller::EFillType::PolygonEarClipping;
				int32 NewGroupID = Mesh->AllocateTriangleGroup();
				Filler.Fill(NewGroupID);
				if (!bTriangleMode)
				{
					NewSelection.SelectedGroupIDs.Add(NewGroupID);
				}
				else
				{
					NewSelection.SelectedGroupIDs.Append(Filler.NewTriangles);
				}

				// Compute normals and UVs
				if (Mesh->HasAttributes())
				{
					TArray<FVector3d> VertexPositions;
					Loop.GetVertices(VertexPositions);
					FVector3d PlaneOrigin;
					FVector3d PlaneNormal;
					PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);

					FDynamicMeshEditor Editor(Mesh);
					FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
					Editor.SetTriangleNormals(Filler.NewTriangles);
					Editor.SetTriangleUVsFromProjection(Filler.NewTriangles, ProjectionFrame, UVScaleFactor);
				}
			}
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshFillHoleChange", "Fill Hole"),
		ChangeTracker.EndChange(), NewSelection);
}

void UEditMeshPolygonsTool::ApplyBridgeEdges()
{
	using namespace EditMeshPolygonsToolLocals;

	const FText BridgeFailMessage = LOCTEXT("OnEdgeBridgeFailed", "Cannot Bridge current selection");

	FGroupTopologySelection CurrentSelection = SelectionMechanic->GetActiveSelection();

	TSet<int32> GroupEdges;
	if (!CurrentSelection.SelectedCornerIDs.IsEmpty())
	{
		ConvertCornerSelectionToGroupEdgeSelection(Topology.Get(), CurrentSelection.SelectedCornerIDs, GroupEdges);
	}
	else
	{
		GroupEdges = CurrentSelection.SelectedEdgeIDs;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	TArray<int32> GroupEdgesA;
	TArray<int32> GroupEdgesB;
	bool bShouldReverseA = false;
	if (!LinkBoundaryGroupEdges(Topology.Get(), Mesh, GroupEdges.Array(), GroupEdgesA, GroupEdgesB, bShouldReverseA))
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnEdgeBridgeFailedInvalidSelection", 
			"Cannot bridge current selection, selection could not be partitioned into a single hole or two "
			"non-loop open-boundary sequences."), EToolMessageLevel::UserWarning);
		return;
	}

	TArray<int32> TrianglesToSelect;

	auto BridgeEdges = [&](const TArray<int32>& LoopVids)
	{
		TArray<int32> LoopEdges;
		FEdgeLoop::VertexLoopToEdgeLoop(Mesh, LoopVids, LoopEdges);
		FEdgeLoop Loop(Mesh, LoopVids, LoopEdges);

		// We could always use the minimal hole filler, but it doesn't quite do what "bridge" would suggest when
		// the area to be bridged is concave (across two curved-inward edges). Meanwhile simple ear clipping
		// seems to fail in some common cases for reasons that we should investigate. For now, start with ear
		// clipping, and revert to minimal if needed.
		FSimpleHoleFiller SimpleHoleFiller(Mesh, Loop, FSimpleHoleFiller::EFillType::PolygonEarClipping);
		TArray<int32> NewTriangles;

		// Fill the hole
		if (!SimpleHoleFiller.Fill())
		{
			//Ear clipping doesn't add vertices, so don't need to delete isolated verts
			FDynamicMeshEditor Editor(Mesh);
			Editor.RemoveTriangles(SimpleHoleFiller.NewTriangles, false);

			FMinimalHoleFiller MinimalHoleFiller(Mesh, Loop);

			if (!MinimalHoleFiller.Fill())
			{
				Editor.RemoveTriangles(MinimalHoleFiller.NewTriangles, false);
				GetToolManager()->DisplayMessage(BridgeFailMessage, EToolMessageLevel::UserWarning);
				// Even though we've manually 'undone' the changes, this will still change mesh timestamps, so we need to register the mesh update
				UpdateFromCurrentMesh(false);
				return;
			}
			NewTriangles = MinimalHoleFiller.NewTriangles;
		}
		else 
		{
			NewTriangles = SimpleHoleFiller.NewTriangles;
		}

		TrianglesToSelect.Append(NewTriangles);

		// Compute normals and UVs
		if (Mesh->HasAttributes())
		{
			TArray<FVector3d> VertexPositions;
			Loop.GetVertices(VertexPositions);
			FVector3d PlaneOrigin;
			FVector3d PlaneNormal;
			PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);

			FDynamicMeshEditor Editor(Mesh);
			FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
			Editor.SetTriangleNormals(NewTriangles);
			Editor.SetTriangleUVsFromProjection(NewTriangles, ProjectionFrame, UVScaleFactor);
		}
	};

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();

	if (GroupEdgesB.IsEmpty())
	{
		// This means that there was just one big hole. Concatenate everything into one
		//  big loop to triangulate.
		TArray<int32> LoopVertices;
		for (int32 i = 0; i < GroupEdgesA.Num(); ++i)
		{
			FEdgeSpan Span = Topology->Edges[GroupEdgesA[i]].Span;
			Span.SetCorrectOrientation();
			if (LoopVertices.Num() > 0 && LoopVertices.Last() == Span.Vertices[0])
			{
				LoopVertices.Pop();
			}
			LoopVertices.Append(Span.Vertices);
		}
		if (!ensure(LoopVertices.Num() > 0 && LoopVertices.Last() == LoopVertices[0]))
		{
			GetToolManager()->DisplayMessage(BridgeFailMessage, EToolMessageLevel::UserWarning);
			return;
		}
		LoopVertices.Pop();

		BridgeEdges(LoopVertices);
	}
	else
	{
		// We will bridge as many pairs as we can, and then do one big "triangular" bridge for
		//  the unmatched ones.

		// The two sequences are given in boundary orientation, which will be in the opposite direction.
		//  So, we need to process one of the sequences in reverse order.
		Algo::Reverse(bShouldReverseA ? GroupEdgesA : GroupEdgesB);

		// Keeps track of the endpoints so we can use it for the final triangular bridge
		FIndex2i LastVids(IndexConstants::InvalidID, IndexConstants::InvalidID);

		int32 NumMatched = FMath::Min(GroupEdgesA.Num(), GroupEdgesB.Num());
		for (int32 i = 0; i < NumMatched; ++i)
		{
			FEdgeSpan SpanA = Topology->Edges[GroupEdgesA[i]].Span;
			SpanA.SetCorrectOrientation();
			FEdgeSpan SpanB = Topology->Edges[GroupEdgesB[i]].Span;
			SpanB.SetCorrectOrientation();

			TArray<int32> LoopVertices;
			LoopVertices.Append(SpanA.Vertices);
			LoopVertices.Append(SpanB.Vertices);

			LastVids.A = SpanA.Vertices[bShouldReverseA ? 0 : SpanA.Vertices.Num() - 1];
			LastVids.B = SpanB.Vertices[!bShouldReverseA ? 0 : SpanB.Vertices.Num() - 1];

			BridgeEdges(LoopVertices);
		}

		if (GroupEdgesA.Num() != GroupEdgesB.Num())
		{
			bool bAIsLonger = GroupEdgesA.Num() > GroupEdgesB.Num();
			TArray<int32>& LongerSequence = bAIsLonger ? GroupEdgesA : GroupEdgesB;
			TArray<int32> RemainingEdges;
			for (int32 i = NumMatched; i < LongerSequence.Num(); ++i)
			{
				RemainingEdges.Add(LongerSequence[i]);
			}
			if (bAIsLonger == bShouldReverseA)
			{
				// If the remaining edges are the ones we reversed for the pairwise iteration, we
				//  need to reverse them back so that they are in proper boundary ordering.
				Algo::Reverse(RemainingEdges);
			}

			int32 OtherLastVid = bAIsLonger ? LastVids.B : LastVids.A;
			if (ensure(OtherLastVid != IndexConstants::InvalidID))
			{
				TArray<int32> LoopVertices;
				for (int32 i = 0; i < RemainingEdges.Num(); ++i)
				{
					FEdgeSpan Span = Topology->Edges[RemainingEdges[i]].Span;
					Span.SetCorrectOrientation();
					if (LoopVertices.Num() > 0 && ensure(LoopVertices.Last() == Span.Vertices[0]))
					{
						LoopVertices.Pop();
					}
					LoopVertices.Append(Span.Vertices);
				}
				LoopVertices.Add(OtherLastVid);

				BridgeEdges(LoopVertices);
			}
		}//end if have unmatched edges
	}

	FText TransactionName = LOCTEXT("PolyMeshBridgeEdgeChange", "Bridge Edge");
	GetToolManager()->BeginUndoTransaction(TransactionName);

	EmitCurrentMeshChangeAndUpdate(TransactionName, ChangeTracker.EndChange(), FGroupTopologySelection());

	// Now that topology is updated, set the new selection
	FGroupTopologySelection NewSelection;
	for (int32 Tid : TrianglesToSelect)
	{
		NewSelection.SelectedGroupIDs.Add(Topology->GetGroupID(Tid));
	}
	if (ensure(!NewSelection.IsEmpty()))
	{
		SelectionMechanic->SetSelection(NewSelection);
	}

	GetToolManager()->EndUndoTransaction();
}


void UEditMeshPolygonsTool::ApplyPokeSingleFace()
{
	if (BeginMeshFaceEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnPokeFailedMessage", "Cannot Poke Current Selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	ChangeTracker.SaveTriangles(ActiveTriangleSelection, true);
	FGroupTopologySelection NewSelection;
	for (int32 tid : ActiveTriangleSelection)
	{
		FDynamicMesh3::FPokeTriangleInfo PokeInfo;
		NewSelection.SelectedGroupIDs.Add(tid);
		if (Mesh->PokeTriangle(tid, PokeInfo) == EMeshResult::Ok)
		{
			NewSelection.SelectedGroupIDs.Add(PokeInfo.NewTriangles.A);
			NewSelection.SelectedGroupIDs.Add(PokeInfo.NewTriangles.B);
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshPokeChange", "Poke Faces"),
		ChangeTracker.EndChange(), NewSelection);
}



void UEditMeshPolygonsTool::ApplyFlipSingleEdge()
{
	FDynamicMesh3* Mesh = CurrentMesh.Get();

	if (!BeginMeshEdgeEditChange([Mesh](int32 Eid) {return !Mesh->IsBoundaryEdge(Eid) && !Mesh->Attributes()->IsSeamEdge(Eid); }))
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnFlipFailedMessage", "Cannot Flip Current Selection (no non-seam edges selected)"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	for (FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		int32 eid = Edge.EdgeIDs[0];
		if (Mesh->IsEdge(eid) && Mesh->IsBoundaryEdge(eid) == false && Mesh->Attributes()->IsSeamEdge(eid) == false)
		{
			FIndex2i et = Mesh->GetEdgeT(eid);
			ChangeTracker.SaveTriangle(et.A, true);
			ChangeTracker.SaveTriangle(et.B, true);
			FDynamicMesh3::FEdgeFlipInfo FlipInfo;
			Mesh->FlipEdge(eid, FlipInfo);
		}
	}

	// After flipping edges, edge ID's stay the same but group edge id's in our topology end up swapping around after
	// the topology rebuild (because they are assigned in order of iteration through triangles). In order to update them,
	// we need to go ahead and do the topology rebuild now. This means that we don't actually need to do the rebuild
	// that happens inside EmitCurrentMeshChangeAndUpdate, but it's not worth trying to refactor to avoid it, so we
	// just end up doing an extraneous update.
	FGroupTopologySelection NewSelection;
	Topology->RebuildTopology();
	for (FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		int32 Eid = Edge.EdgeIDs[0];
		if (Mesh->IsEdge(Eid))
		{
			NewSelection.SelectedEdgeIDs.Add(Topology->FindGroupEdgeID(Eid));
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshFlipChange", "Flip Edges"),
		ChangeTracker.EndChange(), FGroupTopologySelection());
}

void UEditMeshPolygonsTool::ApplyCollapseEdge()
{
	using namespace EditMeshPolygonsToolLocals;

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FGroupTopologySelection ActiveSelection = SelectionMechanic->GetActiveSelection();
	if (ActiveSelection.IsEmpty())
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnCollapseFailedMessage", "Cannot collapse empty selection"), EToolMessageLevel::UserWarning);
		return;
	}

	GetToolManager()->BeginUndoTransaction(CollapseEdgeTransactionLabel);

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();

	TSet<int32> GroupEdgesToCollapse = ActiveSelection.SelectedEdgeIDs;
	if (!ActiveSelection.SelectedGroupIDs.IsEmpty())
	{
		// Retriangulating can be thought of as equivalent to collapsing all the interior
		//  edges, except without the risk of failing because of some kind of awful topology
		//  on the inside of the group.
		int32 NumCompleted = RetriangulateGroups(Mesh, Topology.Get(), ActiveSelection.SelectedGroupIDs, ChangeTracker);
		if (NumCompleted != ActiveSelection.SelectedGroupIDs.Num())
		{
			GetToolManager()->DisplayMessage(PartialCollapseFailureMessage, EToolMessageLevel::UserWarning);
			// continue on and try to collapse the boundary
		}
		// After retriangulation, our topology object may have incorrect eids (if the group was on the boundary,
		//  so the edges got removed during triangle deletion and then recreated). So we have to update it.
		Topology->RebuildTopology();

		for (int32 GroupID : ActiveSelection.SelectedGroupIDs)
		{
			Topology->ForGroupEdges(GroupID, [&GroupEdgesToCollapse](const FGroupTopology::FGroupEdge&, int32 GroupEdgeID)
			{
				GroupEdgesToCollapse.Add(GroupEdgeID);
			});
		}
	}
	else if (!ActiveSelection.SelectedCornerIDs.IsEmpty())
	{
		ConvertCornerSelectionToGroupEdgeSelection(Topology.Get(), ActiveSelection.SelectedCornerIDs, GroupEdgesToCollapse);
	}

	CollapseGroupEdges(GroupEdgesToCollapse, ChangeTracker);

	GetToolManager()->EndUndoTransaction();
}

void UEditMeshPolygonsTool::CollapseGroupEdges(TSet<int32>& GroupEdgesToCollapse, 
	UE::Geometry::FDynamicMeshChangeTracker& ChangeTracker)
{
	using namespace EditMeshPolygonsToolLocals;

	FDynamicMesh3* Mesh = CurrentMesh.Get();

	FDynamicMesh3::FCollapseEdgeOptions CollapseOptions;
	CollapseOptions.bAllowHoleCollapse = true;
	CollapseOptions.bAllowCollapsingInternalEdgeWithBoundaryVertices = true;
	CollapseOptions.bAllowTetrahedronCollapse = true;

	TSet<int32> EidsToCollapse;
	for (int32 GroupEdgeID : GroupEdgesToCollapse)
	{
		EidsToCollapse.Append(Topology->GetGroupEdgeEdges(GroupEdgeID));
	}

	// Partition our edges into connected components so that we can collapse into their
	//  individual centroids.
	TArray<TSet<int32>> EidComponents;
	TSet<int32> PartitionedEids;
	TArray<int32> TempQueue;
	for (int32 Eid : EidsToCollapse)
	{
		if (PartitionedEids.Contains(Eid))
		{
			continue;
		}

		TSet<int32>& ComponentEids = EidComponents.Emplace_GetRef();
		ComponentEids.Add(Eid);
		FMeshConnectedComponents::GrowToConnectedEdges(*Mesh, { Eid }, ComponentEids, &TempQueue,
			[&EidsToCollapse](int32 CurrentEid, int32 NeighborEid)
			{
				return EidsToCollapse.Contains(NeighborEid);
			});
		PartitionedEids.Append(ComponentEids);
	}

	// Now process our components.
	bool bAllCollapsesSuccessful = true;
	TSet<int32> NewSelectionVids;
	for (TSet<int32>& Component : EidComponents)
	{
		FVector3d Centroid = FVector3d::Zero();
		for (int32 Eid : Component)
		{
			Centroid += Mesh->GetEdgePoint(Eid, 0.5);
		}
		Centroid /= Component.Num();

		// Unfiltered because vids will disappear in subsequent collapses
		TSet<int32> UnfilteredVidsToMove;

		for (int32 Eid : Component)
		{
			// Some edges might be collapsed away by other collapses
			if (!Mesh->IsEdge(Eid))
			{
				continue;
			}

			FIndex2i EdgeVids = Mesh->GetEdgeV(Eid);
			ChangeTracker.SaveVertexOneRingTriangles(EdgeVids.A, true);
			ChangeTracker.SaveVertexOneRingTriangles(EdgeVids.B, true);
			FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
			EMeshResult Result = Mesh->CollapseEdge(EdgeVids.A, EdgeVids.B, CollapseOptions, CollapseInfo);

			// Certain collapses of isolated triangles/quads are not currently allowed by CollapseEdge,
			//  but we allow them if the user asks for them.
			if (Result == EMeshResult::Failed_CollapseTriangle
				|| Result == EMeshResult::Failed_CollapseQuad
				|| Result == EMeshResult::Failed_FoundDuplicateTriangle)
			{
				bAllCollapsesSuccessful = RemoveEdgeTrisIfNotLast(*CurrentMesh, Eid) && bAllCollapsesSuccessful;
			}
			// We could also check for EMeshResult::InvalidTopology and do the "move with seam"
			//  approach we do for welding, but it seems like it would be harder to notice this
			//  for collapses because the degenerate triangles are harder to find than open boundaries.
			//  So for now we won't fake a collapse in that case.
			else if (Result == EMeshResult::Ok)
			{
				UnfilteredVidsToMove.Add(CollapseInfo.KeptVertex);
			}
			else
			{
				bAllCollapsesSuccessful = false;
			}
		}

		for (int32 Vid : UnfilteredVidsToMove)
		{
			if (Mesh->IsVertex(Vid))
			{
				Mesh->SetVertex(Vid, Centroid);
				NewSelectionVids.Add(Vid);
			}
		}
	}

	if (!bAllCollapsesSuccessful)
	{
		GetToolManager()->DisplayMessage(PartialCollapseFailureMessage, EToolMessageLevel::UserWarning);
	}

	EmitCurrentMeshChangeAndUpdate(CollapseEdgeTransactionLabel, ChangeTracker.EndChange(), FGroupTopologySelection());

	// Now that the topology is updated, we can get the new corner id's to
	//  set the new selection.
	FGroupTopologySelection NewSelection;
	for (int32 Vid : NewSelectionVids)
	{
		// Even though we filtered each component, it's possible for one component's collapses to indirectly
		//  destroy verts in another, hence the check here.
		if (!Mesh->IsVertex(Vid))
		{
			continue;
		}
		int32 CornerID = Topology->GetCornerIDFromVertexID(Vid);
		if (CornerID != IndexConstants::InvalidID)
		{
			NewSelection.SelectedCornerIDs.Add(CornerID);
		}
	}
	// Seems possible to end up with an empty selection if we collapsed a triangle hole in a group,
	//  so the new vertex is not part of a group boundary.
	if (!NewSelection.IsEmpty())
	{
		SelectionMechanic->SetSelection(NewSelection);
	}
}

void UEditMeshPolygonsTool::ApplySplitSingleEdge()
{
	if (BeginMeshEdgeEditChange() == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("OnSplitFailedMessage", "Cannot Split Current Selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FGroupTopologySelection NewSelection;
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	for (FSelectedEdge& Edge : ActiveEdgeSelection)
	{
		int32 eid = Edge.EdgeIDs[0];
		if (Mesh->IsEdge(eid))
		{
			FIndex2i et = Mesh->GetEdgeT(eid);
			ChangeTracker.SaveTriangle(et.A, true);
			NewSelection.SelectedGroupIDs.Add(et.A);
			if (et.B != FDynamicMesh3::InvalidID)
			{
				ChangeTracker.SaveTriangle(et.B, true);
				NewSelection.SelectedGroupIDs.Add(et.B);
			}
			FDynamicMesh3::FEdgeSplitInfo SplitInfo;
			if (Mesh->SplitEdge(eid, SplitInfo) == EMeshResult::Ok)
			{
				NewSelection.SelectedGroupIDs.Add(SplitInfo.NewTriangles.A);
				if (SplitInfo.NewTriangles.B != FDynamicMesh3::InvalidID)
				{
					NewSelection.SelectedGroupIDs.Add(SplitInfo.NewTriangles.B);
				}
			}
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshSplitChange", "Split Edges"),
		ChangeTracker.EndChange(), NewSelection);
}




bool UEditMeshPolygonsTool::BeginMeshFaceEditChange()
{
	ActiveTriangleSelection.Reset();

	// need some selected faces
	const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
	Topology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);
	if (ActiveSelection.SelectedGroupIDs.Num() == 0 || ActiveTriangleSelection.Num() == 0)
	{
		return false;
	}

	const FDynamicMesh3* Mesh = CurrentMesh.Get();
	ActiveSelectionBounds = FAxisAlignedBox3d::Empty();
	for (int tid : ActiveTriangleSelection)
	{
		ActiveSelectionBounds.Contain(Mesh->GetTriBounds(tid));
	}

	// world and local frames
	ActiveSelectionFrameLocal = Topology->GetSelectionFrame(ActiveSelection);
	ActiveSelectionFrameWorld = ActiveSelectionFrameLocal;
	ActiveSelectionFrameWorld.Transform(WorldTransform);

	return true;
}

void UEditMeshPolygonsTool::EmitCurrentMeshChangeAndUpdate(const FText& TransactionLabel,
	TUniquePtr<FDynamicMeshChange> MeshChangeIn,
	const FGroupTopologySelection& OutputSelection)
{
	// We used to take this as a paremeter, but even if we happen to know that the FDynamicMeshChange doesn't
	// involve topology changes, it acts via deleting/reinserting triangles in undo/redo, which changes the
	// eids in a mesh and causes problems. So we always treat the group topology as modified in this function.
	// TODO: Have an overload that uses a vertex change for non-topology-modifying cases.
	constexpr bool bGroupTopologyModified = true;

	// open top-level transaction
	GetToolManager()->BeginUndoTransaction(TransactionLabel);

	// Since we clear the selection in the selection mechanic when topology changes, we need to know
	// when OutputSelection is pointing to the selection in the selection mechanic and is not empty,
	// so that we can copy it ahead of time and reinstate it.
	bool bReferencingSameSelection = (&SelectionMechanic->GetActiveSelection() == &OutputSelection);

	// Not actually relevant since our assumption of topology being modified means we always clear existing selection.
	// bool bSelectionModified = !bReferencingSameSelection && SelectionMechanic->GetActiveSelection() != OutputSelection;

	// In case we need to make a selection copy
	FGroupTopologySelection TempSelection;
	const FGroupTopologySelection* OutputSelectionToUse = &OutputSelection;

	// Emit a selection clear before emitting the mesh change, so that undo restores it properly.
	if (!SelectionMechanic->GetActiveSelection().IsEmpty() /* && (bSelectionModified || bGroupTopologyModified) */)
	{
		if (bReferencingSameSelection)
		{
			// Need to make a copy because OutputSelection will get cleared
			TempSelection = OutputSelection;
			OutputSelectionToUse = &TempSelection;
		}

		SelectionMechanic->BeginChange();
		SelectionMechanic->ClearSelection();
		GetToolManager()->EmitObjectChange(SelectionMechanic, SelectionMechanic->EndChange(), LOCTEXT("ClearSelection", "Clear Selection"));
	}

	// Prep and emit the mesh change. This needs to be bookended by the change in extra corners, since
	// those get regenerated in the topology rebuild.
	TUniquePtr<FEditMeshPolygonsToolMeshChange> ChangeToEmit = MakeUnique<FEditMeshPolygonsToolMeshChange>(MoveTemp(MeshChangeIn));
	if (!Topology->GetCurrentExtraCornerVids().IsEmpty())
	{
		ChangeToEmit->ExtraCornerVidsBefore = Topology->GetCurrentExtraCornerVids();
	}
	Topology->RebuildTopology();
	if (!Topology->GetCurrentExtraCornerVids().IsEmpty())
	{
		ChangeToEmit->ExtraCornerVidsAfter = Topology->GetCurrentExtraCornerVids();
	}
	GetToolManager()->EmitObjectChange(this, 
		MoveTemp(ChangeToEmit),
		TransactionLabel);

	// Update other related structures
	UpdateFromCurrentMesh(false);
	SelectionMechanic->NotifyMeshChanged(true); // This wasn't updated in UpdateFromCurrentMesh because we didn't ask to rebuild topology
	ModifiedTopologyCounter += bGroupTopologyModified;

	// Set output selection if there's a non-empty one. We know we've cleared the selection by
	// this point due to treating topology as always modified.
	if (!OutputSelectionToUse->IsEmpty() /* && (bSelectionModified || bGroupTopologyModified) */)
	{
		SelectionMechanic->BeginChange();
		SelectionMechanic->SetSelection(*OutputSelectionToUse);
		GetToolManager()->EmitObjectChange(SelectionMechanic, SelectionMechanic->EndChange(), LOCTEXT("SetSelection", "Set Selection"));
	}

	GetToolManager()->EndUndoTransaction();
}

void UEditMeshPolygonsTool::EmitActivityStart(const FText& TransactionLabel)
{
	++ActivityTimestamp;

	GetToolManager()->BeginUndoTransaction(TransactionLabel);
	GetToolManager()->EmitObjectChange(this,
		MakeUnique<FPolyEditActivityStartChange>(ActivityTimestamp),
		TransactionLabel);
	GetToolManager()->EndUndoTransaction();
}


bool UEditMeshPolygonsTool::BeginMeshEdgeEditChange()
{
	return BeginMeshEdgeEditChange([](int32) {return true; });
}

bool UEditMeshPolygonsTool::BeginMeshBoundaryEdgeEditChange(bool bOnlySimple)
{
	if (bOnlySimple)
	{
		return BeginMeshEdgeEditChange(
			[&](int32 GroupEdgeID) { return Topology->IsBoundaryEdge(GroupEdgeID) && Topology->IsSimpleGroupEdge(GroupEdgeID); });
	}
	else
	{
		return BeginMeshEdgeEditChange(
			[&](int32 GroupEdgeID) { return Topology->IsBoundaryEdge(GroupEdgeID); });
	}
}

bool UEditMeshPolygonsTool::BeginMeshEdgeEditChange(TFunctionRef<bool(int32)> GroupEdgeIDFilterFunc)
{
	ActiveEdgeSelection.Reset();

	const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
	int NumEdges = ActiveSelection.SelectedEdgeIDs.Num();
	if (NumEdges == 0)
	{
		return false;
	}
	ActiveEdgeSelection.Reserve(NumEdges);
	for (int32 EdgeID : ActiveSelection.SelectedEdgeIDs)
	{
		if (GroupEdgeIDFilterFunc(EdgeID))
		{
			FSelectedEdge& Edge = ActiveEdgeSelection.Emplace_GetRef();
			Edge.EdgeTopoID = EdgeID;
			Edge.EdgeIDs = Topology->GetGroupEdgeEdges(EdgeID);
		}
	}

	return ActiveEdgeSelection.Num() > 0;
}

void UEditMeshPolygonsTool::SetActionButtonPanelsVisible(bool bVisible)
{
	if (bTriangleMode == false)
	{
		if (EditActions)
		{
			SetToolPropertySourceEnabled(EditActions, bVisible);
		}
		if (EditEdgeActions)
		{
			SetToolPropertySourceEnabled(EditEdgeActions, bVisible);
		}
		if (EditUVActions)
		{
			SetToolPropertySourceEnabled(EditUVActions, bVisible);
		}
	}
	else
	{
		if (EditActions_Triangles)
		{
			SetToolPropertySourceEnabled(EditActions_Triangles, bVisible);
		}
		if (EditEdgeActions_Triangles)
		{
			SetToolPropertySourceEnabled(EditEdgeActions_Triangles, bVisible);
		}
	}
}


bool UEditMeshPolygonsTool::CanCurrentlyNestedCancel()
{
	if ( bTerminateOnPendingActionComplete ) return false;

	return CurrentActivity != nullptr 
		|| (SelectionMechanic && !SelectionMechanic->GetActiveSelection().IsEmpty());
}

bool UEditMeshPolygonsTool::ExecuteNestedCancelCommand()
{
	if (CurrentActivity)
	{
		EndCurrentActivity(EToolShutdownType::Cancel);
		return true;
	}
	else if (SelectionMechanic && !SelectionMechanic->GetActiveSelection().IsEmpty())
	{
		SelectionMechanic->BeginChange();
		SelectionMechanic->ClearSelection();
		GetToolManager()->EmitObjectChange(SelectionMechanic, SelectionMechanic->EndChange(), LOCTEXT("ClearSelection", "Clear Selection"));
		return true;
	}
	return false;
}

bool UEditMeshPolygonsTool::CanCurrentlyNestedAccept()
{
	if ( bTerminateOnPendingActionComplete ) return false;
	return CurrentActivity != nullptr;
}

bool UEditMeshPolygonsTool::ExecuteNestedAcceptCommand()
{
	if (CurrentActivity)
	{
		EndCurrentActivity(EToolShutdownType::Accept);
		return true;
	}
	return false;
}

void FEditMeshPolygonsToolMeshChange::Apply(UObject* Object)
{
	UEditMeshPolygonsTool* Tool = Cast<UEditMeshPolygonsTool>(Object);
	
	MeshChange->Apply(Tool->CurrentMesh.Get(), false);
	Tool->UpdateFromCurrentMesh(false);
	++Tool->ModifiedTopologyCounter;

	Tool->RebuildTopologyWithGivenExtraCorners(ExtraCornerVidsAfter);

	Tool->ActivityContext->OnUndoRedo.Broadcast(true);
}

void FEditMeshPolygonsToolMeshChange::Revert(UObject* Object)
{
	UEditMeshPolygonsTool* Tool = Cast<UEditMeshPolygonsTool>(Object);
	
	MeshChange->Apply(Tool->CurrentMesh.Get(), true);
	Tool->UpdateFromCurrentMesh(false);
	++Tool->ModifiedTopologyCounter;

	Tool->RebuildTopologyWithGivenExtraCorners(ExtraCornerVidsBefore);

	Tool->ActivityContext->OnUndoRedo.Broadcast(true);
}

FString FEditMeshPolygonsToolMeshChange::ToString() const
{
	return TEXT("FEditMeshPolygonsToolMeshChange");
}

void FPolyEditActivityStartChange::Revert(UObject* Object)
{
	//note: previously called EndCurrentActivity() which defauled to Cancel, so leaving that behavior here...
	Cast<UEditMeshPolygonsTool>(Object)->EndCurrentActivity(EToolShutdownType::Cancel);
	bHaveDoneUndo = true;
}
bool FPolyEditActivityStartChange::HasExpired(UObject* Object) const
{
	return bHaveDoneUndo 
		|| Cast<UEditMeshPolygonsTool>(Object)->ActivityTimestamp != ActivityTimestamp;
}
FString FPolyEditActivityStartChange::ToString() const
{
	return TEXT("FPolyEditActivityStartChange");
}

#undef LOCTEXT_NAMESPACE

