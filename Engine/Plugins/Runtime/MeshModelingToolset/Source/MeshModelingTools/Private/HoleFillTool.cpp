// Copyright Epic Games, Inc. All Rights Reserved.

#include "HoleFillTool.h"
#include "ToolBuilderUtil.h"
#include "InteractiveToolManager.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "Changes/DynamicMeshChangeTarget.h"
#include "DynamicMeshToMeshDescription.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "MeshBoundaryLoops.h"
#include "MeshOpPreviewHelpers.h"
#include "Selection/BoundarySelectionMechanic.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HoleFillTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UHoleFillTool"

/*
 * ToolBuilder
 */

USingleSelectionMeshEditingTool* UHoleFillToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UHoleFillTool>(SceneState.ToolManager);
}

bool UHoleFillToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	// We're disallowing volumes because they shouldn't have holes to fill intrinsically.
	return USingleSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

/*
 * Tool properties
 */
void UHoleFillToolActions::PostAction(EHoleFillToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


void UHoleFillStatisticsProperties::Initialize(const UHoleFillTool& HoleFillTool)
{
	if (!HoleFillTool.BoundaryLoops.IsValid())
	{
		return;
	}

	int Initial = HoleFillTool.BoundaryLoops->Loops.Num();
	int Selected = 0;
	int Successful = 0;
	int Failed = 0;
	int Remaining = Initial;
	
	InitialHoles = FString::FromInt(Initial);
	SelectedHoles = FString::FromInt(Selected);
	SuccessfulFills = FString::FromInt(Successful);
	FailedFills = FString::FromInt(Failed);
	RemainingHoles = FString::FromInt(Remaining);
}

void UHoleFillStatisticsProperties::Update(const UHoleFillTool& HoleFillTool, const FHoleFillOp& Op)
{
	if (!HoleFillTool.BoundaryLoops.IsValid())
	{
		return;
	}

	int Initial = HoleFillTool.BoundaryLoops->Loops.Num();
	int Selected = Op.Loops.Num();
	int Failed = Op.NumFailedLoops;
	int Successful = Selected - Failed;
	int Remaining = Initial - Successful;

	InitialHoles = FString::FromInt(Initial);
	SelectedHoles = FString::FromInt(Selected);
	SuccessfulFills = FString::FromInt(Successful);
	FailedFills = FString::FromInt(Failed);
	RemainingHoles = FString::FromInt(Remaining);
}

/*
* Op Factory
*/

TUniquePtr<FDynamicMeshOperator> UHoleFillOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FHoleFillOp> FillOp = MakeUnique<FHoleFillOp>();

	FTransform LocalToWorld = Cast<IPrimitiveComponentBackedTarget>(FillTool->Target)->GetWorldTransform();
	FillOp->SetResultTransform((FTransformSRT3d)LocalToWorld);
	FillOp->OriginalMesh = FillTool->OriginalMesh;
	FillOp->MeshUVScaleFactor = FillTool->MeshUVScaleFactor;
	FillTool->GetLoopsToFill(FillOp->Loops);
	FillOp->FillType = FillTool->Properties->FillType;

	FillOp->FillOptions.bRemoveIsolatedTriangles = FillTool->Properties->bRemoveIsolatedTriangles;
	FillOp->FillOptions.bQuickFillSmallHoles = FillTool->Properties->bQuickFillSmallHoles;

	// Smooth fill properties
	FillOp->SmoothFillOptions = FillTool->SmoothHoleFillProperties->ToSmoothFillOptions();

	return FillOp;
}


/*
 * Tool
 */

void UHoleFillTool::Setup()
{
	USingleSelectionTool::Setup();

	if (!Target)
	{
		return;
	}

	// create mesh to operate on
	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*OriginalMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);

	// initialize properties
	Properties = NewObject<UHoleFillToolProperties>(this, TEXT("Hole Fill Settings"));
	Properties->RestoreProperties(this);
	AddToolPropertySource(Properties);
	SetToolPropertySourceEnabled(Properties, true);

	SmoothHoleFillProperties = NewObject<USmoothHoleFillProperties>(this, TEXT("Smooth Fill Settings"));
	SmoothHoleFillProperties->RestoreProperties(this);
	AddToolPropertySource(SmoothHoleFillProperties);
	SetToolPropertySourceEnabled(SmoothHoleFillProperties, Properties->FillType == EHoleFillOpFillType::Smooth);

	// Set up a callback for when the type of fill changes
	Properties->WatchProperty(Properties->FillType,
		[this](EHoleFillOpFillType NewType)
	{
		SetToolPropertySourceEnabled(SmoothHoleFillProperties, (NewType == EHoleFillOpFillType::Smooth));
	});

	Actions = NewObject<UHoleFillToolActions>(this, TEXT("Hole Fill Actions"));
	Actions->Initialize(this);
	AddToolPropertySource(Actions);
	SetToolPropertySourceEnabled(Actions, true);

	Statistics = NewObject<UHoleFillStatisticsProperties>();
	AddToolPropertySource(Statistics);
	SetToolPropertySourceEnabled(Statistics, true);

	ToolPropertyObjects.Add(this);

	// initialize hit query
	MeshSpatial.SetMesh(OriginalMesh.Get());

	// initialize topology
	constexpr bool bAutoComputeLoops = true;
	BoundaryLoops = MakeUnique<UE::Geometry::FMeshBoundaryLoops>(OriginalMesh.Get(), bAutoComputeLoops);
	const bool bLoopsOK = !BoundaryLoops->bAborted;

	if (bLoopsOK && BoundaryLoops->Spans.Num() > 0)
	{
		// Boundary loop finding finished but some boundaries were too degenerate to form simple loops (this might be due to the 
		// presence of bowtie vertices on boundaries, for example)
		// Since this tool can tolerate failing to fill a boundary loop, just convert the spans to loops and let them fail
		for (const FEdgeSpan& Span : BoundaryLoops->Spans)
		{
			// skip span if there is no edge connecting the first and last vertices, as these cannot be initialized as loops
			if (Span.Vertices.Num() < 2 || FDynamicMesh3::InvalidID == OriginalMesh->FindEdge(Span.Vertices[0], Span.Vertices.Last()))
			{
				continue;
			}
			FEdgeLoop SpanLoop(OriginalMesh.Get());
			if (SpanLoop.InitializeFromVertices(Span.Vertices))
			{
				BoundaryLoops->Loops.Add(SpanLoop);
			}
		}
		BoundaryLoops->Spans.Empty();
	}


	// Set up selection mechanic to find and select edges
	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	SelectionMechanic = NewObject<UBoundarySelectionMechanic>(this);
	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	SelectionMechanic->Setup(this);
	SelectionMechanic->Properties->bSelectEdges = true;
	SelectionMechanic->Properties->bSelectFaces = false;
	SelectionMechanic->Properties->bSelectVertices = false;
	SelectionMechanic->Initialize(OriginalMesh.Get(),
		(FTransform3d)TargetComponent->GetWorldTransform(),
		GetTargetWorld(),
		BoundaryLoops.Get(),
		[this]() { return &MeshSpatial; }
	);
	// allow toggling selection without modifier key
	SelectionMechanic->SetShouldAddToSelectionFunc([]() {return true; });
	SelectionMechanic->SetShouldRemoveFromSelectionFunc([]() {return true; });

	SelectionMechanic->OnSelectionChanged.AddUObject(this, &UHoleFillTool::OnSelectionModified);
	
	// Store a UV scale based on the original mesh bounds
	MeshUVScaleFactor = (1.0 / OriginalMesh->GetBounds().MaxDim());

	Statistics->Initialize(*this);

	// initialize the PreviewMesh+BackgroundCompute object
	SetupPreview();
	InvalidatePreviewResult();

	if (!bLoopsOK)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("LoopFindError", "Error finding hole boundary loops."),
			EToolMessageLevel::UserWarning);

		SetToolPropertySourceEnabled(Properties, false);
		SetToolPropertySourceEnabled(SmoothHoleFillProperties, false);
		SetToolPropertySourceEnabled(Actions, false);
	}
	else if (BoundaryLoops->Loops.Num() == 0)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoHoleNotification", "This mesh has no holes to fill."),
			EToolMessageLevel::UserWarning);

		SetToolPropertySourceEnabled(Properties, false);
		SetToolPropertySourceEnabled(SmoothHoleFillProperties, false);
		SetToolPropertySourceEnabled(Actions, false);
	}
	else
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("HoleFillToolHighlighted", "Holes in the mesh are highlighted. Select individual holes to fill or use the Select All or Clear buttons."),
			EToolMessageLevel::UserNotification);

		// Hide all meshes except the Preview
		TargetComponent->SetOwnerVisibility(false);
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Fill Holes"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("HoleFillToolDescription", "Fill Holes in the selected Mesh by adding triangles. Click on individual holes to fill them, or use the Select All button to fill all holes."),
		EToolMessageLevel::UserNotification);
}

void UHoleFillTool::OnTick(float DeltaTime)
{
	if (Preview)
	{
		Preview->Tick(DeltaTime);
	}

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EHoleFillToolActions::NoAction;
	}
}

void UHoleFillTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	InvalidatePreviewResult();
}

bool UHoleFillTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidResult();
}

void UHoleFillTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Properties->SaveProperties(this);
	SmoothHoleFillProperties->SaveProperties(this);

	if (SelectionMechanic)
	{
		SelectionMechanic->Shutdown();
	}

	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);

	FDynamicMeshOpResult Result = Preview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("HoleFillToolTransactionName", "Hole Fill Tool"));

		check(Result.Mesh.Get() != nullptr);
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, *Result.Mesh.Get(), true);

		GetToolManager()->EndUndoTransaction();
	}
}

void UHoleFillTool::OnSelectionModified()
{
	UpdateActiveBoundaryLoopSelection();
	InvalidatePreviewResult();
}

void UHoleFillTool::RequestAction(EHoleFillToolActions ActionType)
{
	if (bHavePendingAction)
	{
		return;
	}

	PendingAction = ActionType;
	bHavePendingAction = true;
}


void UHoleFillTool::InvalidatePreviewResult()
{
	// Clear any warning message
	GetToolManager()->DisplayMessage({}, EToolMessageLevel::UserWarning);
	Preview->InvalidateResult();
}

void UHoleFillTool::SetupPreview()
{
	UHoleFillOperatorFactory* OpFactory = NewObject<UHoleFillOperatorFactory>();
	OpFactory->FillTool = this;

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(OpFactory, "Preview");
	Preview->Setup(GetTargetWorld(), OpFactory);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target);

	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);
	Preview->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	// configure secondary render material
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.8f, 0.75f, 0.0f), GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		Preview->PreviewMesh->SetSecondaryRenderMaterial(SelectionMaterial);
	}

	// enable secondary triangle buffers
	Preview->OnOpCompleted.AddLambda(
		[this](const FDynamicMeshOperator* Op)
	{
		const FHoleFillOp* HoleFillOp = (const FHoleFillOp*)(Op);
		NewTriangleIDs = TSet<int32>(HoleFillOp->NewTriangles);

		// Notify the user if any holes could not be filled
		if (HoleFillOp->NumFailedLoops > 0)
		{
			GetToolManager()->DisplayMessage(
				FText::Format(LOCTEXT("FillFailNotification", "Failed to fill {0} holes."), HoleFillOp->NumFailedLoops),
				EToolMessageLevel::UserWarning);
		}

		Statistics->Update(*this, *HoleFillOp);
	});

	Preview->PreviewMesh->EnableSecondaryTriangleBuffers(
		[this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return NewTriangleIDs.Contains(TriangleID);
	});

	// set initial preview to un-processed mesh
	Preview->PreviewMesh->SetTransform(Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform());
	Preview->PreviewMesh->UpdatePreview(OriginalMesh.Get());

	Preview->SetVisibility(true);
}


void UHoleFillTool::ApplyAction(EHoleFillToolActions ActionType)
{
	switch (ActionType)
	{
	case EHoleFillToolActions::SelectAll:
		SelectAll();
		break;
	case EHoleFillToolActions::ClearSelection:
		ClearSelection();
		break;
	}
}

void UHoleFillTool::SelectAll()
{
	FGroupTopologySelection NewSelection;
	for (int32 i = 0; i < BoundaryLoops->Loops.Num(); ++i)
	{
		NewSelection.SelectedEdgeIDs.Add(i);
	}

	SelectionMechanic->SetSelection(NewSelection);
	UpdateActiveBoundaryLoopSelection();
	InvalidatePreviewResult();
}


void UHoleFillTool::ClearSelection()
{
	SelectionMechanic->ClearSelection();
	UpdateActiveBoundaryLoopSelection();
	InvalidatePreviewResult();
}

void UHoleFillTool::UpdateActiveBoundaryLoopSelection()
{
	ActiveBoundaryLoopSelection.Reset();

	const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
	int NumEdges = ActiveSelection.SelectedEdgeIDs.Num();
	if (NumEdges == 0)
	{
		return;
	}

	ActiveBoundaryLoopSelection.Reserve(NumEdges);
	for (int32 EdgeID : ActiveSelection.SelectedEdgeIDs)
	{
		FSelectedBoundaryLoop& Loop = ActiveBoundaryLoopSelection.Emplace_GetRef();
		Loop.EdgeTopoID = EdgeID;
		Loop.EdgeIDs = BoundaryLoops->Loops[EdgeID].Edges;
	}
}


void UHoleFillTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (SelectionMechanic)
	{
		SelectionMechanic->Render(RenderAPI);
	}
}


void UHoleFillTool::GetLoopsToFill(TArray<FEdgeLoop>& OutLoops) const
{
	OutLoops.Reset();
	check(BoundaryLoops.IsValid());

	for (const FSelectedBoundaryLoop& FillEdge : ActiveBoundaryLoopSelection)
	{
		if (OriginalMesh->IsBoundaryEdge(FillEdge.EdgeIDs[0]))		// may no longer be boundary due to previous fill
		{
			int32 LoopID = BoundaryLoops->FindLoopContainingEdge(FillEdge.EdgeIDs[0]);
			if (LoopID >= 0)
			{
				OutLoops.Add(BoundaryLoops->Loops[LoopID]);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

