// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatticeDeformerTool.h"

#include "Mechanics/LatticeControlPointsMechanic.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "BaseGizmos/GizmoRenderingUtil.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "DeformationOps/LatticeDeformerOp.h"
#include "Properties/MeshMaterialProperties.h"
#include "Selection/ToolSelectionUtil.h"
#include "MeshOpPreviewHelpers.h" //FDynamicMeshOpResult
#include "ToolSceneQueriesUtil.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicSubmesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Algo/ForEach.h"
#include "Operations/FFDLattice.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Properties/MeshSculptLayerProperties.h"
#include "MeshSculptLayersManagerAPI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LatticeDeformerTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "ULatticeDeformerTool"


namespace
{
	void MakeLatticeGraph(const FFFDLattice& Lattice, FDynamicGraph3d& Graph)
	{
		const FVector3i& Dims = Lattice.GetDimensions();
		const FVector3d& CellSize = Lattice.GetCellSize();
		const FAxisAlignedBox3d& InitialBounds = Lattice.GetInitialBounds();

		// Add cell corners as vertices

		for (int i = 0; i < Dims.X; ++i)
		{
			const double X = CellSize.X * i;
			for (int j = 0; j < Dims.Y; ++j)
			{
				const double Y = CellSize.Y * j;
				for (int k = 0; k < Dims.Z; ++k)
				{
					const double Z = CellSize.Z * k;

					const FVector3d Position = InitialBounds.Min + FVector3d{ X,Y,Z };
					const int P = Lattice.ControlPointIndexFromCoordinates(i, j, k);
					const int VID = Graph.AppendVertex(Position);
					ensure(VID == P);
				}
			}
		}

		// Connect cell corners with edges

		for (int i = 0; i < Dims.X; ++i)
		{
			for (int j = 0; j < Dims.Y; ++j)
			{
				for (int k = 0; k < Dims.Z; ++k)
				{
					const int P = Lattice.ControlPointIndexFromCoordinates(i, j, k);
					if (i + 1 < Dims.X)
					{
						const int Pi = Lattice.ControlPointIndexFromCoordinates(i + 1, j, k);
						Graph.AppendEdge(P, Pi);
					}

					if (j + 1 < Dims.Y)
					{
						const int Pj = Lattice.ControlPointIndexFromCoordinates(i, j + 1, k);
						Graph.AppendEdge(P, Pj);
					}

					if (k + 1 < Dims.Z)
					{
						const int Pk = Lattice.ControlPointIndexFromCoordinates(i, j, k + 1);
						Graph.AppendEdge(P, Pk);
					}
				}
			}
		}
	}
}

// Tool properties/actions

void ULatticeDeformerToolProperties::PostAction(ELatticeDeformerToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


// Tool builder

UMultiTargetWithSelectionTool* ULatticeDeformerToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	const TObjectPtr<ULatticeDeformerTool> Tool = NewObject<ULatticeDeformerTool>(SceneState.ToolManager);

	// Check if the selected component implements ILatticeStateStorage 
	ensure(SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1);		// should be enforced by CanBuildTool
	SceneState.TargetManager->EnumerateSelectedAndTargetableComponents(SceneState, GetTargetRequirements(), [&Tool](UActorComponent* Component)
	{
		if (Component->GetClass()->ImplementsInterface(ULatticeStateStorage::StaticClass()))
		{
			TScriptInterface<ILatticeStateStorage> LatticeInterface;
			LatticeInterface.SetObject(Component);
			LatticeInterface.SetInterface(Cast<ILatticeStateStorage>(Component));
			Tool->SetLatticeStorage(LatticeInterface);
		}
	});

	return Tool;
}

bool ULatticeDeformerToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (RequiresInputSelection() && UE::Geometry::HaveAvailableGeometrySelection(SceneState) == false )
	{
		return false;
	}

	// disable multi-selection for now
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

const FToolTargetTypeRequirements& ULatticeDeformerToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
	UMaterialProvider::StaticClass(),
	UDynamicMeshProvider::StaticClass(),
	UDynamicMeshCommitter::StaticClass(),
	USceneComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


// Operator factory

TUniquePtr<FDynamicMeshOperator> ULatticeDeformerOperatorFactory::MakeNewOperator()
{
	ELatticeInterpolation OpInterpolationType =
		(LatticeDeformerTool->Settings->InterpolationType == ELatticeInterpolationType::Cubic) ?
		ELatticeInterpolation::Cubic :
		ELatticeInterpolation::Linear;

	TUniquePtr<FLatticeDeformerOp> LatticeDeformOp = nullptr;

	if (!LatticeDeformerTool->bHasSelection)
	{
		LatticeDeformOp = MakeUnique<FLatticeDeformerOp>(
		LatticeDeformerTool->OriginalMesh,
		LatticeDeformerTool->Lattice,
		LatticeDeformerTool->ControlPointsMechanic->GetControlPoints(),
		OpInterpolationType,
		LatticeDeformerTool->Settings->bDeformNormals);
	}
	else
	{
		LatticeDeformOp = MakeUnique<FLatticeDeformerOp>(
		LatticeDeformerTool->OriginalMesh,
		LatticeDeformerTool->Submesh,
		LatticeDeformerTool->WorldTransform,
		LatticeDeformerTool->Lattice,
		LatticeDeformerTool->ControlPointsMechanic->GetControlPoints(),
		OpInterpolationType,
		LatticeDeformerTool->Settings->bDeformNormals);
	}

	return LatticeDeformOp;
}


// Tool itself

FVector3i ULatticeDeformerTool::GetLatticeResolution() const
{
	return FVector3i{ Settings->XAxisResolution, Settings->YAxisResolution, Settings->ZAxisResolution };
}

void ULatticeDeformerTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	ControlPointsMechanic->DrawHUD(Canvas, RenderAPI);
}

bool ULatticeDeformerTool::CanAccept() const
{
	return Preview != nullptr && Preview->HaveValidResult();
}

void ULatticeDeformerTool::InitializeLattice(TArray<FVector3d>& OutLatticePoints, TArray<FVector2i>& OutLatticeEdges)
{
	UE::Geometry::FDynamicMesh3* MeshToDeform = nullptr;
	
	if (bHasSelection && Submesh)
	{
		MeshToDeform = &Submesh->GetSubmesh();
	}
	else
	{
		MeshToDeform = OriginalMesh.Get();
	}

	if (!LatticeStorage)
	{
		Lattice = MakeShared<FFFDLattice, ESPMode::ThreadSafe>(GetLatticeResolution(), *MeshToDeform, Settings->Padding);
	}
	else
	{
		Lattice = MakeShared<FFFDLattice, ESPMode::ThreadSafe>(GetLatticeResolution(), MeshToDeform, 0.0f, LatticeStorage->GetInitialBounds(), LatticeStorage->GetTransform());
	}

	Lattice->GenerateInitialLatticePositions(OutLatticePoints);

	// Put the lattice in world space. (If we have LatticeStorage they will be copied over in world space later.)
	if (!LatticeStorage)
	{
		FTransform3d LocalToWorld(Cast<ISceneComponentBackedTarget>(Targets[0])->GetWorldTransform());
		Algo::ForEach(OutLatticePoints, [&LocalToWorld](FVector3d& Point)
		{
			Point = LocalToWorld.TransformPosition(Point);
		});
	}

	Lattice->GenerateLatticeEdges(OutLatticeEdges);
}

void ULatticeDeformerTool::Setup()
{
	UInteractiveTool::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Lattice Deform"));
	GetToolManager()->DisplayMessage(LOCTEXT("LatticeDeformerToolMessage", 
		"Drag the lattice control points to deform the mesh"), EToolMessageLevel::UserNotification);

	// for now only supports one target
	// TODO: include support for multiple targets
	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*OriginalMesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[0]);
	
	bHasSelection = HasGeometrySelection(0);
	if (bHasSelection)
	{
		TSet<int32> SelectionTriangleROI;
		const FGeometrySelection& InputSelection = GetGeometrySelection(0);
		EnumerateSelectionTriangles(InputSelection, *OriginalMesh,
			[&](int32 TriangleID) { SelectionTriangleROI.Add(TriangleID);});

		Submesh = MakeShared<FDynamicSubmesh3, ESPMode::ThreadSafe>(OriginalMesh.Get(), SelectionTriangleROI.Array());
	}

	// Note: Mesh will be implicitly transformed to world space by transforming the lattice; we account for whether that would invert the mesh here
	MeshTransforms::ReverseOrientationIfNeeded(*OriginalMesh, (Cast<ISceneComponentBackedTarget>(Targets[0])->GetWorldTransform()));

	Settings = NewObject<ULatticeDeformerToolProperties>(this, TEXT("Lattice Deformer Tool Settings"));
	Settings->Initialize(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	// Watch for property changes
	Settings->WatchProperty(Settings->XAxisResolution, [this](int) { bShouldRebuild = true; });
	Settings->WatchProperty(Settings->YAxisResolution, [this](int) { bShouldRebuild = true; });
	Settings->WatchProperty(Settings->ZAxisResolution, [this](int) { bShouldRebuild = true; });
	Settings->WatchProperty(Settings->Padding, [this](float) { bShouldRebuild = true; });
	Settings->WatchProperty(Settings->InterpolationType, [this](ELatticeInterpolationType)
	{
		Preview->InvalidateResult();
	});
	Settings->WatchProperty(Settings->bDeformNormals, [this](bool)
	{
		Preview->InvalidateResult();
	});
	Settings->WatchProperty(Settings->GizmoCoordinateSystem, [this](EToolContextCoordinateSystem)
	{
		ControlPointsMechanic->SetCoordinateSystem(Settings->GizmoCoordinateSystem);
	});
	Settings->WatchProperty(Settings->bSetPivotMode, [this](bool)
	{
		ControlPointsMechanic->UpdateSetPivotMode(Settings->bSetPivotMode);
	});
	Settings->WatchProperty(Settings->bSoftDeformation, [this](bool)
	{
		if (Settings->bSoftDeformation)
		{
			RebuildDeformer();
		}
	});

	if (LatticeStorage)
	{
		const FVector3i StoredResolution = LatticeStorage->GetResolution();
		Settings->XAxisResolution = StoredResolution[0];
		Settings->YAxisResolution = StoredResolution[1];
		Settings->ZAxisResolution = StoredResolution[2];
	}

	TArray<FVector3d> LatticePoints;
	TArray<FVector2i> LatticeEdges;
	InitializeLattice(LatticePoints, LatticeEdges);

	// Set up control points mechanic
	ControlPointsMechanic = NewObject<ULatticeControlPointsMechanic>(this);
	ControlPointsMechanic->Setup(this);
	ControlPointsMechanic->SetWorld(GetTargetWorld());
	FTransform3d LocalToWorld(Cast<ISceneComponentBackedTarget>(Targets[0])->GetWorldTransform());
	WorldTransform = LocalToWorld;
	ControlPointsMechanic->Initialize(LatticePoints, LatticeEdges, LocalToWorld);

	auto OnPointsChangedLambda = [this]()
	{
		if (Settings->bSoftDeformation)
		{
			SoftDeformLattice();
		}
		ResetConstrainedPoints();
		Preview->InvalidateResult();
		Settings->bCanChangeResolution = !ControlPointsMechanic->bHasChanged;
		if (SculptLayerProperties)
		{
			SculptLayerProperties->bCanEditLayers = !ControlPointsMechanic->bHasChanged;
		}
	};
	ControlPointsMechanic->OnPointsChanged.AddLambda(OnPointsChangedLambda);

	ControlPointsMechanic->OnSelectionChanged.AddLambda([this]()
	{
		if (Settings->bSoftDeformation)
		{
			RebuildDeformer();
		}
	});

	ControlPointsMechanic->SetCoordinateSystem(Settings->GizmoCoordinateSystem);
	ControlPointsMechanic->UpdateSetPivotMode(Settings->bSetPivotMode);


	ControlPointsMechanic->ShouldHideGizmo = ULatticeControlPointsMechanic::FShouldHideGizmo::CreateLambda([this]()->bool
	{
		for (int32 VID : ControlPointsMechanic->GetSelectedPointIDs())
		{
			if (!ConstrainedLatticePoints.Contains(VID))
			{
				return false;	// found a selected point that is not constrained
			}
		}
		return true;
	});


	StartPreview();


	if (ISceneComponentBackedTarget* SceneComponentTarget = Cast<ISceneComponentBackedTarget>(Targets[0]))
	{
		if (IMeshSculptLayersManager* SculptLayersManager = Cast<IMeshSculptLayersManager>(SceneComponentTarget->GetOwnerSceneComponent()))
		{
			if (SculptLayersManager->HasSculptLayers())
			{
				SculptLayerProperties = NewObject<UMeshSculptLayerProperties>(this);
				SculptLayerProperties->Init(this, SculptLayersManager->NumLockedBaseSculptLayers());
				AddToolPropertySource(SculptLayerProperties);
			}
		}
	}

	if (LatticeStorage)
	{
		TArray<FVector3d> SavedPoints;
		LatticeStorage->ReadLatticePoints(SavedPoints);
		if (SavedPoints.Num() == LatticePoints.Num())
		{
			ControlPointsMechanic->UpdatePointLocations(SavedPoints);
			ControlPointsMechanic->bHasChanged = true;
			OnPointsChangedLambda();
		}

		Settings->InterpolationType = LatticeStorage->ReadInterpolationType();

		LatticeStorage->InteractiveToolStarted();
	}

	if (Settings->bSoftDeformation)
	{
		RebuildDeformer();
	}

	Settings->SilentUpdateWatched();		// On next tick, don't react to any settings changes made in this function
}


void ULatticeDeformerTool::RebuildDeformer()
{
	LatticeGraph = MakePimpl<UE::Geometry::FDynamicGraph3d>();
	MakeLatticeGraph(*Lattice, *LatticeGraph);

	const TArray<FVector3d>& CurrentLatticePoints = ControlPointsMechanic->GetControlPoints();
	check(LatticeGraph->VertexCount() == CurrentLatticePoints.Num());

	for (int VID : LatticeGraph->VertexIndicesItr())
	{
		LatticeGraph->SetVertex(VID, CurrentLatticePoints[VID]);
	}

	DeformationSolver = UE::MeshDeformation::ConstructUniformConstrainedMeshDeformer(*LatticeGraph);

	for (int LatticePointIndex = 0; LatticePointIndex < CurrentLatticePoints.Num(); ++LatticePointIndex)
	{
		if(ConstrainedLatticePoints.Contains(LatticePointIndex))
		{
			// Pin constraint
			DeformationSolver->AddConstraint(LatticePointIndex, 1.0, ConstrainedLatticePoints[LatticePointIndex], true);
		}
		else 
		{
			if (ControlPointsMechanic->ControlPointIsSelected(LatticePointIndex))
			{
				const FVector3d& MovePosition = CurrentLatticePoints[LatticePointIndex];
				DeformationSolver->AddConstraint(LatticePointIndex, 1.0, MovePosition, true);
			}
		}
	}
}


void ULatticeDeformerTool::ResetConstrainedPoints()
{
	ControlPointsMechanic->UpdatePointLocations(ConstrainedLatticePoints);
}

void ULatticeDeformerTool::SoftDeformLattice()
{
	if (!ensure(Lattice))
	{
		return;
	}

	if (!ensure(ControlPointsMechanic))
	{
		return;
	}

	if (!ensure(DeformationSolver))
	{
		return;
	}
	
	const TArray<FVector3d>& CurrentLatticePoints = ControlPointsMechanic->GetControlPoints();

	if (!ensure(LatticeGraph->VertexCount() == CurrentLatticePoints.Num()))
	{
		return;
	}

	for (int LatticePointIndex = 0; LatticePointIndex < CurrentLatticePoints.Num(); ++LatticePointIndex)
	{
		if (ControlPointsMechanic->ControlPointIsSelected(LatticePointIndex))
		{
			// Don't move pinned points
			if (ConstrainedLatticePoints.Contains(LatticePointIndex))
			{
				continue;
			}

			if (!ensure(DeformationSolver->IsConstrained(LatticePointIndex)))
			{
				continue;
			}

			const FVector3d& MovePosition = CurrentLatticePoints[LatticePointIndex];
			DeformationSolver->UpdateConstraintPosition(LatticePointIndex, MovePosition, true);
		}
	}

	TArray<FVector3d> DeformedLatticePoints;
	DeformationSolver->Deform(DeformedLatticePoints);

	ControlPointsMechanic->UpdateControlPointPositions(DeformedLatticePoints);
}


void ULatticeDeformerTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept && LatticeStorage)
	{
		LatticeStorage->StoreLatticePoints(ControlPointsMechanic->GetControlPoints());
		LatticeStorage->StoreInterpolationType(Settings->InterpolationType);
	}

	Settings->SaveProperties(this);
	ControlPointsMechanic->Shutdown();

	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Targets[0]);
	TargetComponent->SetOwnerVisibility(true);

	if (Preview)
	{
		FDynamicMeshOpResult Result = Preview->Shutdown();

		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("LatticeDeformerTool", "Lattice Deformer"));

			FDynamicMesh3* const DynamicMeshResult = Result.Mesh.Get();
			check(DynamicMeshResult != nullptr);

			// The lattice and its output mesh are in world space, so get them in local space.
			// TODO: Would it make more sense to do all the lattice computation in local space?
			// Note: We skip transforming sculpt layers, since they were never transformed to world space
			FTransform3d LocalToWorld(TargetComponent->GetWorldTransform());
			MeshTransforms::ApplyTransformInverse(*DynamicMeshResult, LocalToWorld, true, ~MeshTransforms::ETransformAttributes::SculptLayers);



			UE::ToolTarget::CommitDynamicMeshUpdate(Targets[0], *DynamicMeshResult, true);

			GetToolManager()->EndUndoTransaction();
		}
	}

	if (LatticeStorage)
	{
		LatticeStorage->InteractiveToolShutDown();
	}
}


void ULatticeDeformerTool::StartPreview()
{
	ULatticeDeformerOperatorFactory* LatticeDeformOpCreator = NewObject<ULatticeDeformerOperatorFactory>();
	LatticeDeformOpCreator->LatticeDeformerTool = this;

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(LatticeDeformOpCreator);
	Preview->Setup(GetTargetWorld(), LatticeDeformOpCreator);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Targets[0]);

	Preview->SetIsMeshTopologyConstant(true, EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals);

	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(Targets[0])->GetMaterialSet(MaterialSet);
	Preview->ConfigureMaterials(MaterialSet.Materials,
								ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	// configure secondary render material
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.8f, 0.75f, 0.0f), GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		Preview->PreviewMesh->SetSecondaryRenderMaterial(SelectionMaterial);
	}

	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::NoTangents);
	Preview->SetVisibility(true);
	Preview->InvalidateResult();

	Cast<ISceneComponentBackedTarget>(Targets[0])->SetOwnerVisibility(false);
}


void ULatticeDeformerTool::ApplyAction(ELatticeDeformerToolAction Action)
{
	switch (Action)
	{
	case ELatticeDeformerToolAction::ClearConstraints:
		ClearConstrainedPoints();
		break;
	case ELatticeDeformerToolAction::Constrain:
		ConstrainSelectedPoints();
		break;
	default:
		break;
	}
}


void ULatticeDeformerTool::OnTick(float DeltaTime)
{
	if (PendingAction != ELatticeDeformerToolAction::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = ELatticeDeformerToolAction::NoAction;
	}

	if (Preview)
	{
		if (bShouldRebuild)
		{
			ClearConstrainedPoints();
			TArray<FVector3d> LatticePoints;
			TArray<FVector2i> LatticeEdges;
			InitializeLattice(LatticePoints, LatticeEdges);
			FTransform3d LocalToWorld(Cast<ISceneComponentBackedTarget>(Targets[0])->GetWorldTransform());
			ControlPointsMechanic->Initialize(LatticePoints, LatticeEdges, LocalToWorld);
			Preview->InvalidateResult();
			bShouldRebuild = false;
		}

		Preview->Tick(DeltaTime);
	}
}


void ULatticeDeformerTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (ControlPointsMechanic != nullptr)
	{
		ControlPointsMechanic->Render(RenderAPI);
	}
}

void ULatticeDeformerTool::SetLatticeStorage(const TScriptInterface<ILatticeStateStorage>& InLatticeStorage)
{
	LatticeStorage = InLatticeStorage;
}

void ULatticeDeformerTool::RequestAction(ELatticeDeformerToolAction Action)
{
	if (PendingAction == ELatticeDeformerToolAction::NoAction)
	{
		PendingAction = Action;
	}
}


static const FText LatticeConstraintChangeTransactionText = LOCTEXT("LatticeConstraintChange", "Lattice Constraint Change");

void ULatticeDeformerTool::ConstrainSelectedPoints()
{
	TMap<int, FVector3d> PrevConstrainedLatticePoints = ConstrainedLatticePoints;
	const TArray<FVector3d>& CurrentControlPointPositions = ControlPointsMechanic->GetControlPoints();
	for (int32 VID : ControlPointsMechanic->GetSelectedPointIDs())
	{
		ConstrainedLatticePoints.FindOrAdd(VID) = CurrentControlPointPositions[VID];
	}
	UpdateMechanicColorOverrides();

	GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeDeformerToolConstrainedPointsChange>(PrevConstrainedLatticePoints,
																									 ConstrainedLatticePoints, 
																									 CurrentChangeStamp), 
									   LatticeConstraintChangeTransactionText);
}

void ULatticeDeformerTool::ClearConstrainedPoints()
{
	TMap<int, FVector3d> PrevConstrainedLatticePoints = ConstrainedLatticePoints;
	ConstrainedLatticePoints.Reset();
	UpdateMechanicColorOverrides();

	GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeDeformerToolConstrainedPointsChange>(PrevConstrainedLatticePoints,
																									 ConstrainedLatticePoints,
																									 CurrentChangeStamp),
									   LatticeConstraintChangeTransactionText);
}


void ULatticeDeformerTool::UpdateMechanicColorOverrides()
{
	ControlPointsMechanic->ClearAllPointColorOverrides();
	for ( const TPair<int32,FVector3d>& Constraint : ConstrainedLatticePoints)
	{
		ControlPointsMechanic->SetPointColorOverride(Constraint.Key, FColor::Cyan);
	}
	RebuildDeformer();
	ControlPointsMechanic->UpdateDrawables();
}

bool ULatticeDeformerTool::AllowToolMeshUpdates() const
{
	return !ControlPointsMechanic->IsGizmoBeingDragged() && !ControlPointsMechanic->bHasChanged;
}

void ULatticeDeformerTool::UpdateToolMeshes(TFunctionRef<TUniquePtr<FMeshRegionChangeBase>(FDynamicMesh3&, int32 MeshIdx)> UpdateMesh)
{
	if (AllowToolMeshUpdates())
	{
		UpdateMesh(*OriginalMesh, 0);
		bShouldRebuild = true;
	}
}

void ULatticeDeformerTool::ProcessToolMeshes(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&, int32 MeshIdx)> ProcessMesh) const
{
	ProcessMesh(*OriginalMesh, 0);
}

int32 ULatticeDeformerTool::NumToolMeshes() const
{
	return 1;
}



void FLatticeDeformerToolConstrainedPointsChange::Apply(UObject* Object)
{
	ULatticeDeformerTool* Tool = Cast<ULatticeDeformerTool>(Object);
	if (!ensure(Tool))
	{
		return;
	}

	Tool->ConstrainedLatticePoints = NewConstrainedLatticePoints;
	Tool->UpdateMechanicColorOverrides();
}

void FLatticeDeformerToolConstrainedPointsChange::Revert(UObject* Object)
{
	ULatticeDeformerTool* Tool = Cast<ULatticeDeformerTool>(Object);
	if (!ensure(Tool))
	{
		return;
	}

	Tool->ConstrainedLatticePoints = PrevConstrainedLatticePoints;
	Tool->UpdateMechanicColorOverrides();
}

FString FLatticeDeformerToolConstrainedPointsChange::ToString() const
{
	return TEXT("FLatticeDeformerToolConstrainedPointsChange");
}


#undef LOCTEXT_NAMESPACE

