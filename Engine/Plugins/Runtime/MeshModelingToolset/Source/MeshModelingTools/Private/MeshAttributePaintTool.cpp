// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshAttributePaintTool.h"
#include "Engine/Engine.h" // for GEngine->VertexColorViewModeMaterial_ColorOnly
#include "Materials/Material.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "Drawing/MeshDebugDrawing.h"
#include "ToolSetupUtil.h"
#include "Selection/StoredMeshSelectionUtil.h" // GetCurrentGeometrySelectionForTarget
#include "Selections/GeometrySelectionUtil.h"
#include "Selections/MeshConnectedComponents.h"

#include "MeshDescription.h"

#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/SceneComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshAttributePaintTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshAttributePaintTool"

namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution AttributePaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution AttributePaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}

class FMeshDescriptionVertexAttributeAdapter : public IMeshVertexAttributeAdapter
{
public:
	FMeshDescription* Mesh;
	FName AttributeName;
	TVertexAttributesRef<float> Attribute;

	FMeshDescriptionVertexAttributeAdapter(FMeshDescription* MeshIn, FName AttribNameIn, TVertexAttributesRef<float> AttribIn)
		: Mesh(MeshIn), AttributeName(AttribNameIn), Attribute(AttribIn)
	{
	}

	virtual int32 ElementNum() const override
	{
		return Attribute.GetNumElements();
	}

	virtual float GetValue(int32 Index) const override
	{
		return Attribute.Get(FVertexID(Index));
	}

	virtual void SetValue(int32 Index, float Value) override
	{
		Attribute.Set(FVertexID(Index), Value);
	}

	virtual FInterval1f GetValueRange() override
	{
		return FInterval1f(0.0f, 1.0f);
	}
};

class FDynamicMeshVertexAttributeAdapter : public IMeshVertexAttributeAdapter
{
public:
	FDynamicMesh3* Mesh;
	FDynamicMeshWeightAttribute* WeightAttribute;

	FDynamicMeshVertexAttributeAdapter(FDynamicMesh3* InMesh, FDynamicMeshWeightAttribute* InWeightAttribute)
		: Mesh(InMesh), WeightAttribute(InWeightAttribute)
	{
	}

	virtual int32 ElementNum() const override
	{
		return Mesh->MaxVertexID();
	}

	virtual float GetValue(int32 Index) const override
	{
		float Wt;
		WeightAttribute->GetValue(Index, &Wt);
		return Wt;
	}

	virtual void SetValue(int32 Index, float Value) override
	{
		WeightAttribute->SetScalarValue(Index, Value);
	}

	virtual FInterval1f GetValueRange() override
	{
		return FInterval1f(0.0f, 1.0f);
	}
};

class FDynamicMeshVertexAttributeSource : public IMeshVertexAttributeSource
{
public:
	FDynamicMesh3* Mesh = nullptr;

	FDynamicMeshVertexAttributeSource(FDynamicMesh3* MeshIn)
	{
		Mesh = MeshIn;
	}

	virtual int32 GetAttributeElementNum() override
	{
		return Mesh->MaxVertexID();
	}

	virtual TArray<FName> GetAttributeList() override
	{
		TArray<FName> Result;
		
		if (FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
		{
			const int32 NumLayers = Attributes->NumWeightLayers();
			Result.Reserve(NumLayers);
			for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
			{
				Result.Add(Attributes->GetWeightLayer(LayerIdx)->GetName());
			}
		}
		return Result;
	}


	virtual TUniquePtr<IMeshVertexAttributeAdapter> GetAttribute(FName AttributeName) override
	{
		if (FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
		{
			const int32 NumLayers = Attributes->NumWeightLayers();
			for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
			{
				FDynamicMeshWeightAttribute* WeightLayer = Attributes->GetWeightLayer(LayerIdx);
				if (WeightLayer->GetName() == AttributeName)
				{
					return MakeUnique<FDynamicMeshVertexAttributeAdapter>(Mesh, WeightLayer);
				}
			}
		}
		return nullptr;
	}

};


class FMeshDescriptionVertexAttributeSource : public IMeshVertexAttributeSource
{
public:
	FMeshDescription* Mesh = nullptr;

	FMeshDescriptionVertexAttributeSource(FMeshDescription* MeshIn)
	{
		Mesh = MeshIn;
	}

	virtual int32 GetAttributeElementNum() override
	{
		return Mesh->Vertices().Num();
	}

	virtual TArray<FName> GetAttributeList() override
	{
		TAttributesSet<FVertexID>& VertexAttribs = Mesh->VertexAttributes();
		TArray<FName> Result;

		VertexAttribs.ForEach([&](const FName AttributeName, auto AttributesRef)
		{
			if (VertexAttribs.HasAttributeOfType<float>(AttributeName))
			{
				Result.Add(AttributeName);
			}
		});
		return Result;
	}


	virtual TUniquePtr<IMeshVertexAttributeAdapter> GetAttribute(FName AttributeName) override
	{
		TAttributesSet<FVertexID>& VertexAttribs = Mesh->VertexAttributes();
		TVertexAttributesRef<float> Attrib = VertexAttribs.GetAttributesRef<float>(AttributeName);
		if (Attrib.IsValid())
		{
			return MakeUnique<FMeshDescriptionVertexAttributeAdapter>(Mesh, AttributeName, Attrib);

		}
		return nullptr;
	}

};

void UMeshAttributePaintToolProperties::Initialize(const TArray<FName>& AttributeNames, bool bInitialize)
{
	Attributes.Reset(AttributeNames.Num());
	for (const FName& AttributeName : AttributeNames)
	{
		Attributes.Add(AttributeName.ToString());
	}

	if (bInitialize) {
		Attribute = (Attributes.Num() > 0) ? Attributes[0] : TEXT("");
	}
}


bool UMeshAttributePaintToolProperties::ValidateSelectedAttribute(bool bUpdateIfInvalid)
{
	int32 FoundIndex = Attributes.IndexOfByKey(Attribute);
	if (FoundIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			Attribute = (Attributes.Num() > 0) ? Attributes[0] : TEXT("");
		}
		return false;
	}
	return true;
}

int32 UMeshAttributePaintToolProperties::GetSelectedAttributeIndex()
{
	ensure(INDEX_NONE == -1);
	int32 FoundIndex = Attributes.IndexOfByKey(Attribute);
	return FoundIndex;
}


void UMeshAttributePaintEditActions::PostAction(EMeshAttributePaintToolActions Action)
{
	if (ParentTool.IsValid() && Cast<UMeshAttributePaintTool>(ParentTool))
	{
		Cast<UMeshAttributePaintTool>(ParentTool)->RequestAction(Action);
	}
}




/*
 * ToolBuilder
 */

UMeshSurfacePointTool* UMeshAttributePaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMeshAttributePaintTool* NewTool = NewObject<UMeshAttributePaintTool>(SceneState.ToolManager);
	NewTool->SetWorld(SceneState.World);

	if (ColorMapFactory)
	{
		NewTool->SetColorMap(ColorMapFactory());
	}

	return NewTool;
}

// TODO: May want a base class for brush tools with selection?
void UMeshAttributePaintToolBuilder::InitializeNewTool(UMeshSurfacePointTool* NewTool, const FToolBuilderState& SceneState) const
{
	Super::InitializeNewTool(NewTool, SceneState);

	UMeshAttributePaintTool* Tool = Cast< UMeshAttributePaintTool>(NewTool);
	if (ensure(Tool && Tool->GetTarget()))
	{
		UE::Geometry::FGeometrySelection Selection;
		bool bHaveSelection = UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, Tool->GetTarget(), Selection);
		if (bHaveSelection)
		{
			Tool->SetGeometrySelection(MoveTemp(Selection));
		}
	}
}

bool UMeshAttributePaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

const FToolTargetTypeRequirements& UMeshAttributePaintToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		USceneComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


void UMeshAttributePaintTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UMeshAttributePaintTool::SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn)
{
	GeometrySelection = SelectionIn;
}



void UMeshAttributePaintTool::Setup()
{
	const FString PropertyCacheId = GetPropertyCacheIdentifier();
	
	// want this before brush size/etc
	BrushActionProps = NewObject<UMeshAttributePaintBrushOperationProperties>(this);
	BrushActionProps->RestoreProperties(this, PropertyCacheId);
	AddToolPropertySource(BrushActionProps);

	ViewProperties = NewObject<UMeshAttributePaintToolVisualizationProperties>(this);
	ViewProperties->RestoreProperties(this, PropertyCacheId);
	AddToolPropertySource(ViewProperties);

	// Bake the rotate and scale into the mesh to avoid skewed stamps.
	bBakeRotateScale = true;
	UDynamicMeshBrushTool::Setup();
	if (ensure(BrushProperties))
	{
		BrushProperties->bToolSupportsPressureSensitivity = true;
		BrushProperties->bShowHitBackFaces = true;
		BrushProperties->bSpecifyRadius = true;
	}

	// hide strength and falloff
	BrushProperties->RestoreProperties(this, PropertyCacheId);

	AttribProps = NewObject<UMeshAttributePaintToolProperties>(this);
	AttribProps->RestoreProperties(this, PropertyCacheId);
	AddToolPropertySource(AttribProps);

	//AttributeEditActions = NewObject<UMeshAttributePaintEditActions>(this);
	//AttributeEditActions->Initialize(this);
	//AddToolPropertySource(AttributeEditActions);

	// configure preview mesh
	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	//PreviewMesh->EnableWireframe(SelectionProps->bShowWireframe);
	PreviewMesh->SetShadowsEnabled(false);

	BrushActionProps->bToolHasSelection = GeometrySelection.IsSet();
	SelectionTids.Reset();
	SelectionVids.Reset();
	
	PreviewMesh->EditMesh([this](FDynamicMesh3& Mesh)
	{
		// enable vtx colors on preview mesh
		Mesh.EnableAttributes();
		Mesh.Attributes()->DisablePrimaryColors();
		Mesh.Attributes()->EnablePrimaryColors();
		// Create an overlay that has no split elements, init with zero value.
		Mesh.Attributes()->PrimaryColors()->CreatePerVertex(0.f, true);

		if (GeometrySelection.IsSet())
		{
			UE::Geometry::EnumerateSelectionTriangles(GeometrySelection.GetValue(), *PreviewMesh->GetMesh(),
				[this, &Mesh](int32 TriangleID)
			{
				SelectionTids.Add(TriangleID);
				FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
				for (int i = 0; i < 3; ++i)
				{
					SelectionVids.Add(Triangle[i]);
				}
			});

			PreviewMesh->EnableSecondaryTriangleBuffers([this](const UE::Geometry::FDynamicMesh3*, int32 Tid)
			{
				return !SelectionTids.Contains(Tid);
			});

			PreviewMesh->SetSecondaryBuffersVisibility(!BrushActionProps->bIsolateGeometrySelection);
		}

	});

	// build octree
	VerticesOctree.Initialize(PreviewMesh->GetMesh(), true);

	// assign materials
	if (IMaterialProvider* MaterialProvider = Cast<IMaterialProvider>(Target))
	{
		FComponentMaterialSet MaterialSet;
		MaterialProvider->GetMaterialSet(MaterialSet);
		for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
		{
			PreviewMesh->SetMaterial(k, MaterialSet.Materials[k]);
		}
	}

	// Enable render mesh decomposition for efficient partial updates during painting
	PreviewMesh->SetEnableRenderMeshDecomposition(true);

	UpdateMaterialMode(ViewProperties->MaterialMode);

	RecalculateBrushRadius();

	SetToolDisplayName(LOCTEXT("ToolName", "Paint WeightMaps"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartAttribPaint", "Paint per-vertex attribute maps. Ctrl to Erase/Subtract, Shift to Smooth. [/] to change Brush Size."),
		EToolMessageLevel::UserNotification);

	ColorMapper = MakeUnique<FFloatAttributeColorMapper>();

	if (bPreferMeshDescription && Cast<IMeshDescriptionProvider>(Target) && Cast<IMeshDescriptionCommitter>(Target))
	{
		EditedMesh = MakeUnique<FMeshDescription>();			
		AttributeSource = MakeUnique<FMeshDescriptionVertexAttributeSource>(EditedMesh.Get());
		*EditedMesh = *UE::ToolTarget::GetMeshDescription(Target);
	}
	else
	{
		EditedDynamicMesh = MakeUnique<FDynamicMesh3>();
		FGetMeshParameters Params;
		Params.bWantMeshTangents = true;
		*EditedDynamicMesh = UE::ToolTarget::GetDynamicMeshCopy(Target, Params);
		AttributeSource = MakeUnique< FDynamicMeshVertexAttributeSource>(EditedDynamicMesh.Get());
	}

	InitializeAttributes();
	if (AttribProps->Attributes.Num() > 0)
	{
		PendingNewSelectedIndex = 0;
	}

	SelectedAttributeWatcher.Initialize([this]() { AttribProps->ValidateSelectedAttribute(true);  return AttribProps->GetSelectedAttributeIndex(); },
		[this](int32 NewValue) { PendingNewSelectedIndex = NewValue; }, AttribProps->GetSelectedAttributeIndex());

	bVisibleAttributeValid = false;

	BrushActionProps->WatchProperty(BrushActionProps->bIsolateGeometrySelection, [this](bool)
	{
		PreviewMesh->SetSecondaryBuffersVisibility(!BrushActionProps->bIsolateGeometrySelection);
	});

	ViewProperties->WatchProperty(ViewProperties->MaterialMode,
		[this](EMeshAttributePaintMaterialMode NewMode) { UpdateMaterialMode(NewMode); });
	ViewProperties->WatchProperty(ViewProperties->bFlatShading,
		[this](bool bNewValue) { UpdateFlatShadingSetting(bNewValue); });
}

void UMeshAttributePaintTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UDynamicMeshBrushTool::RegisterActions(ActionSet);
}




void UMeshAttributePaintTool::RequestAction(EMeshAttributePaintToolActions ActionType)
{
	if (bHavePendingAction)
	{
		return;
	}
	PendingAction = ActionType;
	bHavePendingAction = true;
}


void UMeshAttributePaintTool::SetColorMap(TUniquePtr<FFloatAttributeColorMapper> ColorMap)
{
	ColorMapper = MoveTemp(ColorMap);
}


void UMeshAttributePaintTool::OnTick(float DeltaTime)
{
	SelectedAttributeWatcher.CheckAndUpdate();

	if (bStampPending)
	{
		ApplyStamp(LastStamp);
		bStampPending = false;
	}

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EMeshAttributePaintToolActions::NoAction;
	}

	if (PendingNewSelectedIndex >= 0)
	{
		UpdateSelectedAttribute(PendingNewSelectedIndex);
		PendingNewSelectedIndex = -1;
	}

	if (bVisibleAttributeValid == false)
	{
		UpdateVisibleAttribute();
		bVisibleAttributeValid = true;
	}
}






void UMeshAttributePaintTool::OnBeginDrag(const FRay& WorldRay)
{
	UDynamicMeshBrushTool::OnBeginDrag(WorldRay);

	PreviewBrushROI.Reset();
	bInRemoveStroke = GetCtrlToggle();
	bInSmoothStroke = GetShiftToggle();
	BeginChange();
	StartStamp = UBaseBrushTool::LastBrushStamp;
	LastStamp = StartStamp;
	bStampPending = true;
}



void UMeshAttributePaintTool::OnUpdateDrag(const FRay& WorldRay)
{
	UDynamicMeshBrushTool::OnUpdateDrag(WorldRay);

	LastStamp = UBaseBrushTool::LastBrushStamp;
	bStampPending = true;
}



void UMeshAttributePaintTool::OnEndDrag(const FRay& Ray)
{
	UDynamicMeshBrushTool::OnEndDrag(Ray);

	bInRemoveStroke = bInSmoothStroke = false;
	bStampPending = false;

	// close change record
	if (ActiveChangeBuilder)
	{
		if (TUniquePtr<FMeshAttributePaintChange> Change = EndChange())
		{
			GetToolManager()->EmitObjectChange(this, MoveTemp(Change), LOCTEXT("AttributeValuesChange", "Paint"));
		}
		LongTransactions.Close(GetToolManager());
	}
}


bool UMeshAttributePaintTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	UDynamicMeshBrushTool::OnUpdateHover(DevicePos);

	// todo get rid of this redundant hit test!
	FHitResult OutHit;
	if (UDynamicMeshBrushTool::HitTest(DevicePos.WorldRay, OutHit))
	{
		PreviewBrushROI.Reset();
		CalculateVertexROI(LastBrushStamp, PreviewBrushROI);
	}

	return true;
}

bool UMeshAttributePaintTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	return PreviewMesh->FindRayIntersection(FRay3d(Ray), OutHit, [this](int32 Tid)
		{
			return (IsFilteringTrianglesBySelection() ? SelectionTids.Contains(Tid) : true) &&
				(IsFilteringTrianglesByFrontFacing() ? !IsTriangleBackFacing(Tid, PreviewMesh->GetMesh()) : true);
		});
}


void UMeshAttributePaintTool::CalculateVertexROI(const FBrushStampData& Stamp, TArray<int>& VertexROI)
{
	FTransform3d Transform(PreviewMesh->GetTransform());
	FVector3d StampPosLocal = Transform.InverseTransformPosition((FVector3d)Stamp.WorldPosition);

	float Radius = GetCurrentBrushRadiusLocal();
	float RadiusSqr = Radius * Radius;
	const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
	FAxisAlignedBox3d QueryBox(StampPosLocal, Radius);
	
	if (IsFilteringTrianglesBySelection() || IsFilteringTrianglesByFrontFacing())
	{
		FVector3d LocalEyePosition;
		if (IsFilteringTrianglesByFrontFacing())
		{
			FViewCameraState StateOut;
			GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
			LocalEyePosition = (CurrentTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));
		}
		
		VerticesOctree.ParallelRangeQuery(QueryBox,
			[this, Mesh, &StampPosLocal, RadiusSqr, &LocalEyePosition](int32 VertexID)
		{
			const bool bIsSelectionValid = !IsFilteringTrianglesBySelection() || SelectionVids.Contains(VertexID);
			bool bIsFrontFaceValid = !IsFilteringTrianglesByFrontFacing();
			if (!bIsFrontFaceValid)
			{
				bIsFrontFaceValid = true;
				Mesh->EnumerateVertexTriangles(VertexID, [this, Mesh, &LocalEyePosition, &bIsFrontFaceValid](int32 TriID)
				{
					FVector3d Normal, Centroid;
					double Area;
					Mesh->GetTriInfo(TriID, Normal, Area, Centroid);
					bIsFrontFaceValid = bIsFrontFaceValid && Normal.Dot((Centroid - LocalEyePosition)) < 0;
				});
			}
			return bIsSelectionValid && bIsFrontFaceValid
				&& DistanceSquared(Mesh->GetVertex(VertexID), StampPosLocal) < RadiusSqr; 
		}, VertexROI);
	}
	else
	{
		VerticesOctree.ParallelRangeQuery(QueryBox,
			[Mesh, &StampPosLocal, RadiusSqr](int32 VertexID) 
		{ 
			return DistanceSquared(Mesh->GetVertex(VertexID), StampPosLocal) < RadiusSqr; 
		}, VertexROI);
	}
}








void UMeshAttributePaintTool::InitializeAttributes()
{
	AttribProps->Initialize(AttributeSource->GetAttributeList(), true);

	if (AttribProps->Attributes.Num() == 0)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("StartAttribPaintFailed", "No Float attributes exist for this mesh. Use the Attribute Editor to create one."),
			EToolMessageLevel::UserWarning);
	}
	else
	{
		// Clear any existing error messages.
		GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
	}

	AttributeBufferCount = AttributeSource->GetAttributeElementNum();
	TArray<FName> AttributeNames = AttributeSource->GetAttributeList();

	Attributes.SetNum(AttributeNames.Num());
	for (int32 k = 0; k < AttributeNames.Num(); ++k)
	{
		Attributes[k].Name = AttributeNames[k];
		Attributes[k].Attribute = AttributeSource->GetAttribute(AttributeNames[k]);
		Attributes[k].CurrentValues.SetNum(AttributeBufferCount);
		for (int32 i = 0; i < AttributeBufferCount; ++i)
		{
			Attributes[k].CurrentValues[i] = Attributes[k].Attribute->GetValue(i);
		}
		Attributes[k].InitialValues = Attributes[k].CurrentValues;
	}

	CurrentAttributeIndex = -1;
	PendingNewSelectedIndex = -1;
}




void UMeshAttributePaintTool::StoreCurrentAttribute()
{
	if (CurrentAttributeIndex >= 0)
	{
		FAttributeData& AttribData = Attributes[CurrentAttributeIndex];
		for (int32 k = 0; k < AttributeBufferCount; ++k)
		{
			AttribData.Attribute->SetValue(k, AttribData.CurrentValues[k]);
		}
		CurrentAttributeIndex = -1;
		CurrentValueRange = FInterval1f(0.0f, 1.0f);
	}
}


void UMeshAttributePaintTool::UpdateVisibleAttribute()
{
	// copy current value set back to attribute  (should we just always be doing this??)
	StoreCurrentAttribute();

	CurrentAttributeIndex = AttribProps->GetSelectedAttributeIndex();

	if (CurrentAttributeIndex >= 0)
	{
		FAttributeData& AttribData = Attributes[CurrentAttributeIndex];
		CurrentValueRange = AttribData.Attribute->GetValueRange();

		// update mesh with new value colors
		PreviewMesh->EditMesh([&](FDynamicMesh3& Mesh)
		{
			FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();
			FNonManifoldMappingSupport NonManifoldMappingSupport(Mesh);
			for (int32 vid : Mesh.VertexIndicesItr())
			{
				const int32 srcvid = NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(vid);
				const float Value = AttribData.CurrentValues[srcvid];
				const FVector4f Color4f = ToVector4<float>(ColorMapper->ToColor(Value));
				// Note: This tool specifically constructs the color overlay element IDs to exactly match vertex IDs
				checkSlow(ColorOverlay->GetParentVertex(vid) == vid);
				ColorOverlay->SetElement(vid,  Color4f);
			}
		});

		AttribProps->Attribute = AttribData.Name.ToString();
	}
}











double UMeshAttributePaintTool::CalculateBrushFalloff(double Distance)
{
	double f = FMathd::Clamp(1.0 - BrushProperties->BrushFalloffAmount, 0.0, 1.0);
	double d = Distance / GetCurrentBrushRadiusLocal();
	double w = 1;
	if (d > f)
	{
		d = FMathd::Clamp((d - f) / (1.0 - f), 0.0, 1.0);
		w = (1.0 - d * d);
		w = w * w * w;
	}
	return w;
}



void UMeshAttributePaintTool::ApplyStamp(const FBrushStampData& Stamp)
{
	if (CurrentAttributeIndex < 0)
	{
		return;
	}

	FAttributeData& AttribData = Attributes[CurrentAttributeIndex];

	FStampActionData ActionData;
	CalculateVertexROI(Stamp, ActionData.ROIVertices);

	switch (BrushActionProps->BrushAction)
	{
	case EBrushActionMode::FloodFill:
	{
		const bool bEmptyROIVertices = (ActionData.ROIVertices.Num() == 0);
		if (bEmptyROIVertices) // append a vertex from the hit triangle to start the flood fill.
		{
			const int32 TID = Stamp.HitResult.FaceIndex;
			const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
			if (Mesh->IsTriangle(TID))
			{
				const FIndex3i VIDs = Mesh->GetTriangle(TID);
				ActionData.ROIVertices.Add(VIDs[0]);
			}
		}
		ApplyStamp_FloodFill(Stamp, ActionData);
		break;
	}
	case EBrushActionMode::Paint:
		ApplyStamp_Paint(Stamp, ActionData, {BrushActionProps->BrushValue, bInSmoothStroke, bInRemoveStroke});
		break;
	case EBrushActionMode::Smooth:
		ApplyStamp_Paint(Stamp, ActionData, {BrushActionProps->BrushValue, true, bInRemoveStroke});
		break;
	case EBrushActionMode::Erase:
		ApplyStamp_Paint(Stamp, ActionData, {BrushActionProps->BrushValue, bInSmoothStroke, !bInRemoveStroke});
		break;
	default:
		ensure(false);
	}


	// track changes
	TFuture<void> ChangeBuilderCompute;
	if (ActiveChangeBuilder)
	{
		ChangeBuilderCompute = Async(AttributePaintToolAsyncExecTarget, [this, &ActionData]()
		{
			ActiveChangeBuilder->UpdateValues(ActionData.ROIVertices, ActionData.ROIBefore, ActionData.ROIAfter);
		});
	}

	// update values and colors
	PreviewMesh->DeferredEditMesh([&](FDynamicMesh3& Mesh)
	{
		TArray<int> ElIDs;
		FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();
		FNonManifoldMappingSupport NonManifoldMappingSupport(Mesh);
		int32 NumVertices = ActionData.ROIVertices.Num();
		for (int32 k = 0; k < NumVertices; ++k)
		{
			int32 vid = ActionData.ROIVertices[k];
			int32 srcvid = NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(vid);
			AttribData.CurrentValues[srcvid] = ActionData.ROIAfter[k];
			FVector4f NewColor(ToVector4<float>(ColorMapper->ToColor(ActionData.ROIAfter[k])));
			// Note: This tool specifically constructs the color overlay element IDs to exactly match vertex IDs
			checkSlow(ColorOverlay->GetParentVertex(vid) == vid);
			ColorOverlay->SetElement(vid, NewColor);
			AttribData.Attribute->SetValue(vid, AttribData.CurrentValues[srcvid]);
			ElIDs.Reset();
		}
	}, false);

	// Updating the vertex colors is only required when the paint data is being displayed via the vertex color data
	if (ViewProperties->MaterialMode == EMeshAttributePaintMaterialMode::Shaded
		|| ViewProperties->MaterialMode == EMeshAttributePaintMaterialMode::ColorOnly)
	{
		PreviewMesh->NotifyVertexRegionDeferredEditCompleted(ActionData.ROIVertices, EMeshRenderAttributeFlags::VertexColors);
	}

	if (ActiveChangeBuilder)
	{
		ChangeBuilderCompute.Wait();
	}
}

void UMeshAttributePaintTool::UpdateMaterialMode(EMeshAttributePaintMaterialMode MaterialMode)
{
	if (MaterialMode == EMeshAttributePaintMaterialMode::Shaded)
	{
		UMaterialInterface* SculptMaterial = ToolSetupUtil::GetVertexColorSculptMaterial(GetToolManager());
		if (SculptMaterial != nullptr)
		{
			ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(SculptMaterial, this);
		}
	}
	else if (MaterialMode == EMeshAttributePaintMaterialMode::ColorOnly && 
			GEngine && GEngine->VertexColorViewModeMaterial_ColorOnly)
	{
		// use the engine color-only material, which has no shading
		ActiveOverrideMaterial = nullptr;
		PreviewMesh->SetOverrideRenderMaterial(GEngine->VertexColorViewModeMaterial_ColorOnly);
	}
	else if (MaterialMode == EMeshAttributePaintMaterialMode::ExistingMaterial)
	{
		ActiveOverrideMaterial = nullptr;
		PreviewMesh->ClearOverrideRenderMaterial();
	}

	if (ActiveOverrideMaterial != nullptr)
	{
		PreviewMesh->SetOverrideRenderMaterial(ActiveOverrideMaterial);
		ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (ViewProperties->bFlatShading) ? 1.0f : 0.0f);
	}

	PreviewMesh->SetShadowsEnabled(false);
}

void UMeshAttributePaintTool::UpdateFlatShadingSetting(bool bNewValue)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (bNewValue) ? 1.0f : 0.0f);
	}
}

void UMeshAttributePaintTool::ApplyStamp_Paint(const FBrushStampData& Stamp, FStampActionData& ActionData)
{
	ApplyStamp_Paint(Stamp, ActionData, {1.0f, bInSmoothStroke, bInRemoveStroke});
}

void UMeshAttributePaintTool::ApplyStamp_Paint(const FBrushStampData& Stamp, FStampActionData& ActionData, const FBrushActionData& BrushData)
{
	FTransform3d Transform(PreviewMesh->GetTransform());
	FVector3d StampPosLocal = Transform.InverseTransformPosition((FVector3d)Stamp.WorldPosition);

	int32 NumVertices = ActionData.ROIVertices.Num();
	ActionData.ROIBefore.SetNum(NumVertices);
	ActionData.ROIAfter.SetNum(NumVertices);

	FAttributeData& AttribData = Attributes[CurrentAttributeIndex];

	const FDynamicMesh3* CurrentMesh = PreviewMesh->GetMesh();
	FNonManifoldMappingSupport NonManifoldMappingSupport(*CurrentMesh);
	if (BrushData.bSmooth)
	{
		constexpr float SmoothSpeed = 0.25f;

		for (int32 k = 0; k < NumVertices; ++k)
		{
			int32 vid = ActionData.ROIVertices[k];
			FVector3d Position = CurrentMesh->GetVertex(vid);
			float ValueSum = 0, WeightSum = 0;
			for (int32 NbrVID : CurrentMesh->VtxVerticesItr(vid))
			{
				int32 srcNbrVID =  NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(NbrVID);
				FVector3d NbrPos = CurrentMesh->GetVertex(NbrVID);
				float Weight = FMathf::Clamp(1.0f / DistanceSquared(NbrPos, Position), 0.0001f, 1000.0f);
				ValueSum += Weight * AttribData.CurrentValues[srcNbrVID];
				WeightSum += Weight;
			}
			ValueSum /= WeightSum;

			float Falloff = (float)CalculateBrushFalloff(Distance(Position, StampPosLocal));
			const int32 srcvid =  NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(vid);
			float NewValue = FMathf::Lerp(AttribData.CurrentValues[srcvid], ValueSum, SmoothSpeed*Falloff);

			ActionData.ROIBefore[k] = AttribData.CurrentValues[srcvid];
			ActionData.ROIAfter[k] = CurrentValueRange.Clamp(NewValue);
		}
	}
	else
	{
		const float TargetValue = BrushData.bErase ? CurrentValueRange.Min : BrushData.BrushValue;
		const float UseStrength = BrushProperties->BrushStrength * CurrentValueRange.Length();
		for (int32 k = 0; k < NumVertices; ++k)
		{
			const int32 vid = ActionData.ROIVertices[k];
			const int32 srcvid =  NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(vid);
			const FVector3d Position = CurrentMesh->GetVertex(vid);
			const float Falloff = static_cast<float>(CalculateBrushFalloff(Distance(Position, StampPosLocal)));
			ActionData.ROIBefore[k] = AttribData.CurrentValues[srcvid];
			const float ValueAfter = FMath::Lerp(ActionData.ROIBefore[k], TargetValue, UseStrength*Falloff);
			ActionData.ROIAfter[k] = CurrentValueRange.Clamp(ValueAfter);
		}
	}
}


void UMeshAttributePaintTool::ApplyStamp_FloodFill(const FBrushStampData& Stamp, FStampActionData& ActionData)
{
	FAttributeData& AttribData = Attributes[CurrentAttributeIndex];
	const FDynamicMesh3* CurrentMesh = PreviewMesh->GetMesh();

	// convert to connected triangle set
	TSet<int32> RemainingTriangles;
	for (int32 vid : ActionData.ROIVertices)
	{
		if (IsFilteringTrianglesBySelection())
		{
			CurrentMesh->EnumerateVertexTriangles(vid, [&](int32 tid) 
			{ 
				if (SelectionTids.Contains(tid))
				{ 
					RemainingTriangles.Add(tid); 
				}
			});
		}
		else
		{
			CurrentMesh->EnumerateVertexTriangles(vid, [&](int32 tid) { RemainingTriangles.Add(tid); });
		}
	}

	float SetValue = BrushProperties->BrushStrength * CurrentValueRange.Length();
	if (bInRemoveStroke)
	{
		SetValue = CurrentValueRange.Min;
	}

	FNonManifoldMappingSupport NonManifoldMappingSupport(*CurrentMesh);
	ActionData.ROIVertices.Reset();
	TArray<int32> InputTriROI, OutputTriROI, QueueTempBuffer;
	TSet<int32> DoneTempBuffer, DoneVertices;
	while (RemainingTriangles.Num() > 0)
	{
		OutputTriROI.Reset();
		QueueTempBuffer.Reset();
		DoneTempBuffer.Reset();
		InputTriROI.Reset();
		// get a single set element via an iterator
		InputTriROI.Add(*RemainingTriangles.CreateConstIterator());
		TFunction<bool(int32, int32)> TidFilter = [](int32, int32){ return true; };
		if (IsFilteringTrianglesBySelection())
		{
			TidFilter = [this](int32 Index, int32 Tid) { return SelectionTids.Contains(Tid); };
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(CurrentMesh, InputTriROI, OutputTriROI, 
			&QueueTempBuffer, &DoneTempBuffer, TidFilter);
		for (int32 tid : OutputTriROI)
		{
			RemainingTriangles.Remove(tid);
			FIndex3i TriVertices = CurrentMesh->GetTriangle(tid);
			for (int32 j = 0; j < 3; ++j)
			{
				if (DoneVertices.Contains(TriVertices[j]) == false)
				{
					int32 vid = TriVertices[j];
					ActionData.ROIVertices.Add(vid);
					int32 srcvid = NonManifoldMappingSupport.GetOriginalNonManifoldVertexID(vid);
					ActionData.ROIBefore.Add(AttribData.CurrentValues[srcvid]);
					ActionData.ROIAfter.Add(SetValue);
					DoneVertices.Add(vid);
				}
			}
		}
	}
	

}





void UMeshAttributePaintTool::ApplyAction(EMeshAttributePaintToolActions ActionType)
{
	//switch (ActionType)
	//{
	//}
}



void UMeshAttributePaintTool::UpdateSelectedAttribute(int32 NewSelectedIndex)
{
	AttribProps->Initialize(AttributeSource->GetAttributeList(), false);
	AttribProps->Attribute = AttribProps->Attributes[FMath::Clamp(NewSelectedIndex, 0, AttribProps->Attributes.Num() - 1)];
	bVisibleAttributeValid = false;
}




void UMeshAttributePaintTool::OnShutdown(EToolShutdownType ShutdownType)
{
	const FString PropertyCacheId = GetPropertyCacheIdentifier();
	BrushProperties->SaveProperties(this, PropertyCacheId);
	BrushActionProps->SaveProperties(this, PropertyCacheId);
	ViewProperties->SaveProperties(this, PropertyCacheId);

	StoreCurrentAttribute();

	if (ShutdownType == EToolShutdownType::Accept)
	{
		// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshAttributePaintTool", "Edit Attributes"));

		if (EditedMesh.IsValid())
		{
			UE::ToolTarget::CommitMeshDescriptionUpdate(Target, EditedMesh.Get());
		}
		else
		{
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *EditedDynamicMesh, false);
		}

		GetToolManager()->EndUndoTransaction();
	}
}


FString UMeshAttributePaintTool::GetPropertyCacheIdentifier() const
{
	return TEXT("UMeshAttributePaintTool");
}


void UMeshAttributePaintTool::BeginChange()
{
	if (CurrentAttributeIndex < 0)
	{
		ActiveChangeBuilder.Reset();
		return;
	}
	if (! ActiveChangeBuilder)
	{
		ActiveChangeBuilder = MakeUnique<TIndexedValuesChangeBuilder<float, FMeshAttributePaintChange>>();
	}
	ActiveChangeBuilder->BeginNewChange();
	ActiveChangeBuilder->Change->CustomData = CurrentAttributeIndex;

	LongTransactions.Open(LOCTEXT("AttributeValuesChange", "Paint"), GetToolManager());
}


TUniquePtr<FMeshAttributePaintChange> UMeshAttributePaintTool::EndChange()
{
	if (!ensure(ActiveChangeBuilder)) // should be validated before calling EndChange
	{
		return nullptr;
	}

	TUniquePtr<FMeshAttributePaintChange> Result = ActiveChangeBuilder->ExtractResult();
	if (Result)
	{
		Result->ApplyFunction = [](UObject* Object, const int32& AttribIndex, const TArray<int32>& Indices, const TArray<float>& Values)
		{
			UMeshAttributePaintTool* Tool = CastChecked<UMeshAttributePaintTool>(Object);
			Tool->ExternalUpdateValues(AttribIndex, Indices, Values);
		};
		Result->RevertFunction = [](UObject* Object, const int32& AttribIndex, const TArray<int32>& Indices, const TArray<float>& Values)
		{
			UMeshAttributePaintTool* Tool = CastChecked<UMeshAttributePaintTool>(Object);
			Tool->ExternalUpdateValues(AttribIndex, Indices, Values);
		};
		return MoveTemp(Result);
	}
	
	return nullptr;
}



void UMeshAttributePaintTool::ExternalUpdateValues(int32 AttribIndex, const TArray<int32>& VertexIndices, const TArray<float>& NewValues)
{
	if (!ensure(Attributes.IsValidIndex(AttribIndex)))
	{
		return;
	}
	FAttributeData& AttribData = Attributes[AttribIndex];

	int32 NumV = VertexIndices.Num();
	for (int32 k = 0; k < NumV; ++k)
	{
		AttribData.CurrentValues[VertexIndices[k]] = NewValues[k];
	}

	if (AttribIndex == CurrentAttributeIndex)
	{
		PreviewMesh->EditMesh([&](FDynamicMesh3& Mesh)
		{
			FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();
			for (int32 vid : VertexIndices)
			{
				FVector4f NewColor( ToVector4<float>(ColorMapper->ToColor(AttribData.CurrentValues[vid])) );
				// Note: This tool specifically constructs the color overlay element IDs to exactly match vertex IDs
				checkSlow(ColorOverlay->GetParentVertex(vid) == vid);
				ColorOverlay->SetElement(vid, NewColor);
				AttribData.Attribute->SetValue(vid, AttribData.CurrentValues[vid]);
			}
		});
	}
}

bool UMeshAttributePaintTool::IsFilteringTrianglesBySelection() const
{
	return BrushActionProps->bIsolateGeometrySelection && !SelectionTids.IsEmpty();
}

bool UMeshAttributePaintTool::IsFilteringTrianglesByFrontFacing() const
{
	return !BrushProperties->bHitBackFaces;
}

bool UMeshAttributePaintTool::IsTriangleBackFacing(int32 TriangleID, const FDynamicMesh3* QueryMesh) const
{
	if (TriangleID != IndexConstants::InvalidID)
	{
		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurrentTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));

		FVector3d Normal, Centroid;
		double Area;
		QueryMesh->GetTriInfo(TriangleID, Normal, Area, Centroid);

		return (Normal.Dot((Centroid - LocalEyePosition)) >= 0);
	}
	return false;
}



#undef LOCTEXT_NAMESPACE

