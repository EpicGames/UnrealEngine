// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshGroupPaintTool.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "SceneView.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "Util/BufferUtil.h"
#include "Util/ColorConstants.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Polygroups/PolygroupUtil.h"
#include "Polygon2.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshPolygroupChange.h"
#include "Changes/BasicChanges.h"

#include "Sculpting/MeshGroupPaintBrushOps.h"
#include "Sculpting/StampFalloffs.h"
#include "Sculpting/MeshSculptUtil.h"

#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"  // for GEngine->GetSmallFont()
#include "SceneView.h"
#include "ToolTargetManager.h"
#include "ToolBuilderUtil.h"
#include "Changes/MeshTriangleLabelChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshGroupPaintTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshGroupPaintTool"

namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution GroupPaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution GroupPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}


/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UMeshGroupPaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMeshGroupPaintTool* SculptTool = NewObject<UMeshGroupPaintTool>(SceneState.ToolManager);
	SculptTool->SetWorld(SceneState.World);
	return SculptTool;
}
bool UMeshGroupPaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

/*
 * Properties
 */
void UMeshGroupPaintToolActionPropertySet::PostAction(EMeshGroupPaintToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}





/*
 * Tool
 */

void UMeshGroupPaintTool::Setup()
{
	UMeshSculptToolBase::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Paint PolyGroups"));

	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);
	InitializeSculptMeshComponent(DynamicMeshComponent, PreviewMeshActor);

	// assign materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	DynamicMeshComponent->SetInvalidateProxyOnChangeEnabled(false);
	OnDynamicMeshComponentChangedHandle = DynamicMeshComponent->OnMeshChanged.AddUObject(this, &UMeshGroupPaintTool::OnDynamicMeshComponentChanged);

	FDynamicMesh3* Mesh = GetSculptMesh();
	Mesh->EnableVertexColors(FVector3f::One());
	FAxisAlignedBox3d Bounds = Mesh->GetBounds(true);

	TFuture<void> PrecomputeFuture = Async(GroupPaintToolAsyncExecTarget, [&]()
	{
		PrecomputeFilterData();
	});

	TFuture<void> OctreeFuture = Async(GroupPaintToolAsyncExecTarget, [&]()
	{
		// initialize dynamic octree
		if (Mesh->TriangleCount() > 100000)
		{
			Octree.RootDimension = Bounds.MaxDim() / 10.0;
			Octree.SetMaxTreeDepth(4);
		}
		else
		{
			Octree.RootDimension = Bounds.MaxDim();
			Octree.SetMaxTreeDepth(8);
		}
		Octree.Initialize(Mesh);
		//Octree.CheckValidity(EValidityCheckFailMode::Check, true, true);
		//FDynamicMeshOctree3::FStatistics Stats;
		//Octree.ComputeStatistics(Stats);
		//UE_LOGF(LogTemp, Warning, "Octree Stats: %ls", *Stats.ToString());
	});

	// initialize render decomposition
	TUniquePtr<FMeshRenderDecomposition> Decomp = MakeUnique<FMeshRenderDecomposition>();
	FMeshRenderDecomposition::BuildChunkedDecomposition(Mesh, &MaterialSet, *Decomp);
	Decomp->BuildAssociations(Mesh);
	//UE_LOGF(LogTemp, Warning, "Decomposition has %d groups", Decomp->Num());
	DynamicMeshComponent->SetExternalDecomposition(MoveTemp(Decomp));

	// initialize brush radius range interval, brush properties
	// Base class default is 0.01 which is too coarse for triangle painting at typical mesh scales.
    MinimumBrushAdaptiveSizeRatio = 0.001;
	UMeshSculptToolBase::InitializeBrushSizeRange(Bounds);

	FilterProperties = NewObject<UGroupPaintBrushFilterProperties>(this);
	
	// Set up control points mechanic
	PolyLassoMechanic = NewObject<UPolyLassoMarqueeMechanic>(this);
	PolyLassoMechanic->Setup(this);
	PolyLassoMechanic->SetIsEnabled(false);
	PolyLassoMechanic->SpacingTolerance = 10.0f;
	PolyLassoMechanic->OnDrawPolyLassoFinished.AddUObject(this, &UMeshGroupPaintTool::OnPolyLassoFinished);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->bSupportLabels = true;
	PolygroupLayerProperties->RestoreProperties(this, TEXT("MeshGroupPaintTool"));
	PolygroupLayerProperties->InitializeGroupLayers(GetSculptMesh());
	PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
	UpdateActiveGroupLayer();
	AddToolPropertySource(PolygroupLayerProperties);

	// initialize other properties
	FilterProperties->WatchProperty(FilterProperties->SubToolType,
		[this](EMeshGroupPaintInteractionType NewType) { UpdateSubToolType(NewType); });
	FilterProperties->WatchProperty(FilterProperties->BrushSize,
		[this](float NewSize) { UMeshSculptToolBase::BrushProperties->BrushSize.AdaptiveSize = NewSize; CalculateBrushRadius(); });
	FilterProperties->WatchProperty(FilterProperties->bHitBackFaces,
		[this](bool bNewValue) { UMeshSculptToolBase::BrushProperties->bHitBackFaces = bNewValue; });
	FilterProperties->RestoreProperties(this);
	FilterProperties->BrushSize = UMeshSculptToolBase::BrushProperties->BrushSize.AdaptiveSize;
	FilterProperties->bHitBackFaces = UMeshSculptToolBase::BrushProperties->bHitBackFaces;
	FilterProperties->SetGroup = IsPaintingLabel() ? LabelAdapter->GetMaxGroupID() : ActiveGroupSet->MaxGroupID;
	AddToolPropertySource(FilterProperties);

	InitializeIndicator();

	// initialize our properties
	AddToolPropertySource(UMeshSculptToolBase::BrushProperties);
	UMeshSculptToolBase::BrushProperties->bShowPerBrushProps = false;
	UMeshSculptToolBase::BrushProperties->bShowFalloff = false;
	UMeshSculptToolBase::BrushProperties->bShowLazyness = false;
	CalculateBrushRadius();

	PaintBrushOpProperties = NewObject<UGroupPaintBrushOpProps>(this);
	RegisterBrushType((int32)EMeshGroupPaintBrushType::Paint, LOCTEXT("Paint", "Paint"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FGroupPaintBrushOp>(); }),
		PaintBrushOpProperties);

	// secondary brushes
	EraseBrushOpProperties = NewObject<UGroupEraseBrushOpProps>(this);
	EraseBrushOpProperties->GetCurrentGroupLambda = [this]() { return PaintBrushOpProperties->GetGroup(); };

	RegisterSecondaryBrushType((int32)EMeshGroupPaintBrushType::Erase, LOCTEXT("Erase", "Erase"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FGroupEraseBrushOp>>(),
		EraseBrushOpProperties);

	AddToolPropertySource(UMeshSculptToolBase::ViewProperties);

	AddToolPropertySource(UMeshSculptToolBase::GizmoProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::GizmoProperties, false);


	// register watchers
	FilterProperties->WatchProperty( FilterProperties->PrimaryBrushType,
		[this](EMeshGroupPaintBrushType NewType) { UpdateBrushType(NewType); });

	// must call before updating brush type so that we register all brush properties?
	UMeshSculptToolBase::OnCompleteSetup();

	UpdateBrushType(FilterProperties->PrimaryBrushType);
	SetActiveSecondaryBrushType((int32)EMeshGroupPaintBrushType::Erase);

	FreezeActions = NewObject<UMeshGroupPaintToolFreezeActions>(this);
	FreezeActions->Initialize(this);
	AddToolPropertySource(FreezeActions);

	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(DynamicMeshComponent->GetWorld(), DynamicMeshComponent->GetComponentTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("MeshGroupPaintTool"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		ProcessFunc(*GetSculptMesh());
	});

	// force colors update... ?
	DynamicMeshComponent->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
	{
		return GetColorForGroup(GetGroup(TriangleID));
	});

	// disable view properties
	SetViewPropertiesEnabled(false);
	UpdateMaterialMode(EMeshEditingMaterialModes::VertexColor);
	UpdateWireframeVisibility(false);
	UpdateFlatShadingSetting(true);

	// configure panels
	UpdateSubToolType(FilterProperties->SubToolType);

	PrecomputeFuture.Wait();
	OctreeFuture.Wait();
}

void UMeshGroupPaintTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);
	}

	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, TEXT("MeshGroupPaintTool"));
	}
	MeshElementsDisplay->Disconnect();

	FilterProperties->SaveProperties(this);
	PolygroupLayerProperties->SaveProperties(this, TEXT("MeshGroupPaintTool"));

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

	UMeshSculptToolBase::Shutdown(ShutdownType);
}


void UMeshGroupPaintTool::CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("GroupPaintToolTransactionName", "Paint Groups"));
	Component->ProcessMesh([&](const FDynamicMesh3& CurMesh)
	{
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, CurMesh, true);
	});
	GetToolManager()->EndUndoTransaction();
}


void UMeshGroupPaintTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UMeshSculptToolBase::RegisterActions(ActionSet);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 500,
		TEXT("PickGroupColorUnderCursor"),
		LOCTEXT("PickGroupColorUnderCursor", "Pick PolyGroup"),
		LOCTEXT("PickGroupColorUnderCursorTooltip", "Switch the active PolyGroup to the group currently under the cursor"),
		EModifierKey::Shift, EKeys::G,
		[this]() { bPendingPickGroup = true; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 501,
		TEXT("ToggleFrozenGroup"),
		LOCTEXT("ToggleFrozenGroup", "Toggle Group Frozen State"),
		LOCTEXT("ToggleFrozenGroupTooltip", "Toggle Group Frozen State"),
		EModifierKey::Shift, EKeys::F,
		[this]() { bPendingToggleFreezeGroup = true; });


	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 502,
		TEXT("CreateNewGroup"),
		LOCTEXT("CreateNewGroup", "New Group"),
		LOCTEXT("CreateNewGroupTooltip", "Allocate a new Polygroup and set as Current"),
		EModifierKey::Shift, EKeys::Q,
		[this]() { AllocateNewGroupAndSetAsCurrentAction(); });
};


TUniquePtr<FMeshSculptBrushOp>& UMeshGroupPaintTool::GetActiveBrushOp()
{
	if (GetInEraseStroke())
	{
		return SecondaryBrushOp;
	}
	else
	{
		return PrimaryBrushOp;
	}
}

void UMeshGroupPaintTool::IncreaseBrushRadiusAction()
{
	Super::IncreaseBrushRadiusAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UMeshGroupPaintTool::DecreaseBrushRadiusAction()
{
	Super::DecreaseBrushRadiusAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UMeshGroupPaintTool::IncreaseBrushRadiusSmallStepAction()
{
	Super::IncreaseBrushRadiusSmallStepAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}

void UMeshGroupPaintTool::DecreaseBrushRadiusSmallStepAction()
{
	Super::DecreaseBrushRadiusSmallStepAction();
	FilterProperties->BrushSize = BrushProperties->BrushSize.AdaptiveSize;
	NotifyOfPropertyChangeByTool(FilterProperties);
}


bool UMeshGroupPaintTool::IsInBrushSubMode() const
{
	return FilterProperties->SubToolType == EMeshGroupPaintInteractionType::Brush
		|| FilterProperties->SubToolType == EMeshGroupPaintInteractionType::Fill
		|| FilterProperties->SubToolType == EMeshGroupPaintInteractionType::GroupFill;
}


void UMeshGroupPaintTool::OnBeginStroke(const FRay& WorldRay)
{
	UpdateBrushPosition(WorldRay);

	if (PaintBrushOpProperties)
	{
		PaintBrushOpProperties->Group = GetGroupToSet();
		PaintBrushOpProperties->bOnlyPaintUngrouped = FilterProperties->bOnlySetUngrouped;
	}
	if (EraseBrushOpProperties)
	{
		EraseBrushOpProperties->Group = GetGroupToErase();
		EraseBrushOpProperties->bOnlyEraseCurrent = FilterProperties->bOnlyEraseCurrent;
	}

	// initialize first "Last Stamp", so that we can assume all stamps in stroke have a valid previous stamp
	LastStamp.WorldFrame = GetBrushFrameWorld();
	LastStamp.LocalFrame = GetBrushFrameLocal();
	LastStamp.Radius = GetCurrentBrushRadius();
	LastStamp.Falloff = GetCurrentBrushFalloff();
	LastStamp.Direction = GetInInvertStroke() ? -1.0 : 1.0;
	LastStamp.Depth = GetCurrentBrushDepth();
	LastStamp.Power = GetActivePressure() * GetCurrentBrushStrength();
	LastStamp.TimeStamp = FDateTime::Now();

	FSculptBrushOptions SculptOptions;
	//SculptOptions.bPreserveUVFlow = false; // FilterProperties->bPreserveUVFlow;
	SculptOptions.ConstantReferencePlane = GetCurrentStrokeReferencePlane();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	UseBrushOp->ConfigureOptions(SculptOptions);
	UseBrushOp->BeginStroke(GetSculptMesh(), LastStamp, VertexROI);

	AccumulatedTriangleROI.Reset();

	// begin change here? or wait for first stamp?
	BeginChange();
}

void UMeshGroupPaintTool::OnEndStroke()
{
	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	// close change record
	EndChange();
}

void UMeshGroupPaintTool::OnCancelStroke()
{
	GetActiveBrushOp()->CancelStroke();
	ActiveGroupEditBuilder.Reset();
	ActiveLabelEditBuilder.Reset();
	bDrawGroupsDataValid = false;
}


void UMeshGroupPaintTool::UpdateROI(const FSculptBrushStamp& BrushStamp)
{
	SCOPE_CYCLE_COUNTER(GroupPaintTool_UpdateROI);

	const int32 SetGroupID = GetInEraseStroke() ? GetGroupToErase() : GetGroupToSet();

	const FVector3d& BrushPos = BrushStamp.LocalFrame.Origin;
	const FDynamicMesh3* Mesh = GetSculptMesh();
	float RadiusSqr = GetCurrentBrushRadius() * GetCurrentBrushRadius();
	FAxisAlignedBox3d BrushBox(
		BrushPos - GetCurrentBrushRadius() * FVector3d::One(),
		BrushPos + GetCurrentBrushRadius() * FVector3d::One());

	TriangleROI.Reset();

	int32 CenterTID = GetBrushTriangleID();
	if (Mesh->IsTriangle(CenterTID))
	{
		TriangleROI.Add(CenterTID);
	}

	FVector3d CenterNormal = Mesh->IsTriangle(CenterTID) ? TriNormals[CenterTID] : FVector3d::One();		// One so that normal check always passes

	bool bVolumetric = (FilterProperties->BrushAreaMode == EMeshGroupPaintBrushAreaType::Volumetric);
	bool bUseAngleThreshold = (bVolumetric == false) && (FilterProperties->AngleThreshold < 180.0f);
	double DotAngleThreshold = FMathd::Cos(FilterProperties->AngleThreshold * FMathd::DegToRad);
	bool bStopAtUVSeams = FilterProperties->bUVSeams;
	bool bStopAtNormalSeams = FilterProperties->bNormalSeams;

	auto CheckEdgeCriteria = [&](int32 t1, int32 t2) -> bool
	{
		if (bUseAngleThreshold == false || CenterNormal.Dot(TriNormals[t2]) > DotAngleThreshold)
		{
			int32 eid = Mesh->FindEdgeFromTriPair(t1, t2);
			if (bStopAtUVSeams == false || UVSeamEdges[eid] == false)
			{
				if (bStopAtNormalSeams == false || NormalSeamEdges[eid] == false)
				{
					return true;
				}
			}
		}
		return false;
	};

	bool bFill = (FilterProperties->SubToolType == EMeshGroupPaintInteractionType::Fill);
	bool bGroupFill = (FilterProperties->SubToolType == EMeshGroupPaintInteractionType::GroupFill);

	if (bVolumetric)
	{
		Octree.RangeQuery(BrushBox,
			[&](int TriIdx) {

			if ((Mesh->GetTriCentroid(TriIdx) - BrushPos).SquaredLength() < RadiusSqr)
			{
				TriangleROI.Add(TriIdx);
			}
		});
	}
	else
	{
		if (Mesh->IsTriangle(CenterTID))
		{
			TArray<int32> StartROI;
			StartROI.Add(CenterTID);
			FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
				[&](int t1, int t2) 
			{ 
				if ((Mesh->GetTriCentroid(t2) - BrushPos).SquaredLength() < RadiusSqr)
				{
					return CheckEdgeCriteria(t1, t2);
				}
				return false;
			});
		}
	}

	if (bFill)
	{
		TArray<int32> StartROI;
		for (int32 tid : TriangleROI)
		{
			StartROI.Add(tid);
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
														   [&](int t1, int t2)
		{
			return CheckEdgeCriteria(t1, t2);
		});
	}
	else if (bGroupFill)
	{
		TArray<int32> StartROI;
		TSet<int32> FillGroups;
		for (int32 tid : TriangleROI)
		{
			const int32 Group = GetGroup(tid);
			if (Group != SetGroupID)
			{
				StartROI.Add(tid);
				FillGroups.Add(Group);
			}
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
			[&](int t1, int t2)
		{
			return FillGroups.Contains(GetGroup(t2));
		});
	}


	// apply visibility filter
	if (FilterProperties->VisibilityFilter != EMeshGroupPaintVisibilityType::None)
	{
		TArray<int32> ResultBuffer;
		ApplyVisibilityFilter(TriangleROI, TempROIBuffer, ResultBuffer);
	}

	// construct ROI vertex set
	VertexSetBuffer.Reset();
	for (int32 tid : TriangleROI)
	{
		FIndex3i Tri = Mesh->GetTriangle(tid);
		VertexSetBuffer.Add(Tri.A);  VertexSetBuffer.Add(Tri.B);  VertexSetBuffer.Add(Tri.C);
	}
	VertexROI.SetNum(0, EAllowShrinking::No);
	BufferUtil::AppendElements(VertexROI, VertexSetBuffer);

	// construct ROI triangle and group buffers
	ROITriangleBuffer.Reserve(TriangleROI.Num());
	ROITriangleBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : TriangleROI)
	{
		ROITriangleBuffer.Add(tid);
	}
	ROIGroupBuffer.SetNum(ROITriangleBuffer.Num(), EAllowShrinking::No);
}

bool UMeshGroupPaintTool::UpdateStampPosition(const FRay& WorldRay)
{
	CalculateBrushRadius();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		UpdateBrushPositionOnSculptMesh(WorldRay, true);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		UpdateBrushPositionOnActivePlane(WorldRay);
		break;
	}

	if (UseBrushOp->GetAlignStampToView())
	{
		AlignBrushToView();
	}

	CurrentStamp = LastStamp;
	CurrentStamp.DeltaTime = FMathd::Min((FDateTime::Now() - LastStamp.TimeStamp).GetTotalSeconds(), 1.0);
	CurrentStamp.WorldFrame = GetBrushFrameWorld();
	CurrentStamp.LocalFrame = GetBrushFrameLocal();
	CurrentStamp.Power = GetActivePressure() * GetCurrentBrushStrength();

	CurrentStamp.PrevLocalFrame = LastStamp.LocalFrame;
	CurrentStamp.PrevWorldFrame = LastStamp.WorldFrame;

	FVector3d MoveDelta = CurrentStamp.LocalFrame.Origin - CurrentStamp.PrevLocalFrame.Origin;
	if (UseBrushOp->IgnoreZeroMovements() && MoveDelta.SquaredLength() < FMathd::ZeroTolerance)
	{
		return false;
	}

	return true;
}


bool UMeshGroupPaintTool::ApplyStamp()
{
	SCOPE_CYCLE_COUNTER(GroupPaintToolApplyStamp);

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	// yuck
	FMeshTriangleGroupEditBrushOp* GroupBrushOp = (FMeshTriangleGroupEditBrushOp*)UseBrushOp.Get();

	FDynamicMesh3* Mesh = GetSculptMesh();
	GroupBrushOp->ApplyStampByTriangles(Mesh, CurrentStamp, ROITriangleBuffer, ROIGroupBuffer);

	bool bUpdated = SyncMeshWithGroupBuffer(Mesh);

	LastStamp = CurrentStamp;
	LastStamp.TimeStamp = FDateTime::Now();

	return bUpdated;
}




bool UMeshGroupPaintTool::SyncMeshWithGroupBuffer(FDynamicMesh3* Mesh)
{
	int NumModified = 0;
	const int32 NumT = ROITriangleBuffer.Num();
	// change update could be async here if we collected array of <idx,orig,new> and dispatched independenlty
	for ( int32 k = 0; k < NumT; ++k)
	{
		int TriIdx = ROITriangleBuffer[k];
		const int32 CurGroupID = GetGroup(TriIdx);
		if (FrozenGroups.Contains(CurGroupID))		// skip frozen groups
		{
			continue;
		}

		if (ROIGroupBuffer[k] != CurGroupID)
		{
			if (IsPaintingLabel())
			{
				ActiveLabelEditBuilder->SaveTriangle(TriIdx,
					LabelAdapter->GetValueFromGroup(CurGroupID),
					LabelAdapter->GetValueFromGroup(ROIGroupBuffer[k]));
			}
			else
			{
				ActiveGroupEditBuilder->SaveTriangle(TriIdx, CurGroupID, ROIGroupBuffer[k]);
			}
			
			SetGroup(TriIdx, ROIGroupBuffer[k], *Mesh);
			//ActiveVertexChange->UpdateVertexColor(VertIdx, OrigColor, NewColor);
			NumModified++;
		}
	}

	return (NumModified > 0);
}




void UMeshGroupPaintTool::OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled)
{
	// construct polyline
	TArray<FVector2D> Polyline = Lasso.Polyline;
	int32 N = Polyline.Num();
	if (N < 2)
	{
		return;
	}

	// Try to clip polyline to be closed, or closed-enough for winding evaluation to work.
	// If that returns false, the polyline is "too open". In that case we will extend
	// outwards from the endpoints and then try to create a closed very large polygon
	if (UPolyLassoMarqueeMechanic::ApproximateSelfClipPolyline(Polyline) == false)
	{
		FVector2d StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
		FLine2d StartLine(Polyline[0], StartDirOut);
		FVector2d EndDirOut = UE::Geometry::Normalized(Polyline[N - 1] - Polyline[N - 2]);
		FLine2d EndLine(Polyline[N - 1], EndDirOut);

		// if we did not intersect, we are in ambiguous territory. Check if a segment along either end-direction
		// intersects the polyline. If it does, we have something like a spiral and will be OK. 
		// If not, make a closed polygon by interpolating outwards from each endpoint, and then in perp-directions.
		FPolygon2d Polygon(Polyline);
		float PerpSign = Polygon.IsClockwise() ? -1.0 : 1.0;

		Polyline.Insert(StartLine.PointAt(10000.0f), 0);
		Polyline.Insert(Polyline[0] + 1000 * PerpSign * UE::Geometry::PerpCW(StartDirOut), 0);

		Polyline.Add(EndLine.PointAt(10000.0f));
		Polyline.Add(Polyline.Last() + 1000 * PerpSign * UE::Geometry::PerpCW(EndDirOut));
		FVector2d StartPos = Polyline[0];
		Polyline.Add(StartPos);		// close polyline (cannot use Polyline[0] in case Add resizes!)
	}

	N = Polyline.Num();

	// project each mesh vertex to view plane and evaluate winding integral of polyline
	const FDynamicMesh3* Mesh = GetSculptMesh();
	TempROIBuffer.SetNum(Mesh->MaxVertexID());
	ParallelFor(Mesh->MaxVertexID(), [&](int32 vid)
	{
		if (Mesh->IsVertex(vid))
		{
			FVector3d WorldPos = CurTargetTransform.TransformPosition(Mesh->GetVertex(vid));
			FVector2d PlanePos = (FVector2d)Lasso.GetProjectedPoint((FVector)WorldPos);

			double WindingSum = 0;
			FVector2d a = Polyline[0] - PlanePos, b = FVector2d::Zero();
			for (int32 i = 1; i < N; ++i)
			{
				b = Polyline[i] - PlanePos;
				WindingSum += (double)FMathd::Atan2(a.X*b.Y - a.Y*b.X, a.X*b.X + a.Y*b.Y);
				a = b;
			}
			WindingSum /= FMathd::TwoPi;
			bool bInside = FMathd::Abs(WindingSum) > 0.3;
			TempROIBuffer[vid] = bInside ? 1 : 0;
		}
		else
		{
			TempROIBuffer[vid] = -1;
		}
	});

	// convert to vertex selection, and then select fully-enclosed faces
	FMeshVertexSelection VertexSelection(Mesh);
	VertexSelection.SelectByVertexID([&](int32 vid) { return TempROIBuffer[vid] == 1; });
 	FMeshFaceSelection FaceSelection(Mesh, VertexSelection, FilterProperties->MinTriVertCount);
	if (FaceSelection.Num() == 0)
	{
		return;
	}

	const int32 SetGroupID = GetInEraseStroke() ? GetGroupToErase() : GetGroupToSet();
	SetTrianglesToGroupID(FaceSelection.AsSet(), SetGroupID, GetInEraseStroke());
}




void UMeshGroupPaintTool::SetTrianglesToGroupID(const TSet<int32>& Triangles, int32 ToGroupID, bool bIsErase)
{
	BeginChange();

	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Triangles)
	{
		int32 CurGroupID = GetGroup(tid);
		if (CurGroupID == ToGroupID || FrozenGroups.Contains(CurGroupID))		// skip frozen groups
		{
			continue;
		}
		if (bIsErase == false && FilterProperties->bOnlySetUngrouped && CurGroupID != 0)
		{
			continue;
		}
		if (bIsErase && FilterProperties->bOnlyEraseCurrent && CurGroupID != GetGroupToSet())
		{
			continue;
		}

		TempROIBuffer.Add(tid);
	}

	if (HaveVisibilityFilter())
	{
		TArray<int32> VisibleTriangles;
		VisibleTriangles.Reserve(TempROIBuffer.Num());
		ApplyVisibilityFilter(TempROIBuffer, VisibleTriangles);
		TempROIBuffer = MoveTemp(VisibleTriangles);
	}

	SaveTriangleEdits();
	for (int32 tid : TempROIBuffer)
	{
		SetGroup(tid, ToGroupID, *GetSculptMesh());
	}
	SaveTriangleEdits();

	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}



bool UMeshGroupPaintTool::HaveVisibilityFilter() const
{
	return FilterProperties->VisibilityFilter != EMeshGroupPaintVisibilityType::None;
}


void UMeshGroupPaintTool::ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer)
{
	ROIBuffer.SetNum(0, EAllowShrinking::No);
	ROIBuffer.Reserve(Triangles.Num());
	for (int32 tid : Triangles)
	{
		ROIBuffer.Add(tid);
	}
	
	OutputBuffer.Reset();
	ApplyVisibilityFilter(TempROIBuffer, OutputBuffer);

	Triangles.Reset();
	for (int32 tid : OutputBuffer)
	{
		TriangleROI.Add(tid);
	}
}

void UMeshGroupPaintTool::ApplyVisibilityFilter(const TArray<int32>& Triangles, TArray<int32>& VisibleTriangles)
{
	if (!HaveVisibilityFilter())
	{
		VisibleTriangles = Triangles;
		return;
	}

	FViewCameraState StateOut;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
	FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Target);
	FVector3d LocalEyePosition(LocalToWorld.InverseTransformPosition(StateOut.Position));

	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 NumTriangles = Triangles.Num();

	VisibilityFilterBuffer.SetNum(NumTriangles, EAllowShrinking::No);
	ParallelFor(NumTriangles, [&](int32 idx)
	{
		VisibilityFilterBuffer[idx] = true;
		FVector3d Centroid = Mesh->GetTriCentroid(Triangles[idx]);
		FVector3d FaceNormal = Mesh->GetTriNormal(Triangles[idx]);
		if (FaceNormal.Dot((Centroid - LocalEyePosition)) > 0)
		{
			VisibilityFilterBuffer[idx] = false;
		}
		if (FilterProperties->VisibilityFilter == EMeshGroupPaintVisibilityType::Unoccluded)
		{
			int32 HitTID = Octree.FindNearestHitObject(FRay3d(LocalEyePosition, UE::Geometry::Normalized(Centroid - LocalEyePosition)));
			if (HitTID != Triangles[idx])
			{
				VisibilityFilterBuffer[idx] = false;
			}
		}
	});

	VisibleTriangles.Reset();
	for (int32 k = 0; k < NumTriangles; ++k)
	{
		if (VisibilityFilterBuffer[k])
		{
			VisibleTriangles.Add(Triangles[k]);
		}
	}
}



int32 UMeshGroupPaintTool::FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const
{
	if (!IsInBrushSubMode())
	{
		return IndexConstants::InvalidID;
	}

	if (GetBrushCanHitBackFaces())
	{
		return Octree.FindNearestHitObject(LocalRay);
	}
	else
	{
		const FDynamicMesh3* Mesh = GetSculptMesh();

		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));
		int HitTID = Octree.FindNearestHitObject(LocalRay,
			[this, Mesh, &LocalEyePosition](int TriangleID) {
			FVector3d Normal, Centroid;
			double Area;
			Mesh->GetTriInfo(TriangleID, Normal, Area, Centroid);
			return Normal.Dot((Centroid - LocalEyePosition)) < 0;
		});
		return HitTID;
	}
}

int32 UMeshGroupPaintTool::FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const
{
	check(false);
	return IndexConstants::InvalidID;
}



bool UMeshGroupPaintTool::UpdateBrushPosition(const FRay& WorldRay)
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bool bHit = false; 
	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	if (bHit && UseBrushOp->GetAlignStampToView())
	{
		AlignBrushToView();
	}

	return bHit;
}




bool UMeshGroupPaintTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	PendingStampType = FilterProperties->PrimaryBrushType;

	if(ensure(InStroke() == false))
	{
		UpdateBrushPosition(DevicePos.WorldRay);
	}
	return true;
}

void UMeshGroupPaintTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (PolyLassoMechanic)
	{
		// because the actual group change is deferred until mouse release, color the lasso to let the user know whether it will erase
		PolyLassoMechanic->LineColor = GetInEraseStroke() ? FLinearColor::Red : FLinearColor::Green;
		PolyLassoMechanic->DrawHUD(Canvas, RenderAPI);
	}


	float DPIScale = Canvas->GetDPIScale();
	UFont* UseFont = GEngine->GetSmallFont();
	FViewCameraState CamState = RenderAPI->GetCameraState();
	const FSceneView* SceneView = RenderAPI->GetSceneView();
	FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)CamState.Position));

	FDynamicMesh3* Mesh = GetSculptMesh();
	
	auto GetGroupString = [GroupIdLayer = ActiveGroupSet.Get(), GroupLabelAdapter = LabelAdapter.Get()](int CursorHitTID)
	{
		if (GroupLabelAdapter)
		{
			return FString::Printf(TEXT("('%s')"), *GroupLabelAdapter->GetValue(CursorHitTID).ToString());
		}
		return FString::Printf(TEXT("%d"), GroupIdLayer->GetTriangleGroup(CursorHitTID));
	};

	// draw the group number for the group under the cursor
	if (FilterProperties->bShowHitGroup)
	{
		int CursorHitTID = Octree.FindNearestHitObject(FRay3d(LocalEyePosition, Normalized(HoverStamp.LocalFrame.Origin - LocalEyePosition)));
		if (Mesh->IsTriangle(CursorHitTID))
		{
			FVector2D CursorPixelPos;
			SceneView->WorldToPixel(HoverStamp.WorldFrame.Origin, CursorPixelPos);
			const FString CursorString = GetGroupString(CursorHitTID);
			Canvas->DrawShadowedString(CursorPixelPos.X / (double)DPIScale, CursorPixelPos.Y / (double)DPIScale, *CursorString, UseFont, FLinearColor::White);
		}
	}

	// draw the group number for all groups
	if (FilterProperties->bShowAllGroups)
	{
		if (bDrawGroupsDataValid == false)
		{
			if (IsPaintingLabel())
			{
				GroupVisualizationCache.UpdateGroupInfo_ConnectedComponents(*Mesh, *LabelAdapter);
			}
			else
			{
				GroupVisualizationCache.UpdateGroupInfo_ConnectedComponents(*Mesh, *ActiveGroupSet);
			}
			bDrawGroupsDataValid = true;
		}
		for (const FGroupVisualizationCache::FGroupInfo& GroupInfo : GroupVisualizationCache)
		{
			FRay3d EyeRay;
			EyeRay.Origin = LocalEyePosition;
			EyeRay.Direction = Normalized(GroupInfo.Center - LocalEyePosition);
			int HitTID = Octree.FindNearestHitObject(EyeRay);
			if (HitTID == GroupInfo.CenterTris.A || HitTID == GroupInfo.CenterTris.B)
			{
				FVector2D PixelPos;
				SceneView->WorldToPixel(CurTargetTransform.TransformPosition(GroupInfo.Center), PixelPos);
				FString String = FString::Printf(TEXT("%d"), GroupInfo.GroupID);
				Canvas->DrawShadowedString(PixelPos.X / (double)DPIScale, PixelPos.Y / (double)DPIScale, *String, UseFont, FLinearColor::White);
			}
		}
	}

}


void UMeshGroupPaintTool::OnTick(float DeltaTime)
{
	UMeshSculptToolBase::OnTick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);

	bool bIsLasso = (FilterProperties->SubToolType == EMeshGroupPaintInteractionType::PolyLasso);
	PolyLassoMechanic->SetIsEnabled(bIsLasso);

	ConfigureIndicator(FilterProperties->BrushAreaMode == EMeshGroupPaintBrushAreaType::Volumetric);
	SetIndicatorVisibility(bIsLasso == false);

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EMeshGroupPaintToolActions::NoAction;
	}

	SCOPE_CYCLE_COUNTER(GroupPaintToolTick);

	// process the undo update
	if (bUndoUpdatePending)
	{
		// wait for updates
		WaitForPendingUndoRedoUpdate();

		// post rendering update
		DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(AccumulatedTriangleROI, EMeshRenderAttributeFlags::VertexColors);
		GetToolManager()->PostInvalidation();
		bDrawGroupsDataValid = false;

		// ignore stamp and wait for next tick to do anything else
		bUndoUpdatePending = false;
		return;
	}

	if (bPendingPickGroup || bPendingToggleFreezeGroup)
	{
		if (GetBrushTriangleID() >= 0 && IsStampPending() == false )
		{
			if (GetSculptMesh()->IsTriangle(GetBrushTriangleID()))
			{
				const int32 HitGroupID = GetGroup(GetBrushTriangleID());
				if (bPendingPickGroup)
				{
					FilterProperties->SetGroup = HitGroupID;
					if (IsPaintingLabel())
					{
						FilterProperties->SetLabel = LabelAdapter->GetValueFromGroup(HitGroupID);
					}
					NotifyOfPropertyChangeByTool(FilterProperties);
				}
				else if (bPendingToggleFreezeGroup)
				{
					ToggleFrozenGroup(HitGroupID);
				}
			}
		}
		bPendingPickGroup = bPendingToggleFreezeGroup = false;
	}

	auto ExecuteStampOperation = [this](int StampIndex, const FRay& StampRay)
		{
			SCOPE_CYCLE_COUNTER(GroupPaintTool_Tick_ApplyStampBlock);

			// update sculpt ROI
			UpdateROI(CurrentStamp);

			// append updated ROI to modified region (async)
			TFuture<void> AccumulateROI = Async(GroupPaintToolAsyncExecTarget, [&]()
				{
					AccumulatedTriangleROI.Append(TriangleROI);
				});

			// apply the stamp
			bool bGroupsModified = ApplyStamp();

			if (bGroupsModified)
			{
				SCOPE_CYCLE_COUNTER(GroupPaintTool_Tick_UpdateMeshBlock);
				DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TriangleROI, EMeshRenderAttributeFlags::VertexColors);
				GetToolManager()->PostInvalidation();
			}

			// we don't really need to wait for these to happen to end Tick()...
			AccumulateROI.Wait();
		};

	if (IsInBrushSubMode())
	{
		ProcessPerTickStamps(
			[this](const FRay& StampRay) -> bool {
				return UpdateStampPosition(StampRay);
			}, ExecuteStampOperation);
	}

}




void UMeshGroupPaintTool::AllocateNewGroupAndSetAsCurrentAction()
{
	FilterProperties->SetGroup = IsPaintingLabel() ? LabelAdapter->GetMaxGroupID() : ActiveGroupSet->MaxGroupID;
	NotifyOfPropertyChangeByTool(FilterProperties);
}



FColor UMeshGroupPaintTool::GetColorForGroup(int32 GroupID)
{
	FColor Color = LinearColors::SelectFColor(GroupID);
	if (FrozenGroups.Contains(GroupID))
	{
		int32 GrayValue = (Color.R + Color.G + Color.B) / 3;
		Color.R = Color.G = Color.B = FMath::Clamp(GrayValue, 0, 255);
	}
	return Color;
}

void UMeshGroupPaintTool::ToggleFrozenGroup(int32 FreezeGroupID)
{
	if (FreezeGroupID == 0) return;

	TArray<int32> InitialFrozenGroups = FrozenGroups;
	if (FrozenGroups.Contains(FreezeGroupID))
	{
		FrozenGroups.Remove(FreezeGroupID);
	}
	else
	{
		FrozenGroups.Add(FreezeGroupID);
	}

	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		int32 TriGroupID = GetGroup(tid);
		if (TriGroupID == FreezeGroupID)
		{
			TempROIBuffer.Add(tid);
		}
	}
	EmitFrozenGroupsChange(InitialFrozenGroups, FrozenGroups, LOCTEXT("ToggleFrozenGroupChange", "Toggle Frozen Group"));
	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
}

void UMeshGroupPaintTool::FreezeOtherGroups(int32 KeepGroupID)
{
	TArray<int32> InitialFrozenGroups = FrozenGroups;
	FrozenGroups.Reset();
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		int32 GroupID = GetGroup(tid);
		if ( GroupID != 0 && GroupID != KeepGroupID)
		{
			FrozenGroups.AddUnique(GetGroup(tid));
			TempROIBuffer.Add(tid);
		}
	}
	EmitFrozenGroupsChange(InitialFrozenGroups, FrozenGroups, LOCTEXT("FreezeOtherGroups", "Freeze Other Groups"));
	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
}

void UMeshGroupPaintTool::ClearAllFrozenGroups()
{
	TArray<int32> InitialFrozenGroups = FrozenGroups;
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		if ( FrozenGroups.Contains(GetGroup(tid)) )
		{
			TempROIBuffer.Add(tid);
		}
	}
	FrozenGroups.Reset();
	EmitFrozenGroupsChange(InitialFrozenGroups, FrozenGroups, LOCTEXT("ClearAllFrozenGroups", "Clear Frozen Groups"));
	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
}



void UMeshGroupPaintTool::EmitFrozenGroupsChange(const TArray<int32>& FromGroups, const TArray<int32>& ToGroups, const FText& ChangeText)
{
	if (FromGroups != ToGroups)
	{
		TUniquePtr<TSimpleValueLambdaChange<TArray<int32>>> FrozenGroupsChange = MakeUnique<TSimpleValueLambdaChange<TArray<int32>>>();
		FrozenGroupsChange->FromValue = FromGroups;
		FrozenGroupsChange->ToValue = ToGroups;
		FrozenGroupsChange->ValueChangeFunc = [this](UObject*, const TArray<int32>& FromGroups, const TArray<int32>& ToGroups, bool)
		{	
			FrozenGroups = ToGroups;
			DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
		};
		GetToolManager()->EmitObjectChange(this, MoveTemp(FrozenGroupsChange), ChangeText);
	}
}


void UMeshGroupPaintTool::GrowCurrentGroupAction()
{
	BeginChange();

	const int32 CurrentGroupID = GetGroupToSet();
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FMeshFaceSelection InitialSelection(Mesh);
	InitialSelection.Select([&](int32 tid) { return GetGroup(tid) == CurrentGroupID; });
	FMeshFaceSelection ExpandSelection(InitialSelection);
	ExpandSelection.ExpandToOneRingNeighbours([&](int32 tid) { return FrozenGroups.Contains(GetGroup(tid)) == false; });
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	ExpandSelection.SetDifference(InitialSelection, TempROIBuffer);

	SaveTriangleEdits();
	for (int32 tid : TempROIBuffer)
	{
		SetGroup(tid, CurrentGroupID, *GetSculptMesh());
	}
	SaveTriangleEdits();

	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UMeshGroupPaintTool::ShrinkCurrentGroupAction()
{
	BeginChange();

	const int32 CurrentGroupID = GetGroupToSet();
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FMeshFaceSelection InitialSelection(Mesh);
	InitialSelection.Select([&](int32 tid) { return GetGroup(tid) == CurrentGroupID; });
	FMeshFaceSelection ContractSelection(InitialSelection);
	ContractSelection.ContractBorderByOneRingNeighbours();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	InitialSelection.SetDifference(ContractSelection, TempROIBuffer);

	SaveTriangleEdits();
	for (int32 tid : TempROIBuffer)
	{
		// todo: could probably guess boundary groups here...
		SetGroup(tid, 0, *GetSculptMesh());
	}
	SaveTriangleEdits();

	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UMeshGroupPaintTool::ClearCurrentGroupAction()
{
	BeginChange();

	const int32 CurrentGroupID = GetGroupToSet();
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		if (GetGroup(tid) == CurrentGroupID)
		{
			TempROIBuffer.Add(tid);
		}
	}

	SaveTriangleEdits();
	for (int32 tid : TempROIBuffer)
	{
		SetGroup(tid, 0, *GetSculptMesh());
	}
	SaveTriangleEdits();

	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UMeshGroupPaintTool::FloodFillCurrentGroupAction()
{
	BeginChange();

	const int32 SetGroupID = GetGroupToSet();
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		const int32 GroupID = GetGroup(tid);
		if (GroupID == 0 && GroupID != SetGroupID && FrozenGroups.Contains(GroupID) == false)
		{
			TempROIBuffer.Add(tid);
		}
	}

	SaveTriangleEdits();
	for (int32 tid : TempROIBuffer)
	{
		SetGroup(tid, SetGroupID, *GetSculptMesh());
	}
	SaveTriangleEdits();

	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UMeshGroupPaintTool::ClearAllGroupsAction()
{
	BeginChange();

	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		if (GetGroup(tid) != 0)
		{
			TempROIBuffer.Add(tid);
		}
	}

	SaveTriangleEdits();
	for (int32 tid : TempROIBuffer)
	{
		SetGroup(tid, 0, *GetSculptMesh());
	}
	SaveTriangleEdits();

	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TempROIBuffer, EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}




//
// Change Tracking
//
void UMeshGroupPaintTool::BeginChange()
{
	check(ActiveGroupEditBuilder == nullptr && ActiveLabelEditBuilder == nullptr);
	
	if (IsPaintingLabel())
	{
		ActiveLabelEditBuilder = MakeUnique<FDynamicMeshTriangleLabelEditBuilder>(LabelAdapter.Get());
	}
	else
	{
		ActiveGroupEditBuilder = MakeUnique<FDynamicMeshGroupEditBuilder>(ActiveGroupSet.Get());
	}

	LongTransactions.Open(LOCTEXT("GroupPaintChange", "Group Stroke"), GetToolManager());
}

void UMeshGroupPaintTool::EndChange()
{
	check(ActiveGroupEditBuilder || ActiveLabelEditBuilder);

	static const FText Description = LOCTEXT("GroupPaintChange", "Group Stroke");
	if (ActiveLabelEditBuilder)
	{
		TUniquePtr<FDynamicMeshTriangleLabelEdit> EditResult = ActiveLabelEditBuilder->ExtractResult();
		ActiveLabelEditBuilder = nullptr;

		TUniquePtr<TWrappedToolCommandChange<FMeshTriangleLabelChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshTriangleLabelChange>>();
		NewChange->WrappedChange = MakeUnique<FMeshTriangleLabelChange>(MoveTemp(EditResult));
		NewChange->BeforeModify = [this](bool bRevert)
		{
			this->WaitForPendingUndoRedoUpdate();
		};

		GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(NewChange), Description);
	}
	else
	{
		TUniquePtr<FDynamicMeshGroupEdit> EditResult = ActiveGroupEditBuilder->ExtractResult();
		ActiveGroupEditBuilder = nullptr;

		TUniquePtr<TWrappedToolCommandChange<FMeshPolygroupChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshPolygroupChange>>();
		NewChange->WrappedChange = MakeUnique<FMeshPolygroupChange>(MoveTemp(EditResult));
		NewChange->BeforeModify = [this](bool bRevert)
			{
				this->WaitForPendingUndoRedoUpdate();
			};

		GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(NewChange), Description);
	}
	
	LongTransactions.Close(GetToolManager());

	// debug groups are invalid now
	bDrawGroupsDataValid = false;
}

void UMeshGroupPaintTool::SaveTriangleEdits() const
{
	if (IsPaintingLabel())
	{
		if (ensure(ActiveLabelEditBuilder))
		{
			ActiveLabelEditBuilder->SaveTriangles(TempROIBuffer);
		}
	}
	else
	{
		if (ensure(ActiveGroupEditBuilder))
		{
			ActiveGroupEditBuilder->SaveTriangles(TempROIBuffer);
		}
	}
}

void UMeshGroupPaintTool::WaitForPendingUndoRedoUpdate()
{
	if (bUndoUpdatePending)
	{
		bUndoUpdatePending = false;
	}
}

void UMeshGroupPaintTool::OnDynamicMeshComponentChanged()
{
	// update octree
	FDynamicMesh3* Mesh = GetSculptMesh();

	// make sure any previous async computations are done, and update the undo ROI
	if (bUndoUpdatePending)
	{
		// we should never hit this anymore, because of pre-change calling WaitForPendingUndoRedoUpdate()
		WaitForPendingUndoRedoUpdate();

		// TODO: do we need to read from mesh change here??
		//UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}
	else
	{
		// TODO: do we need to read from mesh change here??
		//UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}

	// note that we have a pending update
	bUndoUpdatePending = true;
}


void UMeshGroupPaintTool::PrecomputeFilterData()
{
	if (const FDynamicMesh3* Mesh = GetSculptMesh())
	{
		TriNormals.SetNum(Mesh->MaxTriangleID());
		ParallelFor(Mesh->MaxTriangleID(), [&](int32 tid)
		{
			if (Mesh->IsTriangle(tid))
			{
				TriNormals[tid] = Mesh->GetTriNormal(tid);
			}
		});

		UVSeamEdges.SetNum(Mesh->MaxEdgeID());
		NormalSeamEdges.SetNum(Mesh->MaxEdgeID());

		if (const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
		{
			const FDynamicMeshNormalOverlay* Normals = Attributes->PrimaryNormals();
			const FDynamicMeshUVOverlay* UVs = Attributes->PrimaryUV();
			if (Normals || UVs)
			{
				ParallelFor(Mesh->MaxEdgeID(), [&](int32 eid)
				{
					if (Mesh->IsEdge(eid))
					{
						UVSeamEdges[eid] = UVs ? UVs->IsSeamEdge(eid) : 0;
						NormalSeamEdges[eid] = Normals ? Normals->IsSeamEdge(eid) : 0;
					}
				});
			}
		}
	}
}

int32 UMeshGroupPaintTool::GetGroup(const int32 TriangleID) const
{
	return IsPaintingLabel() ? LabelAdapter->GetGroup(TriangleID) : ActiveGroupSet->GetGroup(TriangleID);
}

void UMeshGroupPaintTool::SetGroup(const int32 TriangleID, const int32 GroupID, FDynamicMesh3& Mesh) const
{
	return IsPaintingLabel() ? LabelAdapter->SetGroup(TriangleID, GroupID, Mesh) : ActiveGroupSet->SetGroup(TriangleID, GroupID, Mesh);
}

int32 UMeshGroupPaintTool::GetGroupToSet() const
{
	return IsPaintingLabel() ? LabelAdapter->GetGroupFromValue(FilterProperties->SetLabel) : FilterProperties->SetGroup;
}

int32 UMeshGroupPaintTool::GetGroupToErase() const
{
	return IsPaintingLabel() ? LabelAdapter->GetGroupFromValue(FilterProperties->EraseLabel) : FilterProperties->EraseGroup;
}

void UMeshGroupPaintTool::OnSelectedGroupLayerChanged()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("ChangeActiveGroupLayer", "Change Polygroup Layer"));

	TArray<int32> InitialFrozenGroups = FrozenGroups;

	int32 ActiveLayerIndex = (ActiveGroupSet) ? ActiveGroupSet->GetPolygroupIndex() : -1;
	UpdateActiveGroupLayer();
	int32 NewLayerIndex = (ActiveGroupSet) ? ActiveGroupSet->GetPolygroupIndex() : -1;

	if (ActiveLayerIndex != NewLayerIndex)
	{
		// clear frozen groups
		EmitFrozenGroupsChange(InitialFrozenGroups, FrozenGroups, LOCTEXT("ClearAllFrozenGroups", "Clear Frozen Groups"));

		TUniquePtr<TSimpleValueLambdaChange<int32>> GroupLayerChange = MakeUnique<TSimpleValueLambdaChange<int32>>();
		GroupLayerChange->FromValue = ActiveLayerIndex;
		GroupLayerChange->ToValue = NewLayerIndex;
		GroupLayerChange->ValueChangeFunc = [this](UObject*, int32 FromIndex, int32 ToIndex, bool)
		{
			this->PolygroupLayerProperties->SetSelectedFromPolygroupIndex(ToIndex);
			this->PolygroupLayerProperties->SilentUpdateWatched();		// to prevent OnSelectedGroupLayerChanged() from being called immediately
			this->UpdateActiveGroupLayer();
		};
		GetToolManager()->EmitObjectChange(this, MoveTemp(GroupLayerChange), LOCTEXT("ChangeActiveGroupLayer", "Change Polygroup Layer"));
	}

	GetToolManager()->EndUndoTransaction();
}


void UMeshGroupPaintTool::UpdateActiveGroupLayer()
{
	ActiveGroupSet.Reset();
	LabelAdapter.Reset();
	FilterProperties->bPaintingLabels = false;
	
	FDynamicMesh3* Mesh = GetSculptMesh();
	
	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		ActiveGroupSet = MakeUnique<FPolygroupSet>(Mesh);
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		FDynamicMeshPolygroupAttribute* FoundAttrib = FindPolygroupLayerByName(*Mesh, SelectedName);
		if (!FoundAttrib)
		{
			if (FDynamicMeshTriangleLabelAttribute* Labels = FindTriangleLabelLayerByName(*Mesh, SelectedName))
			{
				LabelAdapter = MakeShared<FTriangleLabelAdapter>(Mesh, Labels);
				FilterProperties->bPaintingLabels = true;
			}
		}
		
		if (!LabelAdapter.IsValid())
		{
			ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
			ActiveGroupSet = MakeUnique<FPolygroupSet>(Mesh, FoundAttrib);
		}
	}
	
	FilterProperties->bPaintingLabels = IsPaintingLabel();

	// need to reset everything here...
	FrozenGroups.Reset();

	// update colors
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
}



void UMeshGroupPaintTool::UpdateSubToolType(EMeshGroupPaintInteractionType NewType)
{
	// Currenly we mirror base-brush properties in UGroupPaintBrushFilterProperties, so we never
	// want to show both
	//bool bSculptPropsVisible = (NewType == EMeshGroupPaintInteractionType::Brush);
	//SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, bSculptPropsVisible);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, false);

	SetToolPropertySourceEnabled(FilterProperties, true);
	SetBrushOpPropsVisibility(false);
}


void UMeshGroupPaintTool::UpdateBrushType(EMeshGroupPaintBrushType BrushType)
{
	static const FText BaseMessage = LOCTEXT("OnStartTool", "Hold Shift to Erase. [/] and S/D change Size (+Shift to small-step). Shift+Q for New Group, Shift+G to pick Group, Shift+F to Freeze Group.");
	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	SetActivePrimaryBrushType((int32)BrushType);

	SetToolPropertySourceEnabled(GizmoProperties, false);

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}




void UMeshGroupPaintTool::RequestAction(EMeshGroupPaintToolActions ActionType)
{
	if (!bHavePendingAction)
	{
		PendingAction = ActionType;
		bHavePendingAction = true;
	}
}


void UMeshGroupPaintTool::ApplyAction(EMeshGroupPaintToolActions ActionType)
{
	switch (ActionType)
	{
	case EMeshGroupPaintToolActions::ClearFrozen:
		ClearAllFrozenGroups();
		break;

	case EMeshGroupPaintToolActions::FreezeCurrent:
		ToggleFrozenGroup(GetGroupToSet());
		break;

	case EMeshGroupPaintToolActions::FreezeOthers:
		FreezeOtherGroups(GetGroupToSet());
		break;

	case EMeshGroupPaintToolActions::GrowCurrent:
		GrowCurrentGroupAction();
		break;

	case EMeshGroupPaintToolActions::ShrinkCurrent:
		ShrinkCurrentGroupAction();
		break;

	case EMeshGroupPaintToolActions::ClearCurrent:
		ClearCurrentGroupAction();
		break;

	case EMeshGroupPaintToolActions::FloodFillCurrent:
		FloodFillCurrentGroupAction();
		break;

	case EMeshGroupPaintToolActions::ClearAll:
		ClearAllGroupsAction();
		break;

	}
}




#undef LOCTEXT_NAMESPACE

