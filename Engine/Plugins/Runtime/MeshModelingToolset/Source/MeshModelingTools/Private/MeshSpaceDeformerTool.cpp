// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshSpaceDeformerTool.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ToolBuilderUtil.h"

#include "SegmentTypes.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshOpPreviewHelpers.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "Intersection/IntersectionUtil.h"
#include "PreviewMesh.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "Selection/SelectClickedAction.h"

#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/IntervalGizmo.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "CoreMinimal.h"
#include "Math/Matrix.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSpaceDeformerTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "MeshSpaceDeformerTool"

/*
 * ToolBuilder
 */
USingleSelectionMeshEditingTool* UMeshSpaceDeformerToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshSpaceDeformerTool>(SceneState.ToolManager);
}


TUniquePtr<FDynamicMeshOperator> USpaceDeformerOperatorFactory::MakeNewOperator()
{

	check(SpaceDeformerTool);

	const ENonlinearOperationType OperationType = SpaceDeformerTool->Settings->SelectedOperationType;
	
	// Create the actual operator type based on the requested operation
	TUniquePtr<FMeshSpaceDeformerOp>  DeformerOp;
	
	switch (OperationType)
	{
		case ENonlinearOperationType::Bend: 
		{
			DeformerOp = MakeUnique<FBendMeshOp>();
			static_cast<FBendMeshOp*>(DeformerOp.Get())->BendDegrees = SpaceDeformerTool->Settings->BendDegrees;
			static_cast<FBendMeshOp*>(DeformerOp.Get())->bLockBottom = SpaceDeformerTool->Settings->bLockBottom;
			break;
		}
		case ENonlinearOperationType::Flare:
		{
			DeformerOp = MakeUnique<FFlareMeshOp>();
			static_cast<FFlareMeshOp*>(DeformerOp.Get())->FlarePercentY = SpaceDeformerTool->Settings->FlarePercentY;
			static_cast<FFlareMeshOp*>(DeformerOp.Get())->FlarePercentX = SpaceDeformerTool->Settings->bLockXAndYFlaring ? 
				SpaceDeformerTool->Settings->FlarePercentY : SpaceDeformerTool->Settings->FlarePercentX;
			if (SpaceDeformerTool->Settings->FlareProfileType == EFlareProfileType::SinMode)
			{
				static_cast<FFlareMeshOp*>(DeformerOp.Get())->FlareType = FFlareMeshOp::EFlareType::SinFlare;
			}
			else if (SpaceDeformerTool->Settings->FlareProfileType == EFlareProfileType::SinSquaredMode)
			{
				static_cast<FFlareMeshOp*>(DeformerOp.Get())->FlareType = FFlareMeshOp::EFlareType::SinSqrFlare;
			}
			else
			{
				static_cast<FFlareMeshOp*>(DeformerOp.Get())->FlareType = FFlareMeshOp::EFlareType::LinearFlare;
			}
			break;
		}
		case ENonlinearOperationType::Twist:
		{
			DeformerOp = MakeUnique<FTwistMeshOp>();
			static_cast<FTwistMeshOp*>(DeformerOp.Get())->TwistDegrees = SpaceDeformerTool->Settings->TwistDegrees;
			static_cast<FTwistMeshOp*>(DeformerOp.Get())->bLockBottom = SpaceDeformerTool->Settings->bLockBottom;
			break;
		}
	default:
		check(0);
	}

	// Operator runs on another thread - copy data over that it needs.
	SpaceDeformerTool->UpdateOpParameters(*DeformerOp);
	

	// give the operator
	return DeformerOp;
}


void UMeshSpaceDeformerToolActionPropertySet::PostAction(EMeshSpaceDeformerToolAction Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}


/*
 * Tool
 */
UMeshSpaceDeformerTool::UMeshSpaceDeformerTool()
{
}

bool UMeshSpaceDeformerTool::CanAccept() const 
{
	return Super::CanAccept() && (Preview == nullptr || Preview->HaveValidResult());
}

void UMeshSpaceDeformerTool::Setup()
{
	UInteractiveTool::Setup();

	Settings = NewObject<UMeshSpaceDeformerToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);

	ToolActions = NewObject<UMeshSpaceDeformerToolActionPropertySet>(this);
	ToolActions->Initialize(this);
	AddToolPropertySource(ToolActions);

	// populate the OriginalDynamicMesh with a conversion of the input mesh.
	{
		OriginalDynamicMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		*OriginalDynamicMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
	}

	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	FTransform MeshTransform = TargetComponent->GetWorldTransform();

	// Hide the mesh, and potentially put a semi-transparent copy in its place. We could
	// update the materials and restore them later, but this seems safer.
	TargetComponent->SetOwnerVisibility(false);

	OriginalMeshPreview = NewObject<UPreviewMesh>();
	OriginalMeshPreview->CreateInWorld(GetTargetWorld(), MeshTransform);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(OriginalMeshPreview, Target);
	OriginalMeshPreview->UpdatePreview(OriginalDynamicMesh.Get());
	OriginalMeshPreview->SetMaterial(0, ToolSetupUtil::GetCustomDepthOffsetMaterial(GetToolManager(), FLinearColor::White,
		-0.5f,  // depth offset, 0.5% inward
		0.4f)); // opacity
	OriginalMeshPreview->SetVisible(Settings->bShowOriginalMesh);

	// The gizmo gets initialized to the center point of the object-space bounding box,
	// with the Z axis (along which the deformation acts) aligned to the longest of the
	// bounding box dimensions, after scaling them with the transform.
	FAxisAlignedBox3d BBox = OriginalDynamicMesh->GetBounds();
	FVector3d Dimensions = BBox.IsEmpty() ? FVector3d::Zero() : BBox.Max - BBox.Min;
	MeshCenter = BBox.IsEmpty() ? FVector3d::Zero() : (FVector3d)MeshTransform.TransformPosition((FVector)BBox.Center());

	Dimensions = FVector3d(MeshTransform.GetScale3D().GetAbs()) * Dimensions;
	double WorldMajorLength = 0;

	// Prefer being aligned with the Z axis.
	if (Dimensions.Z >= Dimensions.Y && Dimensions.Z >= Dimensions.X)
	{
		WorldMajorLength = Dimensions.Z;
		GizmoFrame = FFrame3d(MeshCenter, FVector3d::UnitX(), FVector3d::UnitY(), FVector3d::UnitZ());
	}
	else if (Dimensions.Y >= Dimensions.X)
	{
		WorldMajorLength = Dimensions.Y;
		GizmoFrame = FFrame3d(MeshCenter, FVector3d::UnitZ(), FVector3d::UnitX(), FVector3d::UnitY());
	}
	else
	{
		WorldMajorLength = Dimensions.X;
		GizmoFrame = FFrame3d(MeshCenter, FVector3d::UnitY(), FVector3d::UnitZ(), FVector3d::UnitX());
	}

	GizmoFrame.Rotate((FQuaterniond)MeshTransform.GetRotation());

	// The scaling of the modifier gizmo is somewhat arbitrary. We choose for it to be
	// related to the major axis length.
	ModifierGizmoLength = WorldMajorLength;

	// add click to set plane behavior
	SetPointInWorldConnector = MakePimpl<FSelectClickedAction>();
	SetPointInWorldConnector->SnapManager = USceneSnappingManager::Find(GetToolManager());
	SetPointInWorldConnector->InvisibleComponentsToHitTest.Add(TargetComponent->GetOwnerComponent());
	SetPointInWorldConnector->OnClickedPositionFunc = [this](const FHitResult& Hit)
	{
		SetGizmoFrameFromWorldPos(Hit.ImpactPoint, Hit.ImpactNormal, Settings->bAlignToNormalOnCtrlClick);
	};

	USingleClickInputBehavior* ClickToSetPlaneBehavior = NewObject<USingleClickInputBehavior>();
	ClickToSetPlaneBehavior->ModifierCheckFunc = FInputDeviceState::IsCtrlKeyDown;
	ClickToSetPlaneBehavior->Initialize(SetPointInWorldConnector.Get());
	AddInputBehavior(ClickToSetPlaneBehavior);


	// Create a new TransformGizmo and associated TransformProxy. The TransformProxy will not be the
	// parent of any Components in this case, we just use it's transform and change delegate.
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->SetTransform(GizmoFrame.ToFTransform());
	TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(),
		ETransformGizmoSubElements::StandardTranslateRotate, this);

	TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());

	// listen for changes to the proxy and update the preview when that happens
	TransformProxy->OnTransformChanged.AddUObject(this, &UMeshSpaceDeformerTool::TransformProxyChanged);

	// create sources for the interval parameters
	UpIntervalSource      = NewObject< UGizmoLocalFloatParameterSource >(this);
	DownIntervalSource    = NewObject< UGizmoLocalFloatParameterSource >(this);
	ForwardIntervalSource = NewObject< UGizmoLocalFloatParameterSource >(this);

	// Initial Lengths for the interval handle
	UpIntervalSource->Value      = WorldMajorLength/2;
	DownIntervalSource->Value    = -WorldMajorLength / 2;
	ForwardIntervalSource->Value = GetModifierGizmoValue();

	// Sync the properties panel to the interval handles.
	Settings->UpperBoundsInterval = UpIntervalSource->Value;
	Settings->LowerBoundsInterval = DownIntervalSource->Value;

	// Wire up callbacks to update result mesh and the properties panel when these parameters are changed (by gizmo manipulation in viewport).  Note this is just a one-way
	// coupling (Sources to Properties). The OnPropertyModified() method provides the Properties to Souces coupling 
	UpIntervalSource->OnParameterChanged.AddLambda([this](IGizmoFloatParameterSource* ParamSource, FGizmoFloatParameterChange Change)->void
	{
		Settings->UpperBoundsInterval = Change.CurrentValue;
		UpdatePreview();
	});

	DownIntervalSource->OnParameterChanged.AddLambda([this](IGizmoFloatParameterSource* ParamSource, FGizmoFloatParameterChange Change)->void
	{
		Settings->LowerBoundsInterval = Change.CurrentValue;
		UpdatePreview();
	});

	ForwardIntervalSource->OnParameterChanged.AddLambda([this](IGizmoFloatParameterSource* ParamSource, FGizmoFloatParameterChange Change)->void
	{
		ApplyModifierGizmoValue(Change.CurrentValue);
		UpdatePreview();
	});

	// add the interval gizmo
	GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(UIntervalGizmo::GizmoName, NewObject<UIntervalGizmoBuilder>());
	IntervalGizmo = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UIntervalGizmo>(UIntervalGizmo::GizmoName, TEXT("MeshSpaceDefomerInterval"), this);

	// wire in the transform and the interval sources.
	IntervalGizmo->SetActiveTarget(TransformProxy, UpIntervalSource, DownIntervalSource, ForwardIntervalSource, GetToolManager());

	// use the statetarget to track details changes
	StateTarget = IntervalGizmo->StateTarget;

	// Set up the bent line visualizer
	VisualizationRenderer.bDepthTested = false;
	VisualizationRenderer.LineColor = FLinearColor::Yellow;
	VisualizationRenderer.LineThickness = 4.0;
	VisualizationRenderer.SetTransform(GizmoFrame.ToFTransform());

	// Set up the preview object
	{
		// create the operator factory
		USpaceDeformerOperatorFactory* DeformerOperatorFactory = NewObject<USpaceDeformerOperatorFactory>(this);
		DeformerOperatorFactory->SpaceDeformerTool = this; // set the back pointer


		Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(DeformerOperatorFactory, "Preview");
		Preview->Setup(GetTargetWorld(), DeformerOperatorFactory);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target);
		Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

		Preview->SetIsMeshTopologyConstant(true, EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals);

		// Give the preview something to display
		Preview->PreviewMesh->UpdatePreview(OriginalDynamicMesh.Get());
		Preview->PreviewMesh->SetTransform(MeshTransform);

		FComponentMaterialSet MaterialSet;
		MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
		Preview->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

		// show the preview mesh
		Preview->SetVisibility(true);

		// start the compute
		UpdatePreview();
	}

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);

	// We want to align to the original mesh, even though it is hidden (our stand-in preview mesh that
	// we use to display a transparent version does not get hit tested by our normal raycasts into
	// the world).
	TArray<const UPrimitiveComponent*> ComponentsToInclude{ TargetComponent->GetOwnerComponent() } ;
	DragAlignmentMechanic->AddToGizmo(TransformGizmo, nullptr, &ComponentsToInclude);
	DragAlignmentMechanic->AddToGizmo(IntervalGizmo, nullptr, &ComponentsToInclude);

	SetToolDisplayName(LOCTEXT("ToolName", "Space Warp"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("MeshSpaceDeformerToolDescription", "Deform the vertices of the selected Mesh using various spatial deformations. Use the in-viewport Gizmo to control the extents/strength of the deformation. Hold Ctrl while translating/rotating gizmo to align to world."),
		EToolMessageLevel::UserNotification);
}

void UMeshSpaceDeformerTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);

	// Restore source mesh and remove our stand-in
	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);
	OriginalMeshPreview->SetVisible(false);
	OriginalMeshPreview->Disconnect();
	OriginalMeshPreview = nullptr;

	if (Preview != nullptr)
	{
		FDynamicMeshOpResult Result = Preview->Shutdown();
		if (ShutdownType == EToolShutdownType::Accept)
		{

			GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshSpaceDeformer", "Space Deformer"));

			FDynamicMesh3* DynamicMeshResult = Result.Mesh.Get();
			check(DynamicMeshResult != nullptr);

			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *DynamicMeshResult, true);

			GetToolManager()->EndUndoTransaction();
		}
	}

	DragAlignmentMechanic->Shutdown();

	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	GizmoManager->DestroyAllGizmosByOwner(this);
	GizmoManager->DeregisterGizmoType(UIntervalGizmo::GizmoName);
}



void  UMeshSpaceDeformerTool::TransformProxyChanged(UTransformProxy* Proxy, FTransform Transform)
{
	GizmoFrame = FFrame3d(Transform.GetLocation(), Transform.GetRotation());
	VisualizationRenderer.SetTransform(GizmoFrame.ToFTransform());
	UpdatePreview();
}

void  UMeshSpaceDeformerTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	UpIntervalSource->Value = Settings->UpperBoundsInterval;
	DownIntervalSource->Value = Settings->LowerBoundsInterval;
	ForwardIntervalSource->Value = GetModifierGizmoValue();

	UpdatePreview();

	OriginalMeshPreview->SetVisible(Settings->bShowOriginalMesh);
}

void UMeshSpaceDeformerTool::UpdateOpParameters(FMeshSpaceDeformerOp& MeshSpaceDeformerOp) const
{
	MeshSpaceDeformerOp.OriginalMesh = OriginalDynamicMesh;
	MeshSpaceDeformerOp.SetTransform(Cast<IPrimitiveComponentBackedTarget>(Target)->GetWorldTransform());
	MeshSpaceDeformerOp.GizmoFrame = GizmoFrame;

	// set the bound range
	MeshSpaceDeformerOp.UpperBoundsInterval = Settings->UpperBoundsInterval;
	MeshSpaceDeformerOp.LowerBoundsInterval = Settings->LowerBoundsInterval;
}

void UMeshSpaceDeformerTool::RequestAction(EMeshSpaceDeformerToolAction ActionType)
{
	if (PendingAction == EMeshSpaceDeformerToolAction::NoAction)
	{
		PendingAction = ActionType;
	}
}

void UMeshSpaceDeformerTool::ApplyAction(EMeshSpaceDeformerToolAction ActionType)
{
	if (PendingAction == EMeshSpaceDeformerToolAction::ShiftToCenter)
	{
		SetGizmoFrameFromWorldPos((FVector)MeshCenter);
	}
}

void UMeshSpaceDeformerTool::OnTick(float DeltaTime)
{
	// Deal with clicked button
	if (PendingAction != EMeshSpaceDeformerToolAction::NoAction)
	{
		ApplyAction(PendingAction);
		PendingAction = EMeshSpaceDeformerToolAction::NoAction;
	}

	if (Preview != nullptr)
	{
		Preview->Tick(DeltaTime);
	}
}

void UMeshSpaceDeformerTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);

	if (Settings->bDrawVisualization && Settings->SelectedOperationType == ENonlinearOperationType::Bend)
	{
		VisualizationRenderer.BeginFrame(RenderAPI, RenderAPI->GetCameraState());
		for (int32 i = 1; i < VisualizationPoints.Num(); ++i)
		{
			VisualizationRenderer.DrawLine(VisualizationPoints[i - 1], VisualizationPoints[i]);
		}
		VisualizationRenderer.EndFrame();
	}
	else
	{
	}
}

void UMeshSpaceDeformerTool::UpdatePreview()
{
	if (Settings->SelectedOperationType == ENonlinearOperationType::Bend)
	{
		const int32 NUM_RENDER_POINTS = 30;

		VisualizationPoints.SetNumUninitialized(NUM_RENDER_POINTS);

		double BentLength = Settings->UpperBoundsInterval - Settings->LowerBoundsInterval;
		double ArcAngle = Settings->BendDegrees * PI / 180;
		double ArcRadius = BentLength / ArcAngle;

		double RotationCenterZ = Settings->bLockBottom ? Settings->LowerBoundsInterval : 0;
		FVector2d RotationCenterYZ(ArcRadius, RotationCenterZ);

		double PointSpacing = BentLength / (NUM_RENDER_POINTS - 1);

		for (int32 i = 0; i < NUM_RENDER_POINTS; ++i)
		{
			double OriginalZ = Settings->LowerBoundsInterval + PointSpacing * i;
			FVector2d YZToRotate(0, RotationCenterZ);

			// The negative here is because we are rotating clockwise in the direction of the positive Y axis
			double AngleToRotate = -ArcAngle * (OriginalZ - RotationCenterZ) / BentLength;

			FMatrix2d RotationMatrix = FMatrix2d::RotationRad(AngleToRotate);
			FVector2d RotatedYZ = RotationMatrix * (YZToRotate - RotationCenterYZ) + RotationCenterYZ;

			VisualizationPoints[i] = FVector3d(0, RotatedYZ.X, RotatedYZ.Y);
		}
	}

	if (Preview)
	{
		Preview->InvalidateResult();
	}
}


void UMeshSpaceDeformerTool::SetGizmoFrameFromWorldPos(const FVector& Position, const FVector& Normal, bool bAlignNormal)
{
	GizmoFrame.Origin = (FVector3d)Position;
	if (bAlignNormal)
	{
		// It's not clear whether aligning the Z axis to the normal is the right idea here. The Z axis
		// is the main axis on which we operate. On the one hand, setting it to the normal gives the user
		// greater control over its alignment. On the other hand, it seems likely that when clicking the object,
		// the user would want the axis to lie along the object on the side they clicked, not pierce inwards.
		// Still, it's hard to come up with a clean alternative.
		FVector3d FrameZ(Normal);
		FVector3d FrameY = UE::Geometry::Normalized(FrameZ.Cross(FVector3d::UnitZ())); // orthogonal to world Z and frame Z 
		FVector3d FrameX = FrameY.Cross(FrameZ); // safe to not normalize because already orthogonal
		GizmoFrame = FFrame3d((FVector3d)Position, FrameX, FrameY, FrameZ);
	}

	TransformGizmo->ReinitializeGizmoTransform(GizmoFrame.ToFTransform());
	UpdatePreview();
}

/** 
 * These two functions translate to and from the modifier gizmo length
 * to the relevant operator parameters. They should be matched to each
 * other.
 */
double UMeshSpaceDeformerTool::GetModifierGizmoValue() const
{
	switch (Settings->SelectedOperationType)
	{
	case ENonlinearOperationType::Bend:
		return Settings->BendDegrees * ModifierGizmoLength / 180;
	case ENonlinearOperationType::Flare:
		return Settings->FlarePercentY * ModifierGizmoLength / 100;
	case ENonlinearOperationType::Twist:
		return Settings->TwistDegrees * ModifierGizmoLength / 360;
	}

	// Shouldn't get here
	check(false);
	return 0;
}
void UMeshSpaceDeformerTool::ApplyModifierGizmoValue(double Value)
{
	switch (Settings->SelectedOperationType)
	{
	case ENonlinearOperationType::Bend:
		Settings->BendDegrees = 180 * Value / ModifierGizmoLength;
		break;
	case ENonlinearOperationType::Flare:
		Settings->FlarePercentY = 100 * Value / ModifierGizmoLength;
		break;
	case ENonlinearOperationType::Twist:
		Settings->TwistDegrees = 360 * Value/ ModifierGizmoLength;
		break;
	default:
		check(false);
	}
}


#undef LOCTEXT_NAMESPACE


