// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexPaintTool.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "InteractiveGizmoManager.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "SceneView.h"
#include "ToolSetupUtil.h"
#include "ToolDataVisualizer.h"
#include "ModelingToolTargetUtil.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicSubmesh3.h"
#include "Util/BufferUtil.h"
#include "Util/ColorConstants.h"
#include "Selection/StoredMeshSelectionUtil.h" // GetCurrentGeometrySelectionForTarget
#include "Selections/GeometrySelectionUtil.h"
#include "Selections/MeshConnectedComponents.h"
#include "Selections/MeshFaceSelection.h"
#include "Selections/MeshVertexSelection.h"
#include "Parameterization/MeshPlanarSymmetry.h"
#include "Polygroups/PolygroupUtil.h"
#include "Polygon2.h"
#include "DynamicMesh/Operations/SplitAttributeWelder.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshPolygroupChange.h"
#include "Changes/BasicChanges.h"

#include "Sculpting/MeshVertexPaintBrushOps.h"
#include "Sculpting/StampFalloffs.h"
#include "Sculpting/MeshSculptUtil.h"
#include "WeightMapUtil.h"
#include "WeightMapTypes.h"

#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Engine/Engine.h"  // for GEngine->GetSmallFont()
#include "SceneView.h"
#include "Materials/Material.h"
#include "Components/MeshComponent.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Optional.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexPaintTool)

#if WITH_EDITOR
static TAutoConsoleVariable<bool> CVarVertexPaintSuppressPostEditChangeCapture(
	TEXT("modeling.VertexPaint.SuppressPostEditChangeCapture"),
	true,
	TEXT("Suppress PostEditChange capture inside the Paint Vertex Colors commit transaction."));
#endif

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshVertexPaintTool"

namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution VertexPaintToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution VertexPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}


/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UMeshVertexPaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMeshVertexPaintTool* SculptTool = NewObject<UMeshVertexPaintTool>(SceneState.ToolManager);
	SculptTool->SetWorld(SceneState.World);
	return SculptTool;
}

void UMeshVertexPaintToolBuilder::InitializeNewTool(UMeshSurfacePointTool* NewTool, const FToolBuilderState& SceneState) const
{
	Super::InitializeNewTool(NewTool, SceneState);
	UMeshVertexPaintTool* Tool = Cast<UMeshVertexPaintTool>(NewTool);
	if (ensure(Tool && Tool->GetTarget()))
	{
		UE::Geometry::FGeometrySelection Selection;
		bool bHaveSelection = UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, Tool->GetTarget(), Selection);
		if (bHaveSelection)
		{
			Tool->SetGeometrySelection(MoveTemp(Selection));
		}
	}
}

bool UMeshVertexPaintToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMeshSurfacePointMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return Cast<UMeshComponent>(&Component) != nullptr; }) > 0;
}

const FToolTargetTypeRequirements& UMeshVertexPaintToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

/*
 * Properties
 */
void UMeshVertexPaintToolActionPropertySet::PostAction(EMeshVertexPaintToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}





/*
 * Tool
 */

void UMeshVertexPaintTool::Setup()
{
	// abort for volumes?
	//if (ActiveColorOverlay == nullptr)
	//{
	//	GetToolManager()->PostActiveToolShutdownRequest(this, EToolShutdownType::Cancel, true,
	//		LOCTEXT("NoColorsShutdown", "Target Mesh does not have or support Vertex Colors"));
	//	return;
	//}


	UMeshSculptToolBase::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Paint Vertex Colors"));

	// create dynamic mesh component to use for live preview
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);
	InitializeSculptMeshComponent(DynamicMeshComponent, PreviewMeshActor);

	// assign materials
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	DynamicMeshComponent->SetInvalidateProxyOnChangeEnabled(false);
	OnDynamicMeshComponentChangedHandle = DynamicMeshComponent->OnMeshChanged.AddUObject(this, &UMeshVertexPaintTool::OnDynamicMeshComponentChanged);

	FDynamicMesh3* Mesh = GetSculptMesh();
	ActiveColorOverlay = (Mesh->Attributes() != nullptr) ? Mesh->Attributes()->PrimaryColors() : nullptr;

	if (ActiveColorOverlay == nullptr)
	{
		if (Mesh->Attributes() == nullptr)
		{
			Mesh->EnableAttributes();
		}
		if (Mesh->Attributes()->PrimaryColors() == nullptr)
		{
			Mesh->Attributes()->EnablePrimaryColors();
		}
		ActiveColorOverlay = Mesh->Attributes()->PrimaryColors();
		if (ActiveColorOverlay == nullptr)
		{
			GetToolManager()->PostActiveToolShutdownRequest(this, EToolShutdownType::Cancel, true,
				LOCTEXT("InvalidColorsMessage", "Target Mesh does not support Vertex Colors"));
			return;
		}
		ActiveColorOverlay->CreateFromPredicate(
			[](int, int, int) { return false; }, 1.0f);
	}
	else
	{
		ActiveColorOverlay->SplitVerticesWithPredicate(
			[](int ElementIdx, int TriID) { return true; },
			[this](int ElementIdx, int TriID, float* FillVect)
			{
				FVector4f CurValue = ActiveColorOverlay->GetElement(ElementIdx);
				FillVect[0] = CurValue.X; FillVect[1] = CurValue.Y; FillVect[2] = CurValue.Z; FillVect[3] = CurValue.W;
			});
	}


	StrokeInitialColorBuffer.SetNum(ActiveColorOverlay->MaxElementID());
	StrokeAccumColorBuffer.SetNum(ActiveColorOverlay->MaxElementID());

	
	//Mesh->EnableVertexColors(FVector3f::One());
	FAxisAlignedBox3d Bounds = Mesh->GetBounds(true);

	FilterProperties = NewObject<UVertexPaintBrushFilterProperties>(this);
	FilterProperties->RestoreProperties(this);
	FilterProperties->WatchProperty(FilterProperties->MaterialMode, [this](EMeshVertexPaintMaterialMode) { UpdateVertexPaintMaterialMode(); });
	FilterProperties->bToolHasSelection = GeometrySelection.IsSet();
	SelectionTids.Reset();

	TFuture<void> PrecomputeFuture = Async(VertexPaintToolAsyncExecTarget, [&]()
	{
		PrecomputeFilterData();
	});

	TFuture<void> OctreeFuture = Async(VertexPaintToolAsyncExecTarget, [&]()
	{
		// initialize dynamic octree
		if (Mesh->TriangleCount() > 100000)
		{
			Octree.RootDimension = Bounds.MaxDim() / 10.0;
			Octree.SetMaxTreeDepth(4);
		}
		else
		{
			Octree.RootDimension = Bounds.MaxDim();
			Octree.SetMaxTreeDepth(8);
		}
		Octree.Initialize(Mesh);
	});

	TFuture<void> AABBTreeFuture = Async(VertexPaintToolAsyncExecTarget, [&]()
	{
		AABBTree.SetMesh(Mesh, true);
	});

	TFuture<void> InitializeSymmetry = Async(VertexPaintToolAsyncExecTarget, [Mesh, this]()
	{
		TryToInitializeSymmetry();
	});

	// initialize render decomposition
	TUniquePtr<FMeshRenderDecomposition> Decomp = MakeUnique<FMeshRenderDecomposition>();
	FMeshRenderDecomposition::BuildChunkedDecomposition(Mesh, &MaterialSet, *Decomp);
	Decomp->BuildAssociations(Mesh);
	DynamicMeshComponent->SetExternalDecomposition(MoveTemp(Decomp));

	// initialize brush radius range interval, brush properties
	UMeshSculptToolBase::InitializeBrushSizeRange(Bounds);

	// Set up control points mechanic
	PolyLassoMechanic = NewObject<UPolyLassoMarqueeMechanic>(this);
	PolyLassoMechanic->Setup(this);
	PolyLassoMechanic->SetIsEnabled(false);
	PolyLassoMechanic->SpacingTolerance = 10.0f;
	PolyLassoMechanic->OnDrawPolyLassoFinished.AddUObject(this, &UMeshVertexPaintTool::OnPolyLassoFinished);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	PolygroupLayerProperties->InitializeGroupLayers(GetSculptMesh());
	PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
	UpdateActiveGroupLayer();

	BasicProperties = NewObject<UVertexPaintBasicProperties>(this);
	BasicProperties->RestoreProperties(this);
	BasicProperties->WatchProperty(BasicProperties->SubToolType,
		[this](EMeshVertexPaintInteractionType NewType) { UpdateSubToolType(NewType); });
	BasicProperties->WatchProperty(BasicProperties->ChannelFilter,
		[this](const FModelingToolsColorChannelFilter&) { OnChannelFilterModified(); });
	BasicProperties->WatchProperty(BasicProperties->SecondaryActionType,
		[this](EMeshVertexPaintSecondaryActionType NewType) { UpdateSecondaryBrushType(NewType); });
	BasicProperties->WatchProperty(BasicProperties->bIsErasePressureEnabled,
		[this](const bool bEnabled) { EraseBrushOpProperties->bIsStrengthPressureEnabled = bEnabled; });
	BasicProperties->WatchProperty(BasicProperties->bIsPaintPressureEnabled,
		[this](const bool bEnabled) { PaintBrushOpProperties->bIsStrengthPressureEnabled = bEnabled; });

	InitializeIndicator();

	AddToolPropertySource(BasicProperties);
	AddToolPropertySource(UMeshSculptToolBase::BrushProperties);

	// initialize our properties
	UMeshSculptToolBase::BrushProperties->bShowPerBrushProps = false;
	UMeshSculptToolBase::BrushProperties->bShowFalloff = true;
	UMeshSculptToolBase::BrushProperties->bShowLazyness = true;
	UMeshSculptToolBase::BrushProperties->bShowFlowRate = true;
	UMeshSculptToolBase::BrushProperties->bShowSpacing = true;
	UMeshSculptToolBase::BrushProperties->BrushFalloffAmount = 0.66f;
	UMeshSculptToolBase::BrushProperties->BrushSize.bToolSupportsPressureSensitivity = true;
	CalculateBrushRadius();

	SymmetryProperties = NewObject<UMeshSymmetryProperties>(this);
	SymmetryProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	SymmetryProperties->bSymmetryCanBeEnabled = false;
	AddToolPropertySource(SymmetryProperties);

	PaintBrushOpProperties = NewObject<UVertexColorPaintBrushOpProps>(this);
	RegisterBrushType((int32)EMeshVertexPaintBrushType::Paint, LOCTEXT("Paint", "Paint"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FVertexPaintBrushOp>(); }),
		PaintBrushOpProperties);

	// secondary brushes
	EraseBrushOpProperties = NewObject<UVertexColorPaintBrushOpProps>(this);
	RegisterSecondaryBrushType((int32)EMeshVertexPaintBrushType::Erase, LOCTEXT("Erase", "Erase"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FVertexPaintBrushOp>(); }),
		EraseBrushOpProperties);

	RegisterSecondaryBrushType((int32)EMeshVertexPaintBrushType::Soften, LOCTEXT("Soften", "Soften"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FVertexColorSoftenBrushOp>>(),
		NewObject<UVertexColorSoftenBrushOpProps>(this));

	RegisterSecondaryBrushType((int32)EMeshVertexPaintBrushType::Smooth, LOCTEXT("Smooth", "Smooth"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FVertexColorSmoothBrushOp>>(),
		NewObject<UVertexColorSmoothBrushOpProps>(this));

	// falloffs
	RegisterStandardFalloffTypes();


	QuickActions = NewObject<UMeshVertexPaintToolQuickActions>(this);
	QuickActions->Initialize(this);

	UtilityActions = NewObject<UMeshVertexPaintToolUtilityActions>(this);
	UtilityActions->Initialize(this);

	AddToolPropertySource(FilterProperties);
	AddToolPropertySource(UMeshSculptToolBase::ViewProperties);
	AddToolPropertySource(QuickActions);
	AddToolPropertySource(UtilityActions);
	AddToolPropertySource(PolygroupLayerProperties);

	AddToolPropertySource(UMeshSculptToolBase::GizmoProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::GizmoProperties, false);


	// register watchers
	BasicProperties->WatchProperty(BasicProperties->PrimaryBrushType,
		[this](EMeshVertexPaintBrushType NewType) { UpdateBrushType(NewType); });

	// must call before updating brush type so that we register all brush properties?
	UMeshSculptToolBase::OnCompleteSetup();

	UpdateBrushType(BasicProperties->PrimaryBrushType);
	UpdateSecondaryBrushType(BasicProperties->SecondaryActionType);
	SetPrimaryFalloffType(EMeshSculptFalloffType::Smooth);

	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(DynamicMeshComponent->GetWorld(), DynamicMeshComponent->GetComponentTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowColorSeams = false;		// default color seams to false
		MeshElementsDisplay->Settings->RestoreProperties(this, GetPropertyCacheIdentifier());
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) 
	{
		const UE::Geometry::FDynamicMesh3* const FullMesh = GetSculptMesh();
		if (ShouldFilterTrianglesBySelection())
		{
			// Arguably there should be a way to add a filter to the visualizer, but we've done it this way elsewhere...
			UE::Geometry::FDynamicSubmesh3 Submesh(FullMesh, SelectionTids.Array());
			ProcessFunc(Submesh.GetSubmesh());
		}
		else
		{
			ProcessFunc(*FullMesh);
		}
	});
	FilterProperties->WatchProperty(FilterProperties->bIsolateGeometrySelection, [this](bool)
	{
		DynamicMeshComponent->SetSecondaryBuffersVisibility(!ShouldFilterTrianglesBySelection());
		MeshElementsDisplay->NotifyMeshChanged();
	});

	// initialize weightmap list in Utility actions
	TArray<FName> TargetWeightMaps;
	UE::WeightMaps::FindVertexWeightMaps(UE::ToolTarget::GetMeshDescription(Target), TargetWeightMaps);
	for (FName Name : TargetWeightMaps)
	{
		UtilityActions->WeightMapsList.Add(Name.ToString());
	}
	if (UtilityActions->WeightMapsList.Num() > 0)
	{
		UtilityActions->WeightMap = FName(UtilityActions->WeightMapsList[0]);
	}

	// initialize LOD names list in Utility actions
	this->SourceLOD = UE::ToolTarget::GetTargetMeshDescriptionLOD(Target, this->bTargetSupportsLODs);
	if (bTargetSupportsLODs)
	{
		this->AvailableLODs = UE::ToolTarget::GetMeshDescriptionLODs(Target, bTargetSupportsLODs, false, true);
		this->AvailableLODs.Remove(SourceLOD);
	}
	if (this->bTargetSupportsLODs && AvailableLODs.Num() > 0)
	{
		for ( int32 k = 0; k < AvailableLODs.Num(); ++k)
		{
			EMeshLODIdentifier LOD = AvailableLODs[k];
			if (LOD != SourceLOD)
			{
				if (LOD == EMeshLODIdentifier::HiResSource)
				{
					UtilityActions->LODNamesList.Add(TEXT("HiRes"));
				}
				else
				{
					UtilityActions->LODNamesList.Add(FString::Printf(TEXT("LOD %d"), static_cast<int32>(LOD)));
				}
			}
			
		}
		UtilityActions->CopyToLODName = UtilityActions->LODNamesList[0];
	}


	// disable view properties
	SetViewPropertiesEnabled(false);
	UpdateWireframeVisibility(false);
	UpdateFlatShadingSetting(true);
	UpdateVertexPaintMaterialMode();

	// configure panels
	UpdateSubToolType(BasicProperties->SubToolType);
	OnChannelFilterModified();

	PrecomputeFuture.Wait();
	OctreeFuture.Wait();
	AABBTreeFuture.Wait();
	InitializeSymmetry.Wait();

	// update symmetry state based on validity, and then update internal apply-symmetry state
	SymmetryProperties->bSymmetryCanBeEnabled = bMeshSymmetryIsValid;
	bApplySymmetry = bMeshSymmetryIsValid && SymmetryProperties->bEnableSymmetry;

	SymmetryProperties->WatchProperty(SymmetryProperties->bEnableSymmetry,
		[this](bool bNewValue) { bApplySymmetry = bMeshSymmetryIsValid && bNewValue; });
}

void UMeshVertexPaintTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);
	}

	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, GetPropertyCacheIdentifier());
	}
	MeshElementsDisplay->Disconnect();

	BasicProperties->SaveProperties(this);
	FilterProperties->SaveProperties(this);
	PolygroupLayerProperties->SaveProperties(this, GetPropertyCacheIdentifier());
	SymmetryProperties->SaveProperties(this, GetPropertyCacheIdentifier());

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		// weld colors to remove seams where possible
		FDynamicMesh3* Mesh = GetSculptMesh();
		FDynamicMeshColorOverlay* ColorOverlay = GetActiveColorOverlay();
		for (int32 vid : Mesh->VertexIndicesItr())
		{
			FSplitAttributeWelder::WeldSplitColors(vid, *ColorOverlay, static_cast<float>(FMathd::ZeroTolerance));
		}
		Mesh->CompactInPlace();		// to clean up colors we just welded
	}

	UMeshSculptToolBase::Shutdown(ShutdownType);
}


void UMeshVertexPaintTool::CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("VertexPaintToolTransactionName", "Paint Vertex Colors"));

#if WITH_EDITOR
	// Keep PostEditChange side effects out of the commit transaction.
	TOptional<UE::ToolTarget::Internal::FScopedSuppressPostEditChangeCapture> SuppressCapture;
	if (CVarVertexPaintSuppressPostEditChangeCapture.GetValueOnGameThread())
	{
		SuppressCapture.Emplace();
	}
#endif

	Component->ProcessMesh([&](const FDynamicMesh3& CurMesh)
	{
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, CurMesh, true);
	});
	GetToolManager()->EndUndoTransaction();
}


void UMeshVertexPaintTool::SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn)
{
	GeometrySelection = SelectionIn;
}

void UMeshVertexPaintTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	UMeshSculptToolBase::RegisterActions(ActionSet);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 500,
		TEXT("PickColorUnderCursor"),
		LOCTEXT("PickColorUnderCursor", "Pick Color"),
		LOCTEXT("PickColorUnderCursorTooltip", "Switch the Paint Color to the color currently under the cursor"),
		EModifierKey::Shift, EKeys::G,
		[this]() { bPendingPickColor = true; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 501,
		TEXT("PickEraseColorUnderCursor"),
		LOCTEXT("PickEraseColorUnderCursor", "Pick Erase Color"),
		LOCTEXT("PickEraseColorUnderCursorTooltip", "Switch the Erase Color to the color currently under the cursor"),
		EModifierKey::Control | EModifierKey::Shift, EKeys::G,
		[this]() { bPendingPickEraseColor = true; });

	//ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 502,
	//	TEXT("ToggleFrozenGroup"),
	//	LOCTEXT("ToggleFrozenGroup", "Toggle Group Frozen State"),
	//	LOCTEXT("ToggleFrozenGroupTooltip", "Toggle Group Frozen State"),
	//	EModifierKey::Shift, EKeys::F,
	//	[this]() { bPendingToggleFreezeGroup = true; });


	//ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 503,
	//	TEXT("CreateNewGroup"),
	//	LOCTEXT("CreateNewGroup", "New Group"),
	//	LOCTEXT("CreateNewGroupTooltip", "Allocate a new Polygroup and set as Current"),
	//	EModifierKey::Shift, EKeys::Q,
	//	[this]() { AllocateNewGroupAndSetAsCurrentAction(); });
};


TUniquePtr<FMeshSculptBrushOp>& UMeshVertexPaintTool::GetActiveBrushOp()
{
	if (GetInEraseStroke())
	{
		return SecondaryBrushOp;
	}
	else
	{
		return PrimaryBrushOp;
	}
}


void UMeshVertexPaintTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	CalculateBrushRadius();
}



bool UMeshVertexPaintTool::IsInBrushSubMode() const
{
	return BasicProperties->SubToolType == EMeshVertexPaintInteractionType::Brush
		|| BasicProperties->SubToolType == EMeshVertexPaintInteractionType::Fill
		|| BasicProperties->SubToolType == EMeshVertexPaintInteractionType::GroupFill
		|| BasicProperties->SubToolType == EMeshVertexPaintInteractionType::TriFill;
}


void UMeshVertexPaintTool::OnBeginStroke(const FRay& WorldRay)
{
	UpdateBrushPosition(WorldRay);

	if (PaintBrushOpProperties)
	{
		PaintBrushOpProperties->Color = BasicProperties->PaintColor;
		PaintBrushOpProperties->BlendMode = (EVertexColorPaintBrushOpBlendMode)(int32)BasicProperties->BlendMode;
		PaintBrushOpProperties->bApplyFalloff = (BasicProperties->SubToolType == EMeshVertexPaintInteractionType::Brush);
	}
	if (EraseBrushOpProperties)
	{
		EraseBrushOpProperties->Color = BasicProperties->EraseColor;
		EraseBrushOpProperties->BlendMode = EVertexColorPaintBrushOpBlendMode::Lerp;
		EraseBrushOpProperties->bApplyFalloff = (BasicProperties->SubToolType == EMeshVertexPaintInteractionType::Brush);
	}

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	if (GetInSmoothingStroke() 
		&& BasicProperties->SecondaryActionType == EMeshVertexPaintSecondaryActionType::Smooth)
	{
		UseBrushOp->PropertySet->SetStrength(0.01 + BasicProperties->SmoothStrength / 2.0);
	}
	else
	{
		UseBrushOp->PropertySet->SetStrength(
			FMathf::Pow(BasicProperties->Strength, 2.0f));
	}
	UseBrushOp->PropertySet->SetFalloff(UMeshSculptToolBase::BrushProperties->BrushFalloffAmount);

	// initialize first "Last Stamp", so that we can assume all stamps in stroke have a valid previous stamp
	LastStamp.WorldFrame = GetBrushFrameWorld();
	LastStamp.LocalFrame = GetBrushFrameLocal();
	LastStamp.Radius = GetCurrentBrushRadius();
	LastStamp.Falloff = GetCurrentBrushFalloff();
	LastStamp.Direction = GetInInvertStroke() ? -1.0 : 1.0;
	LastStamp.Depth = GetCurrentBrushDepth();
	LastStamp.Power = GetActiveBrushStrength();
	LastStamp.TimeStamp = FDateTime::Now();

	// If applying symmetry, make sure the stamp is on the "positive" side. 
	if (bApplySymmetry)
	{
		LastStamp.LocalFrame = Symmetry->GetPositiveSideFrame(LastStamp.LocalFrame);
		LastStamp.WorldFrame = LastStamp.LocalFrame;
		LastStamp.WorldFrame.Transform(CurTargetTransform);
	}

	FSculptBrushOptions SculptOptions;
	//SculptOptions.bPreserveUVFlow = false; // FilterProperties->bPreserveUVFlow;
	SculptOptions.ConstantReferencePlane = GetCurrentStrokeReferencePlane();

	// to mix colors during a stroke like a proper painting tool, we need to accumulate stroke in a separate buffer 
	// and each frame blend it with the initial color buffer.
	// (this should be done async after previous stroke ended...except initial buffer needs to be updated on other operations then)
	if (GetInSmoothingStroke() == false && PaintBrushOpProperties->BlendMode == EVertexColorPaintBrushOpBlendMode::Mix)
	{
		FDynamicMeshColorOverlay* ColorOverlay = GetActiveColorOverlay();
		int32 NumElements = StrokeAccumColorBuffer.Num();
		for (int32 k = 0; k < NumElements; ++k)
		{
			StrokeAccumColorBuffer[k] = FVector4f::Zero();
			StrokeInitialColorBuffer[k] = ColorOverlay->IsElement(k) ? ColorOverlay->GetElement(k) : FVector4f::One();
		}
	}

	UseBrushOp->ConfigureOptions(SculptOptions);
	UseBrushOp->BeginStroke(GetSculptMesh(), LastStamp, VertexROI);

	AccumulatedTriangleROI.Reset();

	// begin change here? or wait for first stamp?
	BeginChange();
}

void UMeshVertexPaintTool::OnEndStroke()
{
	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	// close change record
	EndChange();
}

void UMeshVertexPaintTool::OnCancelStroke()
{
	GetActiveBrushOp()->CancelStroke();
	ActiveChangeBuilder.Reset();
}



void UMeshVertexPaintTool::UpdateROI(const FSculptBrushStamp& BrushStamp)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_UpdateROI);

	const FVector3d& BrushPos = BrushStamp.LocalFrame.Origin;
	const FDynamicMesh3* Mesh = GetSculptMesh();
	float RadiusSqr = GetCurrentBrushRadius() * GetCurrentBrushRadius();
	FAxisAlignedBox3d BrushBox(
		BrushPos - GetCurrentBrushRadius() * FVector3d::One(),
		BrushPos + GetCurrentBrushRadius() * FVector3d::One());

	TriangleROI.Reset();

	// Theoretically we shouldn't need to do this filtering here since we already filter out
	//  unselected triangles in ApplyVisibilityFilter if needed. But seems odd not to.
	bool bShouldFilterTrianglesBySelection = ShouldFilterTrianglesBySelection();

	// With Symmetry or Lazy Brush, GetBrushTriangleID() may not necessarily return a triangle that contains BrushPos,
	// so in that case we will try to find the triangle containing BrushPos. Note that this may not
	// be correct in some geometric cases...potentially we could check if the initial CenterTID 
	// contains the BrushPos point first
	int32 CenterTID = GetBrushTriangleID();
	if (bApplySymmetry || BrushProperties->Lazyness > 0)
	{
		double NearDistSqr;
		IMeshSpatial::FQueryOptions QueryOptions;
		if (bShouldFilterTrianglesBySelection)
		{
			QueryOptions.TriangleFilterF = [this](int32 Tid) { return SelectionTids.Contains(Tid); };
		}
		CenterTID = AABBTree.FindNearestTriangle(BrushPos, NearDistSqr, QueryOptions);
	}

	if (Mesh->IsTriangle(CenterTID))
	{
		TriangleROI.Add(CenterTID);
	}

	FVector3d CenterNormal = Mesh->IsTriangle(CenterTID) ? TriNormals[CenterTID] : FVector3d::One();		// One so that normal check always passes

	bool bVolumetric = (FilterProperties->BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
	bool bUseAngleThreshold = (bVolumetric == false) && (FilterProperties->AngleThreshold < 180.0f);
	double DotAngleThreshold = FMathd::Cos(FilterProperties->AngleThreshold * FMathd::DegToRad);
	bool bStopAtUVSeams = FilterProperties->bUVSeams;
	bool bStopAtNormalSeams = FilterProperties->bNormalSeams;

	auto CheckEdgeCriteria = [&](int32 t1, int32 t2) -> bool
	{
		if (bUseAngleThreshold == false || CenterNormal.Dot(TriNormals[t2]) > DotAngleThreshold)
		{
			int32 eid = Mesh->FindEdgeFromTriPair(t1, t2);
			if (bStopAtUVSeams == false || UVSeamEdges[eid] == false)
			{
				if (bStopAtNormalSeams == false || NormalSeamEdges[eid] == false)
				{
					return true;
				}
			}
		}
		return false;
	};

	if (bVolumetric)
	{
		Octree.RangeQuery(BrushBox,
			[&](int TriIdx) {
			if (bShouldFilterTrianglesBySelection && !SelectionTids.Contains(TriIdx))
			{
				return;
			}
			if ((Mesh->GetTriCentroid(TriIdx) - BrushPos).SquaredLength() < RadiusSqr)
			{
				TriangleROI.Add(TriIdx);
			}
		});
	}
	else
	{
		if (Mesh->IsTriangle(CenterTID))
		{
			TArray<int32> StartROI;
			StartROI.Add(CenterTID);
			FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, TriangleROI, &TempROIBuffer,
				[&](int t1, int t2) 
			{ 
				if (bShouldFilterTrianglesBySelection && (!SelectionTids.Contains(t1) || !SelectionTids.Contains(t2)))
				{
					return false;
				}
				if ((Mesh->GetTriCentroid(t2) - BrushPos).SquaredLength() < RadiusSqr)
				{
					return CheckEdgeCriteria(t1, t2);
				}
				return false;
			});
		}
	}

	auto FillROI = [this, bShouldFilterTrianglesBySelection, CheckEdgeCriteria, Mesh](TSet<int32>& ResultROI)
	{
		TArray<int32> StartROI;
		for (int32 tid : ResultROI)
		{
			StartROI.Add(tid);
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, ResultROI, &TempROIBuffer,
														   [&](int t1, int t2)
		{
			if (bShouldFilterTrianglesBySelection && (!SelectionTids.Contains(t1) || !SelectionTids.Contains(t2)))
			{
				return false;
			}
			return CheckEdgeCriteria(t1, t2);
		});
	};

	auto GroupFillROI = [this, bShouldFilterTrianglesBySelection, CheckEdgeCriteria, Mesh](TSet<int>& ResultROI)
	{
		TArray<int32> StartROI;
		TSet<int32> FillGroups;
		for (int32 tid : ResultROI)
		{
			StartROI.Add(tid);
			FillGroups.Add(ActiveGroupSet->GetGroup(tid));
		}
		FMeshConnectedComponents::GrowToConnectedTriangles(Mesh, StartROI, ResultROI, &TempROIBuffer,
			[&](int t1, int t2)
		{
			if (bShouldFilterTrianglesBySelection && (!SelectionTids.Contains(t1) || !SelectionTids.Contains(t2)))
			{
				return false;
			}
			return (FillGroups.Contains(ActiveGroupSet->GetGroup(t2)));
		});
	};

	auto PostProcessROI = [this, &FillROI, &GroupFillROI](TSet<int>& ResultROI)
	{
		if (BasicProperties->SubToolType == EMeshVertexPaintInteractionType::Fill)
		{
			FillROI(ResultROI);
		}
		else if (BasicProperties->SubToolType == EMeshVertexPaintInteractionType::GroupFill)
		{
			GroupFillROI(ResultROI);
		}

		// apply visibility filter
		if (FilterProperties->VisibilityFilter != EMeshVertexPaintVisibilityType::None)
		{
			TArray<int32> ResultBuffer;
			ApplyVisibilityFilter(ResultROI, TempROIBuffer, ResultBuffer);
		}
	};

	if (bApplySymmetry)
	{
		TempVertexSet.Reset();
		for (int32 tid : TriangleROI)
		{
			FIndex3i Tri = Mesh->GetTriangle(tid);
			TempVertexSet.Add(Tri.A);  TempVertexSet.Add(Tri.B);  TempVertexSet.Add(Tri.C);
		}
		VertexROI.SetNum(0, EAllowShrinking::No);
		BufferUtil::AppendElements(VertexROI, TempVertexSet);
		
		// Find symmetric Vertex ROI. This will overlap with VertexROI in many cases.
		SymmetricVertexROI.Reset();
		Symmetry->GetMirrorVertexROI(VertexROI, SymmetricVertexROI, true);
		SymmetricTriangleROI.Reset();
		// expand the Triangle ROI to include the symmetric vertex one-rings
		for (int32 VertexID : SymmetricVertexROI)
		{
			if (Mesh->IsVertex(VertexID))
			{
				Mesh->EnumerateVertexTriangles(VertexID, [&](int32 tid)
				{
					SymmetricTriangleROI.Add(tid);
				});
			}
		}

		PostProcessROI(SymmetricTriangleROI);

		// construct ROI triangle and group buffers
		SymmetricROITriangleBuffer.Reserve(SymmetricTriangleROI.Num());
		SymmetricROITriangleBuffer.SetNum(0, EAllowShrinking::No);
		for (int32 tid : SymmetricTriangleROI)
		{
			SymmetricROITriangleBuffer.Add(tid);
		}
	}

	PostProcessROI(TriangleROI);

	// construct ROI vertex set
	TempVertexSet.Reset();
	for (int32 tid : TriangleROI)
	{
		FIndex3i Tri = Mesh->GetTriangle(tid);
		TempVertexSet.Add(Tri.A);  TempVertexSet.Add(Tri.B);  TempVertexSet.Add(Tri.C);
	}
	VertexROI.SetNum(0, EAllowShrinking::No);
	BufferUtil::AppendElements(VertexROI, TempVertexSet);

	// construct ROI triangle and group buffers
	ROITriangleBuffer.Reserve(TriangleROI.Num());
	ROITriangleBuffer.SetNum(0, EAllowShrinking::No);
	for (int32 tid : TriangleROI)
	{
		ROITriangleBuffer.Add(tid);
	}
}

bool UMeshVertexPaintTool::UpdateStampPosition(const FRay& WorldRay)
{
	CalculateBrushRadius();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		UpdateBrushPositionOnSculptMesh(WorldRay, true);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		UpdateBrushPositionOnActivePlane(WorldRay);
		break;
	}

	if (UseBrushOp->GetAlignStampToView())
	{
		AlignBrushToView();
	}

	CurrentStamp = LastStamp;
	CurrentStamp.DeltaTime = FMathd::Min((FDateTime::Now() - LastStamp.TimeStamp).GetTotalSeconds(), 1.0);
	CurrentStamp.WorldFrame = GetBrushFrameWorld();
	CurrentStamp.Radius = GetActiveBrushRadius();
	CurrentStamp.LocalFrame = GetBrushFrameLocal();
	CurrentStamp.Power = GetActiveBrushStrength();

	// If applying symmetry, make sure the stamp is on the "positive" side. 
	if (bApplySymmetry)
	{
		CurrentStamp.LocalFrame = Symmetry->GetPositiveSideFrame(CurrentStamp.LocalFrame);
		CurrentStamp.WorldFrame = CurrentStamp.LocalFrame;
		CurrentStamp.WorldFrame.Transform(CurTargetTransform);
	}

	CurrentStamp.PrevLocalFrame = LastStamp.LocalFrame;
	CurrentStamp.PrevWorldFrame = LastStamp.WorldFrame;

	FVector3d MoveDelta = CurrentStamp.LocalFrame.Origin - CurrentStamp.PrevLocalFrame.Origin;
	if (UseBrushOp->IgnoreZeroMovements() && MoveDelta.SquaredLength() < FMathd::ZeroTolerance)
	{
		return false;
	}

	return true;
}


bool UMeshVertexPaintTool::ApplyStamp()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_ApplyStamp);

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	// yuck
	FMeshVertexColorBrushOp* VertexColorBrushOp = (FMeshVertexColorBrushOp*)UseBrushOp.Get();

	// convert triangle ROI to element IDs, populating ROIElementSet, ROIElementBuffer, ROIColorBuffer
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_ApplyStamp_ElementROI);
		InitializeElementROIFromTriangleROI(ROITriangleBuffer, true);
	}
	
	FDynamicMesh3* Mesh = GetSculptMesh();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_ApplyStamp_ApplyStampToVertexColors);
		VertexColorBrushOp->ApplyStampToVertexColors(Mesh, GetActiveColorOverlay(),
			StrokeInitialColorBuffer, StrokeAccumColorBuffer,
			CurrentStamp, ROIElementBuffer, ROIColorBuffer);
	}

	bool bUpdated = SyncMeshWithColorBuffer(Mesh);

	// TODO: Unify the symmetry and main stamp passes.
	if (bApplySymmetry)
	{
		// convert triangle ROI to element IDs, populating ROIElementSet, ROIElementBuffer, ROIColorBuffer
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_ApplyStamp_ElementROI);
			InitializeElementROIFromTriangleROI(SymmetricROITriangleBuffer, true);
		}

		// Temporarily update CurrentStamp to point to the negative side
		FFrame3d PosLocalFrame = CurrentStamp.LocalFrame;
		FFrame3d PosWorldFrame = CurrentStamp.WorldFrame;
		CurrentStamp.LocalFrame.Origin = Symmetry->GetMirroredPosition(CurrentStamp.LocalFrame.Origin);
		CurrentStamp.LocalFrame.Rotation = Symmetry->GetMirroredOrientation(CurrentStamp.LocalFrame.Rotation);
		CurrentStamp.WorldFrame = CurrentStamp.LocalFrame;
		CurrentStamp.WorldFrame.Transform(CurTargetTransform);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_ApplyStamp_ApplyStampToVertexColors);
			VertexColorBrushOp->ApplyStampToVertexColors(Mesh, GetActiveColorOverlay(),
				StrokeInitialColorBuffer, StrokeAccumColorBuffer,
				CurrentStamp, ROIElementBuffer, ROIColorBuffer);
		}

		// Restore CurrentStamp
		CurrentStamp.LocalFrame = PosLocalFrame;
		CurrentStamp.WorldFrame = PosWorldFrame;

		bUpdated = SyncMeshWithColorBuffer(Mesh) || bUpdated;
	}

	LastStamp = CurrentStamp;
	LastStamp.TimeStamp = FDateTime::Now();
	return bUpdated;
}




bool UMeshVertexPaintTool::SyncMeshWithColorBuffer(FDynamicMesh3* Mesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_SyncMeshWithColorBuffer);

	int NumModified = 0;
	const int32 NumElems = ROIElementBuffer.Num();

	if (FDynamicMeshAttributeSet* AttributeSet = Mesh->Attributes())
	{
		if (FDynamicMeshColorOverlay* ColorAttrib = AttributeSet->PrimaryColors())
		{
			for (int32 k = 0; k < NumElems; ++k)
			{
				// check if color has changed? why bother?
				int32 ElementID = ROIElementBuffer[k];
				FVector4f CurColor = ColorAttrib->GetElement(ElementID);
				FVector4f NewColor = ROIColorBuffer[k];
				ApplyChannelFilter(CurColor, NewColor);
				ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
				ColorAttrib->SetElement(ElementID, NewColor);
				NumModified++;
			}

		}
	}

	return (NumModified > 0);
}





void UMeshVertexPaintTool::OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled)
{
	// construct polyline
	TArray<FVector2D> Polyline = Lasso.Polyline;
	int32 N = Polyline.Num();
	if (N < 2)
	{
		return;
	}

	// Try to clip polyline to be closed, or closed-enough for winding evaluation to work.
	// If that returns false, the polyline is "too open". In that case we will extend
	// outwards from the endpoints and then try to create a closed very large polygon
	if (UPolyLassoMarqueeMechanic::ApproximateSelfClipPolyline(Polyline) == false)
	{
		FVector2d StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
		FLine2d StartLine(Polyline[0], StartDirOut);
		FVector2d EndDirOut = UE::Geometry::Normalized(Polyline[N-1] - Polyline[N-2]);
		FLine2d EndLine(Polyline[N-1], EndDirOut);

		// if we did not intersect, we are in ambiguous territory. Check if a segment along either end-direction
		// intersects the polyline. If it does, we have something like a spiral and will be OK. 
		// If not, make a closed polygon by interpolating outwards from each endpoint, and then in perp-directions.
		FPolygon2d Polygon(Polyline);
		float PerpSign = Polygon.IsClockwise() ? -1.0 : 1.0;

		Polyline.Insert(StartLine.PointAt(10000.0f), 0);
		Polyline.Insert(Polyline[0] + 1000 * PerpSign * UE::Geometry::PerpCW(StartDirOut), 0);

		Polyline.Add(EndLine.PointAt(10000.0f));
		Polyline.Add(Polyline.Last() + 1000 * PerpSign * UE::Geometry::PerpCW(EndDirOut));
		FVector2d StartPos = Polyline[0];
		Polyline.Add(StartPos);		// close polyline (cannot use Polyline[0] in case Add resizes!)
	}

	N = Polyline.Num();

	// project each mesh vertex to view plane and evaluate winding integral of polyline
	const FDynamicMesh3* Mesh = GetSculptMesh();
	TempROIBuffer.SetNum(Mesh->MaxVertexID());
	ParallelFor(Mesh->MaxVertexID(), [&](int32 vid)
	{
		if (Mesh->IsVertex(vid))
		{
			FVector3d WorldPos = CurTargetTransform.TransformPosition(Mesh->GetVertex(vid));
			FVector2d PlanePos = (FVector2d)Lasso.GetProjectedPoint((FVector)WorldPos);

			double WindingSum = 0;
			FVector2d a = Polyline[0] - PlanePos, b = FVector2d::Zero();
			for (int32 i = 1; i < N; ++i)
			{
				b = Polyline[i] - PlanePos;
				WindingSum += (double)FMathd::Atan2(a.X*b.Y - a.Y*b.X, a.X*b.X + a.Y*b.Y);
				a = b;
			}
			WindingSum /= FMathd::TwoPi;
			bool bInside = FMathd::Abs(WindingSum) > 0.3;
			TempROIBuffer[vid] = bInside ? 1 : 0;
		}
		else
		{
			TempROIBuffer[vid] = -1;
		}
	});

	// convert to vertex selection, and then select fully-enclosed faces
	FMeshVertexSelection VertexSelection(Mesh);
	VertexSelection.SelectByVertexID([&](int32 vid) { return TempROIBuffer[vid] == 1; });
 	FMeshFaceSelection FaceSelection(Mesh, VertexSelection, FilterProperties->MinTriVertCount);
	if (FaceSelection.Num() == 0)
	{
		return;
	}

	FLinearColor SetColor = GetInEraseStroke() ? BasicProperties->EraseColor : BasicProperties->PaintColor;
	SetTrianglesToVertexColor(FaceSelection.AsSet(), SetColor);
}

// could get rid of this...
void UMeshVertexPaintTool::SetTrianglesToVertexColor(const TSet<int32>& Triangles, const FLinearColor& ToColor)
{
	TempROIBuffer.Reset();
	for (int32 tid : Triangles)
	{
		TempROIBuffer.Add(tid);
	}
	SetTrianglesToVertexColor(TempROIBuffer, ToColor);
}


void UMeshVertexPaintTool::SetTrianglesToVertexColor(const TArray<int32>& Triangles, const FLinearColor& ToColor)
{
	FDynamicMesh3* Mesh = GetSculptMesh();
	FDynamicMeshColorOverlay* ColorOverlay = GetActiveColorOverlay();

	const TArray<int32>* UseTriangles = &Triangles;
	TArray<int32> VisibleTriangles;
	if (HaveVisibilityFilter())
	{
		VisibleTriangles.Reserve(Triangles.Num());
		ApplyVisibilityFilter(Triangles, VisibleTriangles);
		UseTriangles = &VisibleTriangles;
	}

	// convert triangle ROI to element IDs, populating ROIElementSet
	InitializeElementROIFromTriangleROI(*UseTriangles, false);

	if (ROIElementSet.Num() > 0)
	{
		BeginChange();
		for (int32 ElementID : ROIElementSet)
		{
			FVector4f CurColor = ColorOverlay->GetElement(ElementID);
			FVector4f NewColor = (FVector4f)ToColor;
			ApplyChannelFilter(CurColor, NewColor);
			ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
			ColorOverlay->SetElement(ElementID, NewColor);
		}
		DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(*UseTriangles, EMeshRenderAttributeFlags::VertexColors);
		GetToolManager()->PostInvalidation();
		EndChange();
	}
}



void UMeshVertexPaintTool::InitializeElementROIFromTriangleROI(const TArray<int32>& Triangles, bool bInitializeFlatBuffers)
{
	ROIElementBuffer.Reset();
	ROIColorBuffer.Reset();
	ROIElementSet.Reset();

	FDynamicMesh3* Mesh = GetSculptMesh();
	FDynamicMeshColorOverlay* ColorOverlay = GetActiveColorOverlay();

	if (BasicProperties->bHardEdges)
	{
		for (int32 tid : Triangles)
		{
			if (ColorOverlay->IsSetTriangle(tid))
			{
				FIndex3i TriElementIDs = ColorOverlay->GetTriangle(tid);
				ROIElementSet.Add(TriElementIDs.A);
				ROIElementSet.Add(TriElementIDs.B);
				ROIElementSet.Add(TriElementIDs.C);
			}
		}
	}
	else
	{
		// Need to accumulate all Element IDs attached to all vertices of triangles.
		// Perhaps there is a faster way to do this?
		TArray<int> TempElementIDs;
		TempVertexSet.Reset();
		for (int32 tid : Triangles)
		{
			FIndex3i TriVertIDs = Mesh->GetTriangle(tid);
			for (int32 j = 0; j < 3; ++j)
			{
				int32 vid = TriVertIDs[j];
				if (TempVertexSet.Contains(vid) == false)
				{
					TempVertexSet.Add(vid);
					TempElementIDs.Reset();
					ColorOverlay->GetVertexElements(vid, TempElementIDs);
					for (int32 ElementID : TempElementIDs)
					{
						ROIElementSet.Add(ElementID);
					}
				}
			}
		}
	}

	if (bInitializeFlatBuffers)
	{
		ROIElementBuffer = ROIElementSet.Array();
		ROIColorBuffer.SetNum(ROIElementBuffer.Num());
		for (int32 k = 0; k < ROIElementBuffer.Num(); ++k)
		{
			ROIColorBuffer[k] = ColorOverlay->GetElement(ROIElementBuffer[k]);
		}
	}
}


bool UMeshVertexPaintTool::HaveVisibilityFilter() const
{
	return ShouldFilterTrianglesBySelection()
		|| FilterProperties->VisibilityFilter != EMeshVertexPaintVisibilityType::None;
}


void UMeshVertexPaintTool::ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer)
{
	ROIBuffer.SetNum(0, EAllowShrinking::No);
	ROIBuffer.Reserve(Triangles.Num());
	for (int32 tid : Triangles)
	{
		ROIBuffer.Add(tid);
	}
	
	OutputBuffer.Reset();
	ApplyVisibilityFilter(TempROIBuffer, OutputBuffer);

	Triangles.Reset();
	for (int32 tid : OutputBuffer)
	{
		Triangles.Add(tid);
	}
}

void UMeshVertexPaintTool::ApplyVisibilityFilter(const TArray<int32>& Triangles, TArray<int32>& VisibleTriangles)
{
	if (!HaveVisibilityFilter())
	{
		VisibleTriangles = Triangles;
		return;
	}

	FViewCameraState StateOut;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
	FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition(StateOut.Position));

	const FDynamicMesh3* Mesh = GetSculptMesh();

	int32 NumTriangles = Triangles.Num();
	bool bShouldFilterUnselected = ShouldFilterTrianglesBySelection();

	VisibilityFilterBuffer.SetNum(NumTriangles, EAllowShrinking::No);
	ParallelFor(NumTriangles, [&](int32 idx)
	{
		VisibilityFilterBuffer[idx] = true;
		if (bShouldFilterUnselected && !SelectionTids.Contains(idx))
		{
			VisibilityFilterBuffer[idx] = false;
			return;
		}
		FVector3d Centroid = Mesh->GetTriCentroid(Triangles[idx]);
		FVector3d FaceNormal = Mesh->GetTriNormal(Triangles[idx]);
		if (FaceNormal.Dot((Centroid - LocalEyePosition)) > 0)
		{
			VisibilityFilterBuffer[idx] = false;
			return;
		}
		if (FilterProperties->VisibilityFilter == EMeshVertexPaintVisibilityType::Unoccluded)
		{
			int32 HitTID = Octree.FindNearestHitObject(FRay3d(LocalEyePosition, UE::Geometry::Normalized(Centroid - LocalEyePosition)));
			if (HitTID != Triangles[idx])
			{
				VisibilityFilterBuffer[idx] = false;
			}
		}
	});

	VisibleTriangles.Reset();
	for (int32 k = 0; k < NumTriangles; ++k)
	{
		if (VisibilityFilterBuffer[k])
		{
			VisibleTriangles.Add(Triangles[k]);
		}
	}
}



int32 UMeshVertexPaintTool::FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const
{
	if (!IsInBrushSubMode())
	{
		return IndexConstants::InvalidID;
	}

	bool bShouldFilterUnselected = ShouldFilterTrianglesBySelection();
	if (GetBrushCanHitBackFaces())
	{
		if (bShouldFilterUnselected)
		{
			return Octree.FindNearestHitObject(LocalRay, [this](int32 Tid) { return SelectionTids.Contains(Tid); });
		}
		return Octree.FindNearestHitObject(LocalRay);
	}
	else
	{
		const FDynamicMesh3* Mesh = GetSculptMesh();

		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));
		int HitTID = Octree.FindNearestHitObject(LocalRay,
			[this, Mesh, &LocalEyePosition, bShouldFilterUnselected](int TriangleID) {
			if (bShouldFilterUnselected && !SelectionTids.Contains(TriangleID))
			{
				return false;
			}
			FVector3d Normal, Centroid;
			double Area;
			Mesh->GetTriInfo(TriangleID, Normal, Area, Centroid);
			return Normal.Dot((Centroid - LocalEyePosition)) < 0;
		});
		return HitTID;
	}
}

int32 UMeshVertexPaintTool::FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const
{
	check(false);
	return IndexConstants::InvalidID;
}



bool UMeshVertexPaintTool::UpdateBrushPosition(const FRay& WorldRay)
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bool bHit = false; 
	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
	case ESculptBrushOpTargetType::TargetMesh:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		check(false);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	if (bHit && UseBrushOp->GetAlignStampToView())
	{
		AlignBrushToView();
	}

	return bHit;
}




bool UMeshVertexPaintTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	PendingStampType = BasicProperties->PrimaryBrushType;

	if(ensure(InStroke() == false))
	{
		UpdateBrushPosition(DevicePos.WorldRay);

		// update strength and falloff for current brush
		TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
		UseBrushOp->PropertySet->SetStrength(
			FMathf::Pow(BasicProperties->Strength, 2.0f) );
		UseBrushOp->PropertySet->SetFalloff(UMeshSculptToolBase::BrushProperties->BrushFalloffAmount);

	}
	return true;
}

void UMeshVertexPaintTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (PolyLassoMechanic)
	{
		// because the actual group change is deferred until mouse release, color the lasso to let the user know whether it will erase
		PolyLassoMechanic->LineColor = GetInEraseStroke() ? FLinearColor::Red : FLinearColor::Green;
		PolyLassoMechanic->DrawHUD(Canvas, RenderAPI);
	}


	float DPIScale = Canvas->GetDPIScale();
	UFont* UseFont = GEngine->GetSmallFont();
	FViewCameraState CamState = RenderAPI->GetCameraState();
	const FSceneView* SceneView = RenderAPI->GetSceneView();
	FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)CamState.Position));

	FDynamicMesh3* Mesh = GetSculptMesh();

	if (FilterProperties->bShowHitColor)
	{
		FRay3d LocalRay(LocalEyePosition, Normalized(HoverStamp.LocalFrame.Origin - LocalEyePosition));
		int CursorHitTID = ShouldFilterTrianglesBySelection() ? 
			Octree.FindNearestHitObject(LocalRay, [this](int32 Tid) {return SelectionTids.Contains(Tid); })
			: Octree.FindNearestHitObject(LocalRay);
		if (Mesh->IsTriangle(CursorHitTID))
		{
			FIntrRay3Triangle3d Intersection = TMeshQueries<FDynamicMesh3>::RayTriangleIntersection(*GetSculptMesh(), CursorHitTID, LocalRay);
			FVector3f BaryCoords = (FVector3f)Intersection.TriangleBaryCoords;
			FVector4f InterpColor;
			GetActiveColorOverlay()->GetTriBaryInterpolate<float>(CursorHitTID, &BaryCoords.X, &InterpColor.X);
			FVector2D CursorPixelPos;
			SceneView->WorldToPixel(HoverStamp.WorldFrame.Origin, CursorPixelPos);
			FString CursorString = FString::Printf(TEXT("%.3f %.3f %.3f %.3f"), InterpColor.X, InterpColor.Y, InterpColor.Z, InterpColor.W);
			Canvas->DrawShadowedString(CursorPixelPos.X / (double)DPIScale, CursorPixelPos.Y / (double)DPIScale, *CursorString, UseFont, FLinearColor::White);
		}
	}

}

void UMeshVertexPaintTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSculptToolBase::Render(RenderAPI);

	// draw a dot for the symmetric brush stamp position
	if (bApplySymmetry)
	{
		FToolDataVisualizer Visualizer;
		Visualizer.BeginFrame(RenderAPI);
		FVector3d MirrorPoint = CurTargetTransform.TransformPosition(
			Symmetry->GetMirroredPosition(HoverStamp.LocalFrame.Origin));
		Visualizer.DrawPoint(MirrorPoint, FLinearColor(1.0f, 0.1f, 0.1f, 1), 5.0f, false);
		Visualizer.EndFrame();
	}
}

void UMeshVertexPaintTool::OnTick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_OnTick);

	UMeshSculptToolBase::OnTick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);

	bool bIsLasso = (BasicProperties->SubToolType == EMeshVertexPaintInteractionType::PolyLasso);
	PolyLassoMechanic->SetIsEnabled(bIsLasso);

	ConfigureIndicator(FilterProperties->BrushAreaMode == EMeshVertexPaintBrushAreaType::Volumetric);
	SetIndicatorVisibility(bIsLasso == false);

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EMeshVertexPaintToolActions::NoAction;
	}

	// process the undo update
	if (bUndoUpdatePending)
	{
		// wait for updates
		WaitForPendingUndoRedoUpdate();

		// post rendering update
		DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(AccumulatedTriangleROI, EMeshRenderAttributeFlags::VertexColors);
		GetToolManager()->PostInvalidation();

		// ignore stamp and wait for next tick to do anything else
		bUndoUpdatePending = false;
		return;
	}

	if (bPendingPickColor || bPendingPickEraseColor )
	{
		int32 HitTriangleID = GetBrushTriangleID();
		if (HitTriangleID >= 0 && IsStampPending() == false )
		{
			if (GetSculptMesh()->IsTriangle(HitTriangleID))
			{
				if (bPendingPickColor || bPendingPickEraseColor)
				{
					if (const FDynamicMeshAttributeSet* AttributeSet = GetSculptMesh()->Attributes())
					{
						if (const FDynamicMeshColorOverlay* ColorAttrib = AttributeSet->PrimaryColors())
						{
							if (ColorAttrib->IsSetTriangle(HitTriangleID))
							{
								FVector4f HitColor;
								// TODO: use actual hit point here...
								FVector3f BaryCoords = (1.0f / 3.0f) * FVector3f::One();
								ColorAttrib->GetTriBaryInterpolate<float>(HitTriangleID, &BaryCoords.X, &HitColor.X);
								if (bPendingPickEraseColor)
								{
									BasicProperties->EraseColor = HitColor;
								}
								else
								{
									BasicProperties->PaintColor = HitColor;
								}
								NotifyOfPropertyChangeByTool(BasicProperties);
							}
						}
					}
				}
			}
		}
		bPendingPickColor = bPendingPickEraseColor = false;
	}

	auto ExecuteStampOperation = [this](int StampIndex, const FRay& StampRay)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_OnTick_InStroke);

			// update sculpt ROI
			UpdateROI(CurrentStamp);

			// append updated ROI to modified region (async)
			TFuture<void> AccumulateROI = Async(VertexPaintToolAsyncExecTarget, [&]()
				{
					AccumulatedTriangleROI.Append(TriangleROI);
				});

			// apply the stamp
			bool bColorModified = ApplyStamp();

			if (bColorModified)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshVPaint_OnTick_UpdateComponent);
				DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(TriangleROI, EMeshRenderAttributeFlags::VertexColors);
				GetToolManager()->PostInvalidation();
			}

			// we don't really need to wait for these to happen to end Tick()...
			AccumulateROI.Wait();
		};

	if (IsInBrushSubMode())
	{
		ProcessPerTickStamps(
			[this](const FRay& StampRay) -> bool {
				return UpdateStampPosition(StampRay);
			}, ExecuteStampOperation);
	}

}


void UMeshVertexPaintTool::ApplyChannelFilter(const FVector4f& CurColor, FVector4f& NewColor)
{
	const FModelingToolsColorChannelFilter& ChannelFilter = BasicProperties->ChannelFilter;
	if (ChannelFilter.bRed == false)
	{
		NewColor.X = CurColor.X;
	}
	if (ChannelFilter.bGreen == false)
	{
		NewColor.Y = CurColor.Y;
	}
	if (ChannelFilter.bBlue == false)
	{
		NewColor.Z = CurColor.Z;
	}
	if (ChannelFilter.bAlpha == false)
	{
		NewColor.W = CurColor.W;
	}
}

void UMeshVertexPaintTool::OnChannelFilterModified()
{
	FModelingToolsColorChannelFilter NewFilter = BasicProperties->ChannelFilter;

	// if we are showing original material then we do not want to filter the colors by default (could be an option)
	if (FilterProperties->MaterialMode == EMeshVertexPaintMaterialMode::OriginalMaterial)
	{
		NewFilter = FModelingToolsColorChannelFilter();
	}

	if (NewFilter.bRed && NewFilter.bGreen && NewFilter.bBlue)
	{
		DynamicMeshComponent->ClearVertexColorRemappingFunction();
	}
	else if (NewFilter.bRed || NewFilter.bBlue || NewFilter.bGreen)
	{
		DynamicMeshComponent->SetVertexColorRemappingFunction([Filter = NewFilter](FVector4f& Color)
		{
			Color.X = (Filter.bRed) ? Color.X : 0;
			Color.Y = (Filter.bGreen) ? Color.Y : 0;
			Color.Z = (Filter.bBlue) ? Color.Z : 0;
		});
	}
	else if (NewFilter.bAlpha)
	{
		DynamicMeshComponent->SetVertexColorRemappingFunction([Filter = NewFilter](FVector4f& Color)
		{
			Color.X = Color.W;
			Color.Y = Color.W;
			Color.Z = Color.W;
		});
	}
	else
	{
		DynamicMeshComponent->ClearVertexColorRemappingFunction();
	}
}


FColor UMeshVertexPaintTool::GetColorForGroup(int32 GroupID)
{
	return LinearColors::SelectFColor(GroupID);
}



void UMeshVertexPaintTool::UpdateVertexPaintMaterialMode()
{
	if (FilterProperties->MaterialMode == EMeshVertexPaintMaterialMode::OriginalMaterial)
	{
		UpdateMaterialMode(EMeshEditingMaterialModes::ExistingMaterial);
	}
	else if (FilterProperties->MaterialMode == EMeshVertexPaintMaterialMode::LitVertexColor)
	{
		// TODO: add two-sided vertex color support when it becomes available
		UpdateMaterialMode(EMeshEditingMaterialModes::VertexColor);
	}
	else if (FilterProperties->MaterialMode == EMeshVertexPaintMaterialMode::UnlitVertexColor
		&& GEngine && GEngine->VertexColorViewModeMaterial_ColorOnly)
	{
		UpdateMaterialMode(EMeshEditingMaterialModes::ExistingMaterial);
		GetSculptMeshComponent()->SetOverrideRenderMaterial(GEngine->VertexColorViewModeMaterial_ColorOnly);
	}

	OnChannelFilterModified();		// channel visibility filtering may be disabled for existing material
}



void UMeshVertexPaintTool::FloodFillColorAction(FLinearColor Color)
{
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	TempROIBuffer.Reset();
	TempROIBuffer.Reserve(Mesh->TriangleCount());
	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		TempROIBuffer.Add(tid);
	}

	SetTrianglesToVertexColor(TempROIBuffer, Color);
}


void UMeshVertexPaintTool::ApplyCurrentUtilityAction()
{
	switch (UtilityActions->Operation)
	{
	case EMeshVertexPaintToolUtilityOperations::BlendAllSeams:
		BlendAllSeams();
		return;
	case EMeshVertexPaintToolUtilityOperations::FillChannels:
		FillChannels();
		return;
	case EMeshVertexPaintToolUtilityOperations::InvertChannels:
		InvertChannels();
		return;
	case EMeshVertexPaintToolUtilityOperations::CopyChannelToChannel:
		CopyChannelToChannel();
		return;
	case EMeshVertexPaintToolUtilityOperations::SwapChannels:
		SwapChannels();
		return;
	case EMeshVertexPaintToolUtilityOperations::CopyFromWeightMap:
		CopyFromWeightMap();
		return;
	case EMeshVertexPaintToolUtilityOperations::CopyToOtherLODs:
		CopyToOtherLODs();
		return;
	case EMeshVertexPaintToolUtilityOperations::CopyToSingleLOD:
		CopyToSpecificLOD();
		return;
	}
}



void UMeshVertexPaintTool::BlendAllSeams()
{
	BeginChange();

	TArray<int32> TempElementIDs;
	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FDynamicMeshColorOverlay* VertexColors = GetActiveColorOverlay();
	for (int32 VertexID : Mesh->VertexIndicesItr())
	{
		TempElementIDs.Reset();
		VertexColors->GetVertexElements(VertexID, TempElementIDs);
		if (TempElementIDs.Num() > 1)
		{
			FVector4f SumColor = FVector4f::Zero();
			for (int32 ElementID : TempElementIDs)
			{
				SumColor += VertexColors->GetElement(ElementID);
			}
			SumColor /= (double)TempElementIDs.Num();
			for (int32 ElementID : TempElementIDs)
			{
				ActiveChangeBuilder->UpdateValue(ElementID, VertexColors->GetElement(ElementID), SumColor);
				VertexColors->SetElement(ElementID, SumColor);
			}
		}
	}

	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();

	EndChange();
}

void UMeshVertexPaintTool::FillChannels()
{
	float FillValue = UtilityActions->SourceValue;
	FIndex4i Targets = UtilityActions->TargetChannels.AsFlags();
	if (Targets == FIndex4i::Zero())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoChannelsSelectedMessage", "At least one Target Channel must be selected"), EToolMessageLevel::UserError);
		return;
	}

	BeginChange();
	FDynamicMeshColorOverlay* VertexColors = GetActiveColorOverlay();
	for (int32 ElementID : VertexColors->ElementIndicesItr())
	{
		FVector4f CurColor = VertexColors->GetElement(ElementID);
		FVector4f NewColor(
			Targets[0] ? FillValue : CurColor.X, Targets[1] ? FillValue : CurColor.Y,
			Targets[2] ? FillValue : CurColor.Z, Targets[3] ? FillValue : CurColor.W);
		ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
		VertexColors->SetElement(ElementID, NewColor);
	}

	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}

void UMeshVertexPaintTool::InvertChannels()
{
	FIndex4i Targets = UtilityActions->TargetChannels.AsFlags();
	if (Targets == FIndex4i::Zero())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoChannelsSelectedMessage", "At least one Target Channel must be selected"), EToolMessageLevel::UserError);
		return;
	}

	BeginChange();
	FDynamicMeshColorOverlay* VertexColors = GetActiveColorOverlay();
	for (int32 ElementID : VertexColors->ElementIndicesItr())
	{
		FVector4f CurColor = VertexColors->GetElement(ElementID);
		FVector4f NewColor(
			Targets[0] ? FMathf::Clamp(1.0f-CurColor.X,0,1.0f) : CurColor.X, 
			Targets[1] ? FMathf::Clamp(1.0f-CurColor.Y,0,1.0f) : CurColor.Y,
			Targets[2] ? FMathf::Clamp(1.0f-CurColor.Z,0,1.0f) : CurColor.Z,
			Targets[3] ? FMathf::Clamp(1.0f-CurColor.W,0,1.0f) : CurColor.W);
		ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
		VertexColors->SetElement(ElementID, NewColor);
	}

	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


void UMeshVertexPaintTool::CopyChannelToChannel()
{
	int32 SourceChannelIndex = VectorUtil::Clamp(static_cast<int32>(UtilityActions->SourceChannel), 0, 3);
	FIndex4i Targets = UtilityActions->TargetChannels.AsFlags();
	Targets[SourceChannelIndex] = 0;
	if (Targets == FIndex4i::Zero())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoChannelsSelectedMessage", "At least one Target Channel must be selected"), EToolMessageLevel::UserError);
		return;
	}

	BeginChange();

	FDynamicMeshColorOverlay* VertexColors = GetActiveColorOverlay();
	for (int32 ElementID : VertexColors->ElementIndicesItr())
	{
		FVector4f CurColor = VertexColors->GetElement(ElementID);
		FVector4f NewColor = CurColor;
		float SourceValue = CurColor[SourceChannelIndex];
		for (int32 j = 0; j < 4; ++j)
		{
			if (Targets[j] != 0)
			{
				NewColor[j] = SourceValue;
			}
		}
		ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
		VertexColors->SetElement(ElementID, NewColor);
	}

	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();

	EndChange();
}

void UMeshVertexPaintTool::SwapChannels()
{
	int32 SourceChannelIndex = VectorUtil::Clamp(static_cast<int32>(UtilityActions->SourceChannel), 0, 3);
	int32 DestChannelIndex = VectorUtil::Clamp(static_cast<int32>(UtilityActions->TargetChannel), 0, 3);
	if (SourceChannelIndex == DestChannelIndex)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("SameSourceDestMessage", "Source and Destination must be different Channels"), EToolMessageLevel::UserError);
		return;
	}

	BeginChange();

	FDynamicMeshColorOverlay* VertexColors = GetActiveColorOverlay();
	for (int32 ElementID : VertexColors->ElementIndicesItr())
	{
		FVector4f CurColor = VertexColors->GetElement(ElementID);
		FVector4f NewColor = CurColor;
		Swap(NewColor[SourceChannelIndex], NewColor[DestChannelIndex]);
		ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
		VertexColors->SetElement(ElementID, NewColor);
	}

	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();

	EndChange();
}



void UMeshVertexPaintTool::CopyFromWeightMap()
{
	FIndex4i Targets = UtilityActions->TargetChannels.AsFlags();
	if (Targets == FIndex4i::Zero())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoChannelsSelectedMessage", "At least one Target Channel must be selected"), EToolMessageLevel::UserError);
		return;
	}

	FIndexedWeightMap1f WeightMap;
	bool bFound = UE::WeightMaps::GetVertexWeightMap(UE::ToolTarget::GetMeshDescription(Target),
		UtilityActions->WeightMap, WeightMap, 1.0f);

	if (!bFound)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoWeightMapMessage", "Selected Weight Map could not be found on the Target Mesh"), EToolMessageLevel::UserError);
		return;
	}

	// todo: range mapping

	BeginChange();
	FDynamicMeshColorOverlay* VertexColors = GetActiveColorOverlay();
	for (int32 ElementID : VertexColors->ElementIndicesItr())
	{
		int32 ParentVertex = VertexColors->GetParentVertex(ElementID);
		float Value = WeightMap.GetValue(ParentVertex);
		Value = FMathf::Clamp(Value, 0.0f, 1.0f);

		FVector4f CurColor = VertexColors->GetElement(ElementID);
		FVector4f NewColor(
			Targets[0] ? Value : CurColor.X, Targets[1] ? Value : CurColor.Y,
			Targets[2] ? Value : CurColor.Z, Targets[3] ? Value : CurColor.W);
		ActiveChangeBuilder->UpdateValue(ElementID, CurColor, NewColor);
		VertexColors->SetElement(ElementID, NewColor);
	}

	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
	EndChange();
}


// TODO should be moved to shared location
namespace UELocal
{

static void CopyVertexColorsToLOD(
	const FDynamicMesh3& FromMesh, 
	const FDynamicMeshColorOverlay& FromColors,
	const FDynamicMeshAABBTree3& Spatial,
	FDynamicMesh3& ToMesh,
	const FVector4f& FallbackColor)
{
	if (ToMesh.HasAttributes() == false)
	{
		ToMesh.EnableAttributes();
	}
	if (ToMesh.Attributes()->HasPrimaryColors() == false)
	{
		ToMesh.Attributes()->EnablePrimaryColors();
	}
	FDynamicMeshColorOverlay& ToColors = *ToMesh.Attributes()->PrimaryColors();
	ToColors.ClearElements();

	// maybe should use baking here...?
	// could be parallel...

	TArray<FVector4f> VertexColors;
	VertexColors.SetNum(ToMesh.MaxVertexID());
	for (int32 vid : ToMesh.VertexIndicesItr())
	{
		FVector3d ToPos = ToMesh.GetVertex(vid);
		double NearDistSqr;
		int32 NearestTri = Spatial.FindNearestTriangle(ToPos, NearDistSqr);
		if (FromColors.IsSetTriangle(NearestTri) == false)
		{
			VertexColors[vid] = FallbackColor;
			break;
		}

		FDistPoint3Triangle3d Dist = TMeshQueries<FDynamicMesh3>::TriangleDistance(FromMesh, NearestTri, ToPos);
		FVector3f BaryCoords = (FVector3f)Dist.TriangleBaryCoords;
		FVector4f InterpColor;
		FromColors.GetTriBaryInterpolate<float>(NearestTri, &BaryCoords.X, &InterpColor.X);
		VertexColors[vid] = InterpColor;
	}

	TArray<int32> VertToElemMap;
	VertToElemMap.SetNum(VertexColors.Num());
	for (int32 vid : ToMesh.VertexIndicesItr())
	{
		VertToElemMap[vid] = ToColors.AppendElement(VertexColors[vid]);
	}
	for (int32 tid : ToMesh.TriangleIndicesItr())
	{
		FIndex3i TriVerts = ToMesh.GetTriangle(tid);
		ToColors.SetTriangle(tid, FIndex3i(
			VertToElemMap[TriVerts.A], VertToElemMap[TriVerts.B], VertToElemMap[TriVerts.C]));
	}
}

static void GetMeshDescriptionLODWithCopiedVertexColors(
	UToolTarget* Target, EMeshLODIdentifier CopyToLOD,
	FDynamicMesh3& SourceMesh, FDynamicMeshColorOverlay& ColorOverlay, FDynamicMeshAABBTree3& SourceMeshSpatial,
	FVector4f DefaultColor,
	FMeshDescription& UpdatedMeshDescriptionOut)
{
	UpdatedMeshDescriptionOut = UE::ToolTarget::GetMeshDescriptionCopy(Target, FGetMeshParameters(true, CopyToLOD));

	FDynamicMesh3 LODMesh;
	FMeshDescriptionToDynamicMesh Converter1;
	Converter1.bTransformVertexColorsLinearToSRGB = true;
	Converter1.Convert(&UpdatedMeshDescriptionOut, LODMesh);

	UELocal::CopyVertexColorsToLOD(SourceMesh, ColorOverlay, SourceMeshSpatial, LODMesh, DefaultColor);

	FDynamicMeshToMeshDescription Converter2;
	Converter2.ConversionOptions.bTransformVtxColorsSRGBToLinear = true;
	Converter2.UpdateVertexColors(&LODMesh, UpdatedMeshDescriptionOut);
}


}



void UMeshVertexPaintTool::CopyToOtherLODs()
{
	// have to do this via MeshDescriptions for now
	if (bTargetSupportsLODs == false)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoLODsErrorMessage", "Target Mesh does not support LODs"), EToolMessageLevel::UserError);
		return;
	}
	TArray<EMeshLODIdentifier> CopyToLODs = AvailableLODs;
	if (UtilityActions->bCopyToHiRes == false)
	{
		CopyToLODs.Remove(EMeshLODIdentifier::HiResSource);
	}
	if (CopyToLODs.Num() == 0)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoLODsToCopyToMessage", "Target Mesh does not have any other editable LODs"), EToolMessageLevel::UserError);
		return;
	}

	FDynamicMesh3* SourceMesh = GetSculptMesh();
	FDynamicMeshColorOverlay* ColorOverlay = GetActiveColorOverlay();
	FDynamicMeshAABBTree3 SourceMeshSpatial(SourceMesh, true);

	TArray<FMeshDescription> LODMeshDescriptions;
	LODMeshDescriptions.SetNum(CopyToLODs.Num());
	for (int32 k = 0; k < CopyToLODs.Num(); ++k)
	{
		EMeshLODIdentifier LOD = CopyToLODs[k];
		UELocal::GetMeshDescriptionLODWithCopiedVertexColors(Target, LOD,
			*SourceMesh, *ColorOverlay, SourceMeshSpatial, (FVector4f)BasicProperties->PaintColor, LODMeshDescriptions[k]);
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("CopyToOtherLODs", "Copy Vertex Colors To LODs"));
	for (int32 k = 0; k < CopyToLODs.Num(); ++k)
	{
		EMeshLODIdentifier LOD = CopyToLODs[k];
		UE::ToolTarget::CommitMeshDescriptionUpdate(Target, &LODMeshDescriptions[k], nullptr /*no material set changes*/, FCommitMeshParameters(true, LOD));
	}
	GetToolManager()->EndUndoTransaction();
}


void UMeshVertexPaintTool::CopyToSpecificLOD()
{
	// have to do this via MeshDescriptions for now
	if (bTargetSupportsLODs == false)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("NoLODsErrorMessage", "Target Mesh does not support LODs"), EToolMessageLevel::UserError);
		return;
	}

	int32 Index = UtilityActions->LODNamesList.IndexOfByKey(UtilityActions->CopyToLODName);
	if (Index == INDEX_NONE)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("MissingLODMessage", "Could not find selected LOD on Target Mesh"), EToolMessageLevel::UserError);
		return;
	}

	FDynamicMesh3* SourceMesh = GetSculptMesh();
	FDynamicMeshColorOverlay* ColorOverlay = GetActiveColorOverlay();
	FDynamicMeshAABBTree3 SourceMeshSpatial(SourceMesh, true);

	FMeshDescription LODMeshDescription;
	EMeshLODIdentifier LOD = AvailableLODs[Index];
	UELocal::GetMeshDescriptionLODWithCopiedVertexColors(Target, LOD,
		*SourceMesh, *ColorOverlay, SourceMeshSpatial, (FVector4f)BasicProperties->PaintColor, LODMeshDescription);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("CopyToSpecificLOD", "Copy Vertex Colors To LOD"));
	UE::ToolTarget::CommitMeshDescriptionUpdate(Target, &LODMeshDescription, nullptr /*no material set changes*/, FCommitMeshParameters(true, LOD));
	GetToolManager()->EndUndoTransaction();
}


//
// Change Tracking
//

void UMeshVertexPaintTool::BeginChange()
{
	check(ActiveChangeBuilder == nullptr);

	ActiveChangeBuilder = MakeUnique<TIndexedValuesChangeBuilder<FVector4f, FMeshVertexColorPaintChange>>();
	ActiveChangeBuilder->BeginNewChange();
	LongTransactions.Open(LOCTEXT("VertexPaintChange", "Paint Stroke"), GetToolManager());
}


void UMeshVertexPaintTool::EndChange()
{
	check(ActiveChangeBuilder);

	TUniquePtr<FMeshVertexColorPaintChange> EditResult = ActiveChangeBuilder->ExtractResult();
	ActiveChangeBuilder = nullptr;
	
	EditResult->ApplyFunction = [](UObject* Object, const int32& AttribIndex, const TArray<int32>& Indices, const TArray<FVector4f>& Values)
	{
		UMeshVertexPaintTool* Tool = CastChecked<UMeshVertexPaintTool>(Object);
		Tool->ExternalUpdateValues(Indices, Values);
	};
	EditResult->RevertFunction = [](UObject* Object, const int32& AttribIndex, const TArray<int32>& Indices, const TArray<FVector4f>& Values)
	{
		UMeshVertexPaintTool* Tool = CastChecked<UMeshVertexPaintTool>(Object);
		Tool->ExternalUpdateValues(Indices, Values);
	};

	TUniquePtr<TWrappedToolCommandChange<FMeshVertexColorPaintChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshVertexColorPaintChange>>();
	NewChange->WrappedChange = MoveTemp(EditResult);
	NewChange->BeforeModify = [this](bool bRevert)
	{
		this->WaitForPendingUndoRedoUpdate();
	};

	GetToolManager()->EmitObjectChange(this, MoveTemp(NewChange), LOCTEXT("VertexPaintChange", "Paint Stroke"));
	LongTransactions.Close(GetToolManager());
}



void UMeshVertexPaintTool::ExternalUpdateValues(const TArray<int32>& ElementIDs, const TArray<FVector4f>& NewValues)
{
	DynamicMeshComponent->EditMesh([&](FDynamicMesh3& Mesh)
	{
		FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();

		int32 N = ElementIDs.Num();
		for ( int32 k = 0; k < N; ++k)
		{
			int32 ElementID = ElementIDs[k];
			ColorOverlay->SetElement(ElementID, NewValues[k]);
		}
	});

	GetToolManager()->PostInvalidation();
}



void UMeshVertexPaintTool::WaitForPendingUndoRedoUpdate()
{
	if (bUndoUpdatePending)
	{
		bUndoUpdatePending = false;
	}
}

void UMeshVertexPaintTool::OnDynamicMeshComponentChanged()
{
	// update octree
	FDynamicMesh3* Mesh = GetSculptMesh();

	// make sure any previous async computations are done, and update the undo ROI
	if (bUndoUpdatePending)
	{
		// we should never hit this anymore, because of pre-change calling WaitForPendingUndoRedoUpdate()
		WaitForPendingUndoRedoUpdate();

		// TODO: do we need to read from mesh change here??
		//UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}
	else
	{
		// TODO: do we need to read from mesh change here??
		//UE::Geometry::VertexToTriangleOneRing(Mesh, Change->Vertices, AccumulatedTriangleROI);
	}

	// note that we have a pending update
	bUndoUpdatePending = true;
}


void UMeshVertexPaintTool::PrecomputeFilterData()
{
	const FDynamicMesh3* Mesh = GetSculptMesh();
	
	TriNormals.SetNum(Mesh->MaxTriangleID());
	ParallelFor(Mesh->MaxTriangleID(), [&](int32 tid)
	{
		if (Mesh->IsTriangle(tid))
		{
			TriNormals[tid] = Mesh->GetTriNormal(tid);
		}
	});

	const FDynamicMeshNormalOverlay* Normals = Mesh->Attributes()->PrimaryNormals();
	const FDynamicMeshUVOverlay* UVs = Mesh->Attributes()->PrimaryUV();
	UVSeamEdges.SetNum(Mesh->MaxEdgeID());
	NormalSeamEdges.SetNum(Mesh->MaxEdgeID());
	ParallelFor(Mesh->MaxEdgeID(), [&](int32 eid)
	{
		if (Mesh->IsEdge(eid))
		{
			UVSeamEdges[eid] = UVs ? UVs->IsSeamEdge(eid) : false;
			NormalSeamEdges[eid] = Normals ? Normals->IsSeamEdge(eid) : false;
		}
	});

	if (GeometrySelection.IsSet())
	{
		UE::Geometry::EnumerateSelectionTriangles(GeometrySelection.GetValue(), *Mesh,
			[this, &Mesh](int32 TriangleID)
			{
				SelectionTids.Add(TriangleID);
			});

		DynamicMeshComponent->EnableSecondaryTriangleBuffers([this](const UE::Geometry::FDynamicMesh3*, int32 Tid)
		{
			return !SelectionTids.Contains(Tid);
		});
		DynamicMeshComponent->SetSecondaryBuffersVisibility(!ShouldFilterTrianglesBySelection());
	}
}

bool UMeshVertexPaintTool::ShouldFilterTrianglesBySelection() const
{
	return FilterProperties->bIsolateGeometrySelection && !SelectionTids.IsEmpty();
}


void UMeshVertexPaintTool::OnSelectedGroupLayerChanged()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("ChangeActiveGroupLayer", "Change Polygroup Layer"));

	int32 ActiveLayerIndex = (ActiveGroupSet) ? ActiveGroupSet->GetPolygroupIndex() : -1;
	UpdateActiveGroupLayer();
	int32 NewLayerIndex = (ActiveGroupSet) ? ActiveGroupSet->GetPolygroupIndex() : -1;

	if (ActiveLayerIndex != NewLayerIndex)
	{
		TUniquePtr<TSimpleValueLambdaChange<int32>> GroupLayerChange = MakeUnique<TSimpleValueLambdaChange<int32>>();
		GroupLayerChange->FromValue = ActiveLayerIndex;
		GroupLayerChange->ToValue = NewLayerIndex;
		GroupLayerChange->ValueChangeFunc = [this](UObject*, int32 FromIndex, int32 ToIndex, bool)
		{
			this->PolygroupLayerProperties->SetSelectedFromPolygroupIndex(ToIndex);
			this->PolygroupLayerProperties->SilentUpdateWatched();		// to prevent OnSelectedGroupLayerChanged() from being called immediately
			this->UpdateActiveGroupLayer();
		};
		GetToolManager()->EmitObjectChange(this, MoveTemp(GroupLayerChange), LOCTEXT("ChangeActiveGroupLayer", "Change Polygroup Layer"));
	}

	GetToolManager()->EndUndoTransaction();
}


void UMeshVertexPaintTool::UpdateActiveGroupLayer()
{
	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		ActiveGroupSet = MakeUnique<UE::Geometry::FPolygroupSet>(GetSculptMesh());
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*GetSculptMesh(), SelectedName);
		ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
		ActiveGroupSet = MakeUnique<UE::Geometry::FPolygroupSet>(GetSculptMesh(), FoundAttrib);
	}

	// update colors
	DynamicMeshComponent->FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexColors);
	GetToolManager()->PostInvalidation();
}



void UMeshVertexPaintTool::UpdateSubToolType(EMeshVertexPaintInteractionType NewType)
{
	FilterProperties->CurrentSubToolType = BasicProperties->SubToolType;

	bool bSculptPropsVisible = (NewType != EMeshVertexPaintInteractionType::PolyLasso);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::BrushProperties, bSculptPropsVisible);

	//SetToolPropertySourceEnabled(BasicProperties, true);
	SetBrushOpPropsVisibility(false);
}


void UMeshVertexPaintTool::UpdateBrushType(EMeshVertexPaintBrushType BrushType)
{
	static const FText BaseMessage = LOCTEXT("OnStartTool", "Hold Shift to Erase. [/] and S/D change Size (+Shift to small-step). Shift+G to pick Paint Color, +Ctrl for Erase Color.");
	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	SetActivePrimaryBrushType((int32)BrushType);

	SetToolPropertySourceEnabled(GizmoProperties, false);

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}

void UMeshVertexPaintTool::UpdateSecondaryBrushType(EMeshVertexPaintSecondaryActionType NewType)
{
	if (NewType == EMeshVertexPaintSecondaryActionType::Erase)
	{
		SetActiveSecondaryBrushType((int32)EMeshVertexPaintBrushType::Erase);
	}
	else if (NewType == EMeshVertexPaintSecondaryActionType::Soften)
	{
		SetActiveSecondaryBrushType((int32)EMeshVertexPaintBrushType::Soften);
	}
	else if (NewType == EMeshVertexPaintSecondaryActionType::Smooth)
	{
		SetActiveSecondaryBrushType((int32)EMeshVertexPaintBrushType::Smooth);
	}
}


void UMeshVertexPaintTool::RequestAction(EMeshVertexPaintToolActions ActionType)
{
	if (!bHavePendingAction)
	{
		PendingAction = ActionType;
		bHavePendingAction = true;
	}
}


void UMeshVertexPaintTool::ApplyAction(EMeshVertexPaintToolActions ActionType)
{
	switch (ActionType)
	{

	case EMeshVertexPaintToolActions::PaintAll:
		FloodFillColorAction(BasicProperties->PaintColor);
		break;

	case EMeshVertexPaintToolActions::EraseAll:
		FloodFillColorAction(BasicProperties->EraseColor);
		break;

	case EMeshVertexPaintToolActions::FillBlack:
		FloodFillColorAction(FLinearColor::Black);
		break;

	case EMeshVertexPaintToolActions::FillWhite:
		FloodFillColorAction(FLinearColor::White);
		break;

	case EMeshVertexPaintToolActions::ApplyCurrentUtility:
		ApplyCurrentUtilityAction();
		break;
	}
}

FString UMeshVertexPaintTool::GetPropertyCacheIdentifier() const
{
	return TEXT("MeshVertexPaintTool");
}


void UMeshVertexPaintTool::TryToInitializeSymmetry()
{
	// Attempt to find symmetry, favoring the X axis, then Y axis, if a single symmetry plane is not immediately found
	// Uses local mesh surface (angle sum, normal) to help disambiguate final matches, but does not require exact topology matches across the plane

	FAxisAlignedBox3d Bounds = GetSculptMesh()->GetBounds(true);

	TArray<FVector3d> PreferAxes;
	PreferAxes.Add(this->InitialTargetTransform.GetRotation().AxisX());
	PreferAxes.Add(this->InitialTargetTransform.GetRotation().AxisY());

	FMeshPlanarSymmetry FindSymmetry;
	FFrame3d FoundPlane;
	if (FindSymmetry.FindPlaneAndInitialize(GetSculptMesh(), Bounds, FoundPlane, PreferAxes))
	{
		Symmetry = MakePimpl<FMeshPlanarSymmetry>();
		*Symmetry = MoveTemp(FindSymmetry);
		bMeshSymmetryIsValid = true;
	}
}



#undef LOCTEXT_NAMESPACE

