// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshVertexSculptTool.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "Intersection/ContainmentQueries3.h"
#include "Intersection/IntrCylinderBox3.h"
#include "ToolDataVisualizer.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"
#include "Selections/MeshConnectedComponents.h"
#include "Algo/Unique.h"

#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "Parameterization/MeshPlanarSymmetry.h"
#include "Util/BufferUtil.h"
#include "Util/UniqueIndexSet.h"
#include "AssetUtils/Texture2DUtil.h"
#include "ToolSetupUtil.h"
#include "Drawing/PreviewGeometryActor.h"

#include "BaseGizmos/BrushStampIndicator.h"
#include "PreviewMesh.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "Generators/RectangleMeshGenerator.h"

#include "Properties/MeshSculptLayerProperties.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshRegionChange.h"

#include "MeshSculptLayersManagerAPI.h"
#include "Sculpting/KelvinletBrushOp.h"
#include "Sculpting/MeshSmoothingBrushOps.h"
#include "Sculpting/MeshInflateBrushOps.h"
#include "Sculpting/MeshMoveBrushOps.h"
#include "Sculpting/MeshPlaneBrushOps.h"
#include "Sculpting/MeshPinchBrushOps.h"
#include "Sculpting/MeshSculptBrushOps.h"
#include "Sculpting/MeshEraseSculptLayerBrushOps.h"
#include "Sculpting/StampFalloffs.h"
#include "Sculpting/MeshSculptUtil.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/MaterialProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexSculptTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshVertexSculptTool"

namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution VertexSculptToolAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution VertexSculptToolAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}

namespace MeshVertexSculptToolLocals
{
	const FString& OctreePointSetID(TEXT("OctreePointSet"));
	const FString& OctreeLineSetID(TEXT("OctreeLineSet"));

	static FAutoConsoleVariable EnableOctreeVisuals(TEXT("modeling.Sculpting.EnableOctreeVisuals"), false, TEXT("Enable visualizing the octree used for determining ROIs."));
	static FAutoConsoleVariable DisableOctreeUpdates(TEXT("modeling.Sculpting.DisableOctreeUpdates"), false, TEXT("Disable updating the octree during sculpting"));
	static FAutoConsoleVariable OctreeRootCellSizeOverride(TEXT("modeling.Sculpting.OctreeRootCellSizeOverride"), 0.0f, TEXT("If greater than 0.0, set octree root cell size to this value instead of auto-computing."));
	static FAutoConsoleVariable OctreeTreeDepthOverride(TEXT("modeling.Sculpting.OctreeTreeDepthOverride"), 0, TEXT("If greater than 0, set octree tree depth to this value instead of auto-computing."));

	static TAutoConsoleVariable<bool> CVarSwapSmoothInvertModifiers(
		TEXT("modeling.Sculpting.SwapSmoothInvertModifiers"),
		false,
		TEXT("When true, swaps the Shift/Ctrl modifier key roles in the vertex sculpt tool"));
}

/*
 * ToolBuilder
 */
UMeshSurfacePointTool* UMeshVertexSculptToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UMeshVertexSculptTool* SculptTool = NewObject<UMeshVertexSculptTool>(SceneState.ToolManager);
	SculptTool->SetWorld(SceneState.World);
	SculptTool->SetDefaultPrimaryBrushID(DefaultPrimaryBrushID);
	return SculptTool;
}

FToolTargetTypeRequirements UMeshVertexSculptToolBuilder::VSculptTypeRequirements({
	UMaterialProvider::StaticClass(),
	UDynamicMeshProvider::StaticClass(),
	UDynamicMeshCommitter::StaticClass(),
	USceneComponentBackedTarget::StaticClass()
});

const FToolTargetTypeRequirements& UMeshVertexSculptToolBuilder::GetTargetRequirements() const
{
	return VSculptTypeRequirements;
}

/*
 * internal Change classes
 */

class FVertexSculptNonSymmetricChange : public FToolCommandChange
{
public:
	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;
};


/*
 * Tool
 */

void UMeshVertexSculptTool::SaveActiveStrokeModifiers()
{
	Super::SaveActiveStrokeModifiers();
	if (MeshVertexSculptToolLocals::CVarSwapSmoothInvertModifiers.GetValueOnGameThread())
	{
		Swap(bSmoothing, bInvert);
	}
}

void UMeshVertexSculptTool::Setup()
{
	UMeshSculptToolBase::Setup();

	SetToolDisplayName(LOCTEXT("ToolName", "Sculpt"));

	// create dynamic mesh component to use for live preview
	check(TargetWorld);
	FActorSpawnParameters SpawnInfo;
	PreviewMeshActor = TargetWorld->SpawnActor<AInternalToolFrameworkActor>(FVector::ZeroVector, FRotator::ZeroRotator, SpawnInfo);
	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(PreviewMeshActor);

	InitializeSculptMeshComponent(DynamicMeshComponent, PreviewMeshActor);

	// assign materials
	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);
	for (int k = 0; k < MaterialSet.Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, MaterialSet.Materials[k]);
	}

	DynamicMeshComponent->SetInvalidateProxyOnChangeEnabled(false);
	OnDynamicMeshComponentChangedHandle = DynamicMeshComponent->OnMeshRegionChanged.AddUObject(this, &UMeshVertexSculptTool::OnDynamicMeshComponentChanged);

	FDynamicMesh3* SculptMesh = GetSculptMesh();
	FAxisAlignedBox3d Bounds = SculptMesh->GetBounds(true);
	InitialBoundsMaxDim = Bounds.MaxDim();

	// initialize dynamic octree
	float RootCellSizeOverride = FMath::Abs(MeshVertexSculptToolLocals::OctreeRootCellSizeOverride->GetFloat());	
	int TreeDepthOverride = FMath::Abs(MeshVertexSculptToolLocals::OctreeTreeDepthOverride->GetInt());
	TFuture<void> InitializeOctree = Async(VertexSculptToolAsyncExecTarget, [SculptMesh, Bounds, RootCellSizeOverride, TreeDepthOverride, this]()
	{
		double RootDim;
		int32 Depth;
		if (SculptMesh->TriangleCount() > 100000)
		{
			RootDim = InitialBoundsMaxDim / 10.0;
			Depth = 4;
		}
		else
		{
			RootDim = InitialBoundsMaxDim / 2.0;
			Depth = 8;
		}

		if (!FMath::IsNearlyZero(RootCellSizeOverride))
		{
			RootDim = RootCellSizeOverride;
		}
		if (TreeDepthOverride > 0)
		{
			Depth = TreeDepthOverride;
		}

		// Vertex octree for region-of-interest range queries
		VertexOctree.RootDimension = RootDim;
		VertexOctree.MaxTreeDepth = Depth;
		VertexOctree.Initialize(SculptMesh, false);

		// Triangle octree for hit testing
		Octree.RootDimension = RootDim;
		Octree.SetMaxTreeDepth(Depth);
		Octree.Initialize(SculptMesh);
	});

	// find mesh connected-component index for each triangle
	TFuture<void> InitializeComponents = Async(VertexSculptToolAsyncExecTarget, [SculptMesh, this]()
	{
		TriangleComponentIDs.SetNum(SculptMesh->MaxTriangleID());
		FMeshConnectedComponents Components(SculptMesh);
		Components.FindConnectedTriangles();
		int32 ComponentIdx = 1;
		for (const FMeshConnectedComponents::FComponent& Component : Components)
		{
			for (int32 TriIdx : Component.Indices)
			{
				TriangleComponentIDs[TriIdx] = ComponentIdx;
			}
			ComponentIdx++;
		}
	});

	TFuture<void> InitializeSymmetry = Async(VertexSculptToolAsyncExecTarget, [SculptMesh, this]()
	{
		TryToInitializeSymmetry();
	});

	// currently only supporting default polygroup set
	TFuture<void> InitializeGroups = Async(VertexSculptToolAsyncExecTarget, [SculptMesh, this]()
	{
		ActiveGroupSet = MakeUnique<UE::Geometry::FPolygroupSet>(SculptMesh);
	});

	// initialize target mesh
	TFuture<void> InitializeBaseMesh = Async(VertexSculptToolAsyncExecTarget, [this]()
	{
		UpdateBaseMesh(nullptr);
		bTargetDirty = false;
	});

	// Cache mesh connectivity once up front. This assumes vertex sculpting never changes topology
	// (only vertex positions), so the connectivity remains valid for the lifetime of the tool
	TFuture<void> InitializeCachedConnectivity = Async(VertexSculptToolAsyncExecTarget, [SculptMesh, this]()
	{
		CachedVtxTriConnectivity.SetNum(SculptMesh->MaxVertexID());
		CachedVtxVtxConnectivity.SetNum(SculptMesh->MaxVertexID());
		for (int32 VID : SculptMesh->VertexIndicesItr())
		{
			SculptMesh->EnumerateVertexTriangles(VID, [VID, this](int32 TID) { CachedVtxTriConnectivity[VID].Add(TID); });
			SculptMesh->EnumerateVertexVertices(VID, [VID, this](int32 OtherVID) { CachedVtxVtxConnectivity[VID].Add(OtherVID); });
		}
	});

	// initialize render decomposition
	TFuture<void> InitializeRenderDecomp = Async(VertexSculptToolAsyncExecTarget, [SculptMesh, &MaterialSet, this]()
	{
		if (SculptMesh->TriangleCount() == 0)
		{
			return;
		}
		TUniquePtr<FMeshRenderDecomposition> Decomp = MakeUnique<FMeshRenderDecomposition>();
		FMeshRenderDecomposition::BuildChunkedDecomposition(SculptMesh, &MaterialSet, *Decomp);
		Decomp->BuildAssociations(SculptMesh);
		//UE_LOGF(LogTemp, Warning, "Decomposition has %d groups", Decomp->Num());
		DynamicMeshComponent->SetExternalDecomposition(MoveTemp(Decomp));
	});

	// Wait for above precomputations to finish before continuing
	InitializeOctree.Wait();
	InitializeComponents.Wait();
	InitializeGroups.Wait();
	InitializeBaseMesh.Wait();
	InitializeRenderDecomp.Wait();
	InitializeSymmetry.Wait();
	InitializeCachedConnectivity.Wait();

	// initialize brush radius range interval, brush properties
	UMeshSculptToolBase::InitializeBrushSizeRange(Bounds);

	// initialize other properties
	SculptProperties = NewObject<UVertexBrushSculptProperties>(this);
	SculptProperties->Tool = this;
	
	// init state flags flags
	ActiveVertexChange = nullptr;

	InitializeIndicator();

	// initialize our properties
	AddToolPropertySource(UMeshSculptToolBase::BrushProperties);
	UMeshSculptToolBase::BrushProperties->bShowPerBrushProps = false;
	UMeshSculptToolBase::BrushProperties->bShowFalloff = false;
	UMeshSculptToolBase::BrushProperties->BrushSize.bToolSupportsPressureSensitivity = true;
	SculptProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	
	AddToolPropertySource(SculptProperties);
	CalculateBrushRadius();

	AlphaProperties = NewObject<UVertexBrushAlphaProperties>(this);
	AlphaProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	AlphaProperties->Tool = this;
	AddToolPropertySource(AlphaProperties);

	SymmetryProperties = NewObject<UMeshSymmetryProperties>(this);
	SymmetryProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	SymmetryProperties->bSymmetryCanBeEnabled = false;
	AddToolPropertySource(SymmetryProperties);

	if (ISceneComponentBackedTarget* SceneComponentTarget = Cast<ISceneComponentBackedTarget>(Target))
	{
		if (IMeshSculptLayersManager* SculptLayersManager = Cast<IMeshSculptLayersManager>(SceneComponentTarget->GetOwnerSceneComponent()))
		{
			if (SculptLayersManager->HasSculptLayers())
			{
				SculptLayerProperties = NewObject<UMeshSculptLayerProperties>(this);
				SculptLayerProperties->Init(this, SculptLayersManager->NumLockedBaseSculptLayers());
				AddToolPropertySource(SculptLayerProperties);
			}
		}
	}



	this->BaseMeshQueryFunc = [&](int32 VertexID, const FVector3d& Position, double MaxDist, FVector3d& PosOut, FVector3d& NormalOut)
	{
		return GetBaseMeshNearest(VertexID, Position, MaxDist, PosOut, NormalOut);
	};

	RegisterBrushes();

	if (DefaultPrimaryBrushID >= 0 && ensure(BrushOpFactories.Contains(DefaultPrimaryBrushID)))
	{
		SculptProperties->PrimaryBrushID = DefaultPrimaryBrushID;
	}

	// falloffs
	RegisterStandardFalloffTypes();

	AddToolPropertySource(UMeshSculptToolBase::GizmoProperties);
	SetToolPropertySourceEnabled(UMeshSculptToolBase::GizmoProperties, false);
	// Move the gizmo toward the center of the mesh, without changing the plane it represents
	UMeshSculptToolBase::GizmoProperties->RecenterGizmoIfFar(GetSculptMeshComponent()->GetComponentTransform().TransformPosition(Bounds.Center()), Bounds.MaxDim());

	AddToolPropertySource(UMeshSculptToolBase::ViewProperties);

	// register watchers
	SculptProperties->WatchProperty( SculptProperties->PrimaryBrushID,
		[this](int32 NewType) { UpdateBrushType(NewType); });

	SculptProperties->WatchProperty( SculptProperties->PrimaryFalloffType,
		[this](EMeshSculptFalloffType NewType) { 
			SetPrimaryFalloffType(NewType);
			// Request to have the details panel rebuilt to ensure the new falloff property value is propagated to the details customization
			OnDetailsPanelRequestRebuild.Broadcast();
		});

	SculptProperties->WatchProperty(AlphaProperties->Alpha,
		[this](UTexture2D* NewAlpha) {
			UpdateBrushAlpha(AlphaProperties->Alpha);
			// Request to have the details panel rebuilt to ensure the new alpha property value is propagated to the details customization
			OnDetailsPanelRequestRebuild.Broadcast();
		});

	// must call before updating brush type so that we register all brush properties?
	UMeshSculptToolBase::OnCompleteSetup();

	UpdateBrushType(SculptProperties->PrimaryBrushID);
	SetPrimaryFalloffType(SculptProperties->PrimaryFalloffType);
	UpdateBrushAlpha(AlphaProperties->Alpha);
	SetActiveSecondaryBrushType(0);

	StampRandomStream = FRandomStream(31337);

	// update symmetry state based on validity, and then update internal apply-symmetry state
	SymmetryProperties->bSymmetryCanBeEnabled = bMeshSymmetryIsValid;
	bApplySymmetry = bMeshSymmetryIsValid && SymmetryProperties->bEnableSymmetry;

	SymmetryProperties->WatchProperty(SymmetryProperties->bEnableSymmetry,
		[this](bool bNewValue) { bApplySymmetry = bMeshSymmetryIsValid && bNewValue; });
	SymmetryProperties->WatchProperty(SymmetryProperties->bSymmetryCanBeEnabled,
		[this](bool bNewValue) 
		{
			bApplySymmetry = bMeshSymmetryIsValid && bNewValue && SymmetryProperties->bEnableSymmetry; 
		});
}

void UMeshVertexSculptTool::RegisterBrushes()
{
	RegisterBrushType((int32)EMeshVertexSculptBrushType::Smooth, LOCTEXT("SmoothBrush", "Smooth"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FSmoothBrushOp>>(),
		NewObject<USmoothBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::SmoothFill, LOCTEXT("SmoothFill", "SmoothFill"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FSmoothFillBrushOp>>(),
		NewObject<USmoothFillBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Move, LOCTEXT("Move", "Move"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FMoveBrushOp>>(),
		NewObject<UMoveBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Offset, LOCTEXT("Offset", "SculptN"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<UE::Geometry::FSingleNormalSculptBrushOp>>(),
		NewObject<UStandardSculptBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::SculptView, LOCTEXT("SculptView", "SculptV"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FViewAlignedSculptBrushOp>(BaseMeshQueryFunc); }),
		NewObject<UViewAlignedSculptBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::SculptMax, LOCTEXT("SculptMax", "SculptMx"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FSingleNormalMaxSculptBrushOp>(BaseMeshQueryFunc); }),
		NewObject<USculptMaxBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Inflate, LOCTEXT("Inflate", "Inflate"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FInflateBrushOp>>(),
		NewObject<UInflateBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::InflateStroke, LOCTEXT("InflateStroke", "InflateSt"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FSurfaceSculptBrushOp>(BaseMeshQueryFunc); }),
		NewObject<USculptMaxBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::InflateMax, LOCTEXT("InflateMax", "InflateMax"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<FSurfaceMaxSculptBrushOp>(BaseMeshQueryFunc); }),
		NewObject<USculptMaxBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Pinch, LOCTEXT("Pinch", "Pinch"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FPinchBrushOp>>(),
		NewObject<UPinchBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Flatten, LOCTEXT("Flatten", "Flatten"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FFlattenBrushOp>>(),
		NewObject<UFlattenBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::Plane, LOCTEXT("Plane", "PlaneN"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FPlaneBrushOp>>(),
		NewObject<UPlaneBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::PlaneViewAligned, LOCTEXT("PlaneViewAligned", "PlaneV"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FPlaneBrushOp>>(),
		NewObject<UViewAlignedPlaneBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::FixedPlane, LOCTEXT("FixedPlane", "PlaneW"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FPlaneBrushOp>>(),
		NewObject<UFixedPlaneBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::ScaleKelvin, LOCTEXT("ScaleKelvin", "Scale"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FScaleKelvinletBrushOp>>(),
		NewObject<UScaleKelvinletBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::PullKelvin, LOCTEXT("PullKelvin", "Grab"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FPullKelvinletBrushOp>>(),
		NewObject<UPullKelvinletBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::PullSharpKelvin, LOCTEXT("PullSharpKelvin", "GrabSharp"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FSharpPullKelvinletBrushOp>>(),
		NewObject<USharpPullKelvinletBrushOpProps>(this));

	RegisterBrushType((int32)EMeshVertexSculptBrushType::TwistKelvin, LOCTEXT("TwistKelvin", "Twist"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FTwistKelvinletBrushOp>>(),
		NewObject<UTwistKelvinletBrushOpProps>(this));

	if (DoesTargetHaveSculptLayers())
	{
		RegisterBrushType((int32)EMeshVertexSculptBrushType::EraseSculptLayer, LOCTEXT("EraseSculptLayer", "EraseSculptLayer"),
			MakeUnique<TBasicMeshSculptBrushOpFactory<FEraseSculptLayerBrushOp>>(),
			NewObject<UEraseSculptLayerBrushOpProps>(this));
	}

	// secondary brushes
	// We activate ID 0 as our default secondary brush, so use that as the registration ID
	RegisterSecondaryBrushType(0, LOCTEXT("Smooth", "Smooth"),
		MakeUnique<TBasicMeshSculptBrushOpFactory<FSmoothBrushOp>>(),
		NewObject<USecondarySmoothBrushOpProps>(this));
}

FString UMeshVertexSculptTool::GetPropertyCacheIdentifier() const
{
	return TEXT("UMeshVertexSculptTool");
}

void UMeshVertexSculptTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (OctreeGeometry)
	{
		OctreeGeometry->Disconnect();
		OctreeGeometry = nullptr;
	}

	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->OnMeshChanged.Remove(OnDynamicMeshComponentChangedHandle);
	}

	SculptProperties->SaveProperties(this, GetPropertyCacheIdentifier());
	AlphaProperties->SaveProperties(this, GetPropertyCacheIdentifier());
	SymmetryProperties->SaveProperties(this, GetPropertyCacheIdentifier());

	if (PreviewMeshActor != nullptr)
	{
		PreviewMeshActor->Destroy();
		PreviewMeshActor = nullptr;
	}

	// this call will commit result, unregister and destroy DynamicMeshComponent
	UMeshSculptToolBase::Shutdown(ShutdownType);
}


void UMeshVertexSculptTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	CalculateBrushRadius();
}

void UMeshVertexSculptTool::UpdateToolMeshes(TFunctionRef<TUniquePtr<FMeshRegionChangeBase>(FDynamicMesh3&, int32 MeshIdx)> UpdateMesh)
{
	if (AllowToolMeshUpdates())
	{
		// have to wait for any outstanding stamp/undo update to finish...
		WaitForPendingStampUpdateConst();
		WaitForPendingUndoRedoUpdate();

		TUniquePtr<FMeshRegionChangeBase> Change = UpdateMesh(*GetSculptMesh(), 0);
		// A change was created -- emit it to the tool manager and update associated data structures, etc
		if (Change)
		{
			// pass through the change to trigger standard mesh updates / octree recomputation
			constexpr bool bUpdateRender = true;
			OnDynamicMeshComponentChanged_Internal(DynamicMeshComponent, Change.Get(), false, bUpdateRender);

			TUniquePtr<TWrappedToolCommandChange<FMeshRegionChangeBase>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshRegionChangeBase>>();
			NewChange->WrappedChange = MoveTemp(Change);
			NewChange->BeforeModify = [this](bool bRevert)
			{
				this->WaitForPendingUndoRedoUpdate();
			};
			// refresh sculpt layer UI w/ any external mesh changes
			NewChange->AfterModify = [this](bool bRevert)
			{
				this->RefreshSculptLayerProperties();
			};
			// Note this change should be in the context of a larger transaction, so the text isn't that important
			GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(NewChange), LOCTEXT("UpdateVertexSculptMesh", "Updated Mesh"));
			
			if (bMeshSymmetryIsValid)
			{
				// Re-validate that the symmetry still holds after the external mesh change
				if (!Symmetry->ValidateSymmetry(*GetSculptMesh()))
				{
					GetToolManager()->EmitObjectChange(this, MakeUnique<FVertexSculptNonSymmetricChange>(), LOCTEXT("InvalidateSymmetryChange", "Invalidate Symmetry"));
					bMeshSymmetryIsValid = false;
					SymmetryProperties->bSymmetryCanBeEnabled = bMeshSymmetryIsValid;
				}
			}
		}
		// No change is ready to emit, just update component rendering
		else
		{
			DynamicMeshComponent->FastNotifyPositionsUpdated();
		}
	}
}

UPreviewMesh* UMeshVertexSculptTool::MakeBrushIndicatorMesh(UObject* Parent, UWorld* World)
{
	UPreviewMesh* PlaneMesh = NewObject<UPreviewMesh>(Parent);
	PlaneMesh->CreateInWorld(World, FTransform::Identity);

	FRectangleMeshGenerator RectGen;
	RectGen.Width = RectGen.Height = 2.0;
	RectGen.WidthVertexCount = RectGen.HeightVertexCount = 1;
	FDynamicMesh3 Mesh(&RectGen.Generate());
	FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->PrimaryUV();
	// configure UVs to be in same space as texture pixels when mapped into brush frame (??)
	for (int32 eid : UVOverlay->ElementIndicesItr())
	{
		FVector2f UV = UVOverlay->GetElement(eid);
		UV.X = 1.0 - UV.X;
		UV.Y = 1.0 - UV.Y;
		UVOverlay->SetElement(eid, UV);
	}
	PlaneMesh->UpdatePreview(&Mesh);

	BrushIndicatorMaterial = ToolSetupUtil::GetDefaultBrushAlphaMaterial(GetToolManager());
	if (BrushIndicatorMaterial)
	{
		PlaneMesh->SetMaterial(BrushIndicatorMaterial);
	}

	// make sure raytracing is disabled on the brush indicator
	Cast<UDynamicMeshComponent>(PlaneMesh->GetRootComponent())->SetEnableRaytracing(false);
	PlaneMesh->SetShadowsEnabled(false);

	return PlaneMesh;
}

void UMeshVertexSculptTool::InitializeIndicator()
{
	UMeshSculptToolBase::InitializeIndicator();
	// want to draw radius
	BrushIndicator->bDrawRadiusCircle = true;
}

void UMeshVertexSculptTool::SetActiveBrushType(int32 Identifier)
{
	if (SculptProperties->PrimaryBrushID != Identifier)
	{
		SculptProperties->PrimaryBrushID = Identifier;
		UpdateBrushType(SculptProperties->PrimaryBrushID);
		SculptProperties->SilentUpdateWatched();
	}

	// this forces full rebuild of properties panel (!!)
	//this->NotifyOfPropertyChangeByTool(SculptProperties);
}

void UMeshVertexSculptTool::SetActiveFalloffType(int32 Identifier)
{
	EMeshSculptFalloffType NewFalloffType = static_cast<EMeshSculptFalloffType>(Identifier);
	if (SculptProperties->PrimaryFalloffType != NewFalloffType)
	{
		SculptProperties->PrimaryFalloffType = NewFalloffType;
		SetPrimaryFalloffType(SculptProperties->PrimaryFalloffType);
		SculptProperties->SilentUpdateWatched();
	}

	// this forces full rebuild of properties panel (!!)
	//this->NotifyOfPropertyChangeByTool(SculptProperties);
}

void UMeshVertexSculptTool::SetRegionFilterType(int32 Identifier)
{
	SculptProperties->BrushFilter = static_cast<EMeshVertexSculptBrushFilterType>(Identifier);
}


void UMeshVertexSculptTool::OnBeginStroke(const FRay& WorldRay)
{
	WaitForPendingUndoRedoUpdate();		// cannot start stroke if there is an outstanding undo/redo update

	UpdateBrushPosition(WorldRay);
	
	const FFrame3d LocalFrame = GetSymmetricBrushFrameLocal();
	const FFrame3d WorldFrame = GetSymmetricBrushFrameWorld();

	bool bUpdatedROI = false;
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	FMeshSculptBrushOp::EReferencePlaneType ReferencePlaneType = UseBrushOp->GetReferencePlaneType();
	if (ReferencePlaneType == FMeshSculptBrushOp::EReferencePlaneType::InitialROI ||
		ReferencePlaneType == FMeshSculptBrushOp::EReferencePlaneType::InitialROI_ViewAligned)
	{
		bUpdatedROI = true;
		UpdateROI(LocalFrame);
		UpdateStrokeReferencePlaneForROI(LocalFrame, TriangleROIArray,
			ReferencePlaneType == FMeshSculptBrushOp::EReferencePlaneType::InitialROI_ViewAligned);
	}
	else if (ReferencePlaneType == FMeshSculptBrushOp::EReferencePlaneType::WorkPlane)
	{
		UpdateStrokeReferencePlaneFromWorkPlane();
	}

	if (UseBrushOp->GetROIUpdateType() == FMeshSculptBrushOp::EROIUpdateType::UseInitial)
	{
		if (!bUpdatedROI)
		{
			bUpdatedROI = true;
			UpdateROI(LocalFrame);
		}
	}

	// initialize first "Last Stamp", so that we can assume all stamps in stroke have a valid previous stamp
	LastStamp.WorldFrame = WorldFrame;
	LastStamp.LocalFrame = LocalFrame;
	LastStamp.Radius = GetCurrentBrushRadius();
	LastStamp.Falloff = GetCurrentBrushFalloff();
	LastStamp.Direction = GetInInvertStroke() ? -1.0 : 1.0;
	LastStamp.Depth = GetCurrentBrushDepth();
	LastStamp.Power = GetActiveBrushStrength();
	LastStamp.TimeStamp = FDateTime::Now();

	PreviousRayDirection = FVector3d::ZeroVector;

	LastStamp.PrevLocalFrame = LastStamp.LocalFrame;
	LastStamp.PrevWorldFrame = LastStamp.WorldFrame;

	InitialStrokeTriangleID = -1;
	InitialStrokeTriangleID = GetBrushTriangleID();

	FSculptBrushOptions SculptOptions;
	//SculptOptions.bPreserveUVFlow = false; // SculptProperties->bPreserveUVFlow;
	SculptOptions.ConstantReferencePlane = GetCurrentStrokeReferencePlane();

	UseBrushOp->ConfigureOptions(SculptOptions);
	UseBrushOp->BeginStroke(GetSculptMesh(), LastStamp, VertexROI);

	AccumulatedTriangleROI.Reset();

	// begin change here? or wait for first stamp?
	BeginChange();
}

void UMeshVertexSculptTool::OnEndStroke()
{
	// update spatial
	bTargetDirty = true;

	GetActiveBrushOp()->EndStroke(GetSculptMesh(), LastStamp, VertexROI);

	// close change record
	EndChange();
}


void UMeshVertexSculptTool::OnCancelStroke()
{
	GetActiveBrushOp()->CancelStroke();

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;

	LongTransactions.Close(GetToolManager());
}

// The first part of UpdateROI, which updates TriangleROIArray to be triangles
//  in our region of interest, and VertexROI to be vertices in our region of
//  interest.
void UMeshVertexSculptTool::ComputeVertexAndTriangleROI(const FFrame3d& LocalFrame)
{
	if (RequireConnectivityToHitPointInStamp()
		// It's possible for LastBrushTriangleID to be null if we started a stroke and brushed off the
		//  edge of the mesh.
		&& LastBrushTriangleID == IndexConstants::InvalidID)
	{
		// If we're requiring connectivity, and we didn't hit a triangle to start with, then we shouldn't
		//  move any vertices.
		// Make sure TriangleROIArray is not in use, as we're about to clear it
		WaitForPendingStampUpdateConst();
		
		VertexROI.Reset();
		TriangleROIArray.Reset();
		SymmetricVertexROI.Reset();
		return;
	}

	FDynamicMesh3* Mesh = GetSculptMesh();
	FVector3d BrushPos = LocalFrame.Origin;

	// By default, our brush is a sphere, and we affect vertices inside it
	float RadiusSqr = GetCurrentBrushRadius() * GetCurrentBrushRadius();
	// This is used to filter the gathered verts for ones that are actually in the brush.
	TFunction<bool(int32 Vid)> IsVertInBrush = [Mesh, &BrushPos, RadiusSqr](int32 Vid)
	{
		return DistanceSquared(BrushPos, Mesh->GetVertexRef(Vid)) < RadiusSqr;
	};

	// Conservative bounding box for the octree range query (for default sphere brush)
	FAxisAlignedBox3d BrushBox(
		BrushPos - GetCurrentBrushRadius() * FVector3d::One(),
		BrushPos + GetCurrentBrushRadius() * FVector3d::One());

	// Optional bounds predicate for tighter octree cell culling (used by cylinder brushes)
	TFunction<bool(const FAxisAlignedBox3d&)> BoundsPredicateFunc;

	// Some brush types want their brush to be a cylinder, so we need to change how we evaluate
	//  cells/vertices that are within reach
	TUniquePtr<FMeshSculptBrushOp>& CurrentBrush = GetActiveBrushOp();
	if (CurrentBrush &&
		(CurrentBrush->GetBrushRegionType() == FMeshSculptBrushOp::EBrushRegionType::InfiniteCylinder
		|| CurrentBrush->GetBrushRegionType() == FMeshSculptBrushOp::EBrushRegionType::CylinderOnSphere))
	{
		double CylinderRadius = GetCurrentBrushRadius();
		double CylinderHeight = UE_LARGE_WORLD_MAX;

		FVector3d CylinderCenter, CylinderAxis;
		if (CurrentBrush->GetBrushRegionType() == FMeshSculptBrushOp::EBrushRegionType::InfiniteCylinder)
		{
			CylinderCenter = BrushPos;
			CylinderAxis = LocalFrame.Z();

			// Since cylinder is infinite, just have to check distance from line for the actual vert containment function
			FLine3d CylinderLine(CylinderCenter, CylinderAxis);
			IsVertInBrush = [Mesh, CylinderLine, RadiusSqr](int32 Vid)
			{
				return CylinderLine.DistanceSquared(Mesh->GetVertexRef(Vid)) < RadiusSqr;
			};
		}
		else // if cylinder on sphere
		{
			FVector3d SphereCenter = GizmoProperties ? CurTargetTransform.InverseTransformPosition(GizmoProperties->Position) 
				: FVector3d::ZeroVector;
			CylinderAxis = BrushPos - SphereCenter;
			if (!CylinderAxis.Normalize())
			{
				CylinderAxis = FVector3d::UnitZ();
			}
			// We want the bottom of our cylinder to be at the sphere center, and the top to go infinitely up, 
			//  but we need a non-infinite position for the center. So lets pick our height to be based on mesh bounds,
			//  with some arbitrary minimum instead.
			CylinderHeight = FMath::Max(InitialBoundsMaxDim, 1000);
			CylinderCenter = SphereCenter + CylinderAxis * (CylinderHeight / 2);

			IsVertInBrush = [Mesh, CylinderCenter, CylinderAxis, CylinderHeight, CylinderRadius](int32 Vid)
			{
				return UE::Geometry::DoesCylinderContainPoint(CylinderCenter, CylinderAxis, 
					CylinderRadius, CylinderHeight, Mesh->GetVertexRef(Vid));
			};
		}

		// Change the conservative brush bounding box to bound the query cylinder
		BrushBox = FAxisAlignedBox3d::Empty();
		BrushBox.Contain(CylinderCenter + CylinderAxis * (CylinderHeight / 2));
		BrushBox.Contain(CylinderCenter - CylinderAxis * (CylinderHeight / 2));
		BrushBox.Expand(CylinderRadius);

		// For non-axis-aligned cylinders the UE_LARGE_WORLD_MAX-height cylinder's AABB is likely to cover the entire mesh bounds,
		// so also use a line distance test to conservatively filter
		if (CylinderAxis.GetAbsMax() < 1.-UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			FLine3d CylinderLine(CylinderCenter, CylinderAxis);
			BoundsPredicateFunc = [CylinderLine, CylinderRadius](const FAxisAlignedBox3d& CellBounds)
			{
				FVector3d BoundsCenter = CellBounds.Center();
				double LineDistSq = CylinderLine.DistanceSquared(BoundsCenter);
				double OverlapRadius = (CylinderRadius + CellBounds.Diagonal().Length() * .5);
				return LineDistSq <= OverlapRadius * OverlapRadius;
			};
		}
	}

	// Set up component/polygroup filter IDs
	int32 ActiveComponentID = -1;
	int32 ActiveGroupID = -1;
	if (SculptProperties->BrushFilter == EMeshVertexSculptBrushFilterType::Component)
	{
		ActiveComponentID = (InitialStrokeTriangleID >= 0 && InitialStrokeTriangleID < TriangleComponentIDs.Num()) ?
			TriangleComponentIDs[InitialStrokeTriangleID] : -1;
	}
	else if (SculptProperties->BrushFilter == EMeshVertexSculptBrushFilterType::PolyGroup)
	{
		ActiveGroupID = Mesh->IsTriangle(InitialStrokeTriangleID) ? ActiveGroupSet->GetGroup(InitialStrokeTriangleID) : -1;
	}

	bool bFilterByBoundary = SculptProperties->bLockBoundaries;

	// Query the vertex octree, filtering by brush shape, boundary vertex, and active component/group
	TArray<int32> VertexQueryBuffer;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateROI_RangeQuery);
		auto VertexPredicate = [this, Mesh, &IsVertInBrush, bFilterByBoundary, ActiveComponentID, ActiveGroupID](int32 VID) -> bool
		{
			if (!IsVertInBrush(VID))
			{
				return false;
			}
			if (bFilterByBoundary && Mesh->IsBoundaryVertex(VID))
			{
				return false;
			}
			if (ActiveComponentID >= 0 || ActiveGroupID >= 0)
			{
				for (int32 tid : CachedVtxTriConnectivity[VID])
				{
					if (ActiveComponentID >= 0 && TriangleComponentIDs[tid] != ActiveComponentID)
					{
						continue;
					}
					if (ActiveGroupID >= 0 && ActiveGroupSet->GetGroup(tid) != ActiveGroupID)
					{
						continue;
					}
					return true; // found a tri matching the group and component filters
				}
				return false; // filter(s) were active and didn't find a match
			}
			return true;
		};
		if (BoundsPredicateFunc)
		{
			VertexOctree.ParallelRangeQuery(BrushBox, BoundsPredicateFunc, VertexPredicate, VertexQueryBuffer);
		}
		else
		{
			VertexOctree.ParallelRangeQuery(BrushBox, VertexPredicate, VertexQueryBuffer);
		}
	}

	auto ShouldExcludeVertex = [Mesh, bFilterByBoundary](int32 VID) { return bFilterByBoundary && Mesh->IsBoundaryVertex(VID); };

	// Make sure TriangleROIArray, VertexROI and SymmetricVertexROI are not in use, as we're about to change them
	WaitForPendingStampUpdateConst();

	// Build VertexROI from query results. Optionally filter by connectivity to hit point.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateROI_BuildROI);

		if (RequireConnectivityToHitPointInStamp()
			&& ensure(LastBrushTriangleID != IndexConstants::InvalidID))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateROI_BuildROI_FindConnectedComponents);
			// Build a 'lookup' index set of all vertices in the original brush shape
			{
				VertexROILookup.Initialize(Mesh->MaxVertexID());
				for (int32 VID : VertexQueryBuffer)
				{
					VertexROILookup.Add(VID);
				}
			}

			// Start growing actual ROI from vertices on the hit triangle that are in that 'lookup' set
			FIndex3i HitTriVids = Mesh->GetTriangle(LastBrushTriangleID);
			TArray<int32> SeedVids;
			for (int i = 0; i < 3; ++i)
			{
				if (VertexROILookup.Contains(HitTriVids[i]))
				{
					SeedVids.Add(HitTriVids[i]);
				}
			}

			// grow to connected vertices from the seed
			TArray<int32> LocalQueue;
			LocalQueue.Append(SeedVids);

			VertexROIBuilder.Initialize(Mesh->MaxVertexID());
			for (int32 InputValue : SeedVids)
			{
				VertexROIBuilder.Add(InputValue);
			}

			while (LocalQueue.Num() > 0)
			{
				int32 CurVertexID = LocalQueue.Pop(EAllowShrinking::No);

				for (int32 NbrVertexID : CachedVtxVtxConnectivity[CurVertexID])
				{
					if (!VertexROIBuilder.Contains(NbrVertexID) && VertexROILookup.Contains(NbrVertexID))
					{
						LocalQueue.Add(NbrVertexID);
						VertexROIBuilder.Add(NbrVertexID);
					}
				}
			}

			VertexROIBuilder.SwapValuesWith(VertexROI);
		}
		else
		{
			VertexROI = MoveTemp(VertexQueryBuffer);
		}

		if (bApplySymmetry)
		{
			SymmetricVertexROI.Reset();
			Symmetry->GetMirrorVertexROI(VertexROI, SymmetricVertexROI, true, ShouldExcludeVertex);
		}

		// Build TriangleROIArray by expanding VertexROI (+ symmetric) to one-ring triangles
		TriangleROIBuilder.Initialize(Mesh->MaxTriangleID());
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateROI_BuildROI_ConvertToTriangles);
			for (int32 VID : VertexROI)
			{
				for (int32 TID : CachedVtxTriConnectivity[VID])
				{
					TriangleROIBuilder.Add(TID);
				}
			}
		}
		if (bApplySymmetry)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateROI_BuildROI_ConvertSymmetricROIToTriangles);
			for (int32 VID : SymmetricVertexROI)
			{
				if (Mesh->IsVertex(VID))
				{
					for (int32 TID : CachedVtxTriConnectivity[VID])
					{
						TriangleROIBuilder.Add(TID);
					}
				}
			}
		}
		TriangleROIBuilder.SwapValuesWith(TriangleROIArray);
	}
}

// Second part of UpdateROI, which fills out ROIPrevPositionBuffer, prepares ROIPositionBuffer,
//  and prepares the symmetry buffers if relevant.
void UMeshVertexSculptTool::PrepROIVertPositionBuffers()
{
	FDynamicMesh3* Mesh = GetSculptMesh();
	// set up and populate position buffers for Vertex ROI
	TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshSculptTool_UpdateROI_4ROI);
	int32 ROISize = VertexROI.Num();
	ROIPositionBuffer.SetNum(ROISize, EAllowShrinking::No);
	ROIPrevPositionBuffer.SetNum(ROISize, EAllowShrinking::No);
	ParallelFor(ROISize, [&](int i)
	{
		ROIPrevPositionBuffer[i] = Mesh->GetVertexRef(VertexROI[i]);
	});
	// do the same for the Symmetric Vertex ROI
	if (bApplySymmetry)
	{
		SymmetricROIPositionBuffer.SetNum(ROISize, EAllowShrinking::No);
		SymmetricROIPrevPositionBuffer.SetNum(ROISize, EAllowShrinking::No);
		ParallelFor(ROISize, [&](int i)
		{
			if ( Mesh->IsVertex(SymmetricVertexROI[i]) )
			{
				SymmetricROIPrevPositionBuffer[i] = Mesh->GetVertexRef(SymmetricVertexROI[i]);
			}
		});
	}
}

void UMeshVertexSculptTool::UpdateROI(const FVector3d& BrushPos)
{
	UpdateROI(FFrame3d(BrushPos));
}

void UMeshVertexSculptTool::UpdateROI(const FFrame3d& LocalFrame)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateROI);

	ComputeVertexAndTriangleROI(LocalFrame);
	PrepROIVertPositionBuffers();
}

/*
 * Updates CurrentStamp, LastStamp, bMouseMoved, and LastMovedStamp (if bMouseMoved is true)
 * 
 * @return false if this ray can be ignored because it did not move and brush ignores zero movement.
 */
bool UMeshVertexSculptTool::UpdateStampPosition(const FRay& WorldRay)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_UpdateStampPosition);

	CalculateBrushRadius();

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bMouseMoved = (PreviousRayDirection - WorldRay.Direction).SquaredLength() > FMathd::ZeroTolerance;

	// raycast for new brush position (if mouse moved or if the brush requires it on non-movement)
	if (bMouseMoved || !UseBrushOp->UseLastStampFrameOnZeroMovement())
	{
		ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
		switch (TargetType)
		{
		case ESculptBrushOpTargetType::SculptMesh:
			UpdateBrushPositionOnSculptMesh(WorldRay, true);
			break;
		case ESculptBrushOpTargetType::TargetMesh:
			UpdateBrushPositionOnTargetMesh(WorldRay, true);
			break;
		case ESculptBrushOpTargetType::ActivePlane:
			UpdateBrushPositionOnActivePlane(WorldRay);
			break;
		}

		// Adjust stamp alignment if needed
		RealignBrush(UseBrushOp->GetStampAlignmentType());
	}

	CurrentStamp = LastStamp;
	//CurrentStamp.DeltaTime = FMathd::Min((FDateTime::Now() - LastStamp.TimeStamp).GetTotalSeconds(), 1.0);
	CurrentStamp.DeltaTime = 0.03;		// 30 fps - using actual time is no good now that we support variable stamps!
	CurrentStamp.WorldFrame = GetSymmetricBrushFrameWorld();
	CurrentStamp.Radius = GetActiveBrushRadius();
	CurrentStamp.LocalFrame = GetSymmetricBrushFrameLocal();
	CurrentStamp.Power = GetActiveBrushStrength();

	if (bHaveBrushAlpha && (AlphaProperties->RotationAngle != 0 || AlphaProperties->bRandomize))
	{
		float UseAngle = AlphaProperties->RotationAngle;
		if (AlphaProperties->bRandomize)
		{
			UseAngle += (StampRandomStream.GetFraction() - 0.5f) * 2.0f * AlphaProperties->RandomRange;
		}

		// possibly should be done in base brush...
		CurrentStamp.WorldFrame.Rotate(FQuaterniond(CurrentStamp.WorldFrame.Z(), UseAngle, true));
		CurrentStamp.LocalFrame.Rotate(FQuaterniond(CurrentStamp.LocalFrame.Z(), UseAngle, true));
	}

	CurrentStamp.PrevLocalFrame = LastStamp.LocalFrame;
	CurrentStamp.PrevWorldFrame = LastStamp.WorldFrame;

	if (bMouseMoved)
	{
		LastMovedStamp = CurrentStamp;
		PreviousRayDirection = WorldRay.Direction;
	}
	return bMouseMoved || !UseBrushOp->IgnoreZeroMovements();
}

// Adjusts brush alignment (assumes that currently brush is aligned to hit normal)
void UMeshVertexSculptTool::RealignBrush(FMeshSculptBrushOp::EStampAlignmentType AlignmentType)
{
	switch (AlignmentType)
	{
	case FMeshSculptBrushOp::EStampAlignmentType::HitNormal:
		// Assume this is already aligned
		break;
	case FMeshSculptBrushOp::EStampAlignmentType::Camera:
		AlignBrushToView();
		break;
	case FMeshSculptBrushOp::EStampAlignmentType::ReferencePlane:
		// Note for this and reference sphere: GetCurrentStrokeReferencePlane is not (necessarily)
		//  what we want because we may not have done UpdateStrokeReferencePlaneFromWorkPlane.
		UpdateBrushFrameWorld(GetBrushFrameWorld().Origin, 
			GizmoProperties ? GizmoProperties->Rotation.GetAxisZ() : FVector3d::UnitZ());
		break;
	case FMeshSculptBrushOp::EStampAlignmentType::ReferenceSphere:
	{
		FVector3d BrushLocation = GetBrushFrameWorld().Origin;
		FVector3d SphereCenter = GizmoProperties ? GizmoProperties->Position : FVector3d::ZeroVector;
		FVector3d NormalToUse = BrushLocation - SphereCenter;
		if (!NormalToUse.Normalize())
		{
			NormalToUse = FVector3d::UnitZ();
		}
		UpdateBrushFrameWorld(BrushLocation, NormalToUse);
		break;
	}
	}
}

bool UMeshVertexSculptTool::CanUpdateBrushType() const
{
	return DefaultPrimaryBrushID == -1;
}

TFuture<void> UMeshVertexSculptTool::ApplyStamp()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_ApplyStamp);

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	FSculptBrushStamp* StampToUse = &CurrentStamp;

	// compute region plane if necessary. This may currently be expensive?
	if (UseBrushOp->WantsStampRegionPlane())
	{
		StampToUse->RegionPlane = ComputeStampRegionPlane(StampToUse->LocalFrame, TriangleROIArray, true, false, false);
	}

	// set up alpha function if we have one
	if (bHaveBrushAlpha)
	{
		StampToUse->StampAlphaFunc = [this](const FSculptBrushStamp& Stamp, const FVector3d& Position)
		{
			return this->SampleBrushAlpha(Stamp, Position);
		};
	}

	// apply the stamp, which computes new positions
	FDynamicMesh3* Mesh = GetSculptMesh();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_ApplyStamp_Apply);

		FDynamicMesh3* MeshForBrush = Mesh;
		if (UseBrushOp->GetStampInputMeshType() == FMeshSculptBrushOp::EStampInputMeshType::BaseMesh)
		{
			MeshForBrush = GetBaseMesh();
		}
		
		UseBrushOp->ApplyStamp(MeshForBrush, *StampToUse, VertexROI, ROIPositionBuffer);
	}

	// can discard alpha now
	StampToUse->StampAlphaFunc = nullptr;

	// if we are applying symmetry, we need to update the on-plane positions as they
	// will not be in the SymmetricVertexROI
	if (bApplySymmetry)
	{
		// update position of vertices that are on the symmetry plane
		Symmetry->ApplySymmetryPlaneConstraints(VertexROI, ROIPositionBuffer);

		// currently something gross is that VertexROI/ROIPositionBuffer may have both a vertex and it's mirror vertex,
		// each with a different position. We somehow need to be able to resolve this, but we don't have the mapping 
		// between the two locations in VertexROI, and we have no way to figure out the 'new' position of that mirror vertex
		// until we can look it up by VertexID, not array-index. So, we are going to bake in the new vertex positions for now.
		const int32 NumV = ROIPositionBuffer.Num();
		ParallelFor(NumV, [&](int32 k)
		{
			int VertIdx = VertexROI[k];
			const FVector3d& NewPos = ROIPositionBuffer[k];
			Mesh->SetVertex(VertIdx, NewPos, false);
		});

		// compute all the mirror vertex positions
		Symmetry->ComputeSymmetryConstrainedPositions(VertexROI, SymmetricVertexROI, ROIPositionBuffer, SymmetricROIPositionBuffer);
	}

	// once stamp is applied, we can start updating vertex change, which can happen async as we saved all necessary info
	TFuture<void> SaveVertexFuture;
	if (ActiveVertexChange != nullptr)
	{
		SaveVertexFuture = Async(VertexSculptToolAsyncExecTarget, [this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_SyncMeshWithPositionBuffer_UpdateChange);
			const int32 NumV = ROIPositionBuffer.Num();
			for (int k = 0; k < NumV; ++k)
			{
				int VertIdx = VertexROI[k];
				ActiveVertexChange->UpdateVertex(VertIdx, ROIPrevPositionBuffer[k], ROIPositionBuffer[k]);
			}

			if (bApplySymmetry)
			{
				int32 NumSymV = SymmetricVertexROI.Num();
				for (int32 k = 0; k < NumSymV; ++k)
				{
					if (SymmetricVertexROI[k] >= 0)
					{
						ActiveVertexChange->UpdateVertex(SymmetricVertexROI[k], SymmetricROIPrevPositionBuffer[k], SymmetricROIPositionBuffer[k]);
					}
				}
			}

		});
	}

	// now actually update the mesh, which happens on the game thread
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_ApplyStamp_Sync);
		const int32 NumV = ROIPositionBuffer.Num();

		// If we are applying symmetry, we already baked these positions in in the branch above and
		// can skip it now, otherwise we update the mesh (todo: profile ParallelFor here, is it helping or hurting?)
		if (bApplySymmetry == false)
		{
			ParallelFor(NumV, [&](int32 k)
			{
				int VertIdx = VertexROI[k];
				const FVector3d& NewPos = ROIPositionBuffer[k];
				Mesh->SetVertex(VertIdx, NewPos, false);
			});
		}

		// if applying symmetry, bake in new symmetric positions
		if (bApplySymmetry)
		{
			ParallelFor(NumV, [&](int32 k)
			{
				int VertIdx = SymmetricVertexROI[k];
				if (Mesh->IsVertex(VertIdx))
				{
					const FVector3d& NewPos = SymmetricROIPositionBuffer[k];
					Mesh->SetVertex(VertIdx, NewPos, false);
				}
			});
		}

		Mesh->UpdateChangeStamps(true, false);
	}

	LastStamp = *StampToUse;
	LastStamp.TimeStamp = FDateTime::Now();

	// let caller wait for this to finish
	return SaveVertexFuture;
}




bool UMeshVertexSculptTool::IsHitTriangleBackFacing(int32 TriangleID, const FDynamicMesh3* QueryMesh) const
{
	if (TriangleID != IndexConstants::InvalidID)
	{
		FViewCameraState StateOut;
		GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
		FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));

		FVector3d Normal, Centroid;
		double Area;
		QueryMesh->GetTriInfo(TriangleID, Normal, Area, Centroid);

		return (Normal.Dot((Centroid - LocalEyePosition)) >= 0);
	}
	return false;
}


int32 UMeshVertexSculptTool::FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const
{
	// have to wait for any outstanding stamp update to finish...
	WaitForPendingStampUpdateConst();
	// wait for previous Undo to finish (possibly never hit because the change records do it?)
	WaitForPendingUndoRedoUpdate();

	int32 HitTID = Octree.FindNearestHitObject(LocalRay);
	if (GetBrushCanHitBackFaces() == false && IsHitTriangleBackFacing(HitTID, GetSculptMesh()))
	{
		HitTID = IndexConstants::InvalidID;
	}
	return HitTID;
}

int32 UMeshVertexSculptTool::FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const
{
	int32 HitTID = BaseMeshSpatial.FindNearestHitObject(LocalRay);
	if (GetBrushCanHitBackFaces() == false && IsHitTriangleBackFacing(HitTID, GetBaseMesh()))
	{
		HitTID = IndexConstants::InvalidID;
	}
	return HitTID;
}



bool UMeshVertexSculptTool::UpdateBrushPosition(const FRay& WorldRay)
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();

	bool bHit = false; 
	ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();
	switch (TargetType)
	{
	case ESculptBrushOpTargetType::SculptMesh:
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	case ESculptBrushOpTargetType::TargetMesh:
		bHit = UpdateBrushPositionOnTargetMesh(WorldRay, false);
		break;
	case ESculptBrushOpTargetType::ActivePlane:
		//UpdateBrushPositionOnActivePlane(WorldRay);
		bHit = UpdateBrushPositionOnSculptMesh(WorldRay, false);
		break;
	}

	if (bHit)
	{
		RealignBrush(UseBrushOp->GetStampAlignmentType());
	}

	return bHit;
}




void UMeshVertexSculptTool::UpdateHoverStamp(const FFrame3d& StampFrameWorld)
{
	FFrame3d HoverFrame = StampFrameWorld;
	if (bHaveBrushAlpha && (AlphaProperties->RotationAngle != 0))
	{
		HoverFrame.Rotate(FQuaterniond(HoverFrame.Z(), AlphaProperties->RotationAngle, true));
	}
	UMeshSculptToolBase::UpdateHoverStamp(HoverFrame);
}

bool UMeshVertexSculptTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	// 4.26 HOTFIX: update LastWorldRay position so that we have it for updating WorkPlane position
	UMeshSurfacePointTool::LastWorldRay = DevicePos.WorldRay;

	PendingStampBrushID = SculptProperties->PrimaryBrushID;
	if(ensure(InStroke() == false))
	{
		UpdateBrushPosition(DevicePos.WorldRay);

		if (BrushIndicatorMaterial)
		{
			BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffRatio"), GetCurrentBrushFalloff());

			switch (SculptProperties->PrimaryFalloffType)
			{
			default:
			case EMeshSculptFalloffType::Smooth:
			case EMeshSculptFalloffType::BoxSmooth:
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffMode"), 0.0f);
				break;
			case EMeshSculptFalloffType::Linear:
			case EMeshSculptFalloffType::BoxLinear:
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffMode"), 0.3333333f);
				break;
			case EMeshSculptFalloffType::Inverse:
			case EMeshSculptFalloffType::BoxInverse:
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffMode"), 0.6666666f);
				break;
			case EMeshSculptFalloffType::Round:
			case EMeshSculptFalloffType::BoxRound:
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffMode"), 1.0f);
				break;
			}

			switch (SculptProperties->PrimaryFalloffType)
			{
			default:
			case EMeshSculptFalloffType::Smooth:
			case EMeshSculptFalloffType::Linear:
			case EMeshSculptFalloffType::Inverse:
			case EMeshSculptFalloffType::Round:
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffShape"), 0.0f);
				break;
			case EMeshSculptFalloffType::BoxSmooth:
			case EMeshSculptFalloffType::BoxLinear:
			case EMeshSculptFalloffType::BoxInverse:
			case EMeshSculptFalloffType::BoxRound:
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffShape"), 1.0f);
			}

			// Drive the stamp opacity through IndicatorColor's alpha channel
			FLinearColor IndicatorColor;
			BrushIndicatorMaterial->GetVectorParameterDefaultValue(TEXT("IndicatorColor"), IndicatorColor);
			IndicatorColor.A = SculptProperties->StampOpacity;
			BrushIndicatorMaterial->SetVectorParameterValue(TEXT("IndicatorColor"), IndicatorColor);
		}
	}

	return true;
}


void UMeshVertexSculptTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSculptToolBase::Render(RenderAPI);

	// draw a dot for the symmetric brush stamp position
	if (bApplySymmetry)
	{
		FToolDataVisualizer Visualizer;
		Visualizer.BeginFrame(RenderAPI);
		FVector3d MirrorPoint = CurTargetTransform.TransformPosition(
			Symmetry->GetMirroredPosition(HoverStamp.LocalFrame.Origin));
		Visualizer.DrawPoint(MirrorPoint, FLinearColor(1.0f, 0.1f, 0.1f, 1), 5.0f, false);
		Visualizer.EndFrame();
	}
}


void UMeshVertexSculptTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	Super::DrawHUD(Canvas, RenderAPI);
	if (BrushEditBehavior.IsValid())
	{
		BrushEditBehavior->DrawHUD(Canvas, RenderAPI);	
	}
}

namespace VertexSculptAsyncHelpers
{
	template<typename CallableType, typename TGateFutureType>
	auto ChainedAsync(TFuture<TGateFutureType>&& GateFuture,  EAsyncExecution Execution, CallableType&& Callable) -> TFuture<decltype(Forward<CallableType>(Callable)())>
	{
		GateFuture.Wait();
		return Async(Execution, Callable);
	}

}

void UMeshVertexSculptTool::OnTick(float DeltaTime)
{
	UMeshSculptToolBase::OnTick(DeltaTime);

	TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick);

	// Ensure undo/redo async work is complete before processing stamps
	WaitForPendingUndoRedoUpdate();

	// if user changed to not-frozen, we need to reinitialize the target
	if (bCachedFreezeTarget != SculptProperties->bFreezeTarget)
	{
		UpdateBaseMesh(nullptr);
		bTargetDirty = false;
	}

	FDynamicMesh3* Mesh = GetSculptMesh();
	const FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
	const bool bUsingOverlayNormalsOut = Normals != nullptr;
	TFuture<void> AccumulateROI;
	TFuture<void> NormalsROI;
	TArray<TArray<int>> TriangleROIArrays;
	FUniqueIndexSet TriangleROISet;

	auto PreExecuteStampOperation = [this, Mesh, &TriangleROISet](int StampCount)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_PreStrokeUpdate);
		// need to make sure previous stamp finished
		WaitForPendingStampUpdateConst();

		TriangleROISet.Initialize(Mesh->TriangleCount(), (Mesh->TriangleCount() * 0.1) );
	};


	auto ExecuteStampOperation = [this, Mesh, bUsingOverlayNormalsOut, &AccumulateROI, &NormalsROI, &TriangleROISet](int StampIndex, const FRay& StampRay)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_StrokeUpdate);
		{
			if (GetActiveBrushOp()->GetROIUpdateType() == FMeshSculptBrushOp::EROIUpdateType::UpdatePerStamp)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_UpdateROI);
				// update sculpt ROI
				UpdateROI(CurrentStamp.LocalFrame);
			}
		}

		// Instead of figuring out which triangles are unique here, we're going to simply accumulate the stamp ROIs in place
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_AccumROIWrapper);
			AccumulateROI = VertexSculptAsyncHelpers::ChainedAsync(MoveTemp(AccumulateROI), VertexSculptToolAsyncExecTarget, [this, StampIndex, &TriangleROISet, TriangleROIArrayCopy = TriangleROIArray]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_AccumROI);
					for (const int& Tid : TriangleROIArrayCopy)
					{
						TriangleROISet.Add(Tid);
					}
				});
		}

		// need to make sure previous stamp finished
		WaitForPendingStampUpdateConst();

		// Nathan - Hypothetically we can apply more than one stamp at a time, but this would require
		// understanding which stamps do and don't overlap. Any non-overlapping stamps technically
		// could be applied in parallel.
		TFuture<void> UpdateChangeFuture;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_ApplyStamp);
			UpdateChangeFuture = ApplyStamp();
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_ApplyStampWait);
			UpdateChangeFuture.Wait();
		}
	};

	auto PostExecuteStampOperation = [this, Mesh, bUsingOverlayNormalsOut, &AccumulateROI, &NormalsROI,  &TriangleROISet]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_PostStrokeUpdate);

		// NOTE: you might try to speculatively do the octree remove here, to save doing it later on Reinsert().
		// This will not improve things, as Reinsert() checks if it needs to actually re-insert, which avoids many
		// removes, and does much of the work of Remove anyway.
		// 
		// begin octree rebuild calculation
		if (!MeshVertexSculptToolLocals::DisableOctreeUpdates->GetBool())
		{
			// We've run into cases where we got to here before the previous octree future finished
			//  executing, so make sure that the previous one is done.
			WaitForPendingStampUpdateConst();

			StampUpdateOctreeFuture = Async(VertexSculptToolAsyncExecTarget, [this]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_OctreeReinsert);
				// Vertex octree reinsertion (for next stamp's range query)
				TFuture<void> VertexOctreeUpdateFuture = Async(VertexSculptToolAsyncExecTarget, [this]()
				{
					VertexOctree.ReinsertVertices(VertexROI);
					if (bApplySymmetry)
					{
						const FDynamicMesh3* SculptMesh = GetSculptMesh();
						VertexOctree.ReinsertFilteredVertices(SymmetricVertexROI, [SculptMesh](int32 VID) {return !SculptMesh->IsVertex(VID);});
					}
				});
			
				// Triangle octree reinsertion (for hit testing)
				Octree.ReinsertTrianglesParallel(TriangleROIArray, OctreeUpdateTempBuffer, OctreeUpdateTempFlagBuffer);
				VertexOctreeUpdateFuture.Wait();
				bOctreeUpdated = true;
			});
		}

		// Prepare list of triangles to process after all stamps are applied
		AccumulateROI.Wait();
		TArray<int> AccumulatedTriangleROIArray;
		TriangleROISet.SwapValuesWith(AccumulatedTriangleROIArray);

		// precompute dynamic mesh update info
		TArray<int32> RenderUpdateSets; FAxisAlignedBox3d RenderUpdateBounds;
		TFuture<bool> RenderUpdatePrecompute;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_PrecomputeUpdateMesh);
			RenderUpdatePrecompute = DynamicMeshComponent->FastNotifyTriangleVerticesUpdated_TryPrecompute(
				AccumulatedTriangleROIArray, RenderUpdateSets, RenderUpdateBounds);
		}

		// recalculate normals. This has to complete before we can update component
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_RecalcNormals);
			NormalsROI.Wait();
			UE::SculptUtil::RecalculateROINormalForTriangles(Mesh, AccumulatedTriangleROIArray, bUsingOverlayNormalsOut);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_UpdateMesh);	
			RenderUpdatePrecompute.Wait();
			DynamicMeshComponent->FastNotifyTriangleVerticesUpdated_ApplyPrecompute(AccumulatedTriangleROIArray,
				EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals,
				RenderUpdatePrecompute, RenderUpdateSets, RenderUpdateBounds);

			GetToolManager()->PostInvalidation();
		}

		AccumulatedTriangleROI.Append(AccumulatedTriangleROIArray);
	};
	
	ProcessPerTickStamps(
		[this](const FRay& StampRay) -> bool {
			return UpdateStampPosition(StampRay);
		}, PreExecuteStampOperation, ExecuteStampOperation, PostExecuteStampOperation);


	if (InStroke() == false && bTargetDirty)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Tick_UpdateTarget);
		check(InStroke() == false);

		// this spawns futures that we could allow to run while other things happen...
		UpdateBaseMesh(&AccumulatedTriangleROI);
		AccumulatedTriangleROI.Reset();

		bTargetDirty = false;
	}


	if (MeshVertexSculptToolLocals::EnableOctreeVisuals->GetBool())
	{
		if (!OctreeGeometry)
		{
			// Set up all the components we need to visualize things.
			OctreeGeometry = NewObject<UPreviewGeometry>();
			OctreeGeometry->CreateInWorld(TargetWorld, FTransform());

			// These visualize the current spline edges that would be extracted
			OctreeGeometry->AddPointSet(MeshVertexSculptToolLocals::OctreePointSetID);
			OctreeGeometry->AddLineSet(MeshVertexSculptToolLocals::OctreeLineSetID);
		}

		ULineSetComponent* OctreeLineSet = OctreeGeometry->FindLineSet(MeshVertexSculptToolLocals::OctreeLineSetID);

		if (bOctreeUpdated)
		{
			OctreeLineSet->Clear();

			if (!CutTree)
			{
				CutTree = MakeShared<FDynamicMeshOctree3::FTreeCutSet>(Octree.BuildLevelCutSet(0));
			}
			else
			{
				TArray<FDynamicMeshOctree3::FCellReference> NewCells;
				Octree.UpdateLevelCutSet(*CutTree, NewCells);
			}

			int ColorSeed = 0;
			for (const FDynamicMeshOctree3::FCellReference& CellRef : CutTree->CutCells)
			{
				FColor CutColor = FColor::MakeRandomSeededColor(ColorSeed++);
				Octree.CollectTriangles(CellRef, [this, OctreeLineSet, Mesh, &CutColor](int32 Tid) {
					FVector A, B, C;
					Mesh->GetTriVertices(Tid, A, B, C);
					OctreeLineSet->AddLine(A, B, CutColor, 2.0);
					OctreeLineSet->AddLine(B, C, CutColor, 2.0);
					OctreeLineSet->AddLine(C, A, CutColor, 2.0);
					});
			}

			bOctreeUpdated = false;
		}
	}
	else
	{
		if (OctreeGeometry)
		{
			ULineSetComponent* OctreeLineSet = OctreeGeometry->FindLineSet(MeshVertexSculptToolLocals::OctreeLineSetID);
			if (bOctreeUpdated)
			{
				OctreeLineSet->Clear();
			}
		}
	}

}



void UMeshVertexSculptTool::WaitForPendingStampUpdateConst() const
{
	if (StampUpdateOctreeFuture.IsValid() && !StampUpdateOctreeFuture.IsReady())
	{
		StampUpdateOctreeFuture.Wait();
	}
}



void UMeshVertexSculptTool::UpdateBaseMesh(const TSet<int32>* TriangleSet)
{
	if (SculptProperties != nullptr)
	{
		bCachedFreezeTarget = SculptProperties->bFreezeTarget;
		if (SculptProperties->bFreezeTarget)
		{
			return;   // do not update frozen target
		}
	}

	const FDynamicMesh3* SculptMesh = GetSculptMesh();
	if ( ! TriangleSet )
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Target_FullUpdate);
		BaseMesh.Copy(*SculptMesh, false, false, false, false);
		BaseMesh.EnableVertexNormals(FVector3f::UnitZ());
		FMeshNormals::QuickComputeVertexNormals(BaseMesh);
		BaseMeshSpatial.SetMaxTreeDepth(8);
		BaseMeshSpatial = FDynamicMeshOctree3();   // need to clear...
		BaseMeshSpatial.Initialize(&BaseMesh);
	}
	else
	{
		BaseMeshIndexBuffer.Reset();
		for ( int32 tid : *TriangleSet)
		{ 
			FIndex3i Tri = BaseMesh.GetTriangle(tid);
			BaseMesh.SetVertex(Tri.A, SculptMesh->GetVertex(Tri.A));
			BaseMesh.SetVertex(Tri.B, SculptMesh->GetVertex(Tri.B));
			BaseMesh.SetVertex(Tri.C, SculptMesh->GetVertex(Tri.C));
			BaseMeshIndexBuffer.Add(tid);
		}
		auto UpdateBaseNormals = Async(VertexSculptToolAsyncExecTarget, [this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Target_UpdateBaseNormals);
			FMeshNormals::QuickComputeVertexNormalsForTriangles(BaseMesh, BaseMeshIndexBuffer);
		});
		auto ReinsertTriangles = Async(VertexSculptToolAsyncExecTarget, [TriangleSet, this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(VtxSculptTool_Target_Reinsert);
			BaseMeshSpatial.ReinsertTriangles(*TriangleSet);
		});
		UpdateBaseNormals.Wait();
		ReinsertTriangles.Wait();
	}
}


bool UMeshVertexSculptTool::GetBaseMeshNearest(int32 VertexID, const FVector3d& Position, double SearchRadius, FVector3d& TargetPosOut, FVector3d& TargetNormalOut)
{
	TargetPosOut = BaseMesh.GetVertex(VertexID);
	TargetNormalOut = (FVector3d)BaseMesh.GetVertexNormal(VertexID);
	return true;
}






void UMeshVertexSculptTool::IncreaseBrushSpeedAction()
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	float CurStrength = UseBrushOp->PropertySet->GetStrength();
	float NewStrength = FMath::Clamp(CurStrength + 0.05f, 0.0f, 1.0f);
	UseBrushOp->PropertySet->SetStrength(NewStrength);
	NotifyOfPropertyChangeByTool(UseBrushOp->PropertySet.Get());
}

void UMeshVertexSculptTool::DecreaseBrushSpeedAction()
{
	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	float CurStrength = UseBrushOp->PropertySet->GetStrength();
	float NewStrength = FMath::Clamp(CurStrength - 0.05f, 0.0f, 1.0f);
	UseBrushOp->PropertySet->SetStrength(NewStrength);
	NotifyOfPropertyChangeByTool(UseBrushOp->PropertySet.Get());
}

void UMeshVertexSculptTool::UpdateBrushAlpha(UTexture2D* NewAlpha)
{
	if (this->BrushAlpha != NewAlpha)
	{
		this->BrushAlpha = NewAlpha;
		if (this->BrushAlpha != nullptr)
		{
			TImageBuilder<FVector4f> AlphaValues;

			constexpr bool bPreferPlatformData = false;
			const bool bReadOK = UE::AssetUtils::ReadTexture(this->BrushAlpha, AlphaValues, bPreferPlatformData);
			if (bReadOK)
			{
				BrushAlphaValues = MoveTemp(AlphaValues);
				BrushAlphaDimensions = AlphaValues.GetDimensions();
				bHaveBrushAlpha = true;

				BrushIndicatorMaterial->SetTextureParameterValue(TEXT("BrushAlpha"), NewAlpha);
				BrushIndicatorMaterial->SetScalarParameterValue(TEXT("AlphaPower"), 1.0);

				return;
			}
		}
		bHaveBrushAlpha = false;
		BrushAlphaValues = TImageBuilder<FVector4f>();
		BrushAlphaDimensions = FImageDimensions();

		BrushIndicatorMaterial->SetTextureParameterValue(TEXT("BrushAlpha"), nullptr);
		BrushIndicatorMaterial->SetScalarParameterValue(TEXT("AlphaPower"), 0.0);
	}
}


double UMeshVertexSculptTool::SampleBrushAlpha(const FSculptBrushStamp& Stamp, const FVector3d& Position) const
{
	if (! bHaveBrushAlpha) return 1.0;

	static const FVector4f InvalidValue(0, 0, 0, 0);

	FVector2d AlphaUV = Stamp.LocalFrame.ToPlaneUV(Position, 2);
	double u = AlphaUV.X / Stamp.Radius;
	u = 1.0 - (u + 1.0) / 2.0;
	double v = AlphaUV.Y / Stamp.Radius;
	v = 1.0 - (v + 1.0) / 2.0;
	if (u < 0 || u > 1) return 0.0;
	if (v < 0 || v > 1) return 0.0;
	FVector4f AlphaValue = BrushAlphaValues.BilinearSampleUV<float>(FVector2d(u, v), InvalidValue);
	return FMathd::Clamp(AlphaValue.X, 0.0, 1.0);
}


void UMeshVertexSculptTool::TryToInitializeSymmetry()
{
	// Attempt to find symmetry, favoring the X axis, then Y axis, if a single symmetry plane is not immediately found
	// Uses local mesh surface (angle sum, normal) to help disambiguate final matches, but does not require exact topology matches across the plane

	FAxisAlignedBox3d Bounds = GetSculptMesh()->GetBounds(true);

	TArray<FVector3d> PreferAxes;
	PreferAxes.Add(this->InitialTargetTransform.GetRotation().AxisX());
	PreferAxes.Add(this->InitialTargetTransform.GetRotation().AxisY());

	FMeshPlanarSymmetry FindSymmetry;
	FFrame3d FoundPlane;
	if (FindSymmetry.FindPlaneAndInitialize(GetSculptMesh(), Bounds, FoundPlane, PreferAxes))
	{
		Symmetry = MakePimpl<FMeshPlanarSymmetry>();
		*Symmetry = MoveTemp(FindSymmetry);
		bMeshSymmetryIsValid = true;
	}
}


//
// Change Tracking
//
void UMeshVertexSculptTool::BeginChange()
{
	check(ActiveVertexChange == nullptr);
	ActiveVertexChange = new FMeshVertexChangeBuilder();
	LongTransactions.Open(LOCTEXT("VertexSculptChange", "Brush Stroke"), GetToolManager());
}

void UMeshVertexSculptTool::EndChange()
{
	check(ActiveVertexChange);

	TUniquePtr<TWrappedToolCommandChange<FMeshVertexChange>> NewChange = MakeUnique<TWrappedToolCommandChange<FMeshVertexChange>>();
	NewChange->WrappedChange = MoveTemp(ActiveVertexChange->Change);
	NewChange->BeforeModify = [this](bool bRevert)
	{
		this->WaitForPendingUndoRedoUpdate();
	};

	GetToolManager()->EmitObjectChange(DynamicMeshComponent, MoveTemp(NewChange), LOCTEXT("VertexSculptChange", "Brush Stroke"));
	if (bMeshSymmetryIsValid && bApplySymmetry == false)
	{
		// if we end a stroke while symmetry is possible but disabled, we now have to assume that symmetry is no longer possible
		GetToolManager()->EmitObjectChange(this, MakeUnique<FVertexSculptNonSymmetricChange>(), LOCTEXT("DisableSymmetryChange", "Disable Symmetry"));
		bMeshSymmetryIsValid = false;
		SymmetryProperties->bSymmetryCanBeEnabled = bMeshSymmetryIsValid;
	}
	LongTransactions.Close(GetToolManager());

	delete ActiveVertexChange;
	ActiveVertexChange = nullptr;
}

void UMeshVertexSculptTool::SetDefaultPrimaryBrushID(const int32 InPrimaryBrushID)
{
	DefaultPrimaryBrushID = InPrimaryBrushID;
}


void UMeshVertexSculptTool::WaitForPendingUndoRedoUpdate() const
{
	UndoUpdateFuture.Wait();
}

void UMeshVertexSculptTool::OnDynamicMeshComponentChanged(UDynamicMeshComponent* Component, const FMeshRegionChangeBase* Change, bool bRevert)
{
	OnDynamicMeshComponentChanged_Internal(Component, Change, bRevert, /*bUpdateRender*/false);
}
void UMeshVertexSculptTool::OnDynamicMeshComponentChanged_Internal(UDynamicMeshComponent* Component, const FMeshRegionChangeBase* Change, bool bRevert, bool bUpdateRender)
{
	// have to wait for any outstanding stamp update to finish...
	WaitForPendingStampUpdateConst();
	// wait for previous Undo to finish (possibly never hit because the change records do it?)
	WaitForPendingUndoRedoUpdate();

	FDynamicMesh3* Mesh = GetSculptMesh();

	// Note we rely on the fact that Vertex Sculpt tool does not change topology, so we only compute
	// CachedVtxTriConnectivity/CachedVtxVtxConnectivity once in Setup()
	checkSlow(Mesh->MaxVertexID() == CachedVtxTriConnectivity.Num());

	// figure out the set of modified vertices and triangles
	AccumulatedTriangleROI.Reset();
	TArray<int32> ChangedVertices;
	Change->ProcessChangeVertices(Mesh, [this, &Mesh, &ChangedVertices](TConstArrayView<int32> Vertices)
	{
		AccumulatedTriangleROI.Reserve(Vertices.Num() * 4);
		for (int32 VID : Vertices)
		{
			AccumulatedTriangleROI.Append(CachedVtxTriConnectivity[VID]);
		}
		ChangedVertices.Append(Vertices.GetData(), Vertices.Num());
	}, bRevert);

	UndoUpdateFuture.Reset();

	// Recompute normals synchronously, so we don't end up racing UDynamicMeshComponent's render buffer updates (which happen soon after this method)
	UE::SculptUtil::RecalculateROINormals(Mesh, AccumulatedTriangleROI, NormalsROIBuilder);

	if (bUpdateRender)
	{
		Component->FastNotifyTriangleVerticesUpdated(AccumulatedTriangleROI, EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals);
	}

	// start the octree updates
	// Note: both UndoUpdateOctreeFuture and UndoUpdateBaseMeshFuture read AccumulatedTriangleROI concurrently (read-only after this point)
	UndoUpdateOctreeFuture = Async(VertexSculptToolAsyncExecTarget, [this, Mesh, ChangedVerts = MoveTemp(ChangedVertices)]()
	{
		// Vertex octree reinsert changed vertices
		VertexOctree.ReinsertVertices(ChangedVerts);
		// Triangle octree reinsert affected triangles (for hit testing)
		Octree.ReinsertTriangles(AccumulatedTriangleROI);
		bOctreeUpdated = true;
		return true;
	});

	// start the base mesh update
	UndoUpdateBaseMeshFuture = Async(VertexSculptToolAsyncExecTarget, [this, Mesh]()
	{
		UpdateBaseMesh(&AccumulatedTriangleROI);
		return true;
	});

	UndoUpdateFuture = Async(VertexSculptToolAsyncExecTarget, [this]()
	{
		UndoUpdateOctreeFuture.Wait();
		UndoUpdateBaseMeshFuture.Wait();
	});

	RefreshSculptLayerProperties();
}


void UMeshVertexSculptTool::RefreshSculptLayerProperties()
{
	if (SculptLayerProperties)
	{
		SculptLayerProperties->RefreshFromMesh();
	}
}



void UMeshVertexSculptTool::UpdateBrushType(EMeshVertexSculptBrushType BrushType)
{
	UpdateBrushType((int32)BrushType);
}

void UMeshVertexSculptTool::UpdateBrushType(int32 BrushID)
{	
	static const FText BaseMessage = LOCTEXT("OnStartSculptTool", "Hold Shift to Smooth, Ctrl to Invert (where applicable). [/] change Size (+Shift to small-step).");
	FTextBuilder Builder;
	Builder.AppendLine(BaseMessage);

	SetActivePrimaryBrushType(BrushID);

	// If something went wrong and we were unable to activate the given brush, make sure that we have some kind
	//  of brush active so we don't crash.
	if (!PrimaryBrushOp && ensure(!BrushOpFactories.IsEmpty()))
	{
		UE_LOGF(LogGeometry, Error, "Sculpt tool was unable to activate chosen brush.");
		SetActivePrimaryBrushType(BrushOpFactories.CreateIterator()->Key);
		ensure(PrimaryBrushOp);
	}

	if (BrushEditBehavior.IsValid())
	{
		// todo: Handle Kelvinlet brush props better. At the moment we are just disabling strength editing for Kelvinlet brush ops.
		auto PropertySetSupportsStrength = [this]()
		{
			return PrimaryBrushOp->PropertySet.IsValid() && Cast<UBaseKelvinletBrushOpProps>(PrimaryBrushOp->PropertySet.Get()) == nullptr;
		};
	
		if (PropertySetSupportsStrength())
		{
			BrushEditBehavior->VerticalProperty.Name = LOCTEXT("BrushStrength", "Strength");
            BrushEditBehavior->VerticalProperty.GetValueFunc = [this](){ return PrimaryBrushOp->PropertySet->GetStrength(); };
            BrushEditBehavior->VerticalProperty.SetValueFunc = [this](float NewValue){ PrimaryBrushOp->PropertySet->SetStrength(FMath::Clamp(NewValue, 0.f, 1.f)); };
            BrushEditBehavior->VerticalProperty.bEnabled = true;
		}
		else
		{
			BrushEditBehavior->VerticalProperty.bEnabled = false;
		}
	}
	
	if (ensure(SculptProperties))
	{
		SculptProperties->bCanFreezeTarget = 
			BrushID == (int32)EMeshVertexSculptBrushType::Offset
			|| BrushID == (int32)EMeshVertexSculptBrushType::SculptMax || BrushID == (int32)EMeshVertexSculptBrushType::SculptView
			|| BrushID == (int32)EMeshVertexSculptBrushType::InflateStroke || BrushID == (int32)EMeshVertexSculptBrushType::InflateMax
			|| BrushID == (int32)EMeshVertexSculptBrushType::Pinch;
	}

	SetToolPropertySourceEnabled(GizmoProperties, false);
	if (PrimaryBrushOp && 
		(PrimaryBrushOp->GetReferencePlaneType() == FMeshSculptBrushOp::EReferencePlaneType::WorkPlane
		|| PrimaryBrushOp->GetStampAlignmentType() == FMeshSculptBrushOp::EStampAlignmentType::ReferencePlane
		|| PrimaryBrushOp->GetStampAlignmentType() == FMeshSculptBrushOp::EStampAlignmentType::ReferenceSphere))
	{
		Builder.AppendLine(LOCTEXT("FixedPlaneTip", "Use T to reposition Work Plane at cursor, Shift+T to align to Normal, Ctrl+Shift+T to align to View"));
		SetToolPropertySourceEnabled(GizmoProperties, true);
	}

	TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
	bool bEnableAlpha = UseBrushOp && UseBrushOp->UsesAlpha();
	SetToolPropertySourceEnabled(AlphaProperties, bEnableAlpha);

	GetToolManager()->DisplayMessage(Builder.ToText(), EToolMessageLevel::UserNotification);
}

FFrame3d UMeshVertexSculptTool::GetSymmetricBrushFrameWorld() const
{
	FFrame3d OutFrame = GetBrushFrameWorld();
	if (bApplySymmetry)
	{
		OutFrame = GetSymmetricBrushFrameLocal();
		OutFrame.Transform(CurTargetTransform);
	}
	return OutFrame;
}

FFrame3d UMeshVertexSculptTool::GetSymmetricBrushFrameLocal() const
{
	const FFrame3d LocalFrame = GetBrushFrameLocal();
	return bApplySymmetry && Symmetry ? Symmetry->GetPositiveSideFrame(LocalFrame) : LocalFrame;
}

bool UMeshVertexSculptTool::DoesTargetHaveSculptLayers() const
{
	if (ISceneComponentBackedTarget* SceneComponentTarget = Cast<ISceneComponentBackedTarget>(Target))
	{
		if (IMeshSculptLayersManager* SculptLayersManager = Cast<IMeshSculptLayersManager>(SceneComponentTarget->GetOwnerSceneComponent()))
		{
			return SculptLayersManager->HasSculptLayers();
		}
	}
	return false;
}


void UMeshVertexSculptTool::UndoRedo_RestoreSymmetryPossibleState(bool bSetToValue)
{
	bMeshSymmetryIsValid = bSetToValue;
	SymmetryProperties->bSymmetryCanBeEnabled = bMeshSymmetryIsValid;
}





void FVertexSculptNonSymmetricChange::Apply(UObject* Object)
{
	if (Cast<UMeshVertexSculptTool>(Object))
	{
		Cast<UMeshVertexSculptTool>(Object)->UndoRedo_RestoreSymmetryPossibleState(false);
	}
}
void FVertexSculptNonSymmetricChange::Revert(UObject* Object)
{
	if (Cast<UMeshVertexSculptTool>(Object))
	{
		Cast<UMeshVertexSculptTool>(Object)->UndoRedo_RestoreSymmetryPossibleState(true);
	}
}

#undef LOCTEXT_NAMESPACE

