// Copyright Epic Games, Inc. All Rights Reserved.

#include "OffsetMeshTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "WeightMapTypes.h"
#include "DeformationOps/MeshOffsetOps.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OffsetMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UOffsetMeshTool"

/*
 * Tool Builder
 */

USingleTargetWithSelectionTool* UOffsetMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UOffsetMeshTool>(SceneState.ToolManager);
}

/*
 * Tool
 */

UOffsetMeshTool::UOffsetMeshTool()
{
	SetToolDisplayName(LOCTEXT("ToolName", "Offset"));
}

void UOffsetMeshTool::InitializeProperties()
{
	OffsetProperties = NewObject<UOffsetMeshToolProperties>(this);
	AddToolPropertySource(OffsetProperties);
	OffsetProperties->RestoreProperties(this);
	OffsetProperties->WatchProperty(OffsetProperties->OffsetType,
		[&](EOffsetMeshToolOffsetType) { UpdateOptionalPropertyVisibility(); InvalidateResult();  });
	OffsetProperties->WatchProperty(OffsetProperties->Distance,
		[&](float) { InvalidateResult();  });
	OffsetProperties->WatchProperty(OffsetProperties->bCreateShell,
		[&](bool) { InvalidateResult();  });


	IterativeProperties = AddOptionalPropertySet<UIterativeOffsetProperties>(
		[&]() { return OffsetProperties->OffsetType == EOffsetMeshToolOffsetType::Iterative; });

	ImplicitProperties = AddOptionalPropertySet<UImplicitOffsetProperties>(
		[&]() { return OffsetProperties->OffsetType == EOffsetMeshToolOffsetType::Implicit; });

	WeightMapProperties = AddWeightMapPropertySet<UOffsetWeightMapSetProperties>();
	WeightMapProperties->WatchProperty(WeightMapProperties->MinDistance,
		[&](float) { InvalidateResult(); });
}


void UOffsetMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	OffsetProperties->SaveProperties(this);
}


FText UOffsetMeshTool::GetToolMessageString() const
{
	return LOCTEXT("StartOffsetToolMessage", "Offset the mesh vertex positions using various Offsetting methods.");
}

FText UOffsetMeshTool::GetAcceptTransactionName() const
{
	return LOCTEXT("OffsetMeshToolTransactionName", "Offset Mesh");
}


bool UOffsetMeshTool::HasMeshTopologyChanged() const
{
	return OffsetProperties->bCreateShell;
}



TUniquePtr<FDynamicMeshOperator> UOffsetMeshTool::MakeNewOperator()
{
	const FDynamicMesh3* Mesh = &GetInitialMesh();

	TUniquePtr<FMeshOffsetBaseOp> MeshOp;
	switch (OffsetProperties->OffsetType)
	{
		default:
		case EOffsetMeshToolOffsetType::Iterative:
		{
			TUniquePtr<FIterativeOffsetMeshOp> IterOp = MakeUnique<FIterativeOffsetMeshOp>(Mesh);
			IterOp->Steps = IterativeProperties->Steps;
			IterOp->SmoothAlpha = IterativeProperties->SmoothingPerStep;
			IterOp->bReprojectSmooth = IterativeProperties->bReprojectSmooth;
			IterOp->bFixedBoundary = (!IterativeProperties->bOffsetBoundaries);
			MeshOp = MoveTemp(IterOp);
		}
		break;

		case EOffsetMeshToolOffsetType::Implicit:
		{
			TUniquePtr<FLaplacianOffsetMeshOp> ImplicitOp = MakeUnique<FLaplacianOffsetMeshOp>(Mesh);
			double NonlinearT = FMathd::Pow(ImplicitProperties->Smoothness, 2.0);
			// this is an empirically-determined hack that seems to work OK to normalize the result for variable vertex count...
			double ScaledPower = (NonlinearT / 50.0) * Mesh->VertexCount();
			ImplicitOp->Softness = ScaledPower;
			ImplicitOp->bUniformWeights = (ImplicitProperties->bPreserveUVs == false);
			ImplicitOp->bFixedBoundary = true;
			MeshOp = MoveTemp(ImplicitOp);
		}
		break;
	}

	double MaxDist = FMathd::Abs(OffsetProperties->Distance);
	double MinDist = FMathd::Abs(WeightMapProperties->MinDistance);
	MinDist = FMathd::Min(MinDist, MaxDist);
	MaxDist *= GetScaleNormalizationFactor();
	MinDist *= GetScaleNormalizationFactor();
	MeshOp->OffsetRange = FInterval1d(MinDist, MaxDist);
	MeshOp->OffsetSign = (OffsetProperties->Distance < 0) ? -1.0 : 1.0;

	MeshOp->BaseNormals = this->GetInitialVtxNormals();
	MeshOp->bCreateShell = OffsetProperties->bCreateShell;
	if (MeshOp->bCreateShell)
	{
		MeshOp->BoundaryLoops = GetInitialBoundaryLoops();
	}

	MeshOp->WeightMap = GetActiveWeightMap();
	MeshOp->bUseWeightMap = true;

	MeshOp->SetTransform( FTransform3d(GetPreviewTransform()) );

	return MoveTemp(MeshOp);
}





#undef LOCTEXT_NAMESPACE

