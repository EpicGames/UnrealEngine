// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectToTargetTool.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProjectToTargetTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UProjectToTargetTool"

bool UProjectToTargetToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 2;
}

UMultiSelectionMeshEditingTool* UProjectToTargetToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UProjectToTargetTool>(SceneState.ToolManager);
}

void UProjectToTargetTool::Setup()
{
	// ProjectionTarget and ProjectionTargetSpatial are setup before calling the parent class's Setup
	check(Targets.Num() == 2);
	ProjectionTarget = MakeUnique<FDynamicMesh3>();
	*ProjectionTarget = UE::ToolTarget::GetDynamicMeshCopy(Targets[1]);
	ProjectionTargetSpatial = MakeUnique<FDynamicMeshAABBTree3>(ProjectionTarget.Get(), true);

	// Now setup parent RemeshMeshTool class
	URemeshMeshTool::Setup();

	SetToolDisplayName(LOCTEXT("ProjectToTargetToolName", "Project To Target"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("ProjectToTargetToolDescription", "Incrementally deform the first selected mesh towards the second, while applying Remeshing. This can be used to improve the accuracy of shrink-wrapping strategies."),
		EToolMessageLevel::UserNotification);
}


TUniquePtr<FDynamicMeshOperator> UProjectToTargetTool::MakeNewOperator()
{
	UProjectToTargetToolProperties* ProjectProperties = Cast<UProjectToTargetToolProperties>(BasicProperties);
	if (!ensure(ProjectProperties))
	{
		return nullptr;
	}

	TUniquePtr<FDynamicMeshOperator> Op = URemeshMeshTool::MakeNewOperator();

	FDynamicMeshOperator* RawOp = Op.Get();
	FRemeshMeshOp* RemeshOp = static_cast<FRemeshMeshOp*>(RawOp);
	check(RemeshOp);

	RemeshOp->ProjectionTarget = ProjectionTarget.Get();
	RemeshOp->ProjectionTargetSpatial = ProjectionTargetSpatial.Get();

	RemeshOp->ToolMeshLocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
	RemeshOp->TargetMeshLocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[1]);
	RemeshOp->bUseWorldSpace = ProjectProperties->bWorldSpace;
	RemeshOp->bParallel = ProjectProperties->bParallel;

	if (ProjectProperties->RemeshType == ERemeshType::NormalFlow)
	{
		RemeshOp->FaceProjectionPassesPerRemeshIteration = ProjectProperties->FaceProjectionPassesPerRemeshIteration;
		RemeshOp->SurfaceProjectionSpeed = ProjectProperties->SurfaceProjectionSpeed;
		RemeshOp->NormalAlignmentSpeed = ProjectProperties->NormalAlignmentSpeed;
		RemeshOp->bSmoothInFillAreas = ProjectProperties->bSmoothInFillAreas;
		RemeshOp->FillAreaDistanceMultiplier = ProjectProperties->FillAreaDistanceMultiplier;
		RemeshOp->FillAreaSmoothMultiplier = ProjectProperties->FillAreaSmoothMultiplier;

		// disable convergence check
		RemeshOp->MinActiveEdgeFraction = 0.0;
	}

	return Op;
}


#undef LOCTEXT_NAMESPACE

