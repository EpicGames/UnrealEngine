// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/MeshMaterialProperties.h"

#include "DynamicMesh/DynamicMesh3.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMaterialProperties)

#define LOCTEXT_NAMESPACE "UMeshMaterialProperites"

UNewMeshMaterialProperties::UNewMeshMaterialProperties()
{
	Material = CreateDefaultSubobject<UMaterialInterface>(TEXT("MATERIAL"));
}

const TArray<FString>& UExistingMeshMaterialProperties::GetUVChannelNamesFunc() const
{
	return UVChannelNamesList;
}

void UExistingMeshMaterialProperties::RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier)
{
	Super::RestoreProperties(RestoreToTool, CacheIdentifier);
	Setup();
}

void UExistingMeshMaterialProperties::Setup()
{
	UMaterial* CheckerMaterialBase = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/CheckerMaterial"));
	if (CheckerMaterialBase != nullptr)
	{
		CheckerMaterial = UMaterialInstanceDynamic::Create(CheckerMaterialBase, NULL);
		if (CheckerMaterial != nullptr)
		{
			CheckerMaterial->SetScalarParameterValue("Density", CheckerDensity);
			CheckerMaterial->SetScalarParameterValue("UVChannel", static_cast<float>(UVChannelNamesList.IndexOfByKey(UVChannel)));
		}
	}
}

void UExistingMeshMaterialProperties::UpdateMaterials()
{
	if (CheckerMaterial != nullptr)
	{
		CheckerMaterial->SetScalarParameterValue("Density", CheckerDensity);
		CheckerMaterial->SetScalarParameterValue("UVChannel", static_cast<float>(UVChannelNamesList.IndexOfByKey(UVChannel)));
	}
}


UMaterialInterface* UExistingMeshMaterialProperties::GetActiveOverrideMaterial() const
{
	if (MaterialMode == ESetMeshMaterialMode::Checkerboard && CheckerMaterial != nullptr)
	{
		return CheckerMaterial;
	}
	if (MaterialMode == ESetMeshMaterialMode::Override && OverrideMaterial != nullptr)
	{
		return OverrideMaterial;
	}
	return nullptr;
}

void UExistingMeshMaterialProperties::UpdateUVChannels(int32 UVChannelIndex, const TArray<FString>& UVChannelNames, bool bUpdateSelection)
{
	UVChannelNamesList = UVChannelNames;
	if (bUpdateSelection)
	{
		UVChannel = 0 <= UVChannelIndex && UVChannelIndex < UVChannelNames.Num() ? UVChannelNames[UVChannelIndex] : TEXT("");
	}
}

#undef LOCTEXT_NAMESPACE

