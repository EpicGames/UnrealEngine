// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/MeshSculptLayerProperties.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "ModelingToolExternalMeshUpdateAPI.h"
#include "Changes/MeshReplacementChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSculptLayerProperties)

DEFINE_LOG_CATEGORY(LogSculptLayerProps);

#define LOCTEXT_NAMESPACE "UMeshSculptLayerProperties"

void UMeshSculptLayerProperties::Init(IModelingToolExternalDynamicMeshUpdateAPI* InTool, int32 InNumLockedBaseLayers)
{
	bCanEditLayers = true;
	LayerWeights.Reset();

	Tool = InTool;
	if (!Tool)
	{
		return;
	}
	NumLockedBaseLayers = InNumLockedBaseLayers;

	Tool->ProcessToolMeshes([this](const UE::Geometry::FDynamicMesh3& Mesh, int32 MeshIdx)
		{
			// Sculpt layer UI only supports a single mesh for now
			if (MeshIdx > 0)
			{
				return;
			}
			bCanEditLayers = Mesh.HasAttributes() && Mesh.Attributes()->NumSculptLayers() > NumLockedBaseLayers;
			if (bCanEditLayers)
			{
				UpdateSettingsFromMesh(Mesh.Attributes()->GetSculptLayers());
			}
		}
	);
}

void UMeshSculptLayerProperties::UpdateSettingsFromMesh(const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
{
	TConstArrayView<double> SourceWeights = SculptLayers->GetLayerWeights();
	int32 NumLayers = FMath::Max(0, SourceWeights.Num() - NumLockedBaseLayers);
	LayerWeights.SetNum(NumLayers);
	for (int32 Idx = 0; Idx < LayerWeights.Num(); ++Idx)
	{
		LayerWeights[Idx] = SourceWeights[Idx + NumLockedBaseLayers];
	}
	ActiveLayer = SculptLayers->GetActiveLayer();
}

void UMeshSculptLayerProperties::SetLayerWeights(UE::Geometry::FDynamicMeshSculptLayers* SculptLayers) const
{
	TArray<double> FullLayerWeights(SculptLayers->GetLayerWeights());
	FullLayerWeights.SetNum(LayerWeights.Num() + NumLockedBaseLayers);
	for (int32 Idx = 0; Idx < LayerWeights.Num(); ++Idx)
	{
		FullLayerWeights[Idx + NumLockedBaseLayers] = LayerWeights[Idx];
	}
	SculptLayers->UpdateLayerWeights(FullLayerWeights);
}

bool UMeshSculptLayerProperties::ReadSculptLayers(TFunctionRef<void(const UE::Geometry::FDynamicMesh3& Mesh, const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)> ReadFn)
{
	bool bFoundLayers = false;
	if (Tool)
	{
		Tool->ProcessToolMeshes([this, &ReadFn, &bFoundLayers](const UE::Geometry::FDynamicMesh3& Mesh, int32 Idx) -> void
			{
				// Sculpt layer UI only supports a single mesh for now
				if (Idx > 0)
				{
					return;
				}
				if (const UE::Geometry::FDynamicMeshAttributeSet* Attributes = Mesh.Attributes())
				{
					if (const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers = Attributes->GetSculptLayers())
					{
						ReadFn(Mesh, SculptLayers);
						bFoundLayers = true;
					}
				}
			}
		);
	}
	return bFoundLayers;
}

void UMeshSculptLayerProperties::EditSculptLayers(TFunctionRef<void(UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)> EditFn, bool bEmitChange)
{
	if (Tool && Tool->AllowToolMeshUpdates())
	{
		Tool->UpdateToolMeshes([this, &EditFn, bEmitChange](UE::Geometry::FDynamicMesh3& Mesh, int32 Idx) -> TUniquePtr<FMeshRegionChangeBase>
			{
				// Sculpt layer UI only supports a single mesh for now
				if (Idx > 0)
				{
					return nullptr;
				}

				if (!InitialMesh)
				{
					InitialMesh = MakeShared<UE::Geometry::FDynamicMesh3>(Mesh);
				}
				if (UE::Geometry::FDynamicMeshAttributeSet* Attributes = Mesh.Attributes())
				{
					if (UE::Geometry::FDynamicMeshSculptLayers* SculptLayers = Attributes->GetSculptLayers())
					{
						EditFn(Mesh, SculptLayers);
					}
				}

				if (!bEmitChange)
				{
					return nullptr;
				}
				TUniquePtr<FMeshRegionChangeBase> Result(new FMeshReplacementChange(InitialMesh, MakeShared<FDynamicMesh3>(Mesh)));
				InitialMesh = nullptr;
				return MoveTemp(Result);
			}
		);
	}
}

#if WITH_EDITOR
void UMeshSculptLayerProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	EditSculptLayers([this, &PropertyChangedEvent](UE::Geometry::FDynamicMesh3& Mesh, FDynamicMeshSculptLayers* SculptLayers)
		{
			if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UMeshSculptLayerProperties, LayerWeights))
			{
				SetLayerWeights(SculptLayers);
			}
			if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UMeshSculptLayerProperties, ActiveLayer))
			{
				ActiveLayer = FMath::Clamp(ActiveLayer, NumLockedBaseLayers, SculptLayers->NumLayers() - 1);
				SculptLayers->SetActiveLayer(ActiveLayer);
			}
		},
		PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive
	);
}
#endif

void UMeshSculptLayerProperties::AddLayer()
{
	Modify();
	
	EditSculptLayers([this](UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
		{
			Mesh.Attributes()->EnableSculptLayers(SculptLayers->NumLayers() + 1);
			LayerWeights.Add(1.0);
		}, true);
}

bool UMeshSculptLayerProperties::RemoveLayer()
{
	if (LayerWeights.Num() <= 1)
	{
		return false;
	}
	bool bRemoveSuccessful = false;
	
	EditSculptLayers([this, &bRemoveSuccessful](UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
		{
			if (!SculptLayers->DiscardSculptLayer(ActiveLayer))
			{
				return;
			}
			UpdateSettingsFromMesh(SculptLayers);
			// If the system picked a locked active layer, try to pick a different layer instead
			if (ActiveLayer < NumLockedBaseLayers)
			{
				// We shouldn't set a zero-weight layer as the active layer, so look for a non-zero weight layer to set
				int32 NonZeroLayerIdx = INDEX_NONE;
				for (int32 Idx = 0; Idx < LayerWeights.Num(); ++Idx)
				{
					if (LayerWeights[Idx] != 0.0)
					{
						NonZeroLayerIdx = Idx;
						break;
					}
				}
				// if all layers had zero weight, just pick the first layer and set its weight to 1.0 so it is ready to sculpt on
				if (NonZeroLayerIdx == INDEX_NONE)
				{
					LayerWeights[0] = 1.0;
					NonZeroLayerIdx = 0;
					SetLayerWeights(SculptLayers);
				}
				ActiveLayer = SculptLayers->SetActiveLayer(NumLockedBaseLayers + NonZeroLayerIdx);
			}
			bRemoveSuccessful = true;
		}, true);
	return bRemoveSuccessful;
}

bool UMeshSculptLayerProperties::RemoveLayerAtIndex(const int32 IndexOfLayerToRemove)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(IndexOfLayerToRemove))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not removed. Invalid index: %d.", IndexOfLayerToRemove);
		return false;
	}

	Modify();

	bool bRemoveSuccessful = false;

	// adjust index to take into account any locked base layers
	const int32 AdjustedIndex = IndexOfLayerToRemove + NumLockedBaseLayers;
	
	if (AdjustedIndex == ActiveLayer)
	{
		bRemoveSuccessful = RemoveLayer();
	}
	else
	{
		EditSculptLayers([this, &AdjustedIndex, &IndexOfLayerToRemove, &bRemoveSuccessful](UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
		{
			if (!SculptLayers->DiscardSculptLayer(AdjustedIndex))
			{
				return;
			}
			if (ActiveLayer > IndexOfLayerToRemove)
			{
				// ideally, when deleting the active layer, we should set the prior layer to the new active one,
				// unless that layer's weight is 0.
				double LayerWeightTarget = SculptLayers->GetLayerWeights()[ActiveLayer - 1];
				if (LayerWeightTarget != 0.0)
				{
					ActiveLayer = SculptLayers->SetActiveLayer(ActiveLayer - 1);
				}
				// find a different layer to set as the active one
				else
				{
					int32 NonZeroLayerIdx = INDEX_NONE;
					for (int32 Idx = 0; Idx < LayerWeights.Num(); ++Idx)
					{
						if (LayerWeights[Idx] != 0.0)
						{
							NonZeroLayerIdx = Idx;
							break;
						}
					}
					// if all layers had zero weight, just pick the first layer and set its weight to 1.0 so it is ready to sculpt on
					if (NonZeroLayerIdx == INDEX_NONE)
					{
						LayerWeights[0] = 1.0;
						NonZeroLayerIdx = 0;
						SetLayerWeights(SculptLayers);
					}
					ActiveLayer = SculptLayers->SetActiveLayer(NumLockedBaseLayers + NonZeroLayerIdx);
				}
			}
			UpdateSettingsFromMesh(SculptLayers);
			bRemoveSuccessful = true;
		}, true);
	}
	return bRemoveSuccessful;
}

FName UMeshSculptLayerProperties::GetLayerName(const int32 InIndex)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(InIndex))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not found. Invalid index: %d.", InIndex);
		return NAME_None;
	}
	
	FName LayerName;
	ReadSculptLayers([this, &InIndex, &LayerName](const UE::Geometry::FDynamicMesh3& Mesh, const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)->void
		{
			if (const UE::Geometry::FDynamicMeshSculptLayerAttribute* Layer = SculptLayers->GetLayer(InIndex + NumLockedBaseLayers))
			{
				LayerName = Layer->GetName();
			}
			UpdateSettingsFromMesh(SculptLayers);
		});
	return LayerName;
}

void UMeshSculptLayerProperties::RefreshFromMesh()
{
	ReadSculptLayers([this](const UE::Geometry::FDynamicMesh3& Mesh, const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)->void
	{
		if (SculptLayers)
		{
			UpdateSettingsFromMesh(SculptLayers);
			OnMeshLayersPanelRequestRebuild.Broadcast();
		}
	});
}

void UMeshSculptLayerProperties::SetLayerName(const int32 InIndex, const FName InLayerName)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(InIndex))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not found. Invalid index: %d.", InIndex);
		return;
	}
	
	EditSculptLayers([this, &InIndex, &InLayerName](UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
		{
			SculptLayers->GetLayer(InIndex + NumLockedBaseLayers)->SetName(InLayerName);
			UpdateSettingsFromMesh(SculptLayers);
		}, true);
}

void UMeshSculptLayerProperties::MoveLayer(const int32 InInitIndex, int32 InTargetIndex)
{
	// ensure target is within range
	InTargetIndex = FMath::Clamp(InTargetIndex, 0, LayerWeights.Num() - 1);
	
	if (!LayerWeights.IsValidIndex(InInitIndex))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not moved. Invalid source index, %d.", InInitIndex);
		return;
	}

	if (!LayerWeights.IsValidIndex(InTargetIndex))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not moved. Invalid target index, %d.", InTargetIndex);
		return;
	}

	if (InInitIndex == InTargetIndex)
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not moved. Source and target index cannot be the same.");
		return;
	}
	
	Modify();
	
	// adjust indices to take into account any locked base layers
	int32 AdjustedInitIndex = InInitIndex + NumLockedBaseLayers;
	int32 AdjustedTargetIndex = InTargetIndex + NumLockedBaseLayers;
	EditSculptLayers([this, &AdjustedInitIndex, &AdjustedTargetIndex](UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
		{
			SculptLayers->MoveLayer(AdjustedInitIndex, AdjustedTargetIndex);
			UpdateSettingsFromMesh(SculptLayers);
		}, true);
}

void UMeshSculptLayerProperties::SetActiveLayer(const int32 InIndex)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(InIndex))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not found. Invalid index: %d.", InIndex);
		return;
	}
	// account for the locked base layers
	const int32 AdjustedIndex = InIndex + NumLockedBaseLayers;

	// if the provided index is the already active layer, do nothing
	if (AdjustedIndex == ActiveLayer)
	{
		return;
	}

	ActiveLayer = FMath::Clamp(AdjustedIndex, NumLockedBaseLayers, LayerWeights.Num());
	FProperty* ActiveLayerProp = FindFProperty<FProperty>(UMeshSculptLayerProperties::StaticClass(),
		GET_MEMBER_NAME_CHECKED(UMeshSculptLayerProperties, ActiveLayer));
	FPropertyChangedEvent PropertyChangedEvent(ActiveLayerProp);
	#if WITH_EDITOR
	PostEditChangeProperty(PropertyChangedEvent);
	#endif
}

void UMeshSculptLayerProperties::SetLayerWeight(const int32 InIndex, double InWeight, const EPropertyChangeType::Type ChangeType)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(InIndex))
	{
		UE_LOGF(LogSculptLayerProps, Warning, "Sculpt Layer not found. Invalid index: %d.", InIndex);
		return;
	}
	
	Modify();
	
	// adjust index to take into account any locked base layers
	const int32 AdjustedIndex = InIndex + NumLockedBaseLayers;

	// do not allow the ActiveLayer's weight to be set to 0
	if (AdjustedIndex == ActiveLayer && InWeight == 0.0)
	{
		InWeight = 0.1;
	}
	
	LayerWeights[InIndex] = InWeight;

	EditSculptLayers([this, &AdjustedIndex, &InWeight](UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)
	{
		SetLayerWeights(SculptLayers);
	}, ChangeType != EPropertyChangeType::Interactive);
}

#undef LOCTEXT_NAMESPACE

