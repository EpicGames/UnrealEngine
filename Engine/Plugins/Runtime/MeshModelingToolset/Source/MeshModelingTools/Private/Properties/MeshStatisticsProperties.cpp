// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/MeshStatisticsProperties.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"

#include "Components/DynamicMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshStatisticsProperties)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshStatisticsProperites"


void UMeshStatisticsProperties::Update(const FDynamicMesh3& MeshIn)
{
	this->Mesh = FString::Printf(TEXT("T: %d  V: %d  E: %d"), MeshIn.TriangleCount(), MeshIn.VertexCount(), MeshIn.EdgeCount());

	if (MeshIn.HasAttributes())
	{
		const FDynamicMeshAttributeSet* Attribs = MeshIn.Attributes();

		FString UVString;
		for ( int k = 0; k < Attribs->NumUVLayers(); k++)
		{ 
			const FDynamicMeshUVOverlay* UVLayer = Attribs->GetUVLayer(k);
			UVString += FString::Printf(TEXT("UV%d: %d  "), k, UVLayer->ElementCount());
		}
		this->UV = UVString;

 		this->Attributes = FString::Printf(TEXT("Normals: %d"), Attribs->GetNormalLayer(0)->ElementCount());
	}
	else
	{
		this->UV = FString(TEXT("none"));
		this->Attributes = FString(TEXT("none"));
	}

}



#undef LOCTEXT_NAMESPACE

