// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/MeshUVChannelProperties.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "MeshDescription.h"
#include "StaticMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshUVChannelProperties)


void UMeshUVChannelProperties::Initialize(int32 NumUVChannels, bool bInitializeSelection)
{
	UVChannelNamesList.Reset();
	for (int32 k = 0; k < NumUVChannels; ++k)
	{
		UVChannelNamesList.Add(FString::Printf(TEXT("UV %d"), k));
	}
	if (bInitializeSelection)
	{
		UVChannel = (NumUVChannels > 0) ? UVChannelNamesList[0] : TEXT("");
	}
}


const TArray<FString>& UMeshUVChannelProperties::GetUVChannelNamesFunc() const
{
	return UVChannelNamesList;
}


void UMeshUVChannelProperties::Initialize(const FMeshDescription* MeshDescription, bool bInitializeSelection)
{
	TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs =
		MeshDescription->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	Initialize(InstanceUVs.GetNumChannels(), bInitializeSelection);
}

void UMeshUVChannelProperties::Initialize(const FDynamicMesh3* Mesh, bool bInitializeSelection)
{
	int32 NumUVChannels = Mesh->HasAttributes() ? Mesh->Attributes()->NumUVLayers() : 0;
	Initialize(NumUVChannels, bInitializeSelection);
}



bool UMeshUVChannelProperties::ValidateSelection(bool bUpdateIfInvalid)
{
	int32 FoundIndex = UVChannelNamesList.IndexOfByKey(UVChannel);
	if (FoundIndex == INDEX_NONE)
	{
		if (bUpdateIfInvalid)
		{
			UVChannel = (UVChannelNamesList.Num() > 0) ? UVChannelNamesList[0] : TEXT("");
		}
		return false;
	}
	return true;
}

int32 UMeshUVChannelProperties::GetSelectedChannelIndex(bool bForceToZeroOnFailure)
{
	int32 FoundIndex = UVChannelNamesList.IndexOfByKey(UVChannel);
	if (FoundIndex == INDEX_NONE)
	{
		return (bForceToZeroOnFailure ) ? 0 : -1;
	}
	return FoundIndex;
}
