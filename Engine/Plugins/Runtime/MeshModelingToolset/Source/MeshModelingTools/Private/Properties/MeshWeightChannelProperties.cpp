// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/MeshWeightChannelProperties.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWeightChannelProperties)

void UMeshWeightChannelDensityProperties::InitializeAttributes(const UE::Geometry::FDynamicMesh3* const Mesh)
{
	using namespace UE::Geometry;

	AttributeNames.Reset();
	if (const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
	{
		const int32 NumLayers = Attributes->NumWeightLayers();
		AttributeNames.SetNum(NumLayers);
		for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
		{
			AttributeNames[LayerIdx] = Attributes->GetWeightLayer(LayerIdx)->GetName();
		}
	}
}

void UMeshWeightChannelDensityProperties::ValidateSelectedAttribute()
{
	const int32 FoundIndex = AttributeNames.IndexOfByKey(SelectedAttribute);
	if (FoundIndex == INDEX_NONE)
	{
		SelectedAttribute = (AttributeNames.Num() > 0) ? AttributeNames[0] : TEXT("");
	}
}

int32 UMeshWeightChannelDensityProperties::GetSelectedWeightAttributeIndex(const UE::Geometry::FDynamicMesh3* const Mesh) const
{
	using namespace UE::Geometry;

	int32 WeightMapAttributeLayerIndex = -1;
	if (const FDynamicMeshAttributeSet* const Attribs = Mesh->Attributes())
	{
		for (int32 AttributeIndex = 0; AttributeIndex < Attribs->NumWeightLayers(); ++AttributeIndex)
		{
			const FDynamicMeshWeightAttribute* const AttributeLayer = Attribs->GetWeightLayer(AttributeIndex);

			if (AttributeLayer->GetName() == SelectedAttribute)
			{
				WeightMapAttributeLayerIndex = AttributeIndex;
				break;
			}
		}
	}
	return WeightMapAttributeLayerIndex;
}