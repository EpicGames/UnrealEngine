// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/RevolveProperties.h"

#include "CompositionOps/CurveSweepOp.h"
#include "Properties/MeshMaterialProperties.h"
#include "Util/RevolveUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RevolveProperties)

using namespace UE::Geometry;

void URevolveProperties::ApplyToCurveSweepOp(const UNewMeshMaterialProperties& MaterialProperties,
	const FVector3d& RevolutionAxisOrigin, const FVector3d& RevolutionAxisDirection,
	FCurveSweepOp& CurveSweepOpOut) const
{
	// Reversing the profile curve flips the mesh. This may need to be done if the curve wasn't drawn in the default
	// (counterclockwise) direction or if we're not revolving in the default (counterclockwise) direction, or if
	// the user asks.
	bool bReverseProfileCurve = !RevolveUtil::ProfileIsCCWRelativeRevolve(CurveSweepOpOut.ProfileCurve,
		RevolutionAxisOrigin, RevolutionAxisDirection, CurveSweepOpOut.bProfileCurveIsClosed);
	bReverseProfileCurve = bReverseProfileCurve ^ bFlipMesh ^ bReverseRevolutionDirection;
	if (bReverseProfileCurve)
	{
		for (int32 i = 0; i < CurveSweepOpOut.ProfileCurve.Num() / 2; ++i)
		{
			Swap(CurveSweepOpOut.ProfileCurve[i], CurveSweepOpOut.ProfileCurve[CurveSweepOpOut.ProfileCurve.Num() - 1 - i]);
		}
	}

	const double TotalRevolutionDegrees = (HeightOffsetPerDegree == 0) ? RevolveDegreesClamped : RevolveDegrees;

	const int32 Steps = bExplicitSteps ? NumExplicitSteps : FMath::CeilToInt(TotalRevolutionDegrees / StepsMaxDegrees);
	
	double DegreesPerStep = TotalRevolutionDegrees / Steps;
	double DegreesOffset = RevolveDegreesOffset;
	if (bReverseRevolutionDirection)
	{
		DegreesPerStep *= -1;
		DegreesOffset *= -1;
	}
	const double DownAxisOffsetPerStep = TotalRevolutionDegrees * HeightOffsetPerDegree / Steps;

	if (bPathAtMidpointOfStep && DegreesPerStep != 0 && abs(DegreesPerStep) < 180)
	{
		RevolveUtil::MakeProfileCurveMidpointOfFirstStep(CurveSweepOpOut.ProfileCurve, DegreesPerStep, RevolutionAxisOrigin, RevolutionAxisDirection);
	}

	// Generate the sweep curve
	CurveSweepOpOut.bSweepCurveIsClosed = bWeldFullRevolution && HeightOffsetPerDegree == 0 && TotalRevolutionDegrees == 360;
	const int32 NumSweepFrames = CurveSweepOpOut.bSweepCurveIsClosed ? Steps : Steps + 1; // If closed, last sweep frame is also first
	CurveSweepOpOut.SweepCurve.Reserve(NumSweepFrames);
	RevolveUtil::GenerateSweepCurve(RevolutionAxisOrigin, RevolutionAxisDirection, DegreesOffset,
		DegreesPerStep, DownAxisOffsetPerStep, NumSweepFrames, CurveSweepOpOut.SweepCurve);

	// Weld any vertices that are on the axis
	if (bWeldVertsOnAxis && DownAxisOffsetPerStep == 0)
	{
		RevolveUtil::WeldPointsOnAxis(CurveSweepOpOut.ProfileCurve, RevolutionAxisOrigin,
			RevolutionAxisDirection, AxisWeldTolerance, CurveSweepOpOut.ProfileVerticesToWeld);
	}
	CurveSweepOpOut.bSharpNormals = bSharpNormals;
	CurveSweepOpOut.SharpNormalAngleTolerance = SharpNormalsDegreeThreshold;
	CurveSweepOpOut.DiagonalTolerance = QuadSplitCompactTolerance;
	const double UVScale = MaterialProperties.UVScale;
	CurveSweepOpOut.UVScale = FVector2d(UVScale, UVScale);
	if (bReverseProfileCurve ^ bFlipVs)
	{
		CurveSweepOpOut.UVScale[1] *= -1;
		CurveSweepOpOut.UVOffset = FVector2d(0, UVScale);
	}
	CurveSweepOpOut.bUVsSkipFullyWeldedEdges = bUVsSkipFullyWeldedEdges;
	CurveSweepOpOut.bUVScaleRelativeWorld = MaterialProperties.bWorldSpaceUVScale;
	CurveSweepOpOut.UnitUVInWorldCoordinates = 100; // This seems to be the case in the AddPrimitiveTool
	switch (PolygroupMode)
	{
	case ERevolvePropertiesPolygroupMode::PerShape:
		CurveSweepOpOut.PolygonGroupingMode = EProfileSweepPolygonGrouping::Single;
		break;
	case ERevolvePropertiesPolygroupMode::PerFace:
		CurveSweepOpOut.PolygonGroupingMode = EProfileSweepPolygonGrouping::PerFace;
		break;
	case ERevolvePropertiesPolygroupMode::PerRevolveStep:
		CurveSweepOpOut.PolygonGroupingMode = EProfileSweepPolygonGrouping::PerSweepSegment;
		break;
	case ERevolvePropertiesPolygroupMode::PerPathSegment:
		CurveSweepOpOut.PolygonGroupingMode = EProfileSweepPolygonGrouping::PerProfileSegment;
		break;
	}
	switch (QuadSplitMode)
	{
	case ERevolvePropertiesQuadSplit::Compact:
		CurveSweepOpOut.QuadSplitMode = EProfileSweepQuadSplit::ShortestDiagonal;
		break;
	case ERevolvePropertiesQuadSplit::Uniform:
		CurveSweepOpOut.QuadSplitMode = EProfileSweepQuadSplit::Uniform;
		break;
	}
	switch (GetCapFillMode())
	{
	case ERevolvePropertiesCapFillMode::None:
		CurveSweepOpOut.CapFillMode = FCurveSweepOp::ECapFillMode::None;
		break;
	case ERevolvePropertiesCapFillMode::Delaunay:
		CurveSweepOpOut.CapFillMode = FCurveSweepOp::ECapFillMode::Delaunay;
		break;
	case ERevolvePropertiesCapFillMode::EarClipping:
		CurveSweepOpOut.CapFillMode = FCurveSweepOp::ECapFillMode::EarClipping;
		break;
	case ERevolvePropertiesCapFillMode::CenterFan:
		CurveSweepOpOut.CapFillMode = FCurveSweepOp::ECapFillMode::CenterFan;
		break;
	}
}

