// Copyright Epic Games, Inc. All Rights Reserved.

#include "RecomputeUVsTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Polygroups/PolygroupUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "ParameterizationOps/RecomputeUVsOp.h"
#include "ToolTargetManager.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(RecomputeUVsTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "URecomputeUVsTool"


/*
 * ToolBuilder
 */

USingleSelectionMeshEditingTool* URecomputeUVsToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	URecomputeUVsTool* NewTool = NewObject<URecomputeUVsTool>(SceneState.ToolManager);
	return NewTool;
}

bool URecomputeUVsToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return USingleSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) && 
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return ToolBuilderUtil::ComponentTypeCouldHaveUVs(Component); }) > 0;
}

/*
 * Tool
 */


void URecomputeUVsTool::Setup()
{
	UInteractiveTool::Setup();

	InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*InputMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	FTransform TargetTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);

	// initialize our properties

	Settings = NewObject<URecomputeUVsToolProperties>(this);
	Settings->RestoreProperties(this);
	Settings->bUDIMCVAREnabled = false;
	Settings->bEnableUDIMLayout = false;
	AddToolPropertySource(Settings);

	UVChannelProperties = NewObject<UMeshUVChannelProperties>(this);
	UVChannelProperties->RestoreProperties(this);
	UVChannelProperties->Initialize(InputMesh.Get(), false);
	UVChannelProperties->ValidateSelection(true);
	UVChannelProperties->WatchProperty(UVChannelProperties->UVChannel, [this](const FString& NewValue)
		{
			MaterialSettings->UpdateUVChannels(UVChannelProperties->UVChannelNamesList.IndexOfByKey(UVChannelProperties->UVChannel),
			                                   UVChannelProperties->UVChannelNamesList);
		});
	AddToolPropertySource(UVChannelProperties);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->RestoreProperties(this, TEXT("RecomputeUVsTool"));
	PolygroupLayerProperties->InitializeGroupLayers(InputMesh.Get());
	PolygroupLayerProperties->WatchProperty(PolygroupLayerProperties->ActiveGroupLayer, [&](FName) { OnSelectedGroupLayerChanged(); });
	AddToolPropertySource(PolygroupLayerProperties);
	UpdateActiveGroupLayer();

	MaterialSettings = NewObject<UExistingMeshMaterialProperties>(this);
	MaterialSettings->MaterialMode = ESetMeshMaterialMode::Checkerboard;
	MaterialSettings->RestoreProperties(this, TEXT("ModelingUVTools"));
	AddToolPropertySource(MaterialSettings);

	UE::ToolTarget::HideSourceObject(Target);

	// force update
	MaterialSettings->UpdateMaterials();

	RecomputeUVsOpFactory = NewObject<URecomputeUVsOpFactory>();
	RecomputeUVsOpFactory->OriginalMesh = InputMesh;
	RecomputeUVsOpFactory->InputGroups = ActiveGroupSet;
	RecomputeUVsOpFactory->Settings = Settings;
	RecomputeUVsOpFactory->TargetTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	RecomputeUVsOpFactory->GetSelectedUVChannel = [this]() { return GetSelectedUVChannel(); };

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(GetTargetWorld(), RecomputeUVsOpFactory);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->PreviewMesh->ReplaceMesh(*InputMesh);
	Preview->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	Preview->PreviewMesh->SetTransform(TargetTransform);

	Preview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Op)
		{
			OnPreviewMeshUpdated();
		});

	Preview->OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();

	if (bCreateUVLayoutViewOnSetup)
	{
		UVLayoutView = NewObject<UUVLayoutPreview>(this);
		UVLayoutView->CreateInWorld(GetTargetWorld());
		UVLayoutView->SetSourceMaterials(MaterialSet);
		UVLayoutView->SetSourceWorldPosition(TargetTransform, UE::ToolTarget::GetTargetActor(Target)->GetComponentsBoundingBox());
		UVLayoutView->Settings->bEnabled = false;
		UVLayoutView->Settings->bShowWireframe = false;
		UVLayoutView->Settings->RestoreProperties(this, TEXT("RecomputeUVsTool"));
		AddToolPropertySource(UVLayoutView->Settings);
	}

	Preview->InvalidateResult();    // start compute

	SetToolDisplayName(LOCTEXT("ToolNameLocal", "UV Unwrap"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool_Regions", "Generate UVs for PolyGroups or existing UV islands of the mesh using various strategies."),
		EToolMessageLevel::UserNotification);
}


void URecomputeUVsTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	bool bForceMaterialUpdate = false;
	if (PropertySet == Settings || PropertySet == UVChannelProperties)
	{
		// One of the UV generation properties must have changed.  Dirty the result to force a recompute
		Preview->InvalidateResult();
		bForceMaterialUpdate = true;
	}

	if (PropertySet == MaterialSettings || bForceMaterialUpdate)
	{
		MaterialSettings->UpdateMaterials();
		Preview->OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
	}
}


void URecomputeUVsTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (UVLayoutView)
	{
		UVLayoutView->Settings->SaveProperties(this, TEXT("RecomputeUVsTool"));
		UVLayoutView->Disconnect();
	}

	UVChannelProperties->SaveProperties(this);
	Settings->SaveProperties(this);
	PolygroupLayerProperties->RestoreProperties(this, TEXT("RecomputeUVsTool"));
	MaterialSettings->SaveProperties(this, TEXT("ModelingUVTools"));

	UE::ToolTarget::ShowSourceObject(Target);

	FDynamicMeshOpResult Result = Preview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("RecomputeUVs", "Recompute UVs"));
		FDynamicMesh3* NewDynamicMesh = Result.Mesh.Get();
		if (ensure(NewDynamicMesh))
		{
			UE::ToolTarget::CommitDynamicMeshUVUpdate(Target, NewDynamicMesh);
		}
		GetToolManager()->EndUndoTransaction();
	}

	if (RecomputeUVsOpFactory)
	{
		RecomputeUVsOpFactory = nullptr;
	}

}

void URecomputeUVsTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (UVLayoutView)
	{
		UVLayoutView->Render(RenderAPI);
	}
}


void URecomputeUVsTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);

	if (UVLayoutView)
	{
		UVLayoutView->OnTick(DeltaTime);
	}
}

bool URecomputeUVsTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidResult();
}


int32 URecomputeUVsTool::GetSelectedUVChannel() const
{
	return UVChannelProperties ? UVChannelProperties->GetSelectedChannelIndex(true) : 0;
}

void URecomputeUVsTool::OnSelectedGroupLayerChanged()
{
	UpdateActiveGroupLayer();
	Preview->InvalidateResult();
}


void URecomputeUVsTool::UpdateActiveGroupLayer()
{
	if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
	{
		ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(InputMesh.Get());
	}
	else
	{
		FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
		FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*InputMesh, SelectedName);
		ensureMsgf(FoundAttrib, TEXT("Selected Attribute Not Found! Falling back to Default group layer."));
		ActiveGroupSet = MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(InputMesh.Get(), FoundAttrib);
	}
	if (RecomputeUVsOpFactory)
	{
		RecomputeUVsOpFactory->InputGroups = ActiveGroupSet;
	}
}


void URecomputeUVsTool::OnPreviewMeshUpdated()
{
	if (UVLayoutView)
	{
		int32 UVChannel = UVChannelProperties ? UVChannelProperties->GetSelectedChannelIndex(true) : 0;
		Preview->PreviewMesh->ProcessMesh([&](const FDynamicMesh3& NewMesh)
		{
			UVLayoutView->UpdateUVMesh(&NewMesh, UVChannel);
		});
	}

	if (MaterialSettings)
	{
		MaterialSettings->UpdateMaterials();
	}

}



#undef LOCTEXT_NAMESPACE

