// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemeshMeshTool.h"
#include "ComponentSourceInterfaces.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "Properties/RemeshProperties.h"
#include "Properties/MeshStatisticsProperties.h"
#include "Drawing/MeshElementsVisualizer.h"

#include "Util/ColorConstants.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "DynamicMeshToMeshDescription.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemeshMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "URemeshMeshTool"



/*
 * ToolBuilder
 */

bool URemeshMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

UMultiSelectionMeshEditingTool* URemeshMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<URemeshMeshTool>(SceneState.ToolManager);
}

const FToolTargetTypeRequirements& URemeshMeshToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
	});
	return TypeRequirements;
}

/*
 * Tool
 */
URemeshMeshToolProperties::URemeshMeshToolProperties()
{
	TargetTriangleCount = 5000;
	SmoothingStrength = 0.25;
	RemeshIterations = 20;
	MaxRemeshIterations = 20;
	ExtraProjectionIterations = 5;
	bDiscardAttributes = false;
	RemeshType = ERemeshType::Standard;
	SmoothingType = ERemeshSmoothingType::MeanValue;
	bPreserveSharpEdges = true;
	bShowGroupColors = false;

	TargetEdgeLength = 5.0;
	bFlips = true;
	bSplits = true;
	bCollapses = true;
	bReproject = true;
	bReprojectConstraints = false;
	BoundaryCornerAngleThreshold = 45.0;
	bPreventNormalFlips = true;
	bPreventTinyTriangles = true;
	bUseTargetEdgeLength = false;
}

URemeshMeshTool::URemeshMeshTool(const FObjectInitializer&)
{
	BasicProperties = CreateDefaultSubobject<URemeshMeshToolProperties>(TEXT("RemeshProperties"));
	// CreateDefaultSubobject automatically sets RF_Transactional flag, we need to clear it so that undo/redo doesn't affect tool properties
	BasicProperties->ClearFlags(RF_Transactional);
}


void URemeshMeshTool::Setup()
{
	UInteractiveTool::Setup();

	check(BasicProperties);
	BasicProperties->RestoreProperties(this);
	MeshStatisticsProperties = NewObject<UMeshStatisticsProperties>(this);

	check(Targets.Num() > 0);
	check(Targets[0]);

	// hide component and create + show preview
	UE::ToolTarget::HideSourceObject(Targets[0]);
	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(GetTargetWorld(), this);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Targets[0]);

	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
	Preview->ConfigureMaterials( MaterialSet.Materials,
								 ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);
	BasicProperties->WatchProperty(BasicProperties->bShowGroupColors,
								   [this](bool bNewValue) { UpdateVisualization();});

	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(Targets[0]));

	Preview->PreviewMesh->SetTransform((FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[0]));
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->PreviewMesh->UpdatePreview(OriginalMesh.Get());

	OriginalMeshSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(OriginalMesh.Get(), true);

	// calculate initial mesh area
	InitialMeshArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(*OriginalMesh).Y;

	// set properties defaults

	constexpr int32 MinInitialTriangles = 5000;
	constexpr int32 MaxInitialTriangles = 100'000;
	BasicProperties->TargetTriangleCount = FMath::Clamp(OriginalMesh->TriangleCount(), MinInitialTriangles, MaxInitialTriangles);
	BasicProperties->TargetEdgeLength = FRemeshMeshOp::CalculateTargetEdgeLength(OriginalMesh.Get(), BasicProperties->TargetTriangleCount, InitialMeshArea);

	WeightMapProperties = NewObject<UMeshWeightChannelDensityProperties>(this);
	WeightMapProperties->RestoreProperties(this);
	WeightMapProperties->InitializeAttributes(OriginalMesh.Get());
	WeightMapProperties->ValidateSelectedAttribute();

	// Attribute properties
	WeightMapProperties->WatchProperty(WeightMapProperties->SelectedAttribute,
		[this](const FName& NewName)
		{
			WeightMapProperties->ValidateSelectedAttribute();
		});

	// add properties to GUI
	AddToolPropertySource(BasicProperties);
	AddToolPropertySource(WeightMapProperties);
	AddToolPropertySource(MeshStatisticsProperties);


	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(Preview->PreviewMesh->GetWorld(), Preview->PreviewMesh->GetTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowWireframe = true;
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("Remesh"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		Preview->ProcessCurrentMesh(ProcessFunc);
	});

	Preview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute)
	{
		Compute->ProcessCurrentMesh([&](const FDynamicMesh3& ReadMesh)
		{
			MeshStatisticsProperties->Update(ReadMesh);
			MeshElementsDisplay->NotifyMeshChanged();
		});
	});


	Preview->InvalidateResult();

	SetToolDisplayName(LOCTEXT("ToolName", "Remesh"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Retriangulate the selected Mesh. Use the Boundary Constraints to preserve mesh borders. Enable Discard Attributes to ignore UV/Normal Seams. "),
		EToolMessageLevel::UserNotification);
}

void URemeshMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	BasicProperties->SaveProperties(this);
	WeightMapProperties->SaveProperties(this);

	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, TEXT("Remesh"));
	}
	MeshElementsDisplay->Disconnect();


	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	FDynamicMeshOpResult Result = Preview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("RemeshMeshToolTransactionName", "Remesh Mesh"));
		UE::ToolTarget::CommitDynamicMeshUpdate(Targets[0], *Result.Mesh, true);
		GetToolManager()->EndUndoTransaction();
	}
}

void URemeshMeshTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);
}

TUniquePtr<FDynamicMeshOperator> URemeshMeshTool::MakeNewOperator()
{
	TUniquePtr<FRemeshMeshOp> Op = MakeUnique<FRemeshMeshOp>();

	Op->RemeshType = BasicProperties->RemeshType;

	if (!BasicProperties->bUseTargetEdgeLength)
	{
		Op->TargetEdgeLength = FRemeshMeshOp::CalculateTargetEdgeLength(OriginalMesh.Get(), BasicProperties->TargetTriangleCount, InitialMeshArea);
		Op->TriangleCountHint = 2.0 * BasicProperties->TargetTriangleCount;
	}
	else
	{
		Op->TargetEdgeLength = BasicProperties->TargetEdgeLength;
	}

	Op->bCollapses = BasicProperties->bCollapses;
	Op->bDiscardAttributes = BasicProperties->bDiscardAttributes;
	// We always want attributes enabled on result even if we discard them initially
	Op->bResultMustHaveAttributesEnabled = true;
	Op->bFlips = BasicProperties->bFlips;
	Op->bPreserveSharpEdges = BasicProperties->bPreserveSharpEdges;
	Op->MeshBoundaryConstraint = (EEdgeRefineFlags)BasicProperties->MeshBoundaryConstraint;
	Op->GroupBoundaryConstraint = (EEdgeRefineFlags)BasicProperties->GroupBoundaryConstraint;
	Op->MaterialBoundaryConstraint = (EEdgeRefineFlags)BasicProperties->MaterialBoundaryConstraint;
	Op->bPreventNormalFlips = BasicProperties->bPreventNormalFlips;
	Op->bPreventTinyTriangles = BasicProperties->bPreventTinyTriangles;
	Op->bReproject = BasicProperties->bReproject;
	Op->bReprojectConstraints = BasicProperties->bReprojectConstraints;
	Op->BoundaryCornerAngleThreshold = BasicProperties->BoundaryCornerAngleThreshold;
	Op->bSplits = BasicProperties->bSplits;
	Op->RemeshIterations = BasicProperties->RemeshIterations;
	Op->MaxRemeshIterations = BasicProperties->MaxRemeshIterations;
	Op->ExtraProjectionIterations = BasicProperties->ExtraProjectionIterations;
	Op->SmoothingStrength = BasicProperties->SmoothingStrength;
	Op->SmoothingType = BasicProperties->SmoothingType;

	FTransform LocalToWorld = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
	Op->SetTransform(LocalToWorld);

	Op->OriginalMesh = OriginalMesh;
	Op->OriginalMeshSpatial = OriginalMeshSpatial;

	Op->ProjectionTarget = nullptr;
	Op->ProjectionTargetSpatial = nullptr;


	if (WeightMapProperties->bUseWeightMap)
	{
		int32 WeightMapAttributeLayerIndex = WeightMapProperties->GetSelectedWeightAttributeIndex(OriginalMesh.Get());

		if (WeightMapAttributeLayerIndex >= 0)
		{
			Op->bNeedsWeightMaps = true;
			Op->CustomEdgeLengthScaleF = [WeightMapAttributeLayerIndex, RelativeDensity = WeightMapProperties->RelativeDensity](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
			{
				if (Mesh.HasAttributes() && Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex))
				{
					const FDynamicMeshWeightAttribute* const AttributeLayer = Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex);

					float WeightValueA;
					AttributeLayer->GetValue(VertexA, &WeightValueA);
					WeightValueA = FMath::Clamp(WeightValueA, 0.0f, 1.0f);

					float WeightValueB;
					AttributeLayer->GetValue(VertexB, &WeightValueB);
					WeightValueB = FMath::Clamp(WeightValueB, 0.0f, 1.0f);

					const double AvgWeight = 0.5 * (WeightValueA + WeightValueB);

					return FMath::Pow(0.5, -RelativeDensity * AvgWeight);
				}

				return 1.0;
			};
		}
	}

	return Op;
}


void URemeshMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if ( Property )
	{
		if ( Property->GetFName() != GET_MEMBER_NAME_CHECKED(URemeshMeshToolProperties, bShowGroupColors) )
		{
			Preview->InvalidateResult();
		}
	}
}

void URemeshMeshTool::UpdateVisualization()
{
	if (BasicProperties->bShowGroupColors)
	{
		Preview->OverrideMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager());
		Preview->PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
		{
			return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
		},
		UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	else
	{
		Preview->OverrideMaterial = nullptr;
		Preview->PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
}

bool URemeshMeshTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidResult();
}



#undef LOCTEXT_NAMESPACE

