// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoveOccludedTrianglesTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "Polygroups/PolygroupUtil.h"
#include "BaseBehaviors/MultiClickSequenceInputBehavior.h"
#include "Selection/SelectClickedAction.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "InteractiveGizmoManager.h"

#include "Misc/MessageDialog.h"


#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#endif

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoveOccludedTrianglesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "URemoveOccludedTrianglesTool"

/*
 * ToolBuilder
 */


UMultiSelectionMeshEditingTool* URemoveOccludedTrianglesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<URemoveOccludedTrianglesTool>(SceneState.ToolManager);
}




/*
 * Tool
 */

URemoveOccludedTrianglesToolProperties::URemoveOccludedTrianglesToolProperties()
{

}

URemoveOccludedTrianglesAdvancedProperties::URemoveOccludedTrianglesAdvancedProperties()
{
}


URemoveOccludedTrianglesTool::URemoveOccludedTrianglesTool()
{
	SetToolDisplayName(LOCTEXT("ProjectToTargetToolName", "Jacket"));
}

void URemoveOccludedTrianglesTool::Setup()
{
	UInteractiveTool::Setup();

	// hide input StaticMeshComponent
	for (int Idx = 0; Idx < Targets.Num(); Idx++)
	{
		UE::ToolTarget::HideSourceObject(Targets[Idx]);
	}


	// find components with the same source asset
	TArray<int32> MapToFirstOccurrences;
	bool bAnyHasSameSource = GetMapToSharedSourceData(MapToFirstOccurrences);
	TargetToPreviewIdx.SetNum(Targets.Num());
	PreviewToTargetIdx.Reset();
	for (int32 ComponentIdx = 0; ComponentIdx < MapToFirstOccurrences.Num(); ComponentIdx++)
	{
		if (MapToFirstOccurrences[ComponentIdx] == ComponentIdx)
		{
			int32 NumPreviews = PreviewToTargetIdx.Num();
			PreviewToTargetIdx.Add(ComponentIdx);
			TargetToPreviewIdx[ComponentIdx] = NumPreviews;
		}
		else
		{
			TargetToPreviewIdx[ComponentIdx] = TargetToPreviewIdx[MapToFirstOccurrences[ComponentIdx]];
		}
	}

	if (bAnyHasSameSource)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("JacketingMultipleAssetWithSameSource", "WARNING: Multiple meshes in your selection use the same source asset!  Triangles will be conservatively removed from these meshes only when they are occluded in every selected instance."),
			EToolMessageLevel::UserWarning);
	}

	// initialize the PreviewMesh+BackgroundCompute object
	SetupPreviews();

	BasicProperties = NewObject<URemoveOccludedTrianglesToolProperties>(this, TEXT("Remove Occluded Triangle Settings"));
	AdvancedProperties = NewObject<URemoveOccludedTrianglesAdvancedProperties>(this, TEXT("Advanced Settings"));
	MakePolygroupLayerProperties();

	BasicProperties->WatchProperty(BasicProperties->Action,
		[this](EOccludedAction Action) { SetToolPropertySourceEnabled(PolygroupLayersProperties, Action == EOccludedAction::SetNewGroup); }
	);

	// initialize our properties
	AddToolPropertySource(BasicProperties);
	AddToolPropertySource(PolygroupLayersProperties);
	AddToolPropertySource(AdvancedProperties);

	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}

	GetToolManager()->DisplayMessage(
		LOCTEXT("RemoveOccludedTrianglesToolDescription", "Remove triangles that are fully contained within the selected Meshes, and hence cannot be visible with opaque shading."),
		EToolMessageLevel::UserNotification);
}


void URemoveOccludedTrianglesTool::MakePolygroupLayerProperties()
{
	PolygroupLayersProperties = NewObject<UPolygroupLayersProperties>(this, TEXT("Polygroup Layer"));
	auto GetGroupLayerNames = [](const FDynamicMesh3& Mesh)
	{
		TSet<FName> Names;

		if (Mesh.Attributes())
		{
			for (int32 LayerIdx = 0; LayerIdx < Mesh.Attributes()->NumPolygroupLayers(); LayerIdx++)
			{
				FName Name = Mesh.Attributes()->GetPolygroupLayer(LayerIdx)->GetName();
				Names.Add(Name);
			}
		}

		return Names;
	};
	check(OriginalDynamicMeshes.Num() > 0);
	TSet<FName> CommonLayerNames = GetGroupLayerNames(*OriginalDynamicMeshes[0]);
	for (int32 Idx = 1; Idx < OriginalDynamicMeshes.Num(); Idx++)
	{
		CommonLayerNames = CommonLayerNames.Intersect(GetGroupLayerNames(*OriginalDynamicMeshes[Idx]));
	}
	PolygroupLayersProperties->InitializeGroupLayers(CommonLayerNames);
}


void URemoveOccludedTrianglesTool::SetupPreviews()
{	
	int32 NumTargets = Targets.Num();
	int32 NumPreviews = PreviewToTargetIdx.Num();

#if WITH_EDITOR
	static const FText SlowTaskText = LOCTEXT("RemoveOccludedTrianglesInit", "Building mesh occlusion data...");

	FScopedSlowTask SlowTask(NumTargets, SlowTaskText);
	SlowTask.MakeDialog();

	// Declare progress shortcut lambdas
	auto EnterProgressFrame = [&SlowTask](float Progress)
	{
		SlowTask.EnterProgressFrame(Progress);
	};
#else
	auto EnterProgressFrame = [](float Progress) {};
#endif

	// create a "magic pink" secondary material to mark occluded faces (to be used if we are setting a new triangle group instead of removing faces)
	UMaterialInterface* OccludedMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.9f, 0.1f, 0.9f), GetToolManager());

	OccludedGroupIDs.Init(-1, NumPreviews);
	OccludedGroupLayers.Init(-1, NumPreviews);

	OccluderTrees.SetNum(NumTargets);
	OccluderWindings.SetNum(NumTargets);
	OccluderTransforms.SetNum(NumTargets);

	OriginalDynamicMeshes.SetNum(NumPreviews);
	PreviewToCopyIdx.Reset(); PreviewToCopyIdx.SetNum(NumPreviews);
	for (int32 TargetIdx = 0; TargetIdx < NumTargets; TargetIdx++)
	{
		EnterProgressFrame(1);
		
		int PreviewIdx = TargetToPreviewIdx[TargetIdx];

		// used to choose which triangles need to use the special "OccludedMaterial" secondary material, when the Occluded Action == SetNewGroup
		auto IsOccludedGroupFn = [this, PreviewIdx](const FDynamicMesh3* Mesh, int32 TriangleID)
		{
			int GroupID = OccludedGroupIDs[PreviewIdx];
			if (GroupID >= 0)
			{
				int LayerIndex = OccludedGroupLayers[PreviewIdx];
				if (LayerIndex < 0)
				{
					return Mesh->GetTriangleGroup(TriangleID) == GroupID;
				}
				else
				{
					check(Mesh->HasAttributes() && Mesh->Attributes()->NumPolygroupLayers() > LayerIndex);
					return Mesh->Attributes()->GetPolygroupLayer(LayerIndex)->GetValue(TriangleID) == GroupID;
				}
			}
			return false;
		};

		bool bHasConverted = OriginalDynamicMeshes[PreviewIdx].IsValid();
		
		if (!bHasConverted)
		{
			URemoveOccludedTrianglesOperatorFactory *OpFactory = NewObject<URemoveOccludedTrianglesOperatorFactory>();
			OpFactory->Tool = this;
			OpFactory->PreviewIdx = PreviewIdx;
			OriginalDynamicMeshes[PreviewIdx] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
			*OriginalDynamicMeshes[PreviewIdx] = UE::ToolTarget::GetDynamicMeshCopy(Targets[TargetIdx]);

			UMeshOpPreviewWithBackgroundCompute* Preview = Previews.Add_GetRef(NewObject<UMeshOpPreviewWithBackgroundCompute>(OpFactory, "Preview"));
			Preview->Setup(GetTargetWorld(), OpFactory);
			ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
			Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

			const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[TargetIdx]);
			Preview->ConfigureMaterials(MaterialSet.Materials,
				ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
			);

			Preview->PreviewMesh->SetTransform((FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]));
			Preview->PreviewMesh->UpdatePreview(OriginalDynamicMeshes[PreviewIdx].Get());
			Preview->SetVisibility(true);

			OccluderTrees[TargetIdx] = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(OriginalDynamicMeshes[PreviewIdx].Get());
			OccluderWindings[TargetIdx] = MakeShared<TFastWindingTree<FDynamicMesh3>, ESPMode::ThreadSafe>(OccluderTrees[TargetIdx].Get());
			OccluderTransforms[TargetIdx] = UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]);

			// configure secondary render material
			Preview->SecondaryMaterial = OccludedMaterial;

			// Set occluded layer index and group IDs
			Previews[PreviewIdx]->OnOpCompleted.AddLambda([this, PreviewIdx](const FDynamicMeshOperator* UncastOp) {
				const FRemoveOccludedTrianglesOp* Op = static_cast<const FRemoveOccludedTrianglesOp*>(UncastOp);
				OccludedGroupIDs[PreviewIdx] = Op->CreatedGroupID;
				OccludedGroupLayers[PreviewIdx] = Op->CreatedGroupLayerIndex;
			});
			
			// enable secondary triangle buffers
			Preview->PreviewMesh->EnableSecondaryTriangleBuffers(MoveTemp(IsOccludedGroupFn));
		}
		else
		{
			// already did the conversion for a full UMeshOpPreviewWithBackgroundCompute -- just make a light version of that and hook it up to copy the other's work
			int CopyIdx = PreviewCopies.Num();
			UPreviewMesh* PreviewMesh = PreviewCopies.Add_GetRef(NewObject<UPreviewMesh>(this));
			PreviewMesh->CreateInWorld(GetTargetWorld(), (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]));

			PreviewToCopyIdx[PreviewIdx].Add(CopyIdx);

			PreviewMesh->UpdatePreview(OriginalDynamicMeshes[PreviewIdx].Get());

			const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[TargetIdx]);
			PreviewMesh->SetMaterials(MaterialSet.Materials);
			
			PreviewMesh->SetVisible(true);

			OccluderTrees[TargetIdx] = OccluderTrees[PreviewToTargetIdx[PreviewIdx]];
			OccluderWindings[TargetIdx] = OccluderWindings[PreviewToTargetIdx[PreviewIdx]];
			OccluderTransforms[TargetIdx] = UE::ToolTarget::GetLocalToWorldTransform(Targets[TargetIdx]);

			PreviewMesh->SetSecondaryRenderMaterial(OccludedMaterial);
			PreviewMesh->EnableSecondaryTriangleBuffers(MoveTemp(IsOccludedGroupFn));

			Previews[PreviewIdx]->OnMeshUpdated.AddLambda([this, CopyIdx](UMeshOpPreviewWithBackgroundCompute* Compute) {
				PreviewCopies[CopyIdx]->UpdatePreview(Compute->PreviewMesh->GetPreviewDynamicMesh());
			});
		}
	}
}



void URemoveOccludedTrianglesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept && AreAllTargetsValid() == false)
	{
		UE_LOGF(LogTemp, Error, "Tool Target has become Invalid (possibly it has been Force Deleted). Aborting Tool.");
		ShutdownType = EToolShutdownType::Cancel;
	}

	// Restore (unhide) the source meshes
	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	// clear all the preview copies
	for (UPreviewMesh* PreviewMesh : PreviewCopies)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}
	PreviewCopies.Empty();

	TArray<FDynamicMeshOpResult> Results;
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Results.Add(Preview->Shutdown());
	}
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Results);
	}
}

TUniquePtr<FDynamicMeshOperator> URemoveOccludedTrianglesOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FRemoveOccludedTrianglesOp> Op = MakeUnique<FRemoveOccludedTrianglesOp>();
	Op->NormalOffset = Tool->AdvancedProperties->NormalOffset;
	Op->bSetTriangleGroupInsteadOfRemoving = Tool->BasicProperties->Action == EOccludedAction::SetNewGroup;
	Op->ActiveGroupLayer = Tool->PolygroupLayersProperties->ActiveGroupLayer;
	Op->bActiveGroupLayerIsDefault = !Tool->PolygroupLayersProperties->HasSelectedPolygroup();

	switch (Tool->BasicProperties->OcclusionTestMethod)
	{
	case EOcclusionCalculationUIMode::GeneralizedWindingNumber:
		Op->InsideMode = EOcclusionCalculationMode::FastWindingNumber;
		break;
	case EOcclusionCalculationUIMode::RaycastOcclusionSamples:
		Op->InsideMode = EOcclusionCalculationMode::SimpleOcclusionTest;
		break;
	default:
		ensure(false); // all cases should be handled
	}
	switch (Tool->BasicProperties->TriangleSampling)
	{
	case EOcclusionTriangleSamplingUIMode::Vertices:
		Op->TriangleSamplingMethod = EOcclusionTriangleSampling::Vertices;
		break;
// Centroids sampling not exposed in UI for now
// 	case EOcclusionTriangleSamplingUIMode::Centroids:
// 		Op->TriangleSamplingMethod = EOcclusionTriangleSampling::Centroids;
// 		break;
	case EOcclusionTriangleSamplingUIMode::VerticesAndCentroids:
		Op->TriangleSamplingMethod = EOcclusionTriangleSampling::VerticesAndCentroids;
		break;
	default:
		ensure(false);
	}
	Op->WindingIsoValue = Tool->BasicProperties->WindingIsoValue;

	int ComponentIndex = Tool->PreviewToTargetIdx[PreviewIdx];
	FTransform LocalToWorld = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Tool->Targets[ComponentIndex]);
	Op->OriginalMesh = Tool->OriginalDynamicMeshes[PreviewIdx];

	if (Tool->BasicProperties->bOnlySelfOcclude)
	{
		int32 TargetIdx = Tool->PreviewToTargetIdx[PreviewIdx];
		Op->OccluderTrees.Add(Tool->OccluderTrees[TargetIdx]);
		Op->OccluderWindings.Add(Tool->OccluderWindings[TargetIdx]);
		Op->OccluderTransforms.Add(FTransformSRT3d::Identity());
	}
	else
	{
		Op->OccluderTrees = Tool->OccluderTrees;
		Op->OccluderWindings = Tool->OccluderWindings;
		Op->OccluderTransforms = Tool->OccluderTransforms;
	}

	Op->AddRandomRays = Tool->BasicProperties->AddRandomRays;

	Op->AddTriangleSamples = Tool->BasicProperties->AddTriangleSamples;

	Op->ShrinkRemoval = Tool->BasicProperties->ShrinkRemoval;

	Op->MinAreaConnectedComponent = Tool->BasicProperties->MinAreaIsland;

	Op->MinTriCountConnectedComponent = Tool->BasicProperties->MinTriCountIsland;
	
	Op->SetTransform(LocalToWorld);

	Op->MeshTransforms.Add((FTransformSRT3d)LocalToWorld);
	for (int32 CopyIdx : Tool->PreviewToCopyIdx[PreviewIdx])
	{
		Op->MeshTransforms.Add((FTransformSRT3d)Tool->PreviewCopies[CopyIdx]->GetTransform());
	}

	return Op;
}

void URemoveOccludedTrianglesTool::OnTick(float DeltaTime)
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->Tick(DeltaTime);
	}
	// copy working material state to corresponding copies
	for (int PreviewIdx = 0; PreviewIdx < Previews.Num(); PreviewIdx++)
	{
		UMeshOpPreviewWithBackgroundCompute* Preview = Previews[PreviewIdx];
		bool bIsWorking = Preview->IsUsingWorkingMaterial();
		for (int CopyIdx : PreviewToCopyIdx[PreviewIdx])
		{
			if (bIsWorking)
			{
				PreviewCopies[CopyIdx]->SetOverrideRenderMaterial(Preview->WorkingMaterial);
				PreviewCopies[CopyIdx]->ClearSecondaryRenderMaterial();
			}
			else
			{
				PreviewCopies[CopyIdx]->ClearOverrideRenderMaterial();
				PreviewCopies[CopyIdx]->SetSecondaryRenderMaterial(Preview->SecondaryMaterial);
			}
		}
	}
}



#if WITH_EDITOR
void URemoveOccludedTrianglesTool::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}
#endif

void URemoveOccludedTrianglesTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}



bool URemoveOccludedTrianglesTool::CanAccept() const
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		if (!Preview->HaveValidResult())
		{
			return false;
		}
	}
	return Super::CanAccept();
}


void URemoveOccludedTrianglesTool::GenerateAsset(const TArray<FDynamicMeshOpResult>& Results)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("RemoveOccludedTrianglesToolTransactionName", "Remove Occluded Triangles"));

	check(Results.Num() == Previews.Num());

	// check if we entirely remove away any meshes
	bool bWantDestroy = false;
	for (int32 PreviewIdx = 0; PreviewIdx < Previews.Num(); PreviewIdx++)
	{
		bWantDestroy = bWantDestroy || (Results[PreviewIdx].Mesh.Get()->TriangleCount() == 0);
	}
	// if so ask user what to do
	if (bWantDestroy)
	{
		EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo,
			LOCTEXT("RemoveOccludedDestroyQuestion", "Jacketing has removed all triangles from at least one mesh. Do you actually want to delete these mesh components? Note that either way all actors will remain, and meshes that would not have all triangles removed will still be jacketed as normal."), 
			LOCTEXT("RemoveOccludedDestroyTitle", "Delete mesh components?"));
		if (Ret == EAppReturnType::No || Ret == EAppReturnType::Cancel)
		{
			bWantDestroy = false;
		}
	}

	FSelectedObjectsChangeList NewSelection;
	for (int32 PreviewIdx = 0; PreviewIdx < Previews.Num(); PreviewIdx++)
	{
		check(Results[PreviewIdx].Mesh.Get() != nullptr);
		int ComponentIdx = PreviewToTargetIdx[PreviewIdx];

		if (Results[PreviewIdx].Mesh.Get()->TriangleCount() == 0)
		{
			if (bWantDestroy)
			{
				for (int TargetIdx = 0; TargetIdx < Targets.Num(); TargetIdx++)
				{
					if (TargetToPreviewIdx[TargetIdx] == PreviewIdx)
					{
						UE::ToolTarget::GetTargetComponent(Targets[TargetIdx])->DestroyComponent();
					}
				}
			}
			continue;
		}

		UE::ToolTarget::CommitMeshDescriptionUpdateViaDynamicMesh(Targets[ComponentIdx], *Results[PreviewIdx].Mesh, true);

		NewSelection.Actors.Add(UE::ToolTarget::GetTargetActor(Targets[ComponentIdx]));
	}

	// If we destroyed component(s) for empty mesh(es), ensure we update the selection
	// to avoid the details panels from crashing when updating on next engine tick.
	if (bWantDestroy)
	{
		NewSelection.ModificationType = NewSelection.Actors.Num() > 0 ? ESelectedObjectsModificationType::Replace : ESelectedObjectsModificationType::Clear;
		GetToolManager()->RequestSelectionChange(NewSelection);
	}

	GetToolManager()->EndUndoTransaction();
}




#undef LOCTEXT_NAMESPACE

