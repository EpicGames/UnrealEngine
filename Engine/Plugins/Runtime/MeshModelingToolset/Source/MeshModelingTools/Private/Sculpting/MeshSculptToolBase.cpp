// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Sculpting/MeshSculptToolBase.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "ToolSetupUtil.h"
#include "ToolSceneQueriesUtil.h"
#include "PreviewMesh.h"
#include "BaseGizmos/GizmoComponents.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Drawing/MeshDebugDrawing.h"
#include "Drawing/PreviewGeometryActor.h"


#include "Sculpting/StampFalloffs.h"

#include "Generators/SphereGenerator.h"

#include "ModelingToolTargetUtil.h"
#include "BaseBehaviors/TwoAxisPropertyEditBehavior.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Polyline3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSculptToolBase)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshSculptToolBase"


namespace
{
	const FString VertexSculptIndicatorGizmoType = TEXT("VertexSculptIndicatorGizmoType");
}

namespace MeshSculptBaseLocals
{
	const FString& StrokePointSetID(TEXT("StrokePointSet"));
	const FString& StrokeLineSetID(TEXT("StrokeLineSet"));

	// TODO: This may be useful to have again
	// static FAutoConsoleVariable EnableMouseTrail(TEXT("modeling.Sculpting.EnableMouseTrail"), false, TEXT("Enable visualizing the path the mouse takes while sculpting."));
}

void FBrushToolRadius::InitializeWorldSizeRange(TInterval<float> Range, bool bValidateWorldRadius)
{
	WorldSizeRange = Range;
	if (WorldRadius < WorldSizeRange.Min)
	{
		WorldRadius = WorldSizeRange.Interpolate(0.2f);
	}
	else if (WorldRadius > WorldSizeRange.Max)
	{
		WorldRadius = WorldSizeRange.Interpolate(0.8f);
	}
}

float FBrushToolRadius::GetWorldRadius() const
{
	if (SizeType == EBrushToolSizeType::Adaptive)
	{
		return 0.5 * WorldSizeRange.Interpolate( FMath::Max(0, AdaptiveSize) );
	}
	else
	{
		return WorldRadius;
	}
}

void FBrushToolRadius::IncreaseRadius(bool bSmallStep)
{
	float StepSize = (bSmallStep) ? 0.005f : 0.025f;
	if (SizeType == EBrushToolSizeType::Adaptive)
	{
		AdaptiveSize = FMath::Clamp(AdaptiveSize + StepSize, 0.0f, 1.0f);
	}
	else
	{
		float dt = StepSize * 0.5 * WorldSizeRange.Size();
		WorldRadius = FMath::Clamp(WorldRadius + dt, WorldSizeRange.Min, WorldSizeRange.Max);
	}
}

void FBrushToolRadius::DecreaseRadius(bool bSmallStep)
{
	float StepSize = (bSmallStep) ? 0.005f : 0.025f;
	if (SizeType == EBrushToolSizeType::Adaptive)
	{
		AdaptiveSize = FMath::Clamp(AdaptiveSize - StepSize, 0.0f, 1.0f);
	}
	else
	{
		float dt = StepSize * 0.5 * WorldSizeRange.Size();
		WorldRadius = FMath::Clamp(WorldRadius - dt, WorldSizeRange.Min, WorldSizeRange.Max);
	}
}




void UMeshSculptToolBase::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UMeshSculptToolBase::Setup()
{
	UMeshSurfacePointTool::Setup();
	
	const FString PropertyCacheId = GetPropertyCacheIdentifier();

	BrushProperties = NewObject<USculptBrushProperties>(this);
	if (SharesBrushPropertiesChanges())
	{
		BrushProperties->RestoreProperties(this, PropertyCacheId);
	}
	// Note that brush properties includes BrushRadius, which, when not used as a constant,
	// serves as an output property based on target size and brush size, and so it would need
	// updating after the RestoreProperties() call. But deriving classes will call 
	// InitializeBrushSizeRange after this Setup() call to finish the brush setup, which will
	// update the output property if necessary.

	// work plane
	GizmoProperties = NewObject<UWorkPlaneProperties>();
	GizmoProperties->RestoreProperties(this, PropertyCacheId);

	// create proxy for plane gizmo, but not gizmo itself, as it only appears in FixedPlane brush mode
	// listen for changes to the proxy and update the plane when that happens
	PlaneTransformProxy = NewObject<UTransformProxy>(this);
	PlaneTransformProxy->OnTransformChanged.AddUObject(this, &UMeshSculptToolBase::PlaneTransformChanged);

	//GizmoProperties->WatchProperty(GizmoProperties->Position,
	//	[this](FVector NewPosition) { UpdateGizmoFromProperties(); });
	//GizmoProperties->WatchProperty(GizmoProperties->Rotation,
	//	[this](FQuat NewRotation) { UpdateGizmoFromProperties(); });
	GizmoPositionWatcher.Initialize(
		[this]() { return GizmoProperties->Position; },
		[this](FVector NewPosition) { UpdateGizmoFromProperties(); }, GizmoProperties->Position);
	GizmoRotationWatcher.Initialize(
		[this]() { return GizmoProperties->Rotation; },
		[this](FQuat NewRotation) { UpdateGizmoFromProperties(); }, GizmoProperties->Rotation);



	// display
	ViewProperties = NewObject<UMeshEditingViewProperties>();
	ViewProperties->RestoreProperties(this, PropertyCacheId);

	ViewProperties->WatchProperty(ViewProperties->bShowWireframe,
		[this](bool bNewValue) { UpdateWireframeVisibility(bNewValue); });
	ViewProperties->WatchProperty(ViewProperties->MaterialMode,
		[this](EMeshEditingMaterialModes NewMode) { UpdateMaterialMode(NewMode); });
	ViewProperties->WatchProperty(ViewProperties->CustomMaterial,
		[this](TWeakObjectPtr<UMaterialInterface> NewMaterial) { UpdateCustomMaterial(NewMaterial); });
	ViewProperties->WatchProperty(ViewProperties->bFlatShading,
		[this](bool bNewValue) { UpdateFlatShadingSetting(bNewValue); });
	ViewProperties->WatchProperty(ViewProperties->Color,
		[this](FLinearColor NewColor) { UpdateColorSetting(NewColor); }); 
	// This can actually use the same function since the parameter names for the material are the same
	ViewProperties->WatchProperty(ViewProperties->TransparentMaterialColor,
		[this](FLinearColor NewColor) { UpdateTransparentColorSetting(NewColor); });
	ViewProperties->WatchProperty(ViewProperties->Opacity,
		[this](double NewValue) { UpdateOpacitySetting(NewValue); });
	ViewProperties->WatchProperty(ViewProperties->bTwoSided,
		[this](bool bOn) { UpdateTwoSidedSetting(bOn); });
	ViewProperties->WatchProperty(ViewProperties->Image,
		[this](UTexture2D* NewImage) { UpdateImageSetting(NewImage); });

	// add input behavior to click-drag while holding hotkey to adjust brush size and strength
    {
        BrushEditBehavior = NewObject<ULocalTwoAxisPropertyEditInputBehavior>(this);
		
		SetupBrushEditBehaviorSetup(*BrushEditBehavior);
    	
        BrushEditBehavior->OnDragUpdated.AddWeakLambda(this, [this]()
        {
        	CalculateBrushRadius();
        	NotifyOfPropertyChangeByTool(BrushProperties);
        });
        
        BrushEditBehavior->Initialize();
        AddInputBehavior(BrushEditBehavior.Get());
    }
}

void UMeshSculptToolBase::SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior)
{
	MapHorizontalBrushEditBehaviorToBrushSize(OutBehavior);
}

void UMeshSculptToolBase::MapHorizontalBrushEditBehaviorToBrushSize(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior)
{
	OutBehavior.HorizontalProperty.GetValueFunc = [this]()
		{
			if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
			{
				return BrushProperties->BrushSize.AdaptiveSize;
			}
			else
			{
				return BrushProperties->BrushSize.WorldRadius;
			}
		};
	OutBehavior.HorizontalProperty.SetValueFunc = [this](float NewValue)
		{
			if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
			{
				BrushProperties->BrushSize.AdaptiveSize = FMath::Clamp(NewValue, 0.f, 1.f);
			}
			else
			{
				BrushProperties->BrushSize.WorldRadius = FMath::Max(NewValue, 0.01f);
			}

#if WITH_EDITOR
			FPropertyChangedEvent PropertyChangedEvent(USculptBrushProperties::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(USculptBrushProperties, BrushSize)));
			BrushProperties->PostEditChangeProperty(PropertyChangedEvent);
#endif
		};
	OutBehavior.HorizontalProperty.MutateDeltaFunc = [this](float Delta)
		{
			// Scale delta if brush size is in world units.
			return Delta * (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::World ? (CameraState.Position - LastBrushFrameWorld.Origin).Length() : 1.f);
		};
	OutBehavior.HorizontalProperty.Name = LOCTEXT("BrushRadius", "Radius");
	OutBehavior.HorizontalProperty.bEnabled = true;
}

void UMeshSculptToolBase::OnCompleteSetup()
{
	RestoreAllBrushTypeProperties(this);

	for (auto Pair : BrushOpPropSets)
	{
		SetToolPropertySourceEnabled(Pair.Value, false);
	}
}


void UMeshSculptToolBase::Shutdown(EToolShutdownType ShutdownType)
{
	if (StrokeGeometry)
	{
		StrokeGeometry->Disconnect();
		StrokeGeometry = nullptr;
	}

	if (ShutdownType == EToolShutdownType::Accept && AreAllTargetsValid() == false)
	{
		UE_LOGF(LogTemp, Error, "Tool Target has become Invalid (possibly it has been Force Deleted). Aborting Tool.");
		ShutdownType = EToolShutdownType::Cancel;
	}

	UMeshSurfacePointTool::Shutdown(ShutdownType);
	
	const FString PropertyCacheId = GetPropertyCacheIdentifier();

	BrushIndicatorMesh->Disconnect();
	BrushIndicatorMesh = nullptr;

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	BrushIndicator = nullptr;
	GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(VertexSculptIndicatorGizmoType);

	if (SharesBrushPropertiesChanges())
	{
		BrushProperties->SaveProperties(this, PropertyCacheId);
	}
	if (GizmoProperties)
	{
		GizmoProperties->SaveProperties(this, PropertyCacheId);
	}

	ViewProperties->SaveProperties(this, PropertyCacheId);

	SaveAllBrushTypeProperties(this);


	// bake result
	UBaseDynamicMeshComponent* DynamicMeshComponent = GetSculptMeshComponent();
	if (DynamicMeshComponent != nullptr)
	{
		UE::ToolTarget::ShowSourceObject(Target);

		if (ShutdownType == EToolShutdownType::Accept)
		{
			// safe to do this here because we are about to destroy componeont
			DynamicMeshComponent->ApplyTransform(InitialTargetTransform, true);

			// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
			CommitResult(DynamicMeshComponent, false);
		}

		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->DestroyComponent();
		DynamicMeshComponent = nullptr;
	}

	LongTransactions.CloseAll(GetToolManager());
}



void UMeshSculptToolBase::CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("SculptMeshToolTransactionName", "Sculpt Mesh"));
	Component->ProcessMesh([&](const FDynamicMesh3& CurMesh)
	{
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, CurMesh, bModifiedTopology);
	});
	GetToolManager()->EndUndoTransaction();
}


void UMeshSculptToolBase::OnTick(float DeltaTime)
{
	GizmoPositionWatcher.CheckAndUpdate();
	GizmoRotationWatcher.CheckAndUpdate();

	ActivePressure = GetCurrentDevicePressure();

	if (InStroke() == false)
	{
		SaveActiveStrokeModifiers();
	}
	else
	{
		AccumulateStrokeTime(DeltaTime);
	}

	// update cached falloff
	CurrentBrushFalloff = 0.5;
	if (GetActiveBrushOp()->PropertySet.IsValid())
	{
		CurrentBrushFalloff = FMathd::Clamp(GetActiveBrushOp()->PropertySet->GetFalloff(), 0.0, 1.0);
	}

	UpdateHoverStamp(GetBrushFrameWorld());

	// always using HoverStamp here because it's position should always be up-to-date for the current apply-stamp...
	BrushIndicator->Update((float)GetCurrentBrushRadius(),
		HoverStamp.WorldFrame.ToFTransform(), 1.0f - (float)GetCurrentBrushFalloff());

	UpdateWorkPlane();
}


void UMeshSculptToolBase::Render(IToolsContextRenderAPI* RenderAPI)
{
	UMeshSurfacePointTool::Render(RenderAPI);
	// Cache here for usage during interaction, should probably happen in ::Tick() or elsewhere
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	FViewCameraState RenderCameraState = RenderAPI->GetCameraState();

	if (BrushIndicatorMaterial)
	{
		double FixedDimScale = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(RenderCameraState, HoverStamp.WorldFrame.Origin, 1.5f);
		BrushIndicatorMaterial->SetScalarParameterValue(TEXT("FalloffWidth"), FixedDimScale);
	}

	if (ShowWorkPlane() && bDrawWorkPlaneGridLines)
	{
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		FColor GridColor(128, 128, 128, 32);
		float GridThickness = 0.5f*RenderCameraState.GetPDIScalingFactor();
		int NumGridLines = 10;
		FFrame3d DrawFrame(GizmoProperties->Position, GizmoProperties->Rotation);
		MeshDebugDraw::DrawSimpleFixedScreenAreaGrid(RenderCameraState, DrawFrame, NumGridLines, 45.0, GridThickness, GridColor, false, PDI, FTransform::Identity);
	}
}


FString UMeshSculptToolBase::GetPropertyCacheIdentifier() const
{
	return TEXT("UMeshSculptToolBase");
}


void UMeshSculptToolBase::InitializeSculptMeshComponent(UBaseDynamicMeshComponent* Component, AActor* Actor)
{
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Component, nullptr);

	// disable shadows initially, as changing shadow settings invalidates the SceneProxy
	Component->SetShadowsEnabled(false);
	Component->SetupAttachment(Actor->GetRootComponent());
	Component->RegisterComponent();

	// initialize from LOD-0 MeshDescription
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	Component->SetMesh(UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams));
	double MaxDimension = Component->GetMesh()->GetBounds(true).MaxDim();

	// bake rotation and scaling into mesh because handling these inside sculpting is a mess
	// Note: this transform does not include translation ( so only the 3x3 transform)
	InitialTargetTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	// clamp scaling because if we allow zero-scale we cannot invert this transform on Accept
	InitialTargetTransform.ClampMinimumScale(0.01);
	FVector3d Translation = InitialTargetTransform.GetTranslation();
	InitialTargetTransform.SetTranslation(FVector3d::Zero());
	Component->ApplyTransform(InitialTargetTransform, false);
	CurTargetTransform = FTransform3d(Translation);
	Component->SetWorldTransform((FTransform)CurTargetTransform);

	// hide input Component
	UE::ToolTarget::HideSourceObject(Target);
}





void UMeshSculptToolBase::RegisterBrushType(int32 Identifier, FText Name, TUniquePtr<FMeshSculptBrushOpFactory> Factory, UMeshSculptBrushOpProps* PropSet)
{
	FBrushTypeInfo TypeInfo;
	TypeInfo.Name = Name;
	TypeInfo.Identifier = Identifier;
	RegisteredPrimaryBrushTypes.Add(TypeInfo);

	// ensure we are not adding duplicates of PropSets when brush registration occurs
	if (TObjectPtr<UMeshSculptBrushOpProps>* OldPropSet = BrushOpPropSets.Find(Identifier))
	{
		RemoveToolPropertySource(*OldPropSet);
	}

	BrushOpPropSets.Add(Identifier, PropSet);
	BrushOpFactories.Add(Identifier, MoveTemp(Factory));

	if (!ToolPropertyObjects.Contains(PropSet))
	{
		AddToolPropertySource(PropSet);
	}
	SetToolPropertySourceEnabled(PropSet, false);
}

void UMeshSculptToolBase::RegisterSecondaryBrushType(int32 Identifier, FText Name, TUniquePtr<FMeshSculptBrushOpFactory> Factory, UMeshSculptBrushOpProps* PropSet)
{
	FBrushTypeInfo TypeInfo;
	TypeInfo.Name = Name;
	TypeInfo.Identifier = Identifier;
	RegisteredSecondaryBrushTypes.Add(TypeInfo);

	// ensure we are not adding duplicates of PropSets when brush registration occurs
	if (TObjectPtr<UMeshSculptBrushOpProps>* OldPropSet = SecondaryBrushOpPropSets.Find(Identifier))
	{
		RemoveToolPropertySource(*OldPropSet);
	}
	
	SecondaryBrushOpPropSets.Add(Identifier, PropSet);
	SecondaryBrushOpFactories.Add(Identifier, MoveTemp(Factory));

	if (!ToolPropertyObjects.Contains(PropSet))
	{
		AddToolPropertySource(PropSet);
	}
	SetToolPropertySourceEnabled(PropSet, false);
}



void UMeshSculptToolBase::SaveAllBrushTypeProperties(UInteractiveTool* SaveFromTool)
{
	const FString PropertyCacheId = GetPropertyCacheIdentifier();
	for (auto Pair : BrushOpPropSets)
	{
		Pair.Value->SaveProperties(SaveFromTool, PropertyCacheId);
	}
	for (auto Pair : SecondaryBrushOpPropSets)
	{
		Pair.Value->SaveProperties(SaveFromTool, PropertyCacheId);
	}
}
void UMeshSculptToolBase::RestoreAllBrushTypeProperties(UInteractiveTool* RestoreToTool)
{
	const FString PropertyCacheId = GetPropertyCacheIdentifier();
	for (auto Pair : BrushOpPropSets)
	{
		Pair.Value->RestoreProperties(RestoreToTool, PropertyCacheId);
	}
	for (auto Pair : SecondaryBrushOpPropSets)
	{
		Pair.Value->RestoreProperties(RestoreToTool, PropertyCacheId);
	}
}


void UMeshSculptToolBase::SetActivePrimaryBrushType(int32 Identifier)
{
	TUniquePtr<FMeshSculptBrushOpFactory>* Factory = BrushOpFactories.Find(Identifier);
	if (Factory == nullptr)
	{
		ensure(false);
		return;
	}

	if (PrimaryVisiblePropSet != nullptr)
	{
		SetToolPropertySourceEnabled(PrimaryVisiblePropSet, false);
		PrimaryVisiblePropSet = nullptr;
	}

	PrimaryBrushOp = (*Factory)->Build();
	PrimaryBrushOp->Falloff = PrimaryFalloff;

	TObjectPtr<UMeshSculptBrushOpProps>* FoundProps = BrushOpPropSets.Find(Identifier);
	if (FoundProps != nullptr)
	{
		SetToolPropertySourceEnabled(*FoundProps, bBrushOpPropsVisible);
		PrimaryVisiblePropSet = *FoundProps;

		PrimaryBrushOp->PropertySet = PrimaryVisiblePropSet;
	}
}



void UMeshSculptToolBase::SetActiveSecondaryBrushType(int32 Identifier)
{
	TUniquePtr<FMeshSculptBrushOpFactory>* Factory = SecondaryBrushOpFactories.Find(Identifier);
	if (Factory == nullptr)
	{
		ensure(false);
		return;
	}

	if (SecondaryVisiblePropSet != nullptr)
	{
		SetToolPropertySourceEnabled(SecondaryVisiblePropSet, false);
		SecondaryVisiblePropSet = nullptr;
	}

	SecondaryBrushOp = (*Factory)->Build();
	TSharedPtr<FMeshSculptFallofFunc> SecondaryFalloff = MakeShared<FMeshSculptFallofFunc>();;
	SecondaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeStandardSmoothFalloff();
	SecondaryBrushOp->Falloff = SecondaryFalloff;

	TObjectPtr<UMeshSculptBrushOpProps>* FoundProps = SecondaryBrushOpPropSets.Find(Identifier);
	if (FoundProps != nullptr)
	{
		SetToolPropertySourceEnabled(*FoundProps, bBrushOpPropsVisible);
		SecondaryVisiblePropSet = *FoundProps;

		SecondaryBrushOp->PropertySet = SecondaryVisiblePropSet;
	}
}



TUniquePtr<FMeshSculptBrushOp>& UMeshSculptToolBase::GetActiveBrushOp()
{
	if (GetInSmoothingStroke())
	{
		return SecondaryBrushOp;
	}
	else
	{
		return PrimaryBrushOp;
	}
}

void UMeshSculptToolBase::SetBrushOpPropsVisibility(bool bVisible)
{
	bBrushOpPropsVisible = bVisible;
	if (PrimaryVisiblePropSet)
	{
		SetToolPropertySourceEnabled(PrimaryVisiblePropSet, bVisible);
	}
	if (SecondaryVisiblePropSet)
	{
		SetToolPropertySourceEnabled(SecondaryVisiblePropSet, bVisible);
	}
}


void UMeshSculptToolBase::RegisterStandardFalloffTypes()
{
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("Smooth", "Smooth"), TEXT("Smooth"), (int32)EMeshSculptFalloffType::Smooth});
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("Linear", "Linear"), TEXT("Linear"), (int32)EMeshSculptFalloffType::Linear } );
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("Inverse", "Inverse"), TEXT("Inverse"), (int32)EMeshSculptFalloffType::Inverse } );
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("Round", "Round"), TEXT("Round"), (int32)EMeshSculptFalloffType::Round } );
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("BoxSmooth", "BoxSmooth"), TEXT("BoxSmooth"), (int32)EMeshSculptFalloffType::BoxSmooth } );
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("BoxLinear", "BoxLinear"), TEXT("BoxLinear"), (int32)EMeshSculptFalloffType::BoxLinear } );
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("BoxInverse", "BoxInverse"), TEXT("BoxInverse"), (int32)EMeshSculptFalloffType::BoxInverse } );
	RegisteredPrimaryFalloffTypes.Add( FFalloffTypeInfo{ LOCTEXT("BoxRound", "BoxRound"), TEXT("BoxRound"), (int32)EMeshSculptFalloffType::BoxRound } );
}

void UMeshSculptToolBase::SetPrimaryFalloffType(EMeshSculptFalloffType FalloffType)
{
	PrimaryFalloff = MakeShared<FMeshSculptFallofFunc>();
	switch (FalloffType)
	{
	default:
	case EMeshSculptFalloffType::Smooth:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeStandardSmoothFalloff();
		break;
	case EMeshSculptFalloffType::Linear:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeLinearFalloff();
		break;
	case EMeshSculptFalloffType::Inverse:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeInverseFalloff();
		break;
	case EMeshSculptFalloffType::Round:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeRoundFalloff();
		break;
	case EMeshSculptFalloffType::BoxSmooth:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeSmoothBoxFalloff();
		break;
	case EMeshSculptFalloffType::BoxLinear:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeLinearBoxFalloff();
		break;
	case EMeshSculptFalloffType::BoxInverse:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeInverseBoxFalloff();
		break;
	case EMeshSculptFalloffType::BoxRound:
		PrimaryFalloff->FalloffFunc = UE::SculptFalloffs::MakeRoundBoxFalloff();
		break;
	}
}




bool UMeshSculptToolBase::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	FRay3d LocalRay = GetLocalRay(Ray);

	int HitTID = FindHitSculptMeshTriangle(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		FTriangle3d Triangle;
		FDynamicMesh3* Mesh = GetSculptMesh();
		Mesh->GetTriVertices(HitTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();

		OutHit.FaceIndex = HitTID;
		OutHit.Distance = Query.RayParameter;
		OutHit.Normal = (FVector)CurTargetTransform.TransformNormal(Mesh->GetTriNormal(HitTID));
		OutHit.ImpactPoint = (FVector)CurTargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		return true;
	}
	return false;
}

void UMeshSculptToolBase::OnBeginDrag(const FRay& WorldRay)
{
	SaveActiveStrokeModifiers();

	FHitResult OutHit;
	if (HitTest(WorldRay, OutHit))
	{
		bInStroke = true;
		ResetStrokeTime();
		PendingInputRays.Reset();
		PreviousInputRay.Reset();
		PreviousUnfilteredHitPosition.Reset();

		UpdateBrushTargetPlaneFromHit(WorldRay, OutHit);

		// initialize first stamp
		PendingInputRays.Add({ .Ray = WorldRay, .Timestamp = ActiveStrokeTime, .Pressure = GetActivePressure() });
		PendingStampRay = WorldRay;
		bIsStampPending = true;

		// set falloff
		PrimaryBrushOp->Falloff = PrimaryFalloff;

		OnBeginStroke(WorldRay);
		OnUpdateDrag(WorldRay);
	}
}

void UMeshSculptToolBase::OnUpdateDrag(const FRay& WorldRay)
{
	if (InStroke())
	{
		PendingInputRays.Add({ .Ray = WorldRay, .Timestamp = ActiveStrokeTime, .Pressure = GetActivePressure() });
		PendingStampRay = WorldRay;
	}
}

void UMeshSculptToolBase::OnEndDrag(const FRay& Ray)
{
	bInStroke = false;

	// cancel any outstanding stamps! otherwise change record could become invalid
	bIsStampPending = false;

	OnEndStroke();
}

void UMeshSculptToolBase::OnCancelDrag()
{
	bInStroke = false;

	// cancel any outstanding stamps
	bIsStampPending = false;

	OnCancelStroke();
}

FRay3d UMeshSculptToolBase::GetLocalRay(const FRay& WorldRay) const
{
	FRay3d LocalRay(CurTargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		CurTargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);
	return LocalRay;
}

bool UMeshSculptToolBase::GetBrushSizePressureSensitivityEnabled() const
{
	if (ToolPropertyObjects.Contains(BrushProperties) && BrushProperties->BrushSize.bToolSupportsPressureSensitivity)
	{
		return BrushProperties->BrushSize.bEnablePressureSensitivity;
	}
	return false;
}

bool UMeshSculptToolBase::GetBrushStrengthPressureEnabled()
{
	TUniquePtr<FMeshSculptBrushOp>& BrushOp = GetActiveBrushOp();
	if (BrushOp->PropertySet.IsValid())
	{
		return BrushOp->PropertySet->GetStrengthPressureEnabled();
	}
	return false;
}

void UMeshSculptToolBase::SetBrushStrengthPressureEnabled(const bool bEnabled)
{
	TUniquePtr<FMeshSculptBrushOp>& BrushOp = GetActiveBrushOp();
	if (BrushOp->PropertySet.IsValid())
	{
		BrushOp->PropertySet->bIsStrengthPressureEnabled = bEnabled;
	}
}

void UMeshSculptToolBase::UpdateBrushFrameWorld(const FVector3d& NewPosition, const FVector3d& NewNormal)
{
	FFrame3d PrevBrushFrameWorld = LastBrushFrameWorld;

	bool bTriedFrameRepair = false;
retry_frame_update:
	FFrame3d NewFrame = LastBrushFrameWorld;
	NewFrame.Origin = NewPosition;
	NewFrame.AlignAxis(2, NewNormal);
	FVector3d CameraUp = (FVector3d)CameraState.Up();

	if (FMathd::Abs(CameraUp.Dot(NewNormal)) < 0.98)
	{
		NewFrame.ConstrainedAlignAxis(1, CameraUp, NewFrame.Z());
	}

	if ( (NewFrame.Rotation.Length() - 1.0) > 0.1 )		// try to recover from normalization failure
	{
		LastBrushFrameWorld = FFrame3d(LastBrushFrameWorld.Origin);
		if (bTriedFrameRepair == false)
		{
			bTriedFrameRepair = true;
			goto retry_frame_update;
		}
	}

	if (InStroke() && BrushProperties->Lazyness > 0)
	{
		double t = FMathd::Lerp(1.0, 0.1, (double)BrushProperties->Lazyness);
		LastBrushFrameWorld.Origin = UE::Geometry::Lerp(LastBrushFrameWorld.Origin, NewFrame.Origin, t);
		LastBrushFrameWorld.Rotation = FQuaterniond(LastBrushFrameWorld.Rotation, NewFrame.Rotation, t);
	}
	else
	{
		LastBrushFrameWorld = NewFrame;
	}

	ActiveStrokePathArcLen += Distance(LastBrushFrameWorld.Origin, PrevBrushFrameWorld.Origin);

	LastBrushFrameLocal = LastBrushFrameWorld;
	LastBrushFrameLocal.Transform(CurTargetTransform.InverseUnsafe()); // Note: Unsafe inverse used because we cannot handle scales on a frame regardless.
	// TODO: in the case of a non-uniform scale, consider whether we should do additional work to align the Z axis?
}

void UMeshSculptToolBase::AlignBrushToView()
{
	UpdateBrushFrameWorld(GetBrushFrameWorld().Origin, -(FVector3d)CameraState.Forward());
}



void UMeshSculptToolBase::UpdateBrushTargetPlaneFromHit(const FRay& WorldRayIn, const FHitResult& Hit)
{
	FRay3d WorldRay(WorldRayIn);
	FVector3d WorldPosWithBrushDepth = WorldRay.PointAt(Hit.Distance) + GetCurrentBrushDepth() * GetCurrentBrushRadius() * WorldRay.Direction;
	ActiveBrushTargetPlaneWorld = FFrame3d(WorldPosWithBrushDepth, -WorldRay.Direction);
}

bool UMeshSculptToolBase::ProjectWorldRayOnActivePlane(const FRay& WorldRayIn, FVector3d& ProjectedPosition, FVector3d& ProjectedNormal) const
{
	FRay3d WorldRay(WorldRayIn);	
	ActiveBrushTargetPlaneWorld.RayPlaneIntersection(WorldRay.Origin, WorldRay.Direction, 2, ProjectedPosition);
	ProjectedNormal = ActiveBrushTargetPlaneWorld.Z();
	return true;
}

bool UMeshSculptToolBase::ProjectWorldRayOnTargetMesh(const FRay& WorldRayIn, bool bFallbackToViewPlane, FVector3d& ProjectedPosition, FVector3d& ProjectedNormal, int32& HitTriangle) const
{
	FRay3d WorldRay(WorldRayIn);
	FRay3d LocalRay = GetLocalRay(WorldRayIn);
	int32 HitTID = FindHitTargetMeshTriangleConst(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		const FDynamicMesh3* BaseMesh = GetBaseMesh();
		FIntrRay3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleIntersection(*BaseMesh, HitTID, LocalRay);
		ProjectedNormal = CurTargetTransform.TransformNormal(BaseMesh->GetTriNormal(HitTID));
		ProjectedPosition = CurTargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		HitTriangle = HitTID;
		return true;
	}

	if (bFallbackToViewPlane)
	{
		FFrame3d BrushPlane(GetBrushFrameWorld().Origin, (FVector3d)CameraState.Forward());
		BrushPlane.RayPlaneIntersection(WorldRay.Origin, WorldRay.Direction, 2, ProjectedPosition);
		ProjectedNormal = ActiveBrushTargetPlaneWorld.Z();
		return true;
	}

	return false;
}

bool UMeshSculptToolBase::ProjectWorldRayOnSculptMesh(const FRay& WorldRayIn, bool bFallbackToViewPlane, FVector3d& ProjectedPosition, FVector3d& ProjectedNormal, int32& HitTriangle) const
{
	HitTriangle = IndexConstants::InvalidID;

	FRay3d WorldRay(WorldRayIn);
	FRay3d LocalRay = GetLocalRay(WorldRayIn);
	int32 HitTID = FindHitSculptMeshTriangleConst(LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		const FDynamicMesh3* SculptMesh = GetSculptMesh();
		FIntrRay3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleIntersection(*SculptMesh, HitTID, LocalRay);
		ProjectedNormal = CurTargetTransform.TransformNormal(SculptMesh->GetTriNormal(HitTID));
		ProjectedPosition = CurTargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
		HitTriangle = HitTID;
		return true;
	}

	if (bFallbackToViewPlane)
	{
		FFrame3d BrushPlane(GetBrushFrameWorld().Origin, (FVector3d)CameraState.Forward());
		BrushPlane.RayPlaneIntersection(WorldRay.Origin, WorldRay.Direction, 2, ProjectedPosition);
		ProjectedNormal = ActiveBrushTargetPlaneWorld.Z();
		return true;
	}

	return false;
}

bool UMeshSculptToolBase::UpdateBrushPositionOnActivePlane(const FRay& WorldRayIn)
{
	LastBrushTriangleID = IndexConstants::InvalidID;
	FVector3d NewHitPosWorld;
	FVector3d NewHitNormalWorld;
	bool bHit = ProjectWorldRayOnActivePlane(WorldRayIn, NewHitPosWorld, NewHitNormalWorld);
	UpdateBrushFrameWorld(NewHitPosWorld, ActiveBrushTargetPlaneWorld.Z());
	return bHit;
}

bool UMeshSculptToolBase::UpdateBrushPositionOnTargetMesh(const FRay& WorldRayIn, bool bFallbackToViewPlane)
{
	LastBrushTriangleID = IndexConstants::InvalidID;
	FRay3d LocalRay = GetLocalRay(WorldRayIn);
	FVector3d NewHitPosWorld;
	FVector3d NewHitNormalWorld;
	int32 HitTID;
	bool bHit = ProjectWorldRayOnTargetMesh(WorldRayIn, bFallbackToViewPlane, NewHitPosWorld, NewHitNormalWorld, HitTID);
	UpdateBrushFrameWorld(NewHitPosWorld, NewHitNormalWorld);
	UpdateHitTargetMeshTriangle(HitTID, LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		LastBrushTriangleID = HitTID;
	}
	return bHit;
}

bool UMeshSculptToolBase::UpdateBrushPositionOnSculptMesh(const FRay& WorldRayIn, bool bFallbackToViewPlane)
{
	LastBrushTriangleID = IndexConstants::InvalidID;
	FRay3d LocalRay = GetLocalRay(WorldRayIn);
	FVector3d NewHitPosWorld;
	FVector3d NewHitNormalWorld;
	int32 HitTID;
	bool bHit = ProjectWorldRayOnSculptMesh(WorldRayIn, bFallbackToViewPlane, NewHitPosWorld, NewHitNormalWorld, HitTID);
	UpdateBrushFrameWorld(NewHitPosWorld, NewHitNormalWorld);
	UpdateHitSculptMeshTriangle(HitTID, LocalRay);
	if (HitTID != IndexConstants::InvalidID)
	{
		LastBrushTriangleID = HitTID;
	}
	return bHit;
}

void UMeshSculptToolBase::SaveActiveStrokeModifiers()
{
	bSmoothing = GetShiftToggle();
	bInvert = GetCtrlToggle();
}


void UMeshSculptToolBase::UpdateHoverStamp(const FFrame3d& StampFrameWorld)
{
	HoverStamp.WorldFrame = StampFrameWorld;
	HoverStamp.LocalFrame = HoverStamp.WorldFrame;
	HoverStamp.LocalFrame.Transform(CurTargetTransform.InverseUnsafe());
}

float UMeshSculptToolBase::GetStampTemporalFlowRate() const
{
	return BrushProperties->FlowRate;
}

void UMeshSculptToolBase::ProcessPerTickStamps(TFunction<bool(const FRay& StampRay)> UpdateStampPosition, TFunction<void(int StampIndex, const FRay& StampRay)> ExecuteStampOperation)
{
	ProcessPerTickStamps(UpdateStampPosition, [](int StampCount) {}, ExecuteStampOperation, [] {});
}

void UMeshSculptToolBase::ProcessPerTickStamps(TFunction<bool(const FRay& StampRay)> UpdateStampPosition, TFunction<void(int StampCount)> PreExecuteStampsOperation, TFunction<void(int StampIndex, const FRay& StampRay)> ExecuteStampOperation, TFunction<void()> PostExecuteStampsOperation)
{
	if (InStroke())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BaseSculptTool_Tick_ProcessPerTickStamps);
		FDynamicMesh3* Mesh = GetSculptMesh();

		// We save the active pressure here, so we can then override and fake it during the stamp application to generate interpolated pressures along the stroke.
		TGuardValue<double> CachedActivePressure(ActivePressure, ActivePressure);

		// update brush position
		if (UpdateStampPosition(GetPendingStampRayWorld()) == false)
		{
			return;
		}

		UpdateStampPendingState();
		if (IsStampPending() == false)
		{
			return;
		}

		if (!PendingStampRays.IsEmpty())
		{
			PreExecuteStampsOperation(PendingStampRays.Num());
			for (int Index = 0; Index < PendingStampRays.Num(); ++Index)
			{
				const FStampRayData& StampRay = PendingStampRays[Index];

				ActivePressure = StampRay.Pressure;

				UpdateStampPosition(StampRay.StampRay);
				ExecuteStampOperation(Index, StampRay.StampRay);
			}
			PostExecuteStampsOperation();
		}

	}
}



void UMeshSculptToolBase::UpdateStampPendingState()
{
	if (InStroke() == false) return;
	PendingStampRays.Empty();

	// Fail out early with a stamp if we don't support the currently requested stroke type
	if (!GetActiveBrushOp()->SupportsStrokeType(BrushProperties->StrokeType))
	{
		PendingStampRays.Add({ .StampRay = PendingStampRay, .Pressure = GetActivePressure() });
		bIsStampPending = true;
		return;
	}

	switch (BrushProperties->StrokeType)
	{
	case EMeshSculptStrokeType::Airbrush:
	{
		// TODO: Airbrush mode should actually process PendingInputRays instead of PendingStampRay
		//  so that it properly handles cases where we get multiple drag updates in between a tick.
		bool bFlowStampPending = false;
		float UseStampFlowRate = GetStampTemporalFlowRate();
		if (UseStampFlowRate >= 1.0)
		{
			bFlowStampPending = true;
		}
		else if (UseStampFlowRate == 0.0)
		{
			bFlowStampPending = (LastFlowTimeStamp++ == 0);
		}
		else
		{
			double dt = (1.0 - UseStampFlowRate);
			int FlowTimestamp = (int)(ActiveStrokeTime / dt);
			if (FlowTimestamp > LastFlowTimeStamp)
			{
				LastFlowTimeStamp = FlowTimestamp;
				bFlowStampPending = true;
			}
		}

		PendingStampRays.Add({ .StampRay = PendingStampRay, .Pressure = GetActivePressure() });
		bIsStampPending = bFlowStampPending;
		return;
	}
	case EMeshSculptStrokeType::Spacing:
	{
		// We do two things in this mode: we upsample any input rays that are too far apart, and we filter
		//  out any hit positions that are close together.

		if (PendingInputRays.IsEmpty())
		{
			bIsStampPending = false;
			return;
		}
		bIsStampPending = true;
		
		auto GetHitPosition = [this](const FRay& WorldRay, FVector3d& HitPositionOut)->bool
		{
			TUniquePtr<FMeshSculptBrushOp>& UseBrushOp = GetActiveBrushOp();
			ESculptBrushOpTargetType TargetType = UseBrushOp->GetBrushTargetType();

			bool bHit = false;
			FVector3d HitNormal;
			int32 HitTriangle = IndexConstants::InvalidID;
			switch (TargetType)
			{
			case ESculptBrushOpTargetType::SculptMesh:
				bHit = ProjectWorldRayOnSculptMesh(WorldRay, true, HitPositionOut, HitNormal, HitTriangle);
				break;
			case ESculptBrushOpTargetType::TargetMesh:
				bHit = ProjectWorldRayOnTargetMesh(WorldRay, true, HitPositionOut, HitNormal, HitTriangle);
				break;
			case ESculptBrushOpTargetType::ActivePlane:
				bHit = ProjectWorldRayOnActivePlane(WorldRay, HitPositionOut, HitNormal);
				break;
			}

			return bHit;
		};
		
		// Start by upsampling into PendingStampRays. We will filter them further below.

		if (BrushProperties->RayDensity <= KINDA_SMALL_NUMBER)
		{
			// No upsampling needed, just use the same rays
			Algo::Transform(PendingInputRays, PendingStampRays, [](const FInputRayData& InputRay)
				{ return FStampRayData{ .StampRay = InputRay.Ray, .Pressure = InputRay.Pressure }; });
		}
		else
		{
			FFrame3d CameraPlane(CameraState.Position + CameraState.Forward(), CameraState.Forward());

			// At ray density 1.0, if we snap the ray from the left side of the screen to the right
			//  side, we will say that we want to interpolate this many samples
			constexpr double NumSamplesForScreenWidth = 200;
			double ScreenWidth = CameraState.bIsOrthographic ? CameraState.OrthoWorldCoordinateWidth
				// Width of screen at 1 unit away from camera
				: FMath::Tan(FMath::DegreesToRadians(FMath::Clamp(CameraState.HorizontalFOVDegrees / 2, 0.f, 179.f))) * 2;
			double CameraPlaneSpacing = ScreenWidth / NumSamplesForScreenWidth;

			auto GetCameraPlaneIntersection = [this, &CameraPlane, &GetHitPosition](const FRay& WorldRay, FVector3d& IntersectionOut) -> bool
			{
				// This is simpler if we assume that we're shooting rays from behind the camera plane. But
				//  we'll do the proper thing and imagine that we may be getting VR rays. We'll find the
				//  hit location and then project back from there.
				FVector3d HitPosition;
				if (!GetHitPosition(WorldRay, HitPosition))
				{
					return false;
				}
				FVector3d Direction = HitPosition - CameraState.Position;
				if (!Direction.Normalize())
				{
					return false;
				}
				return CameraPlane.RayPlaneIntersection(CameraState.Position, Direction, 2, IntersectionOut);
			};

			// Appends to PendingStampRays. Does not include start, but does include end.
			auto UpsampleSegment = [this, &GetCameraPlaneIntersection, CameraPlaneSpacing](const FInputRayData& Start, const FInputRayData& End)
			{
				FVector3d CameraPlaneEnd, CameraPlaneStart;
				if (!GetCameraPlaneIntersection(End.Ray, CameraPlaneEnd))
				{
					return;
				}
				if (!GetCameraPlaneIntersection(Start.Ray, CameraPlaneStart))
				{
					// Can at least place the end
					PendingStampRays.Add({ .StampRay = End.Ray, .Pressure = End.Pressure });
					return;
				}

				FVector3d CameraPlaneSegmentVector = CameraPlaneEnd - CameraPlaneStart;
				double DistInCameraPlane = CameraPlaneSegmentVector.Length();
				if (DistInCameraPlane == 0)
				{
					return;
				}

				int32 NumSubSegments = FMath::CeilToInt(DistInCameraPlane / CameraPlaneSpacing);
				for (int i = 1; i <= NumSubSegments; ++i) // 1-based seems simpler here
				{
					double TAlongSegment = (double)i / NumSubSegments;

					FVector3d PositionInCameraPlane = CameraPlaneStart + TAlongSegment * CameraPlaneSegmentVector;
					FVector3d Direction = PositionInCameraPlane - CameraState.Position;
					
					if (ensure(Direction.Normalize()))
					{
						double InterpolatedPressure = FMath::Lerp(Start.Pressure, End.Pressure, TAlongSegment);
						PendingStampRays.Add({ .StampRay = FRay(CameraState.Position, Direction), .Pressure = InterpolatedPressure });
					}
				}
			};

			if (!PreviousInputRay.IsSet())
			{
				PendingStampRays.Add({ .StampRay = PendingInputRays[0].Ray, .Pressure = PendingInputRays[0].Pressure });
			}
			else
			{
				UpsampleSegment(PreviousInputRay.GetValue(), PendingInputRays[0]);
			}

			for (int32 PendingInputIndex = 1; PendingInputIndex < PendingInputRays.Num(); ++PendingInputIndex)
			{
				UpsampleSegment(PendingInputRays[PendingInputIndex - 1], PendingInputRays[PendingInputIndex]);
			}
		}//end upscaling
		PreviousInputRay = PendingInputRays.Last();
		PendingInputRays.Reset();
		
		// Filter our results

		TGuardValue<double> CachedActivePressure(ActivePressure, ActivePressure);
		auto GetRadiusFromPressure = [this](double PressureIn)
		{
			ActivePressure = PressureIn;
			return GetActiveBrushRadius();
		};

		int32 DestIndex = 0;
		for (int32 SourceIndex = 0; SourceIndex < PendingStampRays.Num(); ++SourceIndex)
		{
			FVector3d HitPosition;
			auto PassItem = [SourceIndex, &DestIndex, this]()
			{
				if (SourceIndex != DestIndex) 
				{
					PendingStampRays[DestIndex] = PendingStampRays[SourceIndex]; 
				}
				++DestIndex;
			};

			if (!GetHitPosition(PendingStampRays[SourceIndex].StampRay, HitPosition))
			{
				// It's not our job to filter out misses, only hits so that we can get spacing
				PassItem();
				continue;
			}

			double HitRadius = GetRadiusFromPressure(PendingStampRays[SourceIndex].Pressure);
			if (!PreviousUnfilteredHitPosition.IsSet())
			{
				PassItem();
				PreviousUnfilteredHitPosition = HitPosition;
				PreviousUnfilteredHitRadius = HitRadius;
				continue;
			}

			double DistanceToNotOverlap = PreviousUnfilteredHitRadius + HitRadius;
			double Spacing = BrushProperties->Spacing * DistanceToNotOverlap;
			if (FVector3d::Distance(HitPosition, PreviousUnfilteredHitPosition.GetValue()) >= Spacing)
			{
				PassItem();
				PreviousUnfilteredHitPosition = HitPosition;
				PreviousUnfilteredHitRadius = HitRadius;
				continue;
			}
		}
		PendingStampRays.SetNum(DestIndex);
		return;
	}

	case EMeshSculptStrokeType::Dots:
	{
		PendingStampRays.Add({ .StampRay = PendingStampRay, .Pressure = GetActivePressure() });
		bIsStampPending = true;
		return;
	}

	default:
		break;
	}
}


void UMeshSculptToolBase::ResetStrokeTime()
{
	ActiveStrokeTime = 0.0;
	LastFlowTimeStamp = 0;
	ActiveStrokePathArcLen = 0;
	LastSpacingTimestamp = 0;
}

void UMeshSculptToolBase::AccumulateStrokeTime(float DeltaTime)
{
	ActiveStrokeTime += DeltaTime;
}


FFrame3d UMeshSculptToolBase::ComputeStampRegionPlane(const FFrame3d& StampFrame, const TSet<int32>& StampTriangles, bool bIgnoreDepth, bool bViewAligned, bool bInvDistFalloff)
{
	check(false);
	return FFrame3d();
}

FFrame3d UMeshSculptToolBase::ComputeStampRegionPlane(const FFrame3d& StampFrame, const TArray<int32>& StampTriangles, bool bIgnoreDepth, bool bViewAligned, bool bInvDistFalloff)
{
	const FDynamicMesh3* Mesh = GetSculptMesh();
	double FalloffRadius = GetCurrentBrushRadius();
	if (bInvDistFalloff)
	{
		FalloffRadius *= 0.5;
	}
	FVector3d StampNormal = StampFrame.Z();

	FVector3d AverageNormal(0, 0, 0);
	FVector3d AveragePos(0, 0, 0);
	double WeightSum = 0;
	for (int TriID : StampTriangles)
	{
		FVector3d Normal, Centroid; double Area;
		Mesh->GetTriInfo(TriID, Normal, Area, Centroid);
		if (Normal.Dot(StampNormal) < -0.2)		// ignore back-facing (heuristic to avoid "other side")
		{
			continue;
		}

		double Distance = UE::Geometry::Distance(StampFrame.Origin, Centroid);
		double NormalizedDistance = (Distance / FalloffRadius) + 0.0001;

		double Weight = Area;
		if (bInvDistFalloff)
		{
			double RampT = FMathd::Clamp(1.0 - NormalizedDistance, 0.0, 1.0);
			Weight *= FMathd::Clamp(RampT * RampT * RampT, 0.0, 1.0);
		}
		else
		{
			if (NormalizedDistance > 0.5)
			{
				double d = FMathd::Clamp((NormalizedDistance - 0.5) / (1.0 - 0.5), 0.0, 1.0);
				double t = (1.0 - d * d);
				Weight *= (t * t * t);
			}
		}

		AverageNormal += Weight * Mesh->GetTriNormal(TriID);
		AveragePos += Weight * Centroid;
		WeightSum += Weight;
	}
	UE::Geometry::Normalize(AverageNormal);
	AveragePos /= WeightSum;

	if (bViewAligned)
	{
		AverageNormal = -(FVector3d)CameraState.Forward();
	}

	FFrame3d Result = FFrame3d(AveragePos, AverageNormal);
	if (bIgnoreDepth == false)
	{
		Result.Origin -= GetCurrentBrushDepth() * GetCurrentBrushRadius() * Result.Z();
	}

	return Result;
}



void UMeshSculptToolBase::UpdateStrokeReferencePlaneForROI(const FFrame3d& StampFrame, const TArray<int32>& TriangleROI, bool bViewAligned)
{
	StrokePlane = ComputeStampRegionPlane(StampFrame, TriangleROI, false, bViewAligned);
}

void UMeshSculptToolBase::UpdateStrokeReferencePlaneFromWorkPlane()
{
	StrokePlane = FFrame3d(
		CurTargetTransform.InverseTransformPosition((FVector3d)GizmoProperties->Position),
		CurTargetTransform.GetRotation().Inverse() * (FQuaterniond)GizmoProperties->Rotation);
}



void UMeshSculptToolBase::InitializeBrushSizeRange(const FAxisAlignedBox3d& TargetBounds)
{
	double MaxDimension = TargetBounds.MaxDim();
	BrushRelativeSizeRange = FInterval1d(MaxDimension * MinimumBrushAdaptiveSizeRatio, MaxDimension);
	BrushProperties->BrushSize.InitializeWorldSizeRange(
		TInterval<float>((float)BrushRelativeSizeRange.Min, (float)BrushRelativeSizeRange.Max));
	CalculateBrushRadius();
}


void UMeshSculptToolBase::CalculateBrushRadius()
{
	CurrentBrushRadius = BrushProperties->BrushSize.GetWorldRadius();
}

double UMeshSculptToolBase::GetActiveBrushRadius()
{
	if (GetBrushSizePressureSensitivityEnabled())
	{
		if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::Adaptive)
		{
			return 0.5 * BrushProperties->BrushSize.WorldSizeRange.Interpolate(BrushProperties->BrushSize.AdaptiveSize * GetActivePressure());
		}
		if (BrushProperties->BrushSize.SizeType == EBrushToolSizeType::World)
		{
			const TInterval<float> WorldSizeRange = BrushProperties->BrushSize.WorldSizeRange;
			return FMath::Lerp(WorldSizeRange.Min, BrushProperties->BrushSize.GetWorldRadius(), GetActivePressure());
		}
	}
	return CurrentBrushRadius;
}

double UMeshSculptToolBase::GetActiveBrushStrength()
{
	TUniquePtr<FMeshSculptBrushOp>& BrushOp = GetActiveBrushOp();
	if (BrushOp->PropertySet.IsValid())
	{
		double Strength = GetCurrentBrushStrength();
		if (GetBrushStrengthPressureEnabled())
		{
			Strength *= GetActivePressure();
		}

		return FMathd::Clamp(Strength, 0.0, 1.0);
	}
	return 1.0;
}


double UMeshSculptToolBase::GetCurrentBrushStrength()
{
	TUniquePtr<FMeshSculptBrushOp>& BrushOp = GetActiveBrushOp();
	if (BrushOp->PropertySet.IsValid())
	{
		return FMathd::Clamp(BrushOp->PropertySet->GetStrength(), 0.0, 1.0);
	}
	return 1.0;
}

double UMeshSculptToolBase::GetCurrentBrushDepth()
{
	TUniquePtr<FMeshSculptBrushOp>& BrushOp = GetActiveBrushOp();
	if (BrushOp->PropertySet.IsValid())
	{
		return FMathd::Clamp(BrushOp->PropertySet->GetDepth(), -1.0, 1.0);
	}
	return 0.0;
}





void UMeshSculptToolBase::IncreaseBrushRadiusAction()
{
	BrushProperties->BrushSize.IncreaseRadius(false);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UMeshSculptToolBase::DecreaseBrushRadiusAction()
{
	BrushProperties->BrushSize.DecreaseRadius(false);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UMeshSculptToolBase::IncreaseBrushRadiusSmallStepAction()
{
	BrushProperties->BrushSize.IncreaseRadius(true);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}

void UMeshSculptToolBase::DecreaseBrushRadiusSmallStepAction()
{
	BrushProperties->BrushSize.DecreaseRadius(true);
	NotifyOfPropertyChangeByTool(BrushProperties);
	CalculateBrushRadius();
}



FBox UMeshSculptToolBase::GetWorldSpaceFocusBox()
{
	if (LastBrushTriangleID == INDEX_NONE)
	{
		return Super::GetWorldSpaceFocusBox();
	}
	FVector Center = LastBrushFrameWorld.Origin;
	double Size = GetCurrentBrushRadius();
	return FBox(Center - FVector(Size), Center + FVector(Size));
}




void UMeshSculptToolBase::SetViewPropertiesEnabled(bool bNewValue)
{
	if (ViewProperties)
	{
		SetToolPropertySourceEnabled(ViewProperties, bNewValue);
	}
}

void UMeshSculptToolBase::UpdateWireframeVisibility(bool bNewValue)
{
	GetSculptMeshComponent()->SetEnableWireframeRenderPass(bNewValue);
}

void UMeshSculptToolBase::UpdateFlatShadingSetting(bool bNewValue)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (bNewValue) ? 1.0f : 0.0f);
	}
}

void UMeshSculptToolBase::UpdateColorSetting(FLinearColor NewColor)
{
	if (ViewProperties->MaterialMode != EMeshEditingMaterialModes::Transparent)
	{
		if (ActiveOverrideMaterial != nullptr)
		{
			ActiveOverrideMaterial->SetVectorParameterValue(TEXT("Color"), NewColor);
		}
	}
}

void UMeshSculptToolBase::UpdateTransparentColorSetting(FLinearColor NewColor)
{
	// only want to update the active material if it is the transparent one...
	if (ViewProperties->MaterialMode == EMeshEditingMaterialModes::Transparent)
	{
		if (ActiveOverrideMaterial != nullptr)
		{
			ActiveOverrideMaterial->SetVectorParameterValue(TEXT("Color"), NewColor);
		}
	}
}



void UMeshSculptToolBase::UpdateImageSetting(UTexture2D* NewImage)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetTextureParameterValue(TEXT("ImageTexture"), NewImage);
	}
}

void UMeshSculptToolBase::UpdateOpacitySetting(double Opacity)
{
	if (ActiveOverrideMaterial != nullptr)
	{
		ActiveOverrideMaterial->SetScalarParameterValue(TEXT("Opacity"), Opacity);
	}
}

void UMeshSculptToolBase::UpdateTwoSidedSetting(bool bOn)
{
	if (ViewProperties->MaterialMode == EMeshEditingMaterialModes::Transparent)
	{
		ActiveOverrideMaterial = ToolSetupUtil::GetTransparentSculptMaterial(GetToolManager(),
			ViewProperties->TransparentMaterialColor, ViewProperties->Opacity, bOn);
		if (ActiveOverrideMaterial)
		{
			GetSculptMeshComponent()->SetOverrideRenderMaterial(ActiveOverrideMaterial);
		}
	}
}

void UMeshSculptToolBase::UpdateCustomMaterial(TWeakObjectPtr<UMaterialInterface> NewMaterial)
{
	if (ViewProperties->MaterialMode == EMeshEditingMaterialModes::Custom)
	{
		if (NewMaterial.IsValid())
		{
			ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(NewMaterial.Get(), this);
			GetSculptMeshComponent()->SetOverrideRenderMaterial(ActiveOverrideMaterial);
		}
		else
		{
			GetSculptMeshComponent()->ClearOverrideRenderMaterial();
			ActiveOverrideMaterial = nullptr;
		}
	}
}


void UMeshSculptToolBase::UpdateMaterialMode(EMeshEditingMaterialModes MaterialMode)
{
	if (MaterialMode == EMeshEditingMaterialModes::ExistingMaterial)
	{
		GetSculptMeshComponent()->ClearOverrideRenderMaterial();
		if (UPrimitiveComponent* TargetComponent = UE::ToolTarget::GetTargetComponent(Target))
		{
			GetSculptMeshComponent()->SetShadowsEnabled(TargetComponent->bCastDynamicShadow);
		}
		ActiveOverrideMaterial = nullptr;
	}
	else
	{
		if (MaterialMode == EMeshEditingMaterialModes::Custom)
		{
			if (ViewProperties->CustomMaterial.IsValid())
			{
				ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(ViewProperties->CustomMaterial.Get(), this);
			}
			else
			{
				GetSculptMeshComponent()->ClearOverrideRenderMaterial();
				ActiveOverrideMaterial = nullptr;
			}
		}
		else if (MaterialMode == EMeshEditingMaterialModes::CustomImage)
		{
			ActiveOverrideMaterial = ToolSetupUtil::GetCustomImageBasedSculptMaterial(GetToolManager(), ViewProperties->Image);
			if (ViewProperties->Image != nullptr)
			{
				ActiveOverrideMaterial->SetTextureParameterValue(TEXT("ImageTexture"), ViewProperties->Image);
			}
		}
		else if (MaterialMode == EMeshEditingMaterialModes::VertexColor)
		{
			ActiveOverrideMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager());
		}
		else if (MaterialMode == EMeshEditingMaterialModes::Transparent)
		{
			ActiveOverrideMaterial = ToolSetupUtil::GetTransparentSculptMaterial(GetToolManager(), 
				ViewProperties->TransparentMaterialColor, ViewProperties->Opacity, ViewProperties->bTwoSided);
		}
		else
		{
			UMaterialInterface* SculptMaterial = nullptr;
			switch (MaterialMode)
			{
			case EMeshEditingMaterialModes::Diffuse:
				SculptMaterial = ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager());
				break;
			case EMeshEditingMaterialModes::Grey:
				SculptMaterial = ToolSetupUtil::GetImageBasedSculptMaterial(GetToolManager(), ToolSetupUtil::ImageMaterialType::DefaultBasic);
				break;
			case EMeshEditingMaterialModes::Soft:
				SculptMaterial = ToolSetupUtil::GetImageBasedSculptMaterial(GetToolManager(), ToolSetupUtil::ImageMaterialType::DefaultSoft);
				break;
			case EMeshEditingMaterialModes::TangentNormal:
				SculptMaterial = ToolSetupUtil::GetImageBasedSculptMaterial(GetToolManager(), ToolSetupUtil::ImageMaterialType::TangentNormalFromView);
				break;
			}
			if (SculptMaterial != nullptr )
			{
				ActiveOverrideMaterial = UMaterialInstanceDynamic::Create(SculptMaterial, this);
			}
		}

		if (ActiveOverrideMaterial != nullptr)
		{
			GetSculptMeshComponent()->SetOverrideRenderMaterial(ActiveOverrideMaterial);
			ActiveOverrideMaterial->SetScalarParameterValue(TEXT("FlatShading"), (ViewProperties->bFlatShading) ? 1.0f : 0.0f);
		}

		GetSculptMeshComponent()->SetShadowsEnabled(false);
	}
}









void UMeshSculptToolBase::InitializeIndicator()
{
	// register and spawn brush indicator gizmo
	GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(VertexSculptIndicatorGizmoType, NewObject<UBrushStampIndicatorBuilder>());
	BrushIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UBrushStampIndicator>(VertexSculptIndicatorGizmoType, FString(), this);
	BrushIndicatorMesh = MakeBrushIndicatorMesh(this, TargetWorld);
	BrushIndicator->AttachedComponent = BrushIndicatorMesh->GetRootComponent();
	BrushIndicator->LineThickness = 1.0;
	BrushIndicator->bDrawIndicatorLines = true;
	BrushIndicator->bDrawRadiusCircle = false;
	BrushIndicator->LineColor = FLinearColor(0.9f, 0.4f, 0.4f);

	bIsVolumetricIndicator = true;
}

bool UMeshSculptToolBase::GetIsVolumetricIndicator()
{
	return bIsVolumetricIndicator;
}

void UMeshSculptToolBase::ConfigureIndicator(bool bVolumetric)
{
	if (bIsVolumetricIndicator == bVolumetric) return;
	bIsVolumetricIndicator = bVolumetric;
	BrushIndicatorMesh->SetVisible(GetIndicatorVisibility() && bIsVolumetricIndicator);
	if (bVolumetric)
	{
		BrushIndicator->bDrawRadiusCircle = false;
	}
	else
	{
		BrushIndicator->bDrawRadiusCircle = true;
	}
}


void UMeshSculptToolBase::SetIndicatorVisibility(bool bVisible)
{
	if (GetIndicatorVisibility() != bVisible)
	{
		BrushIndicator->bVisible = bVisible;
		BrushIndicatorMesh->SetVisible(bVisible && bIsVolumetricIndicator);
	}
}

bool UMeshSculptToolBase::GetIndicatorVisibility() const
{
	return BrushIndicator->bVisible;
}

UPreviewMesh* UMeshSculptToolBase::MakeBrushIndicatorMesh(UObject* Parent, UWorld* World)
{
	UPreviewMesh* SphereMesh = NewObject<UPreviewMesh>(Parent);
	SphereMesh->CreateInWorld(World, FTransform::Identity);
	FSphereGenerator SphereGen;
	SphereGen.NumPhi = SphereGen.NumTheta = 32;
	SphereGen.Generate();
	FDynamicMesh3 Mesh(&SphereGen);
	SphereMesh->UpdatePreview(&Mesh);

	BrushIndicatorMaterial = ToolSetupUtil::GetDefaultBrushVolumeMaterial(GetToolManager());
	if (BrushIndicatorMaterial)
	{
		SphereMesh->SetMaterial(BrushIndicatorMaterial);
	}

	// make sure raytracing is disabled on the brush indicator
	Cast<UDynamicMeshComponent>(SphereMesh->GetRootComponent())->SetEnableRaytracing(false);
	SphereMesh->SetShadowsEnabled(false);

	return SphereMesh;
}




void UMeshSculptToolBase::UpdateWorkPlane()
{
	bool bGizmoVisible = ShowWorkPlane() && (GizmoProperties->bShowGizmo);
	UpdateFixedPlaneGizmoVisibility(bGizmoVisible);
	GizmoProperties->bPropertySetEnabled = ShowWorkPlane();

	if (PendingWorkPlaneUpdate != EPendingWorkPlaneUpdate::NoUpdatePending)
	{
		// raycast into scene and current sculpt and place plane at closest hit point
		FRay CursorWorldRay = UMeshSurfacePointTool::LastWorldRay;
		FHitResult Result;
		bool bWorldHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, CursorWorldRay);
		FRay3d LocalRay = GetLocalRay(CursorWorldRay);
		int32 SculptMeshTid = FindHitSculptMeshTriangle(LocalRay);
		bool bObjectHit = (SculptMeshTid != IndexConstants::InvalidID);
		if (bWorldHit && 
			(bObjectHit == false || (CursorWorldRay.GetParameter(Result.ImpactPoint) < CursorWorldRay.GetParameter((FVector)HoverStamp.WorldFrame.Origin))))
		{
			SetFixedSculptPlaneFromWorldPos(Result.ImpactPoint, Result.ImpactNormal, PendingWorkPlaneUpdate);
		}
		else
		{
			FVector3d Normal = HoverStamp.WorldFrame.Z();
			// We're updating from a sculpt mesh hit. We can expect HoverStamp to be at the proper location,
			//  but the alignment might differ depending on how the cient is brushing, so get the triangle
			//  normal if we need it.
			if (bObjectHit && PendingWorkPlaneUpdate == EPendingWorkPlaneUpdate::MoveToHitPositionNormal)
			{
				FDynamicMesh3* Mesh = GetSculptMesh();
				if (ensure(Mesh))
				{
					Normal = Mesh->GetTriNormal(SculptMeshTid);
				}
			}

			SetFixedSculptPlaneFromWorldPos((FVector)HoverStamp.WorldFrame.Origin, Normal, PendingWorkPlaneUpdate);
		}
		PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::NoUpdatePending;
	}
}


void UMeshSculptToolBase::SetFixedSculptPlaneFromWorldPos(const FVector& Position, const FVector& Normal, EPendingWorkPlaneUpdate UpdateType)
{
	if (UpdateType == EPendingWorkPlaneUpdate::MoveToHitPositionNormal)
	{
		UpdateFixedSculptPlanePosition(Position);
		FFrame3d CurFrame(FVector::ZeroVector, GizmoProperties->Rotation);
		CurFrame.AlignAxis(2, (FVector3d)Normal);
		UpdateFixedSculptPlaneRotation((FQuat)CurFrame.Rotation);
	}
	else if (UpdateType == EPendingWorkPlaneUpdate::MoveToHitPositionViewAligned)
	{
		UpdateFixedSculptPlanePosition(Position);
		FFrame3d CurFrame(FVector::ZeroVector, GizmoProperties->Rotation);
		CurFrame.AlignAxis(2, -(FVector3d)CameraState.Forward());
		UpdateFixedSculptPlaneRotation((FQuat)CurFrame.Rotation);
	}
	else
	{
		UpdateFixedSculptPlanePosition(Position);
	}

	if (PlaneTransformGizmo != nullptr)
	{
		PlaneTransformGizmo->SetNewGizmoTransform(FTransform(GizmoProperties->Rotation, GizmoProperties->Position));
	}
}

void UMeshSculptToolBase::PlaneTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	UpdateFixedSculptPlaneRotation(Transform.GetRotation());
	UpdateFixedSculptPlanePosition(Transform.GetLocation());
}

void UMeshSculptToolBase::UpdateFixedSculptPlanePosition(const FVector& Position)
{
	GizmoProperties->Position = Position;
	GizmoPositionWatcher.SilentUpdate();
}

void UMeshSculptToolBase::UpdateFixedSculptPlaneRotation(const FQuat& Rotation)
{
	GizmoProperties->Rotation = Rotation;
	GizmoRotationWatcher.SilentUpdate();
}

void UMeshSculptToolBase::UpdateGizmoFromProperties()
{
	if (PlaneTransformGizmo != nullptr)
	{
		PlaneTransformGizmo->SetNewGizmoTransform(FTransform(GizmoProperties->Rotation, GizmoProperties->Position));
	}
}

void UMeshSculptToolBase::UpdateFixedPlaneGizmoVisibility(bool bVisible)
{
	if (bVisible == false)
	{
		if (PlaneTransformGizmo != nullptr)
		{
			GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(PlaneTransformGizmo);
			PlaneTransformGizmo = nullptr;
		}
	}
	else
	{
		if (PlaneTransformGizmo == nullptr)
		{
			PlaneTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(),
				ETransformGizmoSubElements::StandardTranslateRotate, this);
			PlaneTransformGizmo->bUseContextCoordinateSystem = false;
			PlaneTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
			PlaneTransformGizmo->SetActiveTarget(PlaneTransformProxy, GetToolManager());
			PlaneTransformGizmo->ReinitializeGizmoTransform(FTransform(GizmoProperties->Rotation, GizmoProperties->Position));
		}
	}
}







void UMeshSculptToolBase::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	constexpr EToolActionKeyRepeatMode KeyRepeatOnBrushAdjust = EToolActionKeyRepeatMode::Enabled;
	ActionSet.RegisterAction(this, (int32)EStandardToolActions::IncreaseBrushSize,
		TEXT("SculptIncreaseRadius"),
		LOCTEXT("SculptIncreaseRadius", "Increase Radius"),
		LOCTEXT("SculptIncreaseRadiusTooltip", "Increase Brush Radius"),
		EModifierKey::None, EKeys::RightBracket,
		[this]() { IncreaseBrushRadiusAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::DecreaseBrushSize,
		TEXT("SculptDecreaseRadius"),
		LOCTEXT("SculptDecreaseRadius", "Decrease Radius"),
		LOCTEXT("SculptDecreaseRadiusTooltip", "Decrease Brush Radius"),
		EModifierKey::None, EKeys::LeftBracket,
		[this]() { DecreaseBrushRadiusAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 54,
		TEXT("SculptIncreaseRadiusSmallStep"),
		LOCTEXT("SculptIncreaseRadius", "Increase Radius"),
		LOCTEXT("SculptIncreaseRadiusTooltip", "Increase Brush Radius"),
		EModifierKey::Shift, EKeys::RightBracket,
		[this]() { IncreaseBrushRadiusSmallStepAction(); },
		KeyRepeatOnBrushAdjust);

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 55,
		TEXT("SculptDecreaseRadiusSmallStemp"),
		LOCTEXT("SculptDecreaseRadius", "Decrease Radius"),
		LOCTEXT("SculptDecreaseRadiusTooltip", "Decrease Brush Radius"),
		EModifierKey::Shift, EKeys::LeftBracket,
		[this]() { DecreaseBrushRadiusSmallStepAction(); },
		KeyRepeatOnBrushAdjust);



	ActionSet.RegisterAction(this, (int32)EStandardToolActions::ToggleWireframe,
		TEXT("ToggleWireframe"),
		LOCTEXT("ToggleWireframe", "Toggle Wireframe"),
		LOCTEXT("ToggleWireframeTooltip", "Toggle visibility of wireframe overlay"),
		EModifierKey::Alt, EKeys::W,
		[this]() { ViewProperties->bShowWireframe = !ViewProperties->bShowWireframe; });



	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 100,
		TEXT("SetSculptWorkSurfacePosNormal"),
		LOCTEXT("SetSculptWorkSurfacePosNormal", "Reorient Work Surface"),
		LOCTEXT("SetSculptWorkSurfacePosNormalTooltip", "Move the Sculpting Work Plane/Surface to Position and Normal of World hit point under cursor"),
		EModifierKey::Shift, EKeys::T,
		[this]() { PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::MoveToHitPositionNormal; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 101,
		TEXT("SetSculptWorkSurfacePos"),
		LOCTEXT("SetSculptWorkSurfacePos", "Reposition Work Surface"),
		LOCTEXT("SetSculptWorkSurfacePosTooltip", "Move the Sculpting Work Plane/Surface to World hit point under cursor (keep current Orientation)"),
		EModifierKey::None, EKeys::T,
		[this]() { PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::MoveToHitPosition; });

	ActionSet.RegisterAction(this, (int32)EStandardToolActions::BaseClientDefinedActionID + 102,
		TEXT("SetSculptWorkSurfaceView"),
		LOCTEXT("SetSculptWorkSurfaceView", "View-Align Work Surface"),
		LOCTEXT("SetSculptWorkSurfaceViewTooltip", "Move the Sculpting Work Plane/Surface to World hit point under cursor and align to View"),
		EModifierKey::Control | EModifierKey::Shift, EKeys::T,
		[this]() { PendingWorkPlaneUpdate = EPendingWorkPlaneUpdate::MoveToHitPositionViewAligned; });
}


#undef LOCTEXT_NAMESPACE


