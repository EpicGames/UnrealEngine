// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sculpting/MeshSculptUtil.h"
#include "Async/ParallelFor.h"
#include "Async/Async.h"

using namespace UE::Geometry;

void UE::SculptUtil::RecalculateNormals_Overlay(
	FDynamicMesh3* Mesh, 
	const TSet<int32>& ModifiedTris, 
	TSet<int32>& VertexSetBuffer, 
	TArray<int32>& NormalsBuffer)
{
	FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
	check(Normals != nullptr);

	NormalsBuffer.Reset();
	VertexSetBuffer.Reset();
	for (int32 TriangleID : ModifiedTris)
	{
		FIndex3i TriElems = Normals->GetTriangle(TriangleID);
		if (TriElems.A != INDEX_NONE)
		{
			for (int32 j = 0; j < 3; ++j)
			{
				int32 elemid = TriElems[j];
				if (VertexSetBuffer.Contains(elemid) == false)
				{
					VertexSetBuffer.Add(elemid);
					NormalsBuffer.Add(elemid);
				}
			}
		}
	}

	ParallelFor(NormalsBuffer.Num(), [&](int32 k) {
		int32 elemid = NormalsBuffer[k];
		FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
		Normals->SetElement(elemid, (FVector3f)NewNormal);
	});
}



void UE::SculptUtil::RecalculateNormals_PerVertex(
	FDynamicMesh3* Mesh, 
	const TSet<int32>& ModifiedTris, 
	TSet<int32>& VertexSetBuffer, 
	TArray<int32>& NormalsBuffer)
{
	NormalsBuffer.Reset();
	VertexSetBuffer.Reset();
	for (int32 TriangleID : ModifiedTris)
	{
		FIndex3i TriV = Mesh->GetTriangle(TriangleID);
		for (int32 j = 0; j < 3; ++j)
		{
			int32 vid = TriV[j];
			if (VertexSetBuffer.Contains(vid) == false)
			{
				VertexSetBuffer.Add(vid);
				NormalsBuffer.Add(vid);
			}
		}
	}

	ParallelFor(NormalsBuffer.Num(), [&](int32 k) {
		int32 vid = NormalsBuffer[k];
		FVector3d NewNormal = FMeshNormals::ComputeVertexNormal(*Mesh, vid);
		Mesh->SetVertexNormal(vid, (FVector3f)NewNormal);
	});
}



void UE::SculptUtil::RecalculateROINormals(
	FDynamicMesh3* Mesh, 
	const TSet<int32>& TriangleROI, 
	TSet<int32>& VertexSetBuffer, 
	TArray<int32>& NormalsBuffer, 
	bool bForceVertex)
{
	if (Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals() && bForceVertex == false)
	{
		RecalculateNormals_Overlay(Mesh, TriangleROI, VertexSetBuffer, NormalsBuffer);
	}
	else
	{
		RecalculateNormals_PerVertex(Mesh, TriangleROI, VertexSetBuffer, NormalsBuffer);
	}
}










void UE::SculptUtil::RecalculateNormals_Overlay(
	FDynamicMesh3* Mesh, 
	const TSet<int32>& ModifiedTris, 
	FUniqueIndexSet& VertexSetTemp)
{
	FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
	check(Normals != nullptr);

	VertexSetTemp.Initialize(Normals->MaxElementID());
	for (int32 TriangleID : ModifiedTris)
	{
		FIndex3i TriElems = Normals->GetTriangle(TriangleID);
		if (TriElems.A != INDEX_NONE)
		{
			VertexSetTemp.Add(TriElems.A);
			VertexSetTemp.Add(TriElems.B);
			VertexSetTemp.Add(TriElems.C);
		}
	}

	const TArray<int32>& ElemIndices = VertexSetTemp.Indices();
	ParallelFor(ElemIndices.Num(), [&](int32 k) {
		int32 elemid = ElemIndices[k];
		FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
		Normals->SetElement(elemid, (FVector3f)NewNormal);
	});
}



void UE::SculptUtil::RecalculateNormals_PerVertex(
	FDynamicMesh3* Mesh, 
	const TSet<int32>& ModifiedTris, 
	FUniqueIndexSet& VertexSetTemp)
{
	VertexSetTemp.Initialize(Mesh->MaxVertexID());
	for (int32 TriangleID : ModifiedTris)
	{
		const FIndex3i& Triangle = Mesh->GetTriangleRef(TriangleID);
		VertexSetTemp.Add(Triangle.A);
		VertexSetTemp.Add(Triangle.B);
		VertexSetTemp.Add(Triangle.C);
	}

	const TArray<int32>& ElemIndices = VertexSetTemp.Indices();
	ParallelFor(ElemIndices.Num(), [&](int32 k) {
		int32 vid = ElemIndices[k];
		FVector3d NewNormal = FMeshNormals::ComputeVertexNormal(*Mesh, vid);
		Mesh->SetVertexNormal(vid, (FVector3f)NewNormal);
	});
}



void UE::SculptUtil::RecalculateROINormals(
	FDynamicMesh3* Mesh, 
	const TSet<int32>& TriangleROI, 
	FUniqueIndexSet& VertexSetTemp,
	bool bForceVertex)
{
	if (Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals() && bForceVertex == false)
	{
		RecalculateNormals_Overlay(Mesh, TriangleROI, VertexSetTemp);
	}
	else
	{
		RecalculateNormals_PerVertex(Mesh, TriangleROI, VertexSetTemp);
	}
}




static FAxisAlignedBox3d ParallelComputeROIBounds(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles)
{
	return Mesh.GetBoundsForTriangleSelection(Triangles);
}



namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution MeshSculptUtilAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution MeshSculptUtilAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}



void UE::SculptUtil::PrecalculateNormalsROI(
	const FDynamicMesh3* Mesh,
	const TArray<int32>& TriangleROI,
	FUniqueIndexSet& IndexSetTemp,
	bool& bIsOverlayElements,
	bool bForceVertex)
{
	const FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
	if (Normals != nullptr && bForceVertex == false)
	{
		bIsOverlayElements = true;

		static TArray<int32> List; 
		static TArray<std::atomic<bool>> Flags;
		if (Flags.Num() != Normals->MaxElementID())
		{
			Flags.SetNum(Normals->MaxElementID());
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_InitClear);
			ParallelFor(Normals->MaxElementID(), [&](int32 i) { Flags[i] = false; });
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_FIND);
			List.Reset();
			FCriticalSection ListLock;
			int32 N = TriangleROI.Num();
			ParallelFor(TriangleROI.Num(), [&](int i) {
				FIndex3i TriElems = Normals->GetTriangle(TriangleROI[i]);
				Flags[TriElems.A] = true;
				Flags[TriElems.B] = true;
				Flags[TriElems.C] = true;
				//bool PrevA = false, PrevB = false, PrevC = false;
				//bool SetA = Flags[TriElems.A].compare_exchange_strong(PrevA, true);
				//bool SetB = Flags[TriElems.B].compare_exchange_strong(PrevB, true);
				//bool SetC = Flags[TriElems.C].compare_exchange_strong(PrevC, true);
				//if ( SetA || SetB || SetC )
				//{
				//	ListLock.Lock();
				//	if (SetA) List.Add(TriElems.A);
				//	if (SetB) List.Add(TriElems.B);
				//	if (SetC) List.Add(TriElems.C);
				//	ListLock.Unlock();
				//}
			});
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_NORMALS);
			ParallelFor(Normals->MaxElementID(), [&](int32 elemid) {
				if (Flags[elemid] == true)
				{
					FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
					//Normals->SetElement(elemid, (FVector3f)NewNormal);
				}
			});
		}
		//{
		//	TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_Clear);
		//	for (int32 k : List)
		//	{
		//		Flags[k].store(false);
		//	}
		//}

		//{
		//	TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_InsertTest);
		//	static TArray<int32> ListX;
		//	ListX.Reset();
		//	for (int32 k : List)
		//	{
		//		ListX.Add(k);
		//	}
		//}


		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_BitArraySet);
			IndexSetTemp.Initialize(Normals->MaxElementID());
			for (int32 TriangleID : TriangleROI)
			{
				FIndex3i TriElems = Normals->GetTriangle(TriangleID);
				IndexSetTemp.Add(TriElems.A);
				IndexSetTemp.Add(TriElems.B);
				IndexSetTemp.Add(TriElems.C);
			}
		}

		UE_LOGF(LogTemp, Warning, "LIST %d  INDEXSET %d", List.Num(), IndexSetTemp.Indices().Num());
	}
	else
	{
		bIsOverlayElements = false;

		IndexSetTemp.Initialize(Mesh->MaxVertexID());
		for (int32 TriangleID : TriangleROI)
		{
			const FIndex3i& Triangle = Mesh->GetTriangleRef(TriangleID);
			IndexSetTemp.Add(Triangle.A);
			IndexSetTemp.Add(Triangle.B);
			IndexSetTemp.Add(Triangle.C);
		}
	}
}



void UE::SculptUtil::RecalculateROINormals(FDynamicMesh3* Mesh, const TArray<int32>& Indices, bool bIsOverlayElements)
{
	if (bIsOverlayElements)
	{
		FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
		ParallelFor(Indices.Num(), [&](int32 k) {
			int32 elemid = Indices[k];
			FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
			Normals->SetElement(elemid, (FVector3f)NewNormal);
		});
	}
	else
	{
		ParallelFor(Indices.Num(), [&](int32 k) {
			int32 vid = Indices[k];
			FVector3d NewNormal = FMeshNormals::ComputeVertexNormal(*Mesh, vid);
			Mesh->SetVertexNormal(vid, (FVector3f)NewNormal);
		});
	}
}




void UE::SculptUtil::PrecalculateNormalsROI(const FDynamicMesh3* Mesh, const TArray<int32>& TriangleROI,
	TArray<std::atomic<bool>>& ROIFlags, bool& bIsOverlayElements, bool bForceVertex)
{
	const FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
	if (Normals != nullptr && bForceVertex == false)
	{
		bIsOverlayElements = true;
		
		if (ROIFlags.Num() != Normals->MaxElementID())
		{
			ROIFlags.SetNum(Normals->MaxElementID());
			ParallelFor(Normals->MaxElementID(), [&](int i) { ROIFlags[i] = false; });
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrecalculateNormalsROI_FIND);
			ParallelFor(TriangleROI.Num(), [&](int i) {
				if (Normals->IsSetTriangle(TriangleROI[i]))
				{
					FIndex3i TriElems = Normals->GetTriangle(TriangleROI[i]);
					ROIFlags[TriElems.A] = true;
					ROIFlags[TriElems.B] = true;
					ROIFlags[TriElems.C] = true;
				}
			});
		}
	}
	else
	{
		bIsOverlayElements = false;

		check(false);
	}
}

void UE::SculptUtil::RecalculateROINormals(FDynamicMesh3* Mesh, TArray<std::atomic<bool>>& ROIFlags, bool bIsOverlayElements)
{
	if (bIsOverlayElements)
	{
		FDynamicMeshNormalOverlay* Normals = Mesh->HasAttributes() ? Mesh->Attributes()->PrimaryNormals() : nullptr;
		ParallelFor(ROIFlags.Num(), [&](int32 elemid) 
		{
			if (ROIFlags[elemid])
			{
				FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
				Normals->SetElement(elemid, (FVector3f)NewNormal);
				ROIFlags[elemid] = false;
			}

			//bool bFalse = false;
			//bool bWasTrue = ROIFlags[elemid].compare_exchange_strong(bFalse, true);
			//if (bWasTrue)
			//{
			//	FVector3d NewNormal = FMeshNormals::ComputeOverlayNormal(*Mesh, Normals, elemid);
			//	Normals->SetElement(elemid, (FVector3f)NewNormal);
			//}
		});
	}
	else
	{
		ParallelFor(ROIFlags.Num(), [&](int32 vid)
		{
			if (ROIFlags[vid])
			{
				FVector3d NewNormal = FMeshNormals::ComputeVertexNormal(*Mesh, vid);
				Mesh->SetVertexNormal(vid, (FVector3f)NewNormal);
				ROIFlags[vid] = false;
			}

			//bool bFalse = false;
			//bool bWasTrue = ROIFlags[vid].compare_exchange_strong(bFalse, true);
			//if (bWasTrue)
			//{
			//	FVector3d NewNormal = FMeshNormals::ComputeVertexNormal(*Mesh, vid);
			//	Mesh->SetVertexNormal(vid, (FVector3f)NewNormal);
			//}
		});

	}
}


void UE::SculptUtil::RecalculateROINormalForTriangles(FDynamicMesh3* Mesh, TArray<int> Triangles, bool bIsOverlayElements)
{
	if (bIsOverlayElements)
	{
		FMeshNormals::RecomputeOverlayTriNormals(*Mesh, Triangles);
	}
	else
	{
		FMeshNormals::QuickComputeVertexNormalsForTriangles(*Mesh, Triangles);
	}
}
