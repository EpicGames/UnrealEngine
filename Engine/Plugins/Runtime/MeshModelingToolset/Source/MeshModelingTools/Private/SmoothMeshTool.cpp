// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothMeshTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"

// Smoothing operators
#include "DynamicSubmesh3.h"
#include "Selections/GeometrySelectionUtil.h"
#include "SmoothingOps/IterativeSmoothingOp.h"
#include "SmoothingOps/CotanSmoothingOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SmoothMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USmoothMeshTool"

/*
 * Tool Builder
 */
 
USingleTargetWithSelectionTool* USmoothMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<USmoothMeshTool>(SceneState.ToolManager);
}

/*
 * Tool
 */

USmoothMeshTool::USmoothMeshTool()
{
	SetToolDisplayName(LOCTEXT("ToolName", "Smooth"));

}

void USmoothMeshTool::InitializeProperties()
{
	SmoothProperties = NewObject<USmoothMeshToolProperties>(this);
	AddToolPropertySource(SmoothProperties);
	SmoothProperties->RestoreProperties(this);
	SmoothProperties->WatchProperty(SmoothProperties->SmoothingType,
		[&](ESmoothMeshToolSmoothType) { UpdateOptionalPropertyVisibility(); InvalidateResult();  });

	IterativeProperties = AddOptionalPropertySet<UIterativeSmoothProperties>(
		[&]() { return SmoothProperties->SmoothingType == ESmoothMeshToolSmoothType::Iterative; } );

	DiffusionProperties = AddOptionalPropertySet<UDiffusionSmoothProperties>(
		[&]() { return SmoothProperties->SmoothingType == ESmoothMeshToolSmoothType::Diffusion; });

	ImplicitProperties = AddOptionalPropertySet<UImplicitSmoothProperties>(
		[&]() { return SmoothProperties->SmoothingType == ESmoothMeshToolSmoothType::Implicit; });

	WeightMapProperties = AddWeightMapPropertySet<USmoothWeightMapSetProperties>(
		[&]() { return SmoothProperties->SmoothingType != ESmoothMeshToolSmoothType::Diffusion; });
	WeightMapProperties->WatchProperty(WeightMapProperties->MinSmoothMultiplier,
		[&](float) { InvalidateResult(); });
}


void USmoothMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	SmoothProperties->SaveProperties(this);
}


FText USmoothMeshTool::GetToolMessageString() const
{
	return LOCTEXT("StartSmoothToolMessage", "Smooth the mesh vertex positions using various smoothing methods.");
}

FText USmoothMeshTool::GetAcceptTransactionName() const
{
	return LOCTEXT("SmoothMeshToolTransactionName", "Smooth Mesh");
}


bool USmoothMeshTool::HasMeshTopologyChanged() const 
{ 
	return false; 
}


TUniquePtr<FDynamicMeshOperator> USmoothMeshTool::MakeNewOperator()
{
	TUniquePtr<FSmoothingOpBase> MeshOp;
	
	const FDynamicMesh3* Mesh = &GetInitialMesh();

	bool bUsingSelection = false;

	UE::Geometry::FDynamicSubmesh3 SubmeshToSmooth;
	
	if (HasGeometrySelection())
	{
		// retrieves all triangles in the current selected geometry
		TSet<int> TrianglesSelected;
		const FGeometrySelection& InputSelection = GetGeometrySelection();
		UE::Geometry::EnumerateSelectionTriangles(InputSelection, *Mesh,
		[&](int32 TriangleID){TrianglesSelected.Add(TriangleID); });

		// creates a Submesh from the selection
		SubmeshToSmooth = FDynamicSubmesh3(Mesh,TrianglesSelected.Array());
		bUsingSelection = true;
	}

	FSmoothingOpBase::FOptions Options;
	Options.BaseNormals = this->GetInitialVtxNormals();

	if (HasActiveWeightMap())
	{
		Options.bUseWeightMap = true;
		Options.WeightMap = GetActiveWeightMap();
		Options.WeightMapMinMultiplier = WeightMapProperties->MinSmoothMultiplier;
	}

	switch (SmoothProperties->SmoothingType)
	{
	default:
	case ESmoothMeshToolSmoothType::Iterative:
		Options.SmoothAlpha = IterativeProperties->SmoothingPerStep;
		Options.BoundarySmoothAlpha = FMathd::Lerp(0.0, 0.9, IterativeProperties->SmoothingPerStep);
		Options.Iterations = IterativeProperties->Steps;
		Options.bSmoothBoundary = IterativeProperties->bSmoothBoundary;
		Options.bUniform = true;
		Options.bUseImplicit = false;
		if (bUsingSelection)
		{
			MeshOp = MakeUnique<FIterativeSmoothingOp>(Mesh, Options, SubmeshToSmooth);
		}
		else
		{
			MeshOp = MakeUnique<FIterativeSmoothingOp>(Mesh, Options);
		}
		break;

	case ESmoothMeshToolSmoothType::Diffusion:
		Options.SmoothAlpha = DiffusionProperties->SmoothingPerStep;
		Options.BoundarySmoothAlpha = FMathd::Lerp(0.0, 0.9, IterativeProperties->SmoothingPerStep);
		Options.Iterations = DiffusionProperties->Steps;
		Options.bUniform = DiffusionProperties->bPreserveUVs == false;
		Options.bUseImplicit = true;
		if (bUsingSelection)
		{
			MeshOp = MakeUnique<FIterativeSmoothingOp>(Mesh, Options, SubmeshToSmooth);
		}
		else
		{
			MeshOp = MakeUnique<FIterativeSmoothingOp>(Mesh, Options);
		}
		break;

	case ESmoothMeshToolSmoothType::Implicit:
		{	
		Options.SmoothAlpha = ImplicitProperties->SmoothSpeed;
		Options.BoundarySmoothAlpha = 0.0;
		double NonlinearT = FMathd::Pow(ImplicitProperties->Smoothness, 2.0);
		// this is an empirically-determined hack that seems to work OK to normalize the smoothing result for variable vertex count...
		double ScaledPower = (NonlinearT/50.0) * Mesh->VertexCount();
		Options.SmoothPower = ScaledPower;
		Options.bUniform = ImplicitProperties->bPreserveUVs == false;
		Options.bUseImplicit = true;
		Options.NormalOffset = ImplicitProperties->VolumeCorrection;
		if (bUsingSelection)
		{
			MeshOp = MakeUnique<FCotanSmoothingOp>(Mesh, Options, SubmeshToSmooth);
		}
		else
		{
			MeshOp = MakeUnique<FCotanSmoothingOp>(Mesh, Options);
		}
		}
		break;
	}

	FTransform3d XForm3d(GetPreviewTransform());
	MeshOp->SetTransform(XForm3d);

	return MeshOp;
}




#undef LOCTEXT_NAMESPACE

