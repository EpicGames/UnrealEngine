// Copyright Epic Games, Inc. All Rights Reserved.


//
// Automation testing
//

#include "Misc/AutomationTest.h"
#include "Generators/GridBoxMeshGenerator.h"

// Test helpers
namespace DisplaceMeshToolTestsLocals
{
	using namespace DisplaceMeshToolLocals;

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CreateSingleTriangleMesh()
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> Mesh = MakeShareable<FDynamicMesh3>(new FDynamicMesh3);
		int VertexIndexA = Mesh->AppendVertex(FVector3d{ 0,0,0 });
		int VertexIndexB = Mesh->AppendVertex(FVector3d{ 0,1,0 });
		int VertexIndexC = Mesh->AppendVertex(FVector3d{ 1,0,0 });
		Mesh->AppendTriangle(VertexIndexA, VertexIndexB, VertexIndexC);

		return Mesh;
	}

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> CreateTestBoxMesh()
	{
		FGridBoxMeshGenerator BoxGenerator;
		BoxGenerator.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), FVector3d(1.0, 1.0, 1.0));
		int EdgeNum = 5;
		BoxGenerator.EdgeVertices = FIndex3i(EdgeNum, EdgeNum, EdgeNum);
		BoxGenerator.Generate();

		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> Mesh = MakeShareable<FDynamicMesh3>(new FDynamicMesh3(&BoxGenerator));
		return Mesh;
	}
}

// Test constant mode

IMPLEMENT_SIMPLE_AUTOMATION_TEST(UDisplaceMeshTestConstant,
	"MeshModeling.DisplaceMesh.Constant Displace Type",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool UDisplaceMeshTestConstant::RunTest(const FString& Parameters)
{
	using namespace DisplaceMeshToolTestsLocals;

	// ---- Setup
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = CreateSingleTriangleMesh();

	// Requirements:
	UTEST_TRUE("Input mesh has 3 vertices", InputMesh->VertexCount() == 3);
	UTEST_TRUE("Input mesh has 1 triangle", InputMesh->TriangleCount() == 1);
	UTEST_TRUE("Input mesh has vertex at index 0", InputMesh->IsVertex(0));

	// This is done internally during DisplaceOp::CalculateResult, but do it here so we can verify normals up front
	UE::Geometry::FMeshNormals InputMeshNormals(InputMesh.Get());
	InputMeshNormals.ComputeVertexNormals();
	UTEST_TRUE("Input mesh normals point in +Z direction", FMath::IsNearlyZero((InputMeshNormals.GetNormals()[0] - FVector3d{ 0,0,1 }).Length()));

	EDisplaceMeshToolDisplaceType DisplaceType = EDisplaceMeshToolDisplaceType::Constant;
	DisplaceMeshParameters Params;
	Params.DisplaceIntensity = 10.0f;

	FDisplaceMeshOpFactory DisplaceOpFactory(InputMesh, Params, DisplaceType);
	TUniquePtr<FDynamicMeshOperator> DisplaceOp = DisplaceOpFactory.MakeNewOperator();

	// ---- Act
	FProgressCancel ProgressCancel;
	DisplaceOp->CalculateResult(&ProgressCancel);

	// ---- Verify
	TUniquePtr<FDynamicMesh3> FinalMesh = DisplaceOp->ExtractResult();

	UTEST_TRUE("Resulting mesh has 3 vertices", FinalMesh->VertexCount() == 3);
	UTEST_TRUE("Resulting mesh has 1 triangle", FinalMesh->TriangleCount() == 1);
	UTEST_TRUE("Input mesh has vertex at index 0", FinalMesh->IsVertex(0));

	UTEST_TRUE("Resulting mesh moved in the normal direction", (FinalMesh->GetVertex(0).Z > InputMesh->GetVertex(0).Z));

	return true;
}


// Test random mode

IMPLEMENT_SIMPLE_AUTOMATION_TEST(UDisplaceMeshTestRandom,
	"MeshModeling.DisplaceMesh.Random Displace Type",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool UDisplaceMeshTestRandom::RunTest(const FString& Parameters)
{
	using namespace DisplaceMeshToolTestsLocals;

	// ---- Setup
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = CreateSingleTriangleMesh();

	EDisplaceMeshToolDisplaceType DisplaceType = EDisplaceMeshToolDisplaceType::RandomNoise;
	DisplaceMeshParameters Params;
	Params.DisplaceIntensity = 100.0f;
	Params.RandomSeed = 100;

	FDisplaceMeshOpFactory DisplaceOpFactory(InputMesh, Params, DisplaceType);
	TUniquePtr<FDynamicMeshOperator> DisplaceOp = DisplaceOpFactory.MakeNewOperator();

	// ---- Act
	FProgressCancel ProgressCancel;
	DisplaceOp->CalculateResult(&ProgressCancel);

	// ---- Verify
	TUniquePtr<FDynamicMesh3> FinalMesh = DisplaceOp->ExtractResult();

	// Check that the vertex moved at all. Unless we're really unlucky each vertex should have moved at least a little.
	double Dist0 = (InputMesh->GetVertex(0) - FinalMesh->GetVertex(0)).Length();
	UTEST_TRUE("Mesh vertex moved", Dist0 > 1e-4);

	return true;
}


// Test Perlin noise mode

IMPLEMENT_SIMPLE_AUTOMATION_TEST(UDisplaceMeshTestPerlin,
	"MeshModeling.DisplaceMesh.Perlin Noise Displace Type",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool UDisplaceMeshTestPerlin::RunTest(const FString& Parameters)
{
	using namespace DisplaceMeshToolTestsLocals;

	// ---- Setup
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = CreateSingleTriangleMesh();

	EDisplaceMeshToolDisplaceType DisplaceType = EDisplaceMeshToolDisplaceType::PerlinNoise;
	DisplaceMeshParameters Params;
	Params.RandomSeed = 100;
	Params.PerlinLayerProperties.Add({ 1.0f, 100.0f });	/* {Frequency, Amplitude} */
	Params.DisplaceIntensity = 1.0;

	FDisplaceMeshOpFactory DisplaceOpFactory(InputMesh, Params, DisplaceType);
	TUniquePtr<FDynamicMeshOperator> DisplaceOp = DisplaceOpFactory.MakeNewOperator();

	// ---- Act
	FProgressCancel ProgressCancel;
	DisplaceOp->CalculateResult(&ProgressCancel);
	// ---- Verify
	TUniquePtr<FDynamicMesh3> FinalMesh = DisplaceOp->ExtractResult();

	// Check that the vertex moved at all. Unless we're really unlucky each vertex should have moved at least a little.
	FVector3d FinalMeshVertex = FinalMesh->GetVertex(0);
	double Dist0 = (InputMesh->GetVertex(0) - FinalMeshVertex).Length();
	UTEST_TRUE("Mesh vertex moved", Dist0 > 1e-4);

	return true;
}


// Test sine wave mode

IMPLEMENT_SIMPLE_AUTOMATION_TEST(UDisplaceMeshTestSine,
	"MeshModeling.DisplaceMesh.Sine Displace Type",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool UDisplaceMeshTestSine::RunTest(const FString& Parameters)
{
	using namespace DisplaceMeshToolTestsLocals;

	// ---- Setup
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = CreateSingleTriangleMesh();

	EDisplaceMeshToolDisplaceType DisplaceType = EDisplaceMeshToolDisplaceType::SineWave;
	DisplaceMeshParameters Params;
	Params.DisplaceIntensity = 1.0f;
	Params.SineWaveFrequency = 1.0f;
	Params.SineWavePhaseShift = 0.0f;	// TODO: Test this as well
	Params.SineWaveDirection = { 0.0f, 0.0f, 1.0f };

	FDisplaceMeshOpFactory DisplaceOpFactory(InputMesh, Params, DisplaceType);
	TUniquePtr<FDynamicMeshOperator> DisplaceOp = DisplaceOpFactory.MakeNewOperator();

	// ---- Act
	FProgressCancel ProgressCancel;
	DisplaceOp->CalculateResult(&ProgressCancel);

	// ---- Verify
	TUniquePtr<FDynamicMesh3> FinalMesh = DisplaceOp->ExtractResult();

	double ComputedZ0 = FinalMesh->GetVertex(0).Z;
	double ComputedZ1 = FinalMesh->GetVertex(1).Z;
	double ComputedZ2 = FinalMesh->GetVertex(2).Z;
	double ExpectedZ0 = InputMesh->GetVertex(0).Z + FMath::Sin(0.0);	// vertex at [0,0,0]
	double ExpectedZ1 = InputMesh->GetVertex(1).Z + FMath::Sin(1.0);	// vertex at [0,1,0]
	double ExpectedZ2 = InputMesh->GetVertex(2).Z + FMath::Sin(1.0);	// vertex at [1,0,0]

	UTEST_TRUE("Sine displacement computed as expected", FMath::IsNearlyEqual(ComputedZ0, ExpectedZ0, 1e-4));
	UTEST_TRUE("Sine displacement computed as expected", FMath::IsNearlyEqual(ComputedZ1, ExpectedZ1, 1e-4));
	UTEST_TRUE("Sine displacement computed as expected", FMath::IsNearlyEqual(ComputedZ2, ExpectedZ2, 1e-4));

	return true;
}


// Test sine wave direction

IMPLEMENT_SIMPLE_AUTOMATION_TEST(UDisplaceMeshTestSineDirection,
	"MeshModeling.DisplaceMesh.Sine Displace with Direction",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool UDisplaceMeshTestSineDirection::RunTest(const FString& Parameters)
{
	using namespace DisplaceMeshToolTestsLocals;

	// ---- Setup
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = CreateSingleTriangleMesh();

	EDisplaceMeshToolDisplaceType DisplaceType = EDisplaceMeshToolDisplaceType::SineWave;
	DisplaceMeshParameters Params;
	Params.DisplaceIntensity = 1.0f;
	Params.SineWaveFrequency = 1.0f;
	Params.SineWavePhaseShift = 0.0f;
	Params.SineWaveDirection = { 0.0f, 1.0f, 0.0f };

	FDisplaceMeshOpFactory DisplaceOpFactory(InputMesh, Params, DisplaceType);
	TUniquePtr<FDynamicMeshOperator> DisplaceOp = DisplaceOpFactory.MakeNewOperator();

	// ---- Act
	FProgressCancel ProgressCancel;
	DisplaceOp->CalculateResult(&ProgressCancel);

	// ---- Verify
	TUniquePtr<FDynamicMesh3> FinalMesh = DisplaceOp->ExtractResult();

	double ComputedY0 = FinalMesh->GetVertex(0).Y;
	double ComputedY1 = FinalMesh->GetVertex(1).Y;
	double ComputedY2 = FinalMesh->GetVertex(2).Y;
	double ExpectedY0 = InputMesh->GetVertex(0).Y + FMath::Sin(0.0);      // vertex at [0,0,0]
	double ExpectedY1 = InputMesh->GetVertex(1).Y + FMath::Sin(0.0);		// vertex at [0,1,0]
	double ExpectedY2 = InputMesh->GetVertex(2).Y + FMath::Sin(1.0);      // vertex at [1,0,0]

	UTEST_TRUE("Y-axis sine displacement computed as expected", FMath::IsNearlyEqual(ComputedY0, ExpectedY0, 1e-4));
	UTEST_TRUE("Y-axis sine displacement computed as expected", FMath::IsNearlyEqual(ComputedY1, ExpectedY1, 1e-4));
	UTEST_TRUE("Y-axis sine displacement computed as expected", FMath::IsNearlyEqual(ComputedY2, ExpectedY2, 1e-4));

	return true;
}

// Copyright Epic Games, Inc. All Rights Reserved.


//
// This file is included in DisplaceMeshTool.cpp
//


// Test random + direction filter

IMPLEMENT_SIMPLE_AUTOMATION_TEST(UDisplaceMeshTestDirectionalFilter,
	"MeshModeling.DisplaceMesh.Directional Filter",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	bool UDisplaceMeshTestDirectionalFilter::RunTest(const FString& Parameters)
{
	using namespace DisplaceMeshToolTestsLocals;

	// ---- Setup
	// Create a box mesh, random noise, and a directional filter pointing in the +Y direction
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh = CreateTestBoxMesh();

	UE::Geometry::FMeshNormals InputMeshNormals(InputMesh.Get());
	InputMeshNormals.ComputeVertexNormals();

	EDisplaceMeshToolDisplaceType DisplaceType = EDisplaceMeshToolDisplaceType::RandomNoise;
	DisplaceMeshParameters Params;
	Params.DisplaceIntensity = 100.0f;
	Params.RandomSeed = 100;
	Params.bEnableFilter = true;
	Params.FilterDirection = FVector{ 0.0f, 1.0f, 0.0f };
	Params.FilterWidth = 0.2f;

	FDisplaceMeshOpFactory DisplaceOpFactory(InputMesh, Params, DisplaceType);
	TUniquePtr<FDynamicMeshOperator> DisplaceOp = DisplaceOpFactory.MakeNewOperator();

	// ---- Act
	FProgressCancel ProgressCancel;
	DisplaceOp->CalculateResult(&ProgressCancel);

	// ---- Verify
	TUniquePtr<FDynamicMesh3> FinalMesh = DisplaceOp->ExtractResult();

	UTEST_TRUE("Before and after mesh vertex counts match", InputMesh->VertexCount() == FinalMesh->VertexCount());

	for (auto VertexID : InputMesh->VertexIndicesItr())
	{
		double DistanceMoved = (InputMesh->GetVertex(VertexID) - FinalMesh->GetVertex(VertexID)).Length();
		FVector3d VertexNormal = InputMeshNormals[VertexID];
		double Dot = VertexNormal.Dot((FVector3d)Params.FilterDirection);

		if (Dot < -0.5)
		{
			// Should not move
			UTEST_TRUE("Vertex whose normal doesn't match the filter direction should not move", FMath::IsNearlyZero(DistanceMoved));
		}
		else if (Dot > 0.5)
		{
			// Should move
			UTEST_TRUE("Vertex whose normal matches the filter direction should move", DistanceMoved > 1e-4);
		}
	}

	return true;
}