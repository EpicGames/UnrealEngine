// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditActivityUtil.h"

#include "Drawing/PolyEditPreviewMesh.h"
#include "InteractiveTool.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "MeshOpPreviewHelpers.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolSetupUtil.h"

using namespace UE::Geometry;

UPolyEditPreviewMesh* PolyEditActivityUtil::CreatePolyEditPreviewMesh(UInteractiveTool& Tool, const UPolyEditActivityContext& ActivityContext)
{
	UPolyEditPreviewMesh* EditPreview = NewObject<UPolyEditPreviewMesh>(&Tool);
	EditPreview->CreateInWorld(Tool.GetWorld(), FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(EditPreview, nullptr); 
	UpdatePolyEditPreviewMaterials(Tool, ActivityContext, *EditPreview, EPreviewMaterialType::PreviewMaterial);
	EditPreview->EnableWireframe(true);

	return EditPreview;
}

void PolyEditActivityUtil::UpdatePolyEditPreviewMaterials(UInteractiveTool& Tool, const UPolyEditActivityContext& ActivityContext,
	UPolyEditPreviewMesh& EditPreview, EPreviewMaterialType MaterialType)
{
	if (MaterialType == EPreviewMaterialType::SourceMaterials)
	{
		TArray<UMaterialInterface*> Materials;
		ActivityContext.Preview->PreviewMesh->GetMaterials(Materials);

		EditPreview.ClearOverrideRenderMaterial();
		EditPreview.SetMaterials(Materials);
	}
	else if (MaterialType == EPreviewMaterialType::PreviewMaterial)
	{
		EditPreview.ClearOverrideRenderMaterial();
		EditPreview.SetMaterial(
			ToolSetupUtil::GetSelectionMaterial(FLinearColor(0.8f, 0.75f, 0.0f), Tool.GetToolManager()));
	}
	else if (MaterialType == EPreviewMaterialType::UVMaterial)
	{
		UMaterial* CheckerMaterialBase = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/CheckerMaterial"));
		if (CheckerMaterialBase != nullptr)
		{
			UMaterialInstanceDynamic* CheckerMaterial = UMaterialInstanceDynamic::Create(CheckerMaterialBase, NULL);
			CheckerMaterial->SetScalarParameterValue("Density", 1);
			EditPreview.SetOverrideRenderMaterial(CheckerMaterial);
		}
	}
}
