// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditBevelEdgeActivity.h"

#include "InteractiveToolManager.h"
#include "ContextObjectStore.h"
#include "EditMeshPolygonsTool.h"
#include "MeshOpPreviewHelpers.h"
#include "Operations/MeshBevel.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditBevelEdgeActivity)

#define LOCTEXT_NAMESPACE "UPolyEditBevelEdgeActivity"

using namespace UE::Geometry;



class FBevelOp : public FDynamicMeshOperator
{
public:
	virtual ~FBevelOp() {}

	// Inputs:
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<UE::Geometry::FGroupTopology, ESPMode::ThreadSafe> MeshTopology;
	TArray<int32> BevelGroupEdges;
	TArray<int32> BevelGroupFaces;
	double BevelDistance = 1.0;
	int32 Subdivisions = 0;
	double RoundWeight = 1.0;
	bool bInferMaterialID = true;
	int32 SetMaterialID = 0;

	void SetTransform(const FTransformSRT3d& Transform)
	{
		ResultTransform = Transform;
	}

	// FDynamicMeshOperator implementation 
	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		if (FMath::Abs(BevelDistance) < KINDA_SMALL_NUMBER || (Progress && Progress->Cancelled()))
		{
			return;
		}
		ResultMesh->Copy(*OriginalMesh, true, true, true, true);

		UE::Geometry::FMeshBevel Bevel;
		Bevel.InsetDistance = BevelDistance;
		Bevel.MaterialIDMode = FMeshBevel::EMaterialIDMode::InferMaterialID;
		Bevel.MaterialIDMode = (bInferMaterialID) ? FMeshBevel::EMaterialIDMode::InferMaterialID_ConstantIfAmbiguous : FMeshBevel::EMaterialIDMode::ConstantMaterialID;
		Bevel.SetConstantMaterialID = SetMaterialID;
		Bevel.NumSubdivisions = Subdivisions;
		Bevel.RoundWeight = RoundWeight;
		bool bBevelSetupValid = false;
		if (BevelGroupEdges.Num() > 0)
		{
			Bevel.InitializeFromGroupTopologyEdges(*ResultMesh, *MeshTopology, BevelGroupEdges);
			bBevelSetupValid = true;
		}
		else if (BevelGroupFaces.Num() > 0)
		{
			bBevelSetupValid = Bevel.InitializeFromGroupTopologyFaces(*ResultMesh, *MeshTopology, BevelGroupFaces);
		}
		else
		{
			Bevel.InitializeFromGroupTopology(*ResultMesh, *MeshTopology);
			bBevelSetupValid = true;
		}
		if (bBevelSetupValid)
		{
			Bevel.Apply(*ResultMesh, nullptr);
			SetResultInfo(FGeometryResult());
		}
		else
		{
			SetResultInfo(FGeometryResult::Failed());
		}
	}

};


TUniquePtr<FDynamicMeshOperator> UPolyEditBevelEdgeActivity::MakeNewOperator()
{
	// create operator here
	TUniquePtr<FBevelOp> Op = MakeUnique<FBevelOp>();
	Op->OriginalMesh = ActivityContext->CurrentMesh;
	Op->MeshTopology = ActivityContext->CurrentTopology;
	Op->BevelDistance = BevelProperties->BevelDistance;
	Op->Subdivisions = BevelProperties->Subdivisions;
	Op->RoundWeight = BevelProperties->RoundWeight;
	Op->bInferMaterialID = BevelProperties->bInferMaterialID;
	Op->SetMaterialID = BevelProperties->SetMaterialID;

	// copy edge selection
	if (ActiveSelection.SelectedEdgeIDs.Num() > 0)
	{
		for (int32 eid : ActiveSelection.SelectedEdgeIDs)
		{
			Op->BevelGroupEdges.Add(eid);
		}
	}
	else if (ActiveSelection.SelectedGroupIDs.Num() > 0)
	{
		for (int32 gid : ActiveSelection.SelectedGroupIDs)
		{
			Op->BevelGroupFaces.Add(gid);
		}
	}
	else
	{
		UE_LOGF(LogGeometry, Warning, "UPolyEditBevelEdgeActivity::MakeNewOperator : empty selection");
	}

	FTransform3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());
	Op->SetResultTransform(WorldTransform);
	
	return Op;
}

void UPolyEditBevelEdgeActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();

	BevelProperties = NewObject<UPolyEditBevelEdgeProperties>();
	BevelProperties->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(BevelProperties);
	SetToolPropertySourceEnabled(BevelProperties, false);
	BevelProperties->WatchProperty(BevelProperties->BevelDistance, [this](float) {  
		if (bIsRunning) {
			ActivityContext->Preview->InvalidateResult();
		}
	});
	BevelProperties->WatchProperty(BevelProperties->Subdivisions, [this](int) {
		if (bIsRunning) {
			ActivityContext->Preview->InvalidateResult();
		}
	});
	BevelProperties->WatchProperty(BevelProperties->RoundWeight, [this](float) {
		if (bIsRunning) {
			ActivityContext->Preview->InvalidateResult();
		}
	});
	BevelProperties->WatchProperty(BevelProperties->bInferMaterialID, [this](bool) {
		if (bIsRunning) {
			ActivityContext->Preview->InvalidateResult();
		}
	});
	BevelProperties->WatchProperty(BevelProperties->SetMaterialID, [this](int) {
		if (bIsRunning) {
			ActivityContext->Preview->InvalidateResult();
		}
	});
}

void UPolyEditBevelEdgeActivity::Shutdown(EToolShutdownType ShutdownType)
{
	if (bIsRunning)
	{
		End(ShutdownType);
	}

	BevelProperties->SaveProperties(ParentTool.Get());

	BevelProperties = nullptr;
	ParentTool = nullptr;
	ActivityContext = nullptr;
}

bool UPolyEditBevelEdgeActivity::CanStart() const
{
	if (ActivityContext)
	{
		const FGroupTopologySelection& Selection = ActivityContext->SelectionMechanic->GetActiveSelection();
		// Search for any edges that we can bevel
		for (int32 GroupEdge : Selection.SelectedEdgeIDs)
		{
			// Can't bevel boundary edges
			if (ActivityContext->CurrentTopology->Edges[GroupEdge].Groups.B != IndexConstants::InvalidID)
			{
				return true;
			}
		}
		for (int32 GroupID : Selection.SelectedGroupIDs)
		{
			const FGroupTopology::FGroup* Group = ActivityContext->CurrentTopology->FindGroupByID(GroupID);
			if (Group)
			{
				for (const FGroupTopology::FGroupBoundary& Boundary : Group->Boundaries)
				{
					for (int32 GroupEdgeID : Boundary.GroupEdges)
					{
						const FGroupTopology::FGroupEdge& GroupEdge = ActivityContext->CurrentTopology->Edges[GroupEdgeID];
						int32 OtherGroupID = GroupEdge.OtherGroupID(GroupID);
						// Only bevel non-boundary edges where one side is not in the selection
						if (OtherGroupID != IndexConstants::InvalidID && !Selection.SelectedGroupIDs.Contains(OtherGroupID))
						{
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

EToolActivityStartResult UPolyEditBevelEdgeActivity::Start()
{
	if (!CanStart())
	{
		ParentTool->GetToolManager()->DisplayMessage( LOCTEXT("OnBevelFailedMesssage", "Cannot Bevel with current selection: Make sure your selection includes (or borders) non-boundary edges."), EToolMessageLevel::UserWarning);
		return EToolActivityStartResult::FailedStart;
	}

	// Change the op we use in the preview to an Bevel op.
	ActivityContext->Preview->ChangeOpFactory(this);
	ActivityContext->Preview->OnOpCompleted.AddWeakLambda(this,
		[this](const UE::Geometry::FDynamicMeshOperator* UncastOp) {
			const FBevelOp* Op = static_cast<const FBevelOp*>(UncastOp);
			FGeometryResult Result = Op->GetResultInfo();

			// If bevel failed, print a message and kill the activity
			// Warning: if FBevelOp is modified to allow failure based on the bevel parameters, this could incorrectly terminate the operation, and will require more complex handling
			if (Result.HasFailed())
			{
				ParentTool->GetToolManager()->DisplayMessage( LOCTEXT("OnBevelFailedMessage", "Bevel failed for current selection"), EToolMessageLevel::UserWarning);
				EndInternal();
				Cast<IToolActivityHost>(ParentTool)->NotifyActivitySelfEnded(this);
			}
		});

	SetToolPropertySourceEnabled(BevelProperties, true); 

	BeginBevel();

	bIsRunning = true;

	ActivityContext->EmitActivityStart(LOCTEXT("BeginBevelActivity", "Begin Bevel"));
	return EToolActivityStartResult::Running;
}

void UPolyEditBevelEdgeActivity::BeginBevel()
{
	ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();

	// Temporarily clear the selection to avoid the highlighting on the preview
	ActivityContext->SelectionMechanic->SetSelection(FGroupTopologySelection(), false);

	// force initial compute?
	ActivityContext->Preview->InvalidateResult();
}

bool UPolyEditBevelEdgeActivity::CanAccept() const
{
	return true;
}

EToolActivityEndResult UPolyEditBevelEdgeActivity::End(EToolShutdownType ShutdownType)
{
	if (!ensure(bIsRunning))
	{
		EndInternal();
		return EToolActivityEndResult::ErrorDuringEnd;
	}

	if (ShutdownType == EToolShutdownType::Cancel)
	{
		EndInternal();

		// Reset the preview
		ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());

		return EToolActivityEndResult::Cancelled;
	}
	else
	{
		// Stop the current compute if there is one.
		ActivityContext->Preview->CancelCompute();

		// Apply whatever we happen to have at the moment.
		ApplyBevel();

		EndInternal();
		return EToolActivityEndResult::Completed;
	}
}

// Does whatever kind of cleanup we need to do to end the activity.
void UPolyEditBevelEdgeActivity::EndInternal()
{
	if (ActivityContext->SelectionMechanic->GetActiveSelection() != ActiveSelection)
	{
		// We haven't reset the selection back from when we cleared it to avoid the highlighting.
		ActivityContext->SelectionMechanic->SetSelection(ActiveSelection, false);
	}

	ActivityContext->Preview->ClearOpFactory();
	ActivityContext->Preview->OnOpCompleted.RemoveAll(this);
	SetToolPropertySourceEnabled(BevelProperties, false);
	bIsRunning = false;
}

void UPolyEditBevelEdgeActivity::ApplyBevel()
{
	const FDynamicMesh3* ResultMesh = ActivityContext->Preview->PreviewMesh->GetMesh();

	if (ResultMesh->TriangleCount() == 0)
	{
		ParentTool->GetToolManager()->DisplayMessage( LOCTEXT("OnBevelEmptyMeshMessage", "Bevel created empty mesh, ignoring"), EToolMessageLevel::UserWarning);
		// Reset the preview
		ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());
		return;
	}

	// Prep for undo.
	FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
	ChangeTracker.BeginChange();

	// todo figure out what we need to save, instead of everything

	TArray<int32> AllTids;
	for (int32 Tid : ActivityContext->CurrentMesh->TriangleIndicesItr())
	{
		AllTids.Add(Tid);
	}
	ChangeTracker.SaveTriangles(AllTids, true /*bSaveVertices*/);

	// Update current mesh
	ActivityContext->CurrentMesh->Copy(*ResultMesh, true, true, true, true);

	// discard selection for now
	FGroupTopologySelection NewSelection;
	//NewSelection.SelectedEdgeIDs.Append( ... );

	// We need to reset the old selection back before we give the new one, so
	// that undo reverts back to the correct selection state.
	if (ActivityContext->SelectionMechanic->GetActiveSelection() != ActiveSelection)
	{
		ActivityContext->SelectionMechanic->SetSelection(ActiveSelection, false);
	}

	// Emit undo  (also updates relevant structures)
	ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshBevelChange", "Bevel"),
		ChangeTracker.EndChange(), NewSelection);

	ActiveSelection = NewSelection;
}


void UPolyEditBevelEdgeActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
}

void UPolyEditBevelEdgeActivity::Tick(float DeltaTime)
{
}


#undef LOCTEXT_NAMESPACE

