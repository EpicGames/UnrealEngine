// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "GeometryBase.h"
#include "GroupTopology.h" // FGroupTopologySelection
#include "InteractiveToolActivity.h"
#include "FrameTypes.h"
#include "ModelingOperators.h"

#include "PolyEditBevelEdgeActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UPolyEditActivityContext;
class UPolyEditPreviewMesh;
PREDECLARE_GEOMETRY(class FDynamicMesh3);



UCLASS(MinimalAPI)
class UPolyEditBevelEdgeProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Distance that each beveled mesh edge is inset from its initial position */
	UPROPERTY(EditAnywhere, Category = Bevel, meta = (UIMin = ".001", UIMax = "100", ClampMin = "0.0001", ClampMax = "10000", SliderExponent = "3", ModelingQuickEdit))
	double BevelDistance = 4.0;

	/** Number of edge loops added along the bevel faces */
	UPROPERTY(EditAnywhere, Category = Bevel, meta=(UIMax="10", ClampMin="0", ClampMax="1000"))
	int Subdivisions = 0;

	/** Roundness of the bevel. Ignored if Subdivisions = 0. */
	UPROPERTY(EditAnywhere, Category = Bevel, meta = (UIMin = "-2", UIMax="2"))
	float RoundWeight = 1.0;

	/** If true, when faces on either side of a beveled mesh edges have the same Material ID, beveled edge will be set to that Material ID. Otherwise SetMaterialID is used. */
	UPROPERTY(EditAnywhere, Category = Bevel)
	bool bInferMaterialID = true;

	/** Material ID to set on the new faces introduced by bevel operation, unless bInferMaterialID=true and non-ambiguous MaterialID can be inferred from adjacent faces */
	UPROPERTY(EditAnywhere, Category = Bevel, meta = (ClampMin="0", NoSpinbox))
	int SetMaterialID = 0;
};


/**
 * 
 */
UCLASS(MinimalAPI)
class UPolyEditBevelEdgeActivity : public UInteractiveToolActivity,
	public UE::Geometry::IDynamicMeshOperatorFactory

{
	GENERATED_BODY()

public:

	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UPolyEditBevelEdgeProperties> BevelProperties = nullptr;

protected:

	UE_API virtual void BeginBevel();
	UE_API virtual void ApplyBevel();
	UE_API virtual void EndInternal();

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	bool bIsRunning = false;

	UE::Geometry::FGroupTopologySelection ActiveSelection;
};

#undef UE_API
