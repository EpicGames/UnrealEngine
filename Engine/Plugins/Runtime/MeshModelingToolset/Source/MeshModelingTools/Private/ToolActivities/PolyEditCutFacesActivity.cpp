// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditCutFacesActivity.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "Drawing/PolyEditPreviewMesh.h"
#include "InteractiveToolManager.h"
#include "Mechanics/CollectSurfacePathMechanic.h"
#include "DynamicMesh/MeshIndexUtil.h" // TriangleToVertexIDs
#include "MeshOpPreviewHelpers.h"
#include "Operations/MeshPlaneCut.h"
#include "Selections/MeshEdgeSelection.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolActivities/PolyEditActivityUtil.h"
#include "ToolSceneQueriesUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditCutFacesActivity)

#define LOCTEXT_NAMESPACE "UPolyEditInsetOutsetActivity"

using namespace UE::Geometry;

void UPolyEditCutFacesActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	CutProperties = NewObject<UPolyEditCutProperties>();
	CutProperties->RestoreProperties(ParentToolIn);
	AddToolPropertySource(CutProperties);
	SetToolPropertySourceEnabled(CutProperties, false);

	// Register ourselves to receive clicks and hover
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();
}

void UPolyEditCutFacesActivity::Shutdown(EToolShutdownType ShutdownType)
{
	Clear();
	CutProperties->SaveProperties(ParentTool.Get());

	CutProperties = nullptr;
	ParentTool = nullptr;
	ActivityContext = nullptr;
}

bool UPolyEditCutFacesActivity::CanStart() const
{
	if (!ActivityContext)
	{
		return false;
	}
	const FGroupTopologySelection& Selection = ActivityContext->SelectionMechanic->GetActiveSelection();
	return !Selection.SelectedGroupIDs.IsEmpty();
}

EToolActivityStartResult UPolyEditCutFacesActivity::Start()
{
	if (!CanStart())
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("OnCutFacesFailedMessage", "Cannot cut without face selection."),
			EToolMessageLevel::UserWarning);
		return EToolActivityStartResult::FailedStart;
	}

	Clear();
	BeginCutFaces();
	bIsRunning = true;

	ActivityContext->EmitActivityStart(LOCTEXT("BeginCutFacesActivity", "Begin Cut Faces"));

	return EToolActivityStartResult::Running;
}

bool UPolyEditCutFacesActivity::CanAccept() const
{
	return false;
}

EToolActivityEndResult UPolyEditCutFacesActivity::End(EToolShutdownType)
{
	Clear();
	EToolActivityEndResult ToReturn = bIsRunning ? EToolActivityEndResult::Cancelled 
		: EToolActivityEndResult::ErrorDuringEnd;
	bIsRunning = false;
	return ToReturn;
}

void UPolyEditCutFacesActivity::BeginCutFaces()
{
	const FGroupTopologySelection& ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	TArray<int32> ActiveTriangleSelection;
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	FTransform3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());

	EditPreview = PolyEditActivityUtil::CreatePolyEditPreviewMesh(*ParentTool, *ActivityContext);
	FTransform3d WorldTranslation, WorldRotateScale;
	EditPreview->ApplyTranslationToPreview(WorldTransform, WorldTranslation, WorldRotateScale);
	EditPreview->InitializeStaticType(ActivityContext->CurrentMesh.Get(), ActiveTriangleSelection, &WorldRotateScale);

	FDynamicMesh3 StaticHitTargetMesh;
	EditPreview->MakeInsetTypeTargetMesh(StaticHitTargetMesh);

	SurfacePathMechanic = NewObject<UCollectSurfacePathMechanic>(this);
	SurfacePathMechanic->Setup(ParentTool.Get());
	SurfacePathMechanic->InitializeMeshSurface(MoveTemp(StaticHitTargetMesh));
	SurfacePathMechanic->SetFixedNumPointsMode(2);
	SurfacePathMechanic->bSnapToTargetMeshVertices = true;
	double SnapTol = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	SurfacePathMechanic->SpatialSnapPointsFunc = [this, SnapTol](FVector3d Position1, FVector3d Position2)
	{
		return CutProperties->bSnapToVertices && ToolSceneQueriesUtil::PointSnapQuery(this->CameraState, Position1, Position2, SnapTol);
	};

	SetToolPropertySourceEnabled(CutProperties, true);
}

void UPolyEditCutFacesActivity::ApplyCutFaces()
{
	check(SurfacePathMechanic != nullptr && EditPreview != nullptr);

	const FGroupTopologySelection& ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	TArray<int32> ActiveTriangleSelection;
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	// construct cut plane normal from line points
	FFrame3d Point0(SurfacePathMechanic->HitPath[0]), Point1(SurfacePathMechanic->HitPath[1]);
	FVector3d PlaneNormal;
	if (CutProperties->Orientation == EPolyEditCutPlaneOrientation::ViewDirection)
	{
		FVector3d Direction0 = UE::Geometry::Normalized(Point0.Origin - (FVector3d)CameraState.Position);
		FVector3d Direction1 = UE::Geometry::Normalized(Point1.Origin - (FVector3d)CameraState.Position);
		PlaneNormal = Direction1.Cross(Direction0);
	}
	else
	{
		FVector3d LineDirection = UE::Geometry::Normalized(Point1.Origin - Point0.Origin);
		FVector3d UpVector = UE::Geometry::Normalized(Point0.Z() + Point1.Z());
		PlaneNormal = LineDirection.Cross(UpVector);
	}
	FVector3d PlaneOrigin = 0.5 * (Point0.Origin + Point1.Origin);
	// map into local space of target mesh
	FTransformSRT3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());
	PlaneOrigin = WorldTransform.InverseTransformPosition(PlaneOrigin);
	PlaneNormal = WorldTransform.InverseTransformNormal(PlaneNormal);
	UE::Geometry::Normalize(PlaneNormal);

	// track changes
	FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
	ChangeTracker.BeginChange();
	TArray<int32> VertexSelection;
	UE::Geometry::TriangleToVertexIDs(ActivityContext->CurrentMesh.Get(), ActiveTriangleSelection, VertexSelection);
	ChangeTracker.SaveVertexOneRingTriangles(VertexSelection, true);

	// apply the cut to edges of selected triangles
	FGroupTopologySelection OutputSelection;
	FMeshPlaneCut Cut(ActivityContext->CurrentMesh.Get(), PlaneOrigin, PlaneNormal);
	Cut.bSimplifyAlongNewEdges = !ActivityContext->bTriangleMode;
	Cut.SimplifySettings.bPreserveVertexNormals = false;
	FMeshEdgeSelection Edges(ActivityContext->CurrentMesh.Get());
	Edges.SelectTriangleEdges(ActiveTriangleSelection);
	Cut.EdgeFilterFunc = [&](int EdgeID) { return Edges.IsSelected(EdgeID); };
	TSet<int32> TriangleSelectionSet;
	if (ActivityContext->bTriangleMode)
	{
		TriangleSelectionSet.Append(ActiveTriangleSelection);
	}
	if (Cut.SplitEdgesOnly(true, ActivityContext->bTriangleMode ? &TriangleSelectionSet : nullptr))
	{
		if (!ActivityContext->bTriangleMode)
		{
			for (FMeshPlaneCut::FCutResultRegion& Region : Cut.ResultRegions)
			{
				OutputSelection.SelectedGroupIDs.Add(Region.GroupID);
			}
		}
		else
		{
			OutputSelection.SelectedGroupIDs.Append(TriangleSelectionSet);
		}
	}

	// Emit undo (also updates relevant structures)
	ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshCutFacesChange", "Cut Faces"),
		ChangeTracker.EndChange(), OutputSelection);

	// End activity
	Clear();
	bIsRunning = false;
	Cast<IToolActivityHost>(ParentTool)->NotifyActivitySelfEnded(this);
}

void UPolyEditCutFacesActivity::Clear()
{
	if (EditPreview != nullptr)
	{
		EditPreview->Disconnect();
		EditPreview = nullptr;
	}

	SurfacePathMechanic = nullptr;
	SetToolPropertySourceEnabled(CutProperties, false);
}

void UPolyEditCutFacesActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	ParentTool->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (SurfacePathMechanic != nullptr)
	{
		SurfacePathMechanic->Render(RenderAPI);
	}
}

void UPolyEditCutFacesActivity::Tick(float DeltaTime)
{
}

FInputRayHit UPolyEditCutFacesActivity::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning;
	return OutHit;
}

void UPolyEditCutFacesActivity::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (bIsRunning && SurfacePathMechanic->TryAddPointFromRay((FRay3d)ClickPos.WorldRay))
	{
		if (SurfacePathMechanic->IsDone())
		{
			++ActivityStamp;
			ApplyCutFaces();
		}
		else
		{
			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FPolyEditCutFacesActivityFirstPointChange>(ActivityStamp),
				LOCTEXT("CutLineStart", "Cut Line Start"));
		}
	}
}

FInputRayHit UPolyEditCutFacesActivity::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning;
	return OutHit;
}

bool UPolyEditCutFacesActivity::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	SurfacePathMechanic->UpdatePreviewPoint((FRay3d)DevicePos.WorldRay);
	return bIsRunning;
}

void FPolyEditCutFacesActivityFirstPointChange::Revert(UObject* Object)
{
	UPolyEditCutFacesActivity* Activity = Cast<UPolyEditCutFacesActivity>(Object);
	if (ensure(Activity->SurfacePathMechanic))
	{
		Activity->SurfacePathMechanic->PopLastPoint();
	}
	bHaveDoneUndo = true;
}

#undef LOCTEXT_NAMESPACE

