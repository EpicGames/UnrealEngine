// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditExtrudeActivity.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "Drawing/PolyEditPreviewMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "EditMeshPolygonsTool.h"
#include "InteractiveToolManager.h"
#include "Mechanics/PlaneDistanceFromHitMechanic.h"
#include "MeshOpPreviewHelpers.h"
#include "Operations/OffsetMeshRegion.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolActivities/PolyEditActivityUtil.h"
#include "ToolSceneQueriesUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditExtrudeActivity)

#define LOCTEXT_NAMESPACE "UPolyEditExtrudeActivity"

using namespace UE::Geometry;

namespace PolyEditExtrudeActivityLocals
{
	/** Creates a mesh shared ptr out of the given triangles */
	TSharedPtr<FDynamicMesh3> CreatePatchMesh(const FDynamicMesh3* SourceMesh, TArray<int32> Triangles)
	{
		FDynamicSubmesh3 Submesh(SourceMesh, Triangles, 
			(int32)(EMeshComponents::FaceGroups) | (int32)(EMeshComponents::VertexNormals), true);
		return MakeShared<FDynamicMesh3>(MoveTemp(Submesh.GetSubmesh()));
	}

	template <typename EnumType>
	FExtrudeOp::EDirectionMode ToOpDirectionMode(EnumType Value)
	{
		return static_cast<FExtrudeOp::EDirectionMode>(static_cast<int>(Value));
	}

	EPolyEditExtrudeDistanceMode GetDistanceMode(UPolyEditExtrudeActivity& Activity, UPolyEditExtrudeActivity::EPropertySetToUse PropertySetToUse)
	{
		switch (PropertySetToUse)
		{
		case UPolyEditExtrudeActivity::EPropertySetToUse::Extrude:
			return Activity.ExtrudeProperties->DistanceMode;
		case UPolyEditExtrudeActivity::EPropertySetToUse::Offset:
			return Activity.OffsetProperties->DistanceMode;
		case UPolyEditExtrudeActivity::EPropertySetToUse::PushPull:
			return Activity.PushPullProperties->DistanceMode;
		}

		ensure(false);
		return Activity.ExtrudeProperties->DistanceMode;
	}

	double GetFixedDistance(UPolyEditExtrudeActivity& Activity, UPolyEditExtrudeActivity::EPropertySetToUse PropertySetToUse)
	{
		switch (PropertySetToUse)
		{
		case UPolyEditExtrudeActivity::EPropertySetToUse::Extrude:
			return Activity.ExtrudeProperties->Distance;
		case UPolyEditExtrudeActivity::EPropertySetToUse::Offset:
			return Activity.OffsetProperties->Distance;
		case UPolyEditExtrudeActivity::EPropertySetToUse::PushPull:
			return Activity.PushPullProperties->Distance;
		}

		ensure(false);
		return Activity.ExtrudeProperties->Distance;
	}
}

TUniquePtr<FDynamicMeshOperator> UPolyEditExtrudeActivity::MakeNewOperator()
{
	using namespace PolyEditExtrudeActivityLocals;

	TUniquePtr<FExtrudeOp> Op = MakeUnique<FExtrudeOp>();
	Op->OriginalMesh = ActivityContext->CurrentMesh;
	Op->ExtrudeMode = ExtrudeMode;

	switch (PropertySetToUse)
	{
	case EPropertySetToUse::Extrude:
		Op->DirectionMode = ToOpDirectionMode(ExtrudeProperties->DirectionMode);
		Op->bShellsToSolids = ExtrudeProperties->bShellsToSolids;
		Op->bUseColinearityForSettingBorderGroups = ExtrudeProperties->bUseColinearityForSettingBorderGroups;
		Op->MaxScaleForAdjustingTriNormalsOffset = ExtrudeProperties->MaxDistanceScaleFactor;
		break;
	case EPropertySetToUse::Offset:
		Op->DirectionMode = ToOpDirectionMode(OffsetProperties->DirectionMode);
		Op->bShellsToSolids = OffsetProperties->bShellsToSolids;
		Op->bUseColinearityForSettingBorderGroups = OffsetProperties->bUseColinearityForSettingBorderGroups;
		Op->MaxScaleForAdjustingTriNormalsOffset = OffsetProperties->MaxDistanceScaleFactor;
		break;
	case EPropertySetToUse::PushPull:
		Op->DirectionMode = ToOpDirectionMode(PushPullProperties->DirectionMode);
		Op->bShellsToSolids = PushPullProperties->bShellsToSolids;
		Op->bUseColinearityForSettingBorderGroups = PushPullProperties->bUseColinearityForSettingBorderGroups;
		Op->MaxScaleForAdjustingTriNormalsOffset = PushPullProperties->MaxDistanceScaleFactor;
		break;
	}

	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, Op->TriangleSelection);
	
	Op->ExtrudeDistance = GetDistanceMode(*this, PropertySetToUse) == EPolyEditExtrudeDistanceMode::Fixed ?
		GetFixedDistance(*this, PropertySetToUse) : ExtrudeHeightMechanic->CurrentHeight;
	// TODO: The extruder has been changed such that the boolean and move-and-stitch paths treat UV scaling differently,
	// with the boolean path using a newer path that does its own normalization of UVs, while the move-and-stitch
	// still uses the legacy version. They both probably should just use the newer version, but for now we'll pass
	// the path-appropriate value here...
	Op->UVScaleFactor = Op->ExtrudeMode == FExtrudeOp::EExtrudeMode::MoveAndStitch ? UVScaleFactor : 1.0;

	FTransform3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());
	Op->SetResultTransform(WorldTransform);
	
	Op->MeshSpaceExtrudeDirection = WorldTransform.InverseTransformVector(GetExtrudeDirection());

	return Op;
}

void UPolyEditExtrudeActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	ExtrudeProperties = NewObject<UPolyEditExtrudeProperties>();
	ExtrudeProperties->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(ExtrudeProperties);
	SetToolPropertySourceEnabled(ExtrudeProperties, false);
	ExtrudeProperties->WatchProperty(ExtrudeProperties->Direction,
	[this](EPolyEditExtrudeDirection) { 
		if (bIsRunning && ExtrudeProperties->DirectionMode == EPolyEditExtrudeModeOptions::SingleDirection)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->MeasureDirection,
	[this](EPolyEditExtrudeDirection) {
		if (bIsRunning && ExtrudeProperties->DirectionMode != EPolyEditExtrudeModeOptions::SingleDirection)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->DirectionMode,
	[this](EPolyEditExtrudeModeOptions) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->DistanceMode,
	[this](EPolyEditExtrudeDistanceMode) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->Distance,
	[this](double) {
		if (bIsRunning)
		{
			ActivityContext->Preview->InvalidateResult();
		}
	});
	ExtrudeProperties->WatchProperty(ExtrudeProperties->bShellsToSolids,
	[this](bool) {
		if (bIsRunning)
		{
			ActivityContext->Preview->InvalidateResult();
		}
	});


	OffsetProperties = NewObject<UPolyEditOffsetProperties>();
	OffsetProperties->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(OffsetProperties);
	SetToolPropertySourceEnabled(OffsetProperties, false);
	OffsetProperties->WatchProperty(OffsetProperties->MeasureDirection,
	[this](EPolyEditExtrudeDirection) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	OffsetProperties->WatchProperty(OffsetProperties->DirectionMode,
	[this](EPolyEditOffsetModeOptions) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	OffsetProperties->WatchProperty(OffsetProperties->DistanceMode,
	[this](EPolyEditExtrudeDistanceMode) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	OffsetProperties->WatchProperty(OffsetProperties->Distance,
	[this](double) {
		if (bIsRunning)
		{
			ActivityContext->Preview->InvalidateResult();
		}
	});

	PushPullProperties = NewObject<UPolyEditPushPullProperties>();
	PushPullProperties->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(PushPullProperties);
	SetToolPropertySourceEnabled(PushPullProperties, false);
	PushPullProperties->WatchProperty(PushPullProperties->MeasureDirection,
	[this](EPolyEditExtrudeDirection) {
		if (bIsRunning && PushPullProperties->DirectionMode != EPolyEditPushPullModeOptions::SingleDirection)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	PushPullProperties->WatchProperty(PushPullProperties->SingleDirection,
	[this](EPolyEditExtrudeDirection) {
		if (bIsRunning && PushPullProperties->DirectionMode == EPolyEditPushPullModeOptions::SingleDirection)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	PushPullProperties->WatchProperty(PushPullProperties->DirectionMode,
	[this](EPolyEditPushPullModeOptions) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	PushPullProperties->WatchProperty(PushPullProperties->DistanceMode,
	[this](EPolyEditExtrudeDistanceMode) {
		if (bIsRunning)
		{
			ReinitializeExtrudeHeightMechanic();
			ActivityContext->Preview->InvalidateResult();
		}
	});
	PushPullProperties->WatchProperty(PushPullProperties->Distance,
	[this](double) {
		if (bIsRunning)
		{
			ActivityContext->Preview->InvalidateResult();
		}
	});

	// Register ourselves to receive clicks and hover
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();
}

void UPolyEditExtrudeActivity::Shutdown(EToolShutdownType ShutdownType)
{
	if (bIsRunning)
	{
		End(ShutdownType);
	}

	ExtrudeProperties->SaveProperties(ParentTool.Get());
	OffsetProperties->SaveProperties(ParentTool.Get());
	PushPullProperties->SaveProperties(ParentTool.Get());

	ExtrudeProperties = nullptr;
	OffsetProperties = nullptr;
	PushPullProperties = nullptr;
	ParentTool = nullptr;
	ActivityContext = nullptr;
}

bool UPolyEditExtrudeActivity::CanStart() const
{
	if (!ActivityContext)
	{
		return false;
	}
	const FGroupTopologySelection& Selection = ActivityContext->SelectionMechanic->GetActiveSelection();
	return !Selection.SelectedGroupIDs.IsEmpty();
}

EToolActivityStartResult UPolyEditExtrudeActivity::Start()
{
	using namespace PolyEditExtrudeActivityLocals;

	if (!CanStart())
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("OnExtrudeFailedMesssage", "Action requires face selection."),
			EToolMessageLevel::UserWarning);
		return EToolActivityStartResult::FailedStart;
	}

	// Change the op we use in the preview to an extrude op.
	ActivityContext->Preview->ChangeOpFactory(this);
	ActivityContext->Preview->OnOpCompleted.AddWeakLambda(this,
		[this](const UE::Geometry::FDynamicMeshOperator* UncastOp) {
			const FExtrudeOp* Op = static_cast<const FExtrudeOp*>(UncastOp);
			NewSelectionGids = Op->ExtrudedFaceNewGids;
		});

	switch (PropertySetToUse)
	{
	case EPropertySetToUse::Extrude:
		SetToolPropertySourceEnabled(ExtrudeProperties, true); 
		break;
	case EPropertySetToUse::Offset:
		SetToolPropertySourceEnabled(OffsetProperties, true);
		break;
	case EPropertySetToUse::PushPull:
		SetToolPropertySourceEnabled(PushPullProperties, true);
		break;
	}

	// Set up the basics of the extrude height mechanic.
	// TODO: Allow option to use a gizmo instead
	ExtrudeHeightMechanic = NewObject<UPlaneDistanceFromHitMechanic>(this);
	ExtrudeHeightMechanic->Setup(ParentTool.Get());
	ExtrudeHeightMechanic->WorldHitQueryFunc = [this](const FRay& WorldRay, FHitResult& HitResult)
	{
		return ToolSceneQueriesUtil::FindNearestVisibleObjectHit(GetParentTool(), HitResult, WorldRay);
	};
	ExtrudeHeightMechanic->WorldPointSnapFunc = [this](const FVector3d& WorldPos, FVector3d& SnapPos)
	{
		return ToolSceneQueriesUtil::FindWorldGridSnapPoint(ParentTool.Get(), WorldPos, SnapPos);
	};
	ExtrudeHeightMechanic->CurrentHeight = GetFixedDistance(*this, PropertySetToUse);
	
	BeginExtrude();

	bRequestedApply = false;
	bIsRunning = true;

	ActivityContext->EmitActivityStart(LOCTEXT("BeginExtrudeActivity", "Begin Extrude"));
	return EToolActivityStartResult::Running;
}

// Someday we may want to initialize one extrude after another in the same invocation. Anything that needs
// to be reset in those cases goes here.
void UPolyEditExtrudeActivity::BeginExtrude()
{
	using namespace PolyEditExtrudeActivityLocals;

	ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();

	// Temporarily clear the selection to avoid the highlighting on the preview as we extrude (which 
	// sometimes highlights incorrect triangles in boolean mode). There's currently not a good way to
	// disable those secondary triangle buffers without losing the filter function.
	// The selection gets reset on ApplyExtrude() to a new selection, or to the old selection in EndInternal.
	ActivityContext->SelectionMechanic->SetSelection(FGroupTopologySelection(), false);

	TArray<int32> ActiveTriangleSelection;
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	PatchMesh = CreatePatchMesh(ActivityContext->CurrentMesh.Get(), ActiveTriangleSelection);
	UE::Geometry::FDynamicMeshAABBTree3 PatchSpatial(PatchMesh.Get(), true);

	// Get the world selection frame
	FFrame3d ActiveSelectionFrameLocal = ActivityContext->CurrentTopology->GetSelectionFrame(ActiveSelection);
	ActiveSelectionFrameWorld = ActiveSelectionFrameLocal;
	ActiveSelectionFrameWorld.Origin = PatchSpatial.FindNearestPoint(ActiveSelectionFrameLocal.Origin);
	FTransform3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());
	ActiveSelectionFrameWorld.Transform(WorldTransform);

	ReinitializeExtrudeHeightMechanic();
	ActivityContext->Preview->InvalidateResult();

	float BoundsMaxDim = ActivityContext->CurrentMesh->GetBounds().MaxDim();
	if (BoundsMaxDim > 0)
	{
		UVScaleFactor = 1.0 / BoundsMaxDim;
	}
}

// The height mechanics has to get reinitialized whenever extrude direction changes
void UPolyEditExtrudeActivity::ReinitializeExtrudeHeightMechanic()
{
	using namespace PolyEditExtrudeActivityLocals;

	if (GetDistanceMode(*this, PropertySetToUse) != EPolyEditExtrudeDistanceMode::ClickInViewport)
	{
		return;
	}

	FFrame3d ExtrusionFrameWorld = ActiveSelectionFrameWorld;
	ExtrusionFrameWorld.AlignAxis(2, GetExtrudeDirection());

	FDynamicMesh3 ExtrudeHitTargetMesh;	

	// Make infinite-extent hit-test mesh to use in the mechanic. This might seem like something
	// that would only be applicable in SingleDirection extrude mode, but it is in fact useful
	// for the mechanic in general, as it tends to make the mouse interactions feel better around
	// the extrude measurement line.
	ExtrudeHitTargetMesh = *PatchMesh;
	MeshTransforms::ApplyTransform(ExtrudeHitTargetMesh, 
		(FTransform3d)ActivityContext->Preview->PreviewMesh->GetTransform(), true);

	double Length = 99999.0;
	FVector3d ExtrudeDirection = GetExtrudeDirection();
	MeshTransforms::Translate(ExtrudeHitTargetMesh, -Length * ExtrudeDirection);

	FOffsetMeshRegion Extruder(&ExtrudeHitTargetMesh);
	Extruder.OffsetPositionFunc = [Length, ExtrudeDirection](const FVector3d& Position,
		const FVector3d& Normal, int VertexID)
	{
		return FVector3d(Position + 2.0 * Length * ExtrudeDirection);
	};
	Extruder.Triangles.Reserve(PatchMesh->TriangleCount());
	for (int32 Tid : PatchMesh->TriangleIndicesItr())
	{
		Extruder.Triangles.Add(Tid);
	}
	Extruder.Apply();

	ExtrudeHeightMechanic->Initialize(MoveTemp(ExtrudeHitTargetMesh), ExtrusionFrameWorld, true);
}

bool UPolyEditExtrudeActivity::CanAccept() const
{
	return true;
}

EToolActivityEndResult UPolyEditExtrudeActivity::End(EToolShutdownType ShutdownType)
{
	if (!ensure(bIsRunning))
	{
		EndInternal();
		return EToolActivityEndResult::ErrorDuringEnd;
	}

	if (ShutdownType == EToolShutdownType::Cancel)
	{
		EndInternal();

		// Reset the preview
		ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());

		return EToolActivityEndResult::Cancelled;
	}
	else
	{
		// Stop the current compute if there is one.
		ActivityContext->Preview->CancelCompute();

		// Apply whatever we happen to have at the moment.
		ApplyExtrude();

		EndInternal();
		return EToolActivityEndResult::Completed;
	}
}

// Does whatever kind of cleanup we need to do to end the activity.
void UPolyEditExtrudeActivity::EndInternal()
{
	if (ActivityContext->SelectionMechanic->GetActiveSelection() != ActiveSelection)
	{
		// We haven't reset the selection back from when we cleared it to avoid the highlighting.
		ActivityContext->SelectionMechanic->SetSelection(ActiveSelection, false);
	}

	ActivityContext->Preview->ClearOpFactory();
	ActivityContext->Preview->OnOpCompleted.RemoveAll(this);
	PatchMesh.Reset();
	ExtrudeHeightMechanic->Shutdown();
	ExtrudeHeightMechanic = nullptr;
	SetToolPropertySourceEnabled(ExtrudeProperties, false);
	SetToolPropertySourceEnabled(OffsetProperties, false);
	SetToolPropertySourceEnabled(PushPullProperties, false);
	bIsRunning = false;
}

void UPolyEditExtrudeActivity::ApplyExtrude()
{
	check(ExtrudeHeightMechanic != nullptr);

	const FDynamicMesh3* ResultMesh = ActivityContext->Preview->PreviewMesh->GetMesh();

	if (ResultMesh->TriangleCount() == 0)
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("OnDeleteAllFailedMessage", "Cannot Delete Entire Mesh"),
			EToolMessageLevel::UserWarning);

		// Reset the preview
		ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());
		return;
	}

	// Prep for undo.
	// TODO: It would be nice if we could attach the change tracker to the op and have it actually
	// track the changed triangles instead of us having to know which ones to save here. This is
	// particularly true for the boolean case, where we conservatively save all the triangles,
	// even though we only need the ones that got cut/deleted. Unfortunatley our boolean code
	// doesn't currently track changed triangles.
	FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
	ChangeTracker.BeginChange();
	if (ExtrudeMode == FExtrudeOp::EExtrudeMode::MoveAndStitch)
	{
		TArray<int32> ActiveTriangleSelection;
		ActivityContext->CurrentTopology->GetSelectedTriangles(
			ActiveSelection, ActiveTriangleSelection);

		ChangeTracker.SaveTriangles(ActiveTriangleSelection, true /*bSaveVertices*/);
	}
	else // if boolean
	{
		TArray<int32> AllTids;
		for (int32 Tid : ActivityContext->CurrentMesh->TriangleIndicesItr())
		{
			AllTids.Add(Tid);
		}
		ChangeTracker.SaveTriangles(AllTids, true /*bSaveVertices*/);
	}

	// Update current mesh
	ActivityContext->CurrentMesh->Copy(*ResultMesh,
		true, true, true, true);

	// Construct new selection
	FGroupTopologySelection NewSelection;
	if (ActivityContext->bTriangleMode)
	{
		TSet<int32> GidSet(NewSelectionGids);
		for (int32 Tid : ResultMesh->TriangleIndicesItr())
		{
			if (GidSet.Contains(ResultMesh->GetTriangleGroup(Tid)))
			{
				NewSelection.SelectedGroupIDs.Add(Tid);
			}
		}
	}
	else
	{
		NewSelection.SelectedGroupIDs.Append(NewSelectionGids);
	}

	// We need to reset the old selection back before we give the new one, so
	// that undo reverts back to the correct selection state.
	if (ActivityContext->SelectionMechanic->GetActiveSelection() != ActiveSelection)
	{
		ActivityContext->SelectionMechanic->SetSelection(ActiveSelection, false);
	}

	// Emit undo  (also updates relevant structures)
	ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshExtrudeChange", "Extrude"),
		ChangeTracker.EndChange(), NewSelection);

	ActiveSelection = NewSelection;
}

// Gets used to set the direction along which we measure the distance to extrude.
FVector3d UPolyEditExtrudeActivity::GetExtrudeDirection() const
{
	EPolyEditExtrudeDirection DirectionToUse = EPolyEditExtrudeDirection::SelectionNormal;
	switch (PropertySetToUse)
	{
	case EPropertySetToUse::Extrude:
		DirectionToUse = ExtrudeProperties->DirectionMode == EPolyEditExtrudeModeOptions::SingleDirection ?
			ExtrudeProperties->Direction : ExtrudeProperties->MeasureDirection;
		break;
	case EPropertySetToUse::Offset:
		DirectionToUse = OffsetProperties->MeasureDirection;
		break;
	case EPropertySetToUse::PushPull:
		DirectionToUse = PushPullProperties->DirectionMode == EPolyEditPushPullModeOptions::SingleDirection ?
			PushPullProperties->SingleDirection : PushPullProperties->MeasureDirection;
		break;
	}

	switch (DirectionToUse)
	{
	default:
	case EPolyEditExtrudeDirection::SelectionNormal:
		return ActiveSelectionFrameWorld.Z();
	case EPolyEditExtrudeDirection::WorldX:
		return FVector3d::UnitX();
	case EPolyEditExtrudeDirection::WorldY:
		return FVector3d::UnitY();
	case EPolyEditExtrudeDirection::WorldZ:
		return FVector3d::UnitZ();
	case EPolyEditExtrudeDirection::LocalX:
		return FTransformSRT3d(ActivityContext->Preview->PreviewMesh->GetTransform()).GetRotation().AxisX();
	case EPolyEditExtrudeDirection::LocalY:
		return FTransformSRT3d(ActivityContext->Preview->PreviewMesh->GetTransform()).GetRotation().AxisY();
	case EPolyEditExtrudeDirection::LocalZ:
		return FTransformSRT3d(ActivityContext->Preview->PreviewMesh->GetTransform()).GetRotation().AxisZ();
	}
}

void UPolyEditExtrudeActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace PolyEditExtrudeActivityLocals;

	if (ensure(bIsRunning) 
		&& ExtrudeHeightMechanic != nullptr
		&& GetDistanceMode(*this, PropertySetToUse) == EPolyEditExtrudeDistanceMode::ClickInViewport)
	{
		ExtrudeHeightMechanic->Render(RenderAPI);
	}
}

void UPolyEditExtrudeActivity::Tick(float DeltaTime)
{
	if (!ensure(bIsRunning))
	{
		return;
	}

	if (bRequestedApply && ActivityContext->Preview->HaveValidResult())
	{
		ApplyExtrude();
		bRequestedApply = false;

		// End activity
		EndInternal();
		Cast<IToolActivityHost>(ParentTool)->NotifyActivitySelfEnded(this);
		return;
	}
}

FInputRayHit UPolyEditExtrudeActivity::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	using namespace PolyEditExtrudeActivityLocals;

	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning 
		&& GetDistanceMode(*this, PropertySetToUse) == EPolyEditExtrudeDistanceMode::ClickInViewport;
	return OutHit;
}

void UPolyEditExtrudeActivity::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (bIsRunning && !bRequestedApply)
	{
		bRequestedApply = true;
	}
}

FInputRayHit UPolyEditExtrudeActivity::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	using namespace PolyEditExtrudeActivityLocals;

	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning && GetDistanceMode(*this, PropertySetToUse) == EPolyEditExtrudeDistanceMode::ClickInViewport;
	return OutHit;
}

bool UPolyEditExtrudeActivity::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (bIsRunning && !bRequestedApply)
	{
		ExtrudeHeightMechanic->UpdateCurrentDistance(DevicePos.WorldRay);
		ActivityContext->Preview->InvalidateResult();
	}
	return bIsRunning;
}

#undef LOCTEXT_NAMESPACE

