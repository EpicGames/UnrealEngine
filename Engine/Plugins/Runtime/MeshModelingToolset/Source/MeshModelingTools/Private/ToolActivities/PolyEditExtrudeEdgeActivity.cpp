// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditExtrudeEdgeActivity.h"

#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Containers/Map.h"
#include "ContextObjectStore.h"
#include "Drawing/PreviewGeometryActor.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMeshEditor.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h"
#include "ModelingOperators.h" // FDynamicMeshOperator
#include "PrimitiveDrawingUtils.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolContextInterfaces.h" // IToolsContextRenderAPI

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditExtrudeEdgeActivity)

#define LOCTEXT_NAMESPACE "PolyEditExtrudeEdgeActivity"

namespace PolyEditExtrudeEdgeActivityLocals
{
	// When adjusting extrusion distance to keep edges parallel, maximal allowed increase factor
	const double MAX_VERT_MOVEMENT_ADJUSTMENT_SCALE = 4.0;

	// These should match what is used in UMeshTopologySelectionMechanic (or better, live in a common place)
	const FColor GroupLineColor = FColor::Red;
	const float GroupLineThickness = 1.0f;
	const float GroupLineDepthBias = 0.5f;

	const FColor ExtrudeFrameLineColor = FColor::Orange;
	const float ExtrudeFrameLineThickness = 0.5f;
	const double ExtrudeFrameLineLength = 1000;

	FText TransactionLabel = LOCTEXT("ExtrudeEdgeTransactionLabel", "Extrude Edges");

	class FExtrudeEdgeOp : public UE::Geometry::FDynamicMeshOperator
	{
	public:
		virtual ~FExtrudeEdgeOp() {}

		// Inputs:
		TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
		FTransform OriginalMeshTransform;
		TArray<int32> SelectedEids;
		TArray<int32> GroupsToSetPerEid;

		EPolyEditExtrudeEdgeDirectionMode DirectionMode;
		FVector3d SingleDirectionVector;
		FVector3d LocalFrameParams;
		double ScalingAdjustmentLimit = 1;
		bool bAssignAnyBoundaryNeighborToUnmatched = false;

		bool bGatherGroupBoundariesForRender = false;
		bool bGatherAllEdgesForRender = false;

		// Outputs
		TArray<int32> NewSelectionEids;
		TArray<int32> EdgesToRender;
	protected:

		TMap<int32, FVector3d> DesiredExtrudeDirections;

	public:
		// FDynamicMeshOperator
		virtual void CalculateResult(FProgressCancel* Progress) override
		{
			using namespace UE::Geometry;

			if (Progress && Progress->Cancelled()) { return; }

			ResultMesh->Copy(*OriginalMesh);
			ResultTransform = OriginalMeshTransform;

			if (Progress && Progress->Cancelled()) { return; }

			if ((DirectionMode == EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames && LocalFrameParams.IsNearlyZero())
				|| (DirectionMode == EPolyEditExtrudeEdgeDirectionMode::SingleDirection && SingleDirectionVector.IsNearlyZero()))
			{
				return;
			}

			FExtrudeBoundaryEdges Extruder(ResultMesh.Get());

			switch (DirectionMode)
			{
			case EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames:
				Extruder.OffsetPositionFunc = [this](const FVector3d& Position, const FExtrudeBoundaryEdges::FExtrudeFrame& ExtrudeFrame, int32 SourceVid)
				{
					return ExtrudeFrame.FromFramePoint(LocalFrameParams);
				};
				break;
			case EPolyEditExtrudeEdgeDirectionMode::SingleDirection:
				Extruder.OffsetPositionFunc = [this](const FVector3d& Position, const FExtrudeBoundaryEdges::FExtrudeFrame& ExtrudeFrame, int32 SourceVid)
				{
					return Position + SingleDirectionVector;
				};
				break;
			}

			Extruder.InputEids = SelectedEids;
			Extruder.GroupsToSetPerEid = GroupsToSetPerEid;
			Extruder.ScalingAdjustmentLimit = ScalingAdjustmentLimit;
			Extruder.bUsePerVertexExtrudeFrames = DirectionMode == EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames;
			Extruder.bAssignAnyBoundaryNeighborToUnmatched = bAssignAnyBoundaryNeighborToUnmatched;

			if (Progress && Progress->Cancelled()) { return; }

			Extruder.Apply(Progress);

			if (Progress && Progress->Cancelled()) { return; }

			NewSelectionEids = Extruder.NewExtrudedEids;

			TSet<int32> EdgesToRenderSet;
			if (bGatherAllEdgesForRender)
			{
				for (int32 Tid : Extruder.NewTids)
				{
					FIndex3i TriEids = ResultMesh->GetTriEdges(Tid);
					for (int i = 0; i < 3; ++i)
					{
						EdgesToRenderSet.Add(TriEids[i]);
					}
				}
			}
			else if (bGatherGroupBoundariesForRender)
			{
				for (int32 Tid : Extruder.NewTids)
				{
					FIndex3i TriEids = ResultMesh->GetTriEdges(Tid);
					for (int i = 0; i < 3; ++i)
					{
						if (ResultMesh->IsBoundaryEdge(TriEids[i]) || ResultMesh->IsGroupBoundaryEdge(TriEids[i]))
						{
							EdgesToRenderSet.Add(TriEids[i]);
						}
					}
				}
			}
			EdgesToRender = EdgesToRenderSet.Array();
		}
	};
}

TUniquePtr<UE::Geometry::FDynamicMeshOperator> UPolyEditExtrudeEdgeActivity::MakeNewOperator()
{
	using namespace PolyEditExtrudeEdgeActivityLocals;

	TUniquePtr<FExtrudeEdgeOp> Op = MakeUnique<FExtrudeEdgeOp>();
	Op->OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(*ActivityContext->CurrentMesh);
	Op->OriginalMeshTransform = ActivityContext->Preview->PreviewMesh->GetTransform();
	Op->SelectedEids = SelectedEids;
	Op->GroupsToSetPerEid = GroupsToSetPerEid;
	Op->ScalingAdjustmentLimit = Settings->bAdjustToExtrudeEvenly ? MAX_VERT_MOVEMENT_ADJUSTMENT_SCALE : 1.0;
	Op->DirectionMode = Settings->DirectionMode;
	Op->bGatherAllEdgesForRender = ActivityContext->bTriangleMode;
	Op->bGatherGroupBoundariesForRender = true;
	Op->bAssignAnyBoundaryNeighborToUnmatched = Settings->bUseUnselectedForFrames;

	if (Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::SingleDirection)
	{
		FVector3d WorldDestination = SingleDirectionVectorWorldSpace + ExtrudeFrameForGizmoWorldSpace.Origin;

		Op->SingleDirectionVector = CurrentMeshTransform.InverseTransformPosition(WorldDestination)
			- ExtrudeFrameForGizmoMeshSpace.Frame.Origin;
	}
	else
	{
		FVector3d WorldDestination = ExtrudeFrameForGizmoWorldSpace.FromFramePoint(ParamsInWorldExtrudeFrame);
		
		Op->LocalFrameParams = ExtrudeFrameForGizmoMeshSpace.ToFramePoint(
			CurrentMeshTransform.InverseTransformPosition(WorldDestination));
	}

	return Op;
}


void UPolyEditExtrudeEdgeActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();

	Settings = NewObject<UPolyEditExtrudeEdgeActivityProperties>();
	Settings->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(Settings);
	SetToolPropertySourceEnabled(Settings, false);
	
	ExtrudeFrameProxy = NewObject<UTransformProxy>(this);
	ExtrudeFrameProxy->OnTransformChanged.AddWeakLambda(this, [this](UTransformProxy*, FTransform Transform)
	{
		// TODO: It's possible to get these callbacks from undo while the activity is not running because the
		// transform changes are only expired by exiting the tool, not the activity. This is easy enough to
		// guard against, but may need a broader solution someday.
		if (bIsRunning && ActivityContext && ActivityContext->Preview)
		{
			ParamsInWorldExtrudeFrame = ExtrudeFrameForGizmoWorldSpace.ToFramePoint(Transform.GetLocation());
			ActivityContext->Preview->InvalidateResult();
		}
	});

	SingleDirectionProxy = NewObject<UTransformProxy>(this);
	SingleDirectionProxy->OnTransformChanged.AddWeakLambda(this, [this](UTransformProxy*, FTransform Transform)
	{
		if (bIsRunning && ActivityContext && ActivityContext->Preview)
		{
			SingleDirectionVectorWorldSpace = Transform.GetLocation() - ExtrudeFrameForGizmoWorldSpace.Origin;
			ActivityContext->Preview->InvalidateResult();
		}
	});

	Settings->WatchProperty(Settings->DirectionMode, [this](EPolyEditExtrudeEdgeDirectionMode) 
	{
		ConvertToNewDirectionMode(Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::SingleDirection);
	});
	Settings->WatchProperty(Settings->Distance, [this](double) 
	{
		if (!bIsRunning) { return; }
		
		// When distance changes, we keep the same extrude direction (set by gizmo or automatically) and
		//  just adjust the length of extrusion in world space.

		auto AdjustVectorLength = [this](FVector3d& VectorToAdjust)
		{
			if (bExtrudeDistanceWasNegative)
			{
				// The normlized extrude vector is actually in the opposite direction
				VectorToAdjust *= -1;
			}
			VectorToAdjust.Normalize();
			if (VectorToAdjust.IsZero())
			{
				VectorToAdjust = FVector3d::UnitX();
			}
			VectorToAdjust *= Settings->Distance;
		};

		if (Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::SingleDirection)
		{
			AdjustVectorLength(SingleDirectionVectorWorldSpace);
		}
		else
		{
			AdjustVectorLength(ParamsInWorldExtrudeFrame);
		}
		bExtrudeDistanceWasNegative = Settings->Distance < 0;

		if (bIsRunning)
		{
			ActivityContext->Preview->InvalidateResult();
		}
	});
	Settings->WatchProperty(Settings->DistanceMode, [this](EPolyEditExtrudeEdgeDistanceMode) 
	{
		if (!bIsRunning) { return; }

		// When swapping distance mode, we would like things to stay pretty much where they are.
		// This can be done by updating from params, since both methods of operation are just
		//  setting those.
		if (Settings->DistanceMode == EPolyEditExtrudeEdgeDistanceMode::Fixed)
		{
			UpdateDistanceFromParams();
		}
		else
		{
			UpdateGizmosFromCurrentParams();
		}
		UpdateGizmoVisibility(); 
	});

	auto UpdateExtrudeFrame = [this]()
	{
		if (!bIsRunning) { return; }

		RecalculateGizmoExtrudeFrame();
		UpdateGizmosFromCurrentParams();
		ActivityContext->Preview->InvalidateResult();
	};

	Settings->WatchProperty(Settings->bUseUnselectedForFrames, [UpdateExtrudeFrame](bool)
	{
		UpdateExtrudeFrame();
	});
	Settings->WatchProperty(Settings->bAdjustToExtrudeEvenly, [UpdateExtrudeFrame](bool)
	{
		UpdateExtrudeFrame();
	});

	Settings->SilentUpdateWatched();
}


EToolActivityStartResult UPolyEditExtrudeEdgeActivity::Start()
{
	using namespace PolyEditExtrudeEdgeActivityLocals;

	if (!CanStart())
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("OnExtrudeFailedMesssage", "Action requires boundary edge selection."),
			EToolMessageLevel::UserWarning);
		return EToolActivityStartResult::FailedStart;
	}

	CurrentMeshTransform = ActivityContext->Preview->PreviewMesh->GetTransform();

	ActivityContext->Preview->ChangeOpFactory(this);
	ActivityContext->Preview->OnOpCompleted.AddWeakLambda(this,
		[this](const UE::Geometry::FDynamicMeshOperator* UncastOp)
	{
		const FExtrudeEdgeOp* Op = static_cast<const FExtrudeEdgeOp*>(UncastOp);
		NewSelectionEids = Op->NewSelectionEids;
		EidsToRender = Op->EdgesToRender;
	});
	ActivityContext->Preview->OnMeshUpdated.AddWeakLambda(this,
		[this](UMeshOpPreviewWithBackgroundCompute*)
	{
		// Do this here instead of OnOpCompleted, because we need access to the updated mesh.
		UpdateDrawnPreviewEdges();
	});

	PreviewGeometry = NewObject<UPreviewGeometry>();
	PreviewGeometry->CreateInWorld(ActivityContext->Preview->GetWorld(), FTransform::Identity);
	PreviewGeometry->SetTransform(CurrentMeshTransform);

	SetToolPropertySourceEnabled(Settings, true);
	bIsRunning = true;

	// Things seem to look a little better if we clear the selection highlighting. But this means we need to
	// revert the selection before issuing any undo/redo transactions. This is done either in ApplyExtrude or
	// EndInternal.
	ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	ActivityContext->SelectionMechanic->ClearSelection();
	bRevertedSelection = false;
	GatherSelectedEids();

	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	ExtrudeFrameGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAxisX | ETransformGizmoSubElements::TranslateAxisZ | ETransformGizmoSubElements::TranslatePlaneXZ,
		this);
	ExtrudeFrameGizmo->SetActiveTarget(ExtrudeFrameProxy, GetParentTool()->GetToolManager());
	ExtrudeFrameGizmo->SetVisibility(false);
	// We force the coordinate system to be local so that the gizmo only moves in the plane we specify
	ExtrudeFrameGizmo->bUseContextCoordinateSystem = false;
	ExtrudeFrameGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;

	SingleDirectionGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes,
		this);
	SingleDirectionGizmo->SetActiveTarget(SingleDirectionProxy);
	SingleDirectionGizmo->SetVisibility(false);

	RecalculateGizmoExtrudeFrame();

	ResetParams();
	UpdateGizmosFromCurrentParams();

	UpdateGizmoVisibility();

	// Do this as the last thing so that everything is set up for it.
	ActivityContext->Preview->InvalidateResult();

	ActivityContext->EmitActivityStart(LOCTEXT("BeginExtrudeEdgeActivity", "Begin Extrude Edges"));
	return EToolActivityStartResult::Running;
}


EToolActivityEndResult UPolyEditExtrudeEdgeActivity::End(EToolShutdownType ShutdownType)
{
	if (!ensure(bIsRunning))
	{
		EndInternal();
		return EToolActivityEndResult::ErrorDuringEnd;
	}

	if (ShutdownType == EToolShutdownType::Cancel)
	{
		EndInternal();

		// Reset the preview
		ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());

		return EToolActivityEndResult::Cancelled;
	}
	else
	{
		// Stop the current compute if there is one.
		ActivityContext->Preview->CancelCompute();

		// Apply whatever we happen to have at the moment.
		ApplyExtrude();

		EndInternal();
		return EToolActivityEndResult::Completed;
	}
}


void UPolyEditExtrudeEdgeActivity::ApplyExtrude()
{
	using namespace UE::Geometry;
	using namespace PolyEditExtrudeEdgeActivityLocals;

	const FDynamicMesh3* ResultMesh = ActivityContext->Preview->PreviewMesh->GetMesh();

	UE::Geometry::FMeshEdgesFromTriangleSubIndices StableEdgeIDs;
	StableEdgeIDs.InitializeFromEdgeIDs(*ResultMesh, NewSelectionEids);

	FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
	ChangeTracker.BeginChange();

	ActivityContext->CurrentMesh->Copy(*ResultMesh);

	// We need to reset the old selection back before we give the new one, so that undo reverts back
	//  to the correct selection state. Currently, the boolean check is only necessary in EndInternal,
	//  but we'll keep it in case we move code around.
	if (!bRevertedSelection)
	{
		ActivityContext->SelectionMechanic->SetSelection(ActiveSelection, false);
		bRevertedSelection = true;
	}

	ParentTool->GetToolManager()->BeginUndoTransaction(TransactionLabel);
	FGroupTopologySelection NewSelection;
	ActivityContext->EmitCurrentMeshChangeAndUpdate(TransactionLabel, ChangeTracker.EndChange(), NewSelection);
	ActivityContext->SelectionMechanic->BeginChange();

	StableEdgeIDs.GetEdgeIDs(*ActivityContext->CurrentMesh, NewSelectionEids);

	for (int32 Eid : NewSelectionEids)
	{
		NewSelection.SelectedEdgeIDs.Add(ActivityContext->CurrentTopology->FindGroupEdgeID(Eid));
	}

	ActivityContext->SelectionMechanic->SetSelection(NewSelection);
	ActivityContext->SelectionMechanic->EndChangeAndEmitIfModified();
	ParentTool->GetToolManager()->EndUndoTransaction();
}


void UPolyEditExtrudeEdgeActivity::EndInternal()
{
	// Fix up the selection we cleared to remove the highlighting (if we haven't already done it via an Apply)
	if (!bRevertedSelection)
	{
		ActivityContext->SelectionMechanic->SetSelection(ActiveSelection, false);
		bRevertedSelection = true;
	}

	ActivityContext->Preview->ClearOpFactory();
	ActivityContext->Preview->OnOpCompleted.RemoveAll(this);
	ActivityContext->Preview->OnMeshUpdated.RemoveAll(this);
	SetToolPropertySourceEnabled(Settings, false);

	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	GizmoManager->DestroyAllGizmosByOwner(this);

	PreviewGeometry->Disconnect();
	PreviewGeometry = nullptr;

	bIsRunning = false;
}



void UPolyEditExtrudeEdgeActivity::Shutdown(EToolShutdownType ShutdownType)
{
	if (IsRunning())
	{
		End(ShutdownType);
	}
	Super::Shutdown(ShutdownType);
}

bool UPolyEditExtrudeEdgeActivity::CanStart() const
{
	using namespace UE::Geometry;
	if (!ActivityContext)
	{
		return false;
	}
	const FGroupTopologySelection& Selection = ActivityContext->SelectionMechanic->GetActiveSelection();
	if (Selection.SelectedEdgeIDs.IsEmpty())
	{
		return false;
	}
	for (int32 GroupEdgeID : Selection.SelectedEdgeIDs)
	{
		if (ActivityContext->CurrentTopology->IsBoundaryEdge(GroupEdgeID))
		{
			return true;
		}
	}
	return false;
}

bool UPolyEditExtrudeEdgeActivity::CanAccept() const
{
	return true;
}


void UPolyEditExtrudeEdgeActivity::UpdateGizmoVisibility()
{
	if (!bIsRunning || !ensure(SingleDirectionGizmo && ExtrudeFrameGizmo))
	{
		return;
	}
	if (Settings->DistanceMode == EPolyEditExtrudeEdgeDistanceMode::Fixed)
	{
		SingleDirectionGizmo->SetVisibility(false);
		ExtrudeFrameGizmo->SetVisibility(false);
	}
	else
	{
		ExtrudeFrameGizmo->SetVisibility(Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames);
		SingleDirectionGizmo->SetVisibility(Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::SingleDirection);
	}
}

void UPolyEditExtrudeEdgeActivity::GatherSelectedEids()
{
	SelectedEids.Reset();
	GroupsToSetPerEid.Reset();

	int32 GroupID = ActivityContext->CurrentMesh->MaxGroupID();

	// Only used in triangle mode, to avoid creating a bunch of new groups
	TMap<int32, int32> NeighborGroupToNewGroup; 
	
	for (int32 GroupEdgeID : ActiveSelection.SelectedEdgeIDs)
	{
		if (!ActivityContext->CurrentTopology->IsBoundaryEdge(GroupEdgeID))
		{
			continue;
		}

		if (ActivityContext->bTriangleMode)
		{
			int32 MeshEdgeID = ActivityContext->CurrentTopology->Edges[GroupEdgeID].Span.Edges[0];
			SelectedEids.Add(MeshEdgeID);

			int32 Tid = ActivityContext->CurrentMesh->GetEdgeT(MeshEdgeID).A;
			int32 NeighborGroup = ActivityContext->CurrentMesh->GetTriangleGroup(Tid);

			int32* NewGroupID = NeighborGroupToNewGroup.Find(NeighborGroup);
			if (!NewGroupID)
			{
				NewGroupID = &NeighborGroupToNewGroup.Add(NeighborGroup, GroupID);
				++GroupID;
			}
			GroupsToSetPerEid.Add(*NewGroupID);
		}
		else
		{
			for (int32 Eid : ActivityContext->CurrentTopology->GetGroupEdgeEdges(GroupEdgeID))
			{
				SelectedEids.Add(Eid);
				GroupsToSetPerEid.Add(GroupID);
			}

			++GroupID;
		}
	}
}

// We draw the boundary edges for our extrusion preview. This is not something we do in other
//  cases, for instance while in push/pull and the face extrude activity. But for edge extrusion,
//  it is helpful to see the boundary because you may be looking behind the face that you are
//  extruding, making it invisible.
void UPolyEditExtrudeEdgeActivity::UpdateDrawnPreviewEdges()
{
	using namespace PolyEditExtrudeEdgeActivityLocals;

	if (!bIsRunning 
		|| !ActivityContext || !ActivityContext->Preview || !ActivityContext->Preview->PreviewMesh 
		|| !PreviewGeometry)
	{
		return;
	}

	ActivityContext->Preview->PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh) 
	{
		PreviewGeometry->CreateOrUpdateLineSet(TEXT("NewBoundaries"), EidsToRender.Num(), [this, &Mesh]
			(int32 Index, TArray<FRenderableLine>& Lines)
		{
			int32 Eid = EidsToRender[Index];
			if (!ensure(Mesh.IsEdge(Eid)))
			{
				return;
			}
			
			FIndex2i EdgeVids = Mesh.GetEdgeV(Eid);
			Lines.Add(FRenderableLine(Mesh.GetVertex(EdgeVids.A), Mesh.GetVertex(EdgeVids.B),
				GroupLineColor, GroupLineThickness, GroupLineDepthBias));
		});
	});
}


// Updates Settings->Distance from the parameters
void UPolyEditExtrudeEdgeActivity::UpdateDistanceFromParams()
{
	if (Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::SingleDirection)
	{
		Settings->Distance = SingleDirectionVectorWorldSpace.Length();
	}
	else // if (Settings->DirectionMode == EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames)
	{
		Settings->Distance = ExtrudeFrameForGizmoWorldSpace.FromFrameVector(ParamsInWorldExtrudeFrame).Length();
	}
}

void UPolyEditExtrudeEdgeActivity::UpdateGizmosFromCurrentParams()
{
	FVector3d SingleDirectionGizmoLocation = SingleDirectionVectorWorldSpace + ExtrudeFrameForGizmoWorldSpace.Origin;

	FVector3d ExtrudeFrameGizmoLocation = ExtrudeFrameForGizmoWorldSpace.FromFramePoint(ParamsInWorldExtrudeFrame);

	SingleDirectionGizmo->ReinitializeGizmoTransform(FTransform(
		FQuat(ExtrudeFrameForGizmoWorldSpace.Rotation),
		SingleDirectionGizmoLocation));

	ExtrudeFrameGizmo->ReinitializeGizmoTransform(FTransform(
		FQuat(ExtrudeFrameForGizmoWorldSpace.Rotation),
		ExtrudeFrameGizmoLocation));

	ExtrudeFrameGizmo->SetDisplaySpaceTransform(ExtrudeFrameForGizmoWorldSpace.ToFTransform());
}

void UPolyEditExtrudeEdgeActivity::ConvertToNewDirectionMode(bool bToSingleDirection)
{
	if (bToSingleDirection)
	{
		// Convert fixed parameter
		FVector3d CurrentPosition = ExtrudeFrameForGizmoWorldSpace.FromFramePoint(ParamsInWorldExtrudeFrame);
		SingleDirectionVectorWorldSpace = CurrentPosition - ExtrudeFrameForGizmoWorldSpace.Origin;

		// Convert gizmo location
		CurrentPosition = ExtrudeFrameGizmo->GetGizmoTransform().GetLocation();
		FTransform NewTransform(FQuat(ExtrudeFrameForGizmoWorldSpace.Rotation), CurrentPosition);
		SingleDirectionGizmo->ReinitializeGizmoTransform(NewTransform);
	}
	else
	{
		// Convert fixed parameter
		FVector3d CurrentPosition = ExtrudeFrameForGizmoWorldSpace.Origin + SingleDirectionVectorWorldSpace;
		ParamsInWorldExtrudeFrame = ExtrudeFrameForGizmoWorldSpace.ToFramePoint(CurrentPosition);
		ParamsInWorldExtrudeFrame.Y = 0;

		// Convert gizmo location
		CurrentPosition = SingleDirectionGizmo->GetGizmoTransform().GetLocation();
		FVector3d FramePoint = ExtrudeFrameForGizmoWorldSpace.ToFramePoint(CurrentPosition);
		FramePoint.Y = 0;
		FTransform NewTransform(FQuat(ExtrudeFrameForGizmoWorldSpace.Rotation), 
			ExtrudeFrameForGizmoWorldSpace.FromFramePoint(FramePoint));
		ExtrudeFrameGizmo->ReinitializeGizmoTransform(NewTransform);
	}
	UpdateGizmoVisibility();

	if (bIsRunning)
	{
		ActivityContext->Preview->InvalidateResult();
	}
}

void UPolyEditExtrudeEdgeActivity::ResetParams()
{
	SingleDirectionVectorWorldSpace = ExtrudeFrameForGizmoWorldSpace.X() * Settings->Distance;
	ParamsInWorldExtrudeFrame = FVector3d::UnitX() * Settings->Distance;
}

void UPolyEditExtrudeEdgeActivity::RecalculateGizmoExtrudeFrame()
{
	using namespace PolyEditExtrudeEdgeActivityLocals;
	using namespace UE::Geometry;

	// Getting an extrude frame is actually tricky because we have to grab two boundary edges that are
	// adjacent to some vertex in the selection, and the way these get selected is dependent on the 
	// implementation of FExtrudeBoundaryEdges (because there is ambiguity in the case of bowties).

	TArray<FExtrudeBoundaryEdges::FNewVertSourceData> NewVertData;
	TMap<int32, FIndex2i> EidToIndicesIntoNewVerts;
	FExtrudeBoundaryEdges::GetInputEdgePairings(
		*ActivityContext->CurrentMesh, SelectedEids, Settings->bUseUnselectedForFrames,
		NewVertData, EidToIndicesIntoNewVerts);

	if (!ensure(!NewVertData.IsEmpty()))
	{
		return;
	}

	FExtrudeBoundaryEdges::FExtrudeFrame ExtrudeFrame;
	FExtrudeBoundaryEdges::GetExtrudeFrame(
		*ActivityContext->CurrentMesh, 
		NewVertData[0].SourceVid, 
		NewVertData[0].SourceEidPair.A,
		NewVertData[0].SourceEidPair.B, 
		ExtrudeFrame, Settings->bAdjustToExtrudeEvenly ? MAX_VERT_MOVEMENT_ADJUSTMENT_SCALE : 1.0);

	ExtrudeFrameForGizmoMeshSpace = ExtrudeFrame;

	// This method of finding the extrude frame in world space isn't quite right in the case of nonuniform scale. However
	// it seems hard to find any perfect solution, and in the end, it's not critical because we'll probably be able to get
	// extrusion parameters out of an imperfect frame anyway.
	ExtrudeFrameForGizmoWorldSpace = FFrame3d(
		CurrentMeshTransform.TransformPosition(ExtrudeFrameForGizmoMeshSpace.Frame.Origin),
		CurrentMeshTransform.TransformRotation(FQuat(ExtrudeFrameForGizmoMeshSpace.Frame.Rotation)));
}

void UPolyEditExtrudeEdgeActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace PolyEditExtrudeEdgeActivityLocals;

	if (bIsRunning)
	{
		// Draw a line showing where the X axis of the extrude frame is. We could use the preview actor for this but
		// this is simpler and doesn't require us to set up the foreground material.
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		PDI->DrawLine(ExtrudeFrameForGizmoWorldSpace.Origin,
			ExtrudeFrameForGizmoWorldSpace.FromFramePoint(FVector3d(1000, 0, 0)),
			ExtrudeFrameLineColor, SDPG_Foreground, ExtrudeFrameLineThickness, 1.0, true);
	}
	
}

void UPolyEditExtrudeEdgeActivity::Tick(float DeltaTime)
{
}



#undef LOCTEXT_NAMESPACE
