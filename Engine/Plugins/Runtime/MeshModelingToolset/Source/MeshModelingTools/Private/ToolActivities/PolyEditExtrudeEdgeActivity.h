// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FrameTypes.h"
#include "InteractiveTool.h" // UInteractiveToolPropertySet
#include "InteractiveToolActivity.h"
#include "GroupTopology.h" // FGroupTopologySelection
#include "ModelingOperators.h" // IDynamicMeshOperatorFactory
#include "Operations/ExtrudeBoundaryEdges.h" // FExtrudeFrame

#include "PolyEditExtrudeEdgeActivity.generated.h"

class UCombinedTransformGizmo;
class UPolyEditActivityContext;
class UPreviewGeometry;
class UTransformProxy;

UENUM()
enum class EPolyEditExtrudeEdgeDirectionMode
{
	/** Each vertex gets its own local frame to extrude along. */
	LocalExtrudeFrames,
	/** All vertices are extruded in the same direction */
	SingleDirection,
};


UENUM()
enum class EPolyEditExtrudeEdgeDistanceMode
{
	/** Sets the distance numerically using the Distance parameter. */
	Fixed,
	/** Uses a gizmo attached to one of the vertices to determine distance to extrude. */
	Gizmo
};

UCLASS()
class UPolyEditExtrudeEdgeActivityProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	
	/** How direction to move the vertices is determined. */
	UPROPERTY(EditAnywhere, Category = Extrude)
	EPolyEditExtrudeEdgeDirectionMode DirectionMode = EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames;

	/** How distance to extrude along a given direction is determined. */
	UPROPERTY(EditAnywhere, Category = Extrude)
	EPolyEditExtrudeEdgeDistanceMode DistanceMode = EPolyEditExtrudeEdgeDistanceMode::Gizmo;

	UPROPERTY(EditAnywhere, Category = Extrude, meta = (EditCondition = "DistanceMode == EPolyEditExtrudeEdgeDistanceMode::Fixed", EditConditionHides,
		UIMin = "0", UIMax = "200", ClampMin = "-100000", ClampMax = "100000"))
	double Distance = 20;

	/** When a vertex has both a selected and nonselected neighbor, use the unselected neighbor in picking an extrude frame as well. */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (
		EditCondition = "DirectionMode == EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames", EditConditionHides))
	bool bUseUnselectedForFrames = false;

	/** Adjust individual extrude directions in an effort to make extruded edges parallel to their original edges. */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (
		EditCondition = "DirectionMode == EPolyEditExtrudeEdgeDirectionMode::LocalExtrudeFrames", EditConditionHides))
	bool bAdjustToExtrudeEvenly = false;
};


/**
 * Extrudes boundary edges.
 */
UCLASS()
class UPolyEditExtrudeEdgeActivity : public UInteractiveToolActivity,
	public UE::Geometry::IDynamicMeshOperatorFactory

{
	GENERATED_BODY()

public:
	using FFrame3d = UE::Geometry::FFrame3d;

	// IInteractiveToolActivity
	virtual void Setup(UInteractiveTool* ParentTool) override;
	virtual void Shutdown(EToolShutdownType ShutdownType) override;
	virtual bool CanStart() const override;
	virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override;
	virtual EToolActivityEndResult End(EToolShutdownType) override;
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	virtual void Tick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

private:
	UPROPERTY()
	TObjectPtr<UPolyEditExtrudeEdgeActivityProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	// Support for gizmos.
	UPROPERTY()
	TObjectPtr<UTransformProxy> ExtrudeFrameProxy;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> ExtrudeFrameGizmo;
	UPROPERTY()
	TObjectPtr<UTransformProxy> SingleDirectionProxy;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> SingleDirectionGizmo;

	// Used for drawing the boundaries of the new faces to make them look similar to the
	// rest of the mesh.
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry;

	bool bIsRunning = false;

	void ApplyExtrude();
	void EndInternal();

	UE::Geometry::FExtrudeBoundaryEdges::FExtrudeFrame ExtrudeFrameForGizmoMeshSpace;

	FFrame3d ExtrudeFrameForGizmoWorldSpace;
	
	// These are the parameters we actually end up using, initialized from gizmos or detail panel.
	FVector3d SingleDirectionVectorWorldSpace;
	FVector3d ParamsInWorldExtrudeFrame;

	void UpdateGizmosFromCurrentParams();
	void UpdateDistanceFromParams();
	void ConvertToNewDirectionMode(bool bToSingleDirection);
	void ResetParams();

	void RecalculateGizmoExtrudeFrame();
	void UpdateGizmoVisibility();

	UE::Geometry::FGroupTopologySelection ActiveSelection;
	bool bRevertedSelection = false;
	TArray<int32> SelectedEids;
	TArray<int32> GroupsToSetPerEid;
	void GatherSelectedEids();

	TArray<int32> EidsToRender;
	void UpdateDrawnPreviewEdges();

	TArray<int32> NewSelectionEids;

	// TODO: Might be worth having a getter in ActivityContext
	FTransform CurrentMeshTransform;

	// This is helpful so that in our distance watcher, we know that the previous distance
	//  was negative, and therefore the extrude direction is actually opposite.
	bool bExtrudeDistanceWasNegative = false;
};
