// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditInsertEdgeActivity.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "CuttingOps/GroupEdgeInsertionOp.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditInsertEdgeActivity)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPolyEditInsertEdgeActivity"

namespace PolyEditInsertEdgeActivityLocals
{
	bool GetSharedBoundary(const FGroupTopology& Topology,
		const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint, int32 StartTopologyID, bool bStartIsCorner,
		const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint, int32 EndTopologyID, bool bEndIsCorner,
		int32& GroupIDOut, int32& BoundaryIndexOut);
	bool DoesBoundaryContainPoint(const FGroupTopology& Topology,
		const FGroupTopology::FGroupBoundary& Boundary, int32 PointTopologyID, bool bPointIsCorner);

	FText GroupEdgeStartTransactionName = LOCTEXT("GroupEdgeStartTransactionName", "Group Edge Start");
}

TUniquePtr<FDynamicMeshOperator> UPolyEditInsertEdgeActivity::MakeNewOperator()
{
	TUniquePtr<FGroupEdgeInsertionOp> Op = MakeUnique<FGroupEdgeInsertionOp>();

	Op->OriginalMesh = ComputeStartMesh;
	Op->OriginalTopology = ComputeStartTopology;
	Op->SetTransform(TargetTransform);

	if (Settings->InsertionMode == EGroupEdgeInsertionMode::PlaneCut)
	{
		Op->Mode = FGroupEdgeInserter::EInsertionMode::PlaneCut;
	}
	else
	{
		Op->Mode = FGroupEdgeInserter::EInsertionMode::Retriangulate;
	}

	Op->VertexTolerance = Settings->VertexTolerance;

	Op->StartPoint = StartPoint;
	Op->EndPoint = EndPoint;
	Op->CommonGroupID = CommonGroupID;
	Op->CommonBoundaryIndex = CommonBoundaryIndex;

	return Op;
}

void UPolyEditInsertEdgeActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	// Set up properties
	Settings = NewObject<UGroupEdgeInsertionProperties>(this);
	Settings->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(Settings);
	SetToolPropertySourceEnabled(Settings, false);
	Settings->GetOnModified().AddUObject(this, &UPolyEditInsertEdgeActivity::OnPropertyModified);

	// These draw the group edges and the loops to be inserted
	ExistingEdgesRenderer.LineColor = FLinearColor::Red;
	ExistingEdgesRenderer.LineThickness = 2.0;
	PreviewEdgeRenderer.LineColor = FLinearColor::Green;
	PreviewEdgeRenderer.LineThickness = 4.0;
	PreviewEdgeRenderer.PointColor = FLinearColor::Green;
	PreviewEdgeRenderer.PointSize = 8.0;
	PreviewEdgeRenderer.bDepthTested = false;

	// Set up the topology selector settings
	TopologySelectorSettings.bEnableEdgeHits = true;
	TopologySelectorSettings.bEnableCornerHits = true;
	TopologySelectorSettings.bEnableFaceHits = false;

	// Set up our input routing
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();

	// TODO: When the deprecated function GetTopologySelector is removed from UPolygonSelectionMechanic, we can remove the UMeshTopologySelectionMechanic:: prefix here
	TopologySelector = ActivityContext->SelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector();

	ActivityContext->OnUndoRedo.AddWeakLambda(this, [this](bool bGroupTopologyModified)
	{
		UpdateComputeInputs();
		ToolState = EState::GettingStart;
		ClearPreview(true);
	});
}

void UPolyEditInsertEdgeActivity::UpdateComputeInputs()
{
	ComputeStartMesh = MakeShared<FDynamicMesh3>(*ActivityContext->CurrentMesh);

	TSharedPtr<FGroupTopology> NonConstTopology = MakeShared<FGroupTopology>(*ActivityContext->CurrentTopology);
	NonConstTopology->RetargetOnClonedMesh(ComputeStartMesh.Get());

	ComputeStartTopology = NonConstTopology;
}


void UPolyEditInsertEdgeActivity::Shutdown(EToolShutdownType ShutdownType)
{
	if (bIsRunning)
	{
		End(ShutdownType);
	}

	Settings->SaveProperties(ParentTool.Get());
	Settings->GetOnModified().Clear();

	TopologySelector.Reset();
	Settings = nullptr;
	ActivityContext->OnUndoRedo.RemoveAll(this);
	ActivityContext = nullptr;

	Super::Shutdown(ShutdownType);
}

EToolActivityStartResult UPolyEditInsertEdgeActivity::Start()
{
	ParentTool->GetToolManager()->DisplayMessage(
		LOCTEXT("InsertEdgeActivityDescription", "Click two points on the boundary of a face to "
			"insert a new edge between the points and split the face."),
		EToolMessageLevel::UserNotification);

	SetToolPropertySourceEnabled(Settings, true);

	// We don't use selection, so clear it if necessary (have to issue an undo/redo event)
	if (!ActivityContext->SelectionMechanic->GetActiveSelection().IsEmpty())
	{
		ActivityContext->SelectionMechanic->BeginChange();
		ActivityContext->SelectionMechanic->ClearSelection();
		ParentTool->GetToolManager()->EmitObjectChange(ActivityContext->SelectionMechanic,
			ActivityContext->SelectionMechanic->EndChange(), LOCTEXT("ClearSelection", "Clear Selection"));
	}

	TargetTransform = ActivityContext->Preview->PreviewMesh->GetTransform();

	UpdateComputeInputs();
	SetupPreview();

	ToolState = EState::GettingStart;
	bLastComputeSucceeded = false;

	bIsRunning = true;

	// Emit activity start transaction
	ActivityContext->EmitActivityStart(LOCTEXT("BeginInsertEdgeActivity", "Begin Insert Edge"));

	return EToolActivityStartResult::Running;
}

EToolActivityEndResult UPolyEditInsertEdgeActivity::End(EToolShutdownType ShutdownType)
{
	if (!bIsRunning)
	{
		return EToolActivityEndResult::ErrorDuringEnd;
	}

	SetToolPropertySourceEnabled(Settings, false);
	Settings->GetOnModified().Clear();

	ActivityContext->Preview->OnOpCompleted.RemoveAll(this);
	ActivityContext->Preview->OnMeshUpdated.RemoveAll(this);
	ToolState = EState::GettingStart;
	// Note that this does an ensure on us not being in a ToolState that requires the preview points, hence
	// we do this after resetting ToolState.
	ClearPreview(true); 
	ActivityContext->Preview->ClearOpFactory();

	LatestOpTopologyResult.Reset();
	LatestOpChangedTids.Reset();

	bIsRunning = false;

	return CanAccept() ? EToolActivityEndResult::Completed : EToolActivityEndResult::Cancelled;
}

void UPolyEditInsertEdgeActivity::SetupPreview()
{
	ActivityContext->Preview->ChangeOpFactory(this);

	// Whenever we get a new result from the op, we need to extract the preview edges so that
	// we can draw them if we want to.
	ActivityContext->Preview->OnOpCompleted.AddWeakLambda(this, [this](const FDynamicMeshOperator* UncastOp) {
		const FGroupEdgeInsertionOp* Op = static_cast<const FGroupEdgeInsertionOp*>(UncastOp);

		LatestOpTopologyResult.Reset();
		LatestOpChangedTids.Reset();
		PreviewEdges.Reset();

		// See if this compute is actually outdated, i.e. we changed the mesh
		// out from under it.
		if (Op->OriginalMesh != ComputeStartMesh)
		{
			bLastComputeSucceeded = false;
			return;
		}

		bLastComputeSucceeded = Op->bSucceeded;

		if (bLastComputeSucceeded)
		{
			Op->GetEdgeLocations(PreviewEdges);
			LatestOpTopologyResult = Op->ResultTopology;
			LatestOpChangedTids = Op->ChangedTids;
		}
	});

	// In case of failure, we want to hide the broken preview, since we wouldn't accept it on
	// a click. Note that this can't be fired OnOpCompleted because the preview is updated
	// with the op result after that callback, which would undo the reset. The preview edge
	// extraction can't be lumped in here because it needs the op rather than the preview object.
	ActivityContext->Preview->OnMeshUpdated.AddWeakLambda(this, [this](UMeshOpPreviewWithBackgroundCompute*) {
		if (!bLastComputeSucceeded)
		{
			ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());
		}
	});
}

bool UPolyEditInsertEdgeActivity::CanStart() const
{
	return true;
}

void UPolyEditInsertEdgeActivity::Tick(float DeltaTime)
{
	if (ToolState == EState::WaitingForInsertComplete && ActivityContext->Preview->HaveValidResult())
	{
		if (bLastComputeSucceeded)
		{
			FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
			ChangeTracker.BeginChange();
			ChangeTracker.SaveTriangles(*LatestOpChangedTids, true /*bSaveVertices*/);

			// Update current mesh
			ActivityContext->CurrentMesh->Copy(*ActivityContext->Preview->PreviewMesh->GetMesh(), 
				true, true, true, true);
			
			*ActivityContext->CurrentTopology = *LatestOpTopologyResult;
			ActivityContext->CurrentTopology->RetargetOnClonedMesh(ActivityContext->CurrentMesh.Get());
			
			UpdateComputeInputs();

			// Emit transaction
			FGroupTopologySelection EmptySelection;
			ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("EdgeInsertionTransactionName", "Edge Insertion"),
				ChangeTracker.EndChange(), EmptySelection);

			ToolState = EState::GettingStart;
			
			if (Settings->bContinuousInsertion)
			{
				// If continuous insertion is enabled, try to find information associated with new start point.
				// If found, remain in GettingEnd mode.
				FVector3d PreviewPoint;
				if (GetHoveredItem(LastEndPointWorldRay, StartPoint, StartTopologyID, bStartIsCorner, PreviewPoint))
				{
					PreviewPoints.Reset();
					PreviewPoints.Add(PreviewPoint);
					ToolState = EState::GettingEnd;

					ParentTool->GetToolManager()->EmitObjectChange(this,
						MakeUnique<FGroupEdgeInsertionFirstPointChange>(CurrentChangeStamp),
						PolyEditInsertEdgeActivityLocals::GroupEdgeStartTransactionName);
				}
			}
		}
		else
		{
			ToolState = PreviewPoints.Num() > 0 ? EState::GettingEnd : EState::GettingStart;
		}

		PreviewEdges.Reset();
	}

}

void UPolyEditInsertEdgeActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	ParentTool->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	// Draw the existing group edges
	FViewCameraState RenderCameraState = RenderAPI->GetCameraState();
	ExistingEdgesRenderer.BeginFrame(RenderAPI, RenderCameraState);
	ExistingEdgesRenderer.SetTransform(TargetTransform);

	for (const FGroupTopology::FGroupEdge& Edge : ActivityContext->CurrentTopology->Edges)
	{
		FVector3d A, B;
		for (int32 eid : Edge.Span.Edges)
		{
			ActivityContext->CurrentMesh->GetEdgeV(eid, A, B);
			ExistingEdgesRenderer.DrawLine(A, B);
		}
	}
	ExistingEdgesRenderer.EndFrame();

	// Draw the preview edges and points
	PreviewEdgeRenderer.BeginFrame(RenderAPI, RenderCameraState);
	PreviewEdgeRenderer.SetTransform(TargetTransform);
	for (const TPair<FVector3d, FVector3d>& EdgeVerts : PreviewEdges)
	{
		PreviewEdgeRenderer.DrawLine(EdgeVerts.Key, EdgeVerts.Value);
	}
	for (const FVector3d& Point : PreviewPoints)
	{
		PreviewEdgeRenderer.DrawPoint(Point);
	}
	PreviewEdgeRenderer.EndFrame();
}

bool UPolyEditInsertEdgeActivity::CanAccept() const
{
	return ToolState != EState::WaitingForInsertComplete;
}

void UPolyEditInsertEdgeActivity::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	PreviewEdges.Reset();

	// Don't clear drawn elements because we may be getting the second endpoint, and still
	// need to keep the first.
	ClearPreview(false); 
}

void UPolyEditInsertEdgeActivity::ClearPreview(bool bClearDrawnElements)
{
	ActivityContext->Preview->CancelCompute();
	ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());
	bShowingBaseMesh = true;

	if (bClearDrawnElements)
	{
		PreviewEdges.Reset();
		PreviewPoints.Reset();

		// If we're removing the start point, we shouldn't be in a state that requires
		// it, i.e. getting the second point.
		if (!ensure(ToolState != EState::GettingEnd))
		{
			ToolState = EState::GettingStart;
		}
	}
}

/** 
 * Update the preview unless we've already computed one with the same parameters (such as when snapping to
 * the same vertex despite moving the mouse).
 */
void UPolyEditInsertEdgeActivity::ConditionallyUpdatePreview(
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& NewEndPoint, int32 NewEndTopologyID, bool bNewEndIsCorner,
	int32 NewCommonGroupID, int32 NewBoundaryIndex)
{
	if (bShowingBaseMesh 
		|| bEndIsCorner != bNewEndIsCorner || EndTopologyID != NewEndTopologyID
		|| EndPoint.bIsVertex != NewEndPoint.bIsVertex || EndPoint.ElementID != NewEndPoint.ElementID
		|| (!NewEndPoint.bIsVertex && NewEndPoint.EdgeTValue != EndPoint.EdgeTValue)
		|| CommonGroupID != NewCommonGroupID || CommonBoundaryIndex != NewBoundaryIndex)
	{
		// Update the end variables, since they are apparently different
		EndPoint = NewEndPoint;
		EndTopologyID = NewEndTopologyID;
		bEndIsCorner = bNewEndIsCorner;
		CommonGroupID = NewCommonGroupID;
		CommonBoundaryIndex = NewBoundaryIndex;

		// If either endpoint is a corner, we need to calculate its tangent. This will differ based on which
		// boundary it is a part of.
		if (bStartIsCorner)
		{
			GetCornerTangent(StartTopologyID, CommonGroupID, CommonBoundaryIndex, StartPoint.Tangent);
		}
		if (bEndIsCorner)
		{
			GetCornerTangent(EndTopologyID, CommonGroupID, CommonBoundaryIndex, EndPoint.Tangent);
		}

		bShowingBaseMesh = false;
		PreviewEdges.Reset();
		ActivityContext->Preview->InvalidateResult();
	}
}

FInputRayHit UPolyEditInsertEdgeActivity::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Hit;

	// Early out if the activity is not running. This is actually important because the behavior is
	// always in the behavior list while the tool is running (we don't have a way to add/remove at
	// will).
	if (!bIsRunning)
	{
		return Hit; // Hit.bHit is false
	}

	switch (ToolState)
	{
	case EState::WaitingForInsertComplete:
		break; // Keep hit invalid

	case EState::GettingStart:
	{
		PreviewPoints.Reset();
		FVector3d RayPoint;
		if (TopologyHitTest(PressPos.WorldRay, RayPoint))
		{
			Hit = FInputRayHit(PressPos.WorldRay.GetParameter((FVector)RayPoint));
		}
		break;
	}
	case EState::GettingEnd:
	{
		FVector3d RayPoint;
		FRay3d LocalRay;
		if (TopologyHitTest(PressPos.WorldRay, RayPoint, &LocalRay))
		{
			Hit = FInputRayHit(PressPos.WorldRay.GetParameter((FVector)RayPoint));
		}
		else
		{
			// If we don't hit a valid element, we still do a hover if we hit the mesh.
			// We still do the topology check in the first place because it accepts missing
			// rays that are close enough to snap.
			double RayT = 0;
			int32 Tid = FDynamicMesh3::InvalidID;
			if (ActivityContext->MeshSpatial->FindNearestHitTriangle(LocalRay, RayT, Tid))
			{
				Hit = FInputRayHit(RayT);
			}
		}
		break;
	}
	}

	return Hit;
}

bool UPolyEditInsertEdgeActivity::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	using namespace PolyEditInsertEdgeActivityLocals;

	if (!bIsRunning)
	{
		return false;
	}

	switch (ToolState)
	{
	case EState::WaitingForInsertComplete:
		return false; // Do nothing.

	case EState::GettingStart:
	{
		// Update start variables and show a preview of a point if it's on an edge or corner
		PreviewPoints.Reset();
		FVector3d PreviewPoint;
		if (GetHoveredItem(DevicePos.WorldRay, StartPoint, StartTopologyID, bStartIsCorner, PreviewPoint))
		{
			PreviewPoints.Add(PreviewPoint);
			return true;
		}
		return false;
	}
	case EState::GettingEnd:
	{
		// This shouldn't happen- if it does, we messed up with our state handling somewhere. But don't crash,
		// just reset.
		if (!ensure(PreviewPoints.Num() > 0))
		{
			ToolState = EState::GettingStart;
			ClearPreview(true);
			return false;
		}

		PreviewPoints.SetNum(1); // Keep the first element, which is the start point

		// Don't update the end variables right away so that we can check if they actually changed (they
		// won't when we snap to the same corner as before).
		FGroupEdgeInserter::FGroupEdgeSplitPoint SnappedPoint;
		int32 PointTopologyID, GroupID, BoundaryIndex;
		bool bPointIsCorner;
		FVector3d PreviewPoint;
		FRay3d LocalRay;
		if (GetHoveredItem(DevicePos.WorldRay, SnappedPoint, PointTopologyID, bPointIsCorner, PreviewPoint, &LocalRay))
		{
			// See if the point is not on the same vertex/edge but is on the same boundary
			if (!(SnappedPoint.bIsVertex == StartPoint.bIsVertex && SnappedPoint.ElementID == StartPoint.ElementID)
				&& GetSharedBoundary(*ActivityContext->CurrentTopology, StartPoint, StartTopologyID, bStartIsCorner,
					SnappedPoint, PointTopologyID, bPointIsCorner, GroupID, BoundaryIndex))
			{
				ConditionallyUpdatePreview(SnappedPoint, PointTopologyID, bPointIsCorner, GroupID, BoundaryIndex);
			}
			else
			{
				PreviewEdges.Reset(); // TODO: Maybe we should show a different color edge on a fail, rather than hiding it?
			}
			PreviewPoints.Add(PreviewPoint);
			
			return true;
		}

		// If we're here, then we don't have a valid endpoint. Make sure our preview is reset and draw a line
		// to the current hit location.
		if (!bShowingBaseMesh)
		{
			ClearPreview(false);
		}
		PreviewEdges.Reset();
		double RayT = 0;
		int32 Tid = FDynamicMesh3::InvalidID;
		if (ActivityContext->MeshSpatial->FindNearestHitTriangle(LocalRay, RayT, Tid))
		{
			PreviewEdges.Emplace(PreviewPoints[0], LocalRay.PointAt(RayT));
			return true;
		}
		return false;
	}
	}

	check(false); // Each case has its own return, so shouldn't get here
	return false;
}

void UPolyEditInsertEdgeActivity::OnEndHover()
{
	if (!bIsRunning)
	{
		return;
	}
	switch (ToolState)
	{
	case EState::WaitingForInsertComplete:
	case EState::GettingStart:
		ClearPreview(true);
		break;
	case EState::GettingEnd:
		// Keep the first preview point.
		ClearPreview(false);
		PreviewPoints.SetNum(1);
		PreviewEdges.Reset();
	}
}

FInputRayHit UPolyEditInsertEdgeActivity::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Hit;

	// Early out if the activity is not running. 
	if (!bIsRunning)
	{
		return Hit; // Hit.bHit is false
	}

	switch (ToolState)
	{
	case EState::WaitingForInsertComplete:
		break; // Keep hit invalid

	// Same requirement for the other two cases: the click should go on an edge
	case EState::GettingStart:
	case EState::GettingEnd:
	{
		FVector3d RayPoint;
		if (TopologyHitTest(ClickPos.WorldRay, RayPoint))
		{
			Hit = FInputRayHit(ClickPos.WorldRay.GetParameter((FVector)RayPoint));
		}
		break;
	}
	}
	return Hit;
}

void UPolyEditInsertEdgeActivity::OnClicked(const FInputDeviceRay& ClickPos)
{
	using namespace PolyEditInsertEdgeActivityLocals;

	switch (ToolState)
	{
	case EState::WaitingForInsertComplete:
		break; // Do nothing

	case EState::GettingStart:
	{
		// Update start variables and switch state if successful
		FVector3d PreviewPoint;
		if (GetHoveredItem(ClickPos.WorldRay, StartPoint, StartTopologyID, bStartIsCorner, PreviewPoint))
		{
			PreviewPoints.Reset();
			PreviewPoints.Add(PreviewPoint);
			ToolState = EState::GettingEnd;

			ParentTool->GetToolManager()->BeginUndoTransaction(PolyEditInsertEdgeActivityLocals::GroupEdgeStartTransactionName);
			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FGroupEdgeInsertionFirstPointChange>(CurrentChangeStamp),
				PolyEditInsertEdgeActivityLocals::GroupEdgeStartTransactionName);
			ParentTool->GetToolManager()->EndUndoTransaction();
		}
		break;
	}
	case EState::GettingEnd:
	{
		// Don't update the end variables right away so that we can check if they actually changed (they
		// won't when we snap to the same corner as before).
		FVector3d PreviewPoint;
		FGroupEdgeInserter::FGroupEdgeSplitPoint SnappedPoint;
		int32 PointTopologyID, GroupID, BoundaryIndex;
		bool bPointIsCorner;
		if (GetHoveredItem(ClickPos.WorldRay, SnappedPoint, PointTopologyID, bPointIsCorner, PreviewPoint))
		{
			// See if the point is not on the same vertex/edge but is on the same boundary
			if (!(SnappedPoint.bIsVertex == StartPoint.bIsVertex && SnappedPoint.ElementID == StartPoint.ElementID) 
				&& GetSharedBoundary(*ActivityContext->CurrentTopology, StartPoint, StartTopologyID, bStartIsCorner,
					SnappedPoint, PointTopologyID, bPointIsCorner, GroupID, BoundaryIndex))
			{
				ConditionallyUpdatePreview(SnappedPoint, PointTopologyID, bPointIsCorner, GroupID, BoundaryIndex);
				ToolState = EState::WaitingForInsertComplete;
			}
			else
			{
				ClearPreview(false);
			}
		}

		LastEndPointWorldRay = ClickPos.WorldRay;
			
		break;
	}
	}
}

bool UPolyEditInsertEdgeActivity::TopologyHitTest(const FRay& WorldRay,
	FVector3d& RayPositionOut, FRay3d* LocalRayOut)
{
	FRay3d LocalRay((FVector3d)TargetTransform.InverseTransformPosition(WorldRay.Origin),
		(FVector3d)TargetTransform.InverseTransformVector(WorldRay.Direction), false);

	if (LocalRayOut)
	{
		*LocalRayOut = LocalRay;
	}

	FGroupTopologySelection Selection;
	FVector3d Position, Normal;
	TopologySelectorSettings.bHitBackFaces = ActivityContext->SelectionMechanic->Properties->bHitBackFaces;
	if (TopologySelector->FindSelectedElement(TopologySelectorSettings,
		LocalRay, Selection, Position, Normal))
	{
		RayPositionOut = TargetTransform.TransformPosition(Position);
		return true;
	}
	return false;
}

bool UPolyEditInsertEdgeActivity::GetHoveredItem(const FRay& WorldRay,
	FGroupEdgeInserter::FGroupEdgeSplitPoint& PointOut, 
	int32& TopologyElementIDOut, bool& bIsCornerOut, FVector3d& PositionOut,
	FRay3d* LocalRayOut)
{
	TopologyElementIDOut = FDynamicMesh3::InvalidID;
	PointOut.ElementID = FDynamicMesh3::InvalidID;

	// Cast the ray to see what we hit.
	FRay3d LocalRay((FVector3d)TargetTransform.InverseTransformPosition(WorldRay.Origin),
		(FVector3d)TargetTransform.InverseTransformVector(WorldRay.Direction), false);
	if (LocalRayOut)
	{
		*LocalRayOut = LocalRay;
	}
	FGroupTopologySelection Selection;
	FVector3d Position, Normal;
	int32 EdgeSegmentID;
	TopologySelectorSettings.bHitBackFaces = ActivityContext->SelectionMechanic->Properties->bHitBackFaces;
	if (!TopologySelector->FindSelectedElement(
		TopologySelectorSettings, LocalRay, Selection, Position, Normal, &EdgeSegmentID))
	{
		return false; // Didn't hit anything
	}
	else if (Selection.SelectedCornerIDs.Num() > 0)
	{
		// Point is a corner
		TopologyElementIDOut = Selection.GetASelectedCornerID();
		bIsCornerOut = true;
		PointOut.bIsVertex = true;
		PointOut.ElementID = ActivityContext->CurrentTopology->GetCornerVertexID(TopologyElementIDOut);
		// We can't initialize the tangent yet because the tangent of a corner will
		// depend on which boundary it is a part of.

		PositionOut = ActivityContext->CurrentMesh->GetVertex(PointOut.ElementID);
	}
	else 
	{
		// Point is an edge. We'll need to calculate the t value and some other things.
		check(Selection.SelectedEdgeIDs.Num() > 0);

		TopologyElementIDOut = Selection.GetASelectedEdgeID();
		bIsCornerOut = false;

		const FGroupTopology::FGroupEdge& GroupEdge = ActivityContext->CurrentTopology->Edges[TopologyElementIDOut];

		int32 Eid = GroupEdge.Span.Edges[EdgeSegmentID];
		int32 StartVid = GroupEdge.Span.Vertices[EdgeSegmentID];
		int32 EndVid = GroupEdge.Span.Vertices[EdgeSegmentID + 1];
		FVector3d StartVert = ActivityContext->CurrentMesh->GetVertex(StartVid);
		FVector3d EndVert = ActivityContext->CurrentMesh->GetVertex(EndVid);
		FVector3d EdgeVector = EndVert - StartVert;
		double EdgeLength = EdgeVector.Length();
		check(EdgeLength > 0);

		PointOut.Tangent = EdgeVector / EdgeLength;

		FRay EdgeRay((FVector)StartVert, (FVector)PointOut.Tangent, true);
		float DistDownEdge = EdgeRay.GetParameter((FVector)Position);

		// See if the point is at a vertex in the group edge span.
		if (DistDownEdge <= Settings->VertexTolerance)
		{
			PointOut.bIsVertex = true;
			PointOut.ElementID = StartVid;
			PositionOut = StartVert;
			if (EdgeSegmentID > 0)
			{
				// Average with previous normalized edge vector
				PointOut.Tangent += 
					UE::Geometry::Normalized(StartVert - ActivityContext->CurrentMesh->GetVertex(GroupEdge.Span.Vertices[EdgeSegmentID - 1]));
				UE::Geometry::Normalize(PointOut.Tangent);
			}
		}
		else if (abs(DistDownEdge - EdgeLength) <= Settings->VertexTolerance)
		{
			PointOut.bIsVertex = true;
			PointOut.ElementID = EndVid;
			PositionOut = EndVert;
			if (EdgeSegmentID + 2 < GroupEdge.Span.Vertices.Num())
			{
				PointOut.Tangent += UE::Geometry::Normalized(
					ActivityContext->CurrentMesh->GetVertex(GroupEdge.Span.Vertices[EdgeSegmentID + 2]) - EndVert);
				UE::Geometry::Normalize(PointOut.Tangent);
			}
		}
		else
		{
			PointOut.bIsVertex = false;
			PointOut.ElementID = Eid;
			PointOut.EdgeTValue = DistDownEdge / EdgeLength;
			PositionOut = (FVector3d)EdgeRay.PointAt(DistDownEdge);
			if (ActivityContext->CurrentMesh->GetEdgeV(Eid).A != StartVid)
			{
				PointOut.EdgeTValue = 1 - PointOut.EdgeTValue;
			}
		}
	}
	return true;
}

void UPolyEditInsertEdgeActivity::GetCornerTangent(int32 CornerID, int32 GroupID, int32 BoundaryIndex, FVector3d& TangentOut)
{
	TangentOut = FVector3d::Zero();

	int32 CornerVid = ActivityContext->CurrentTopology->GetCornerVertexID(CornerID);
	check(CornerVid != FDynamicMesh3::InvalidID);

	const FGroupTopology::FGroup* Group = ActivityContext->CurrentTopology->FindGroupByID(GroupID);
	check(Group && BoundaryIndex >= 0 && BoundaryIndex < Group->Boundaries.Num());

	const FGroupTopology::FGroupBoundary& Boundary = Group->Boundaries[BoundaryIndex];
	TArray<FVector3d> AdjacentPoints;
	for (int32 GroupEdgeID : Boundary.GroupEdges)
	{
		TArray<int32> Vertices = ActivityContext->CurrentTopology->Edges[GroupEdgeID].Span.Vertices;
		if (Vertices[0] == CornerVid)
		{
			AdjacentPoints.Add(ActivityContext->CurrentMesh->GetVertex(Vertices[1]));
		}
		else if (Vertices.Last() == CornerVid)
		{
			AdjacentPoints.Add(ActivityContext->CurrentMesh->GetVertex(Vertices[Vertices.Num()-2]));
		}
	}
	check(AdjacentPoints.Num() == 2);

	FVector3d CornerPosition = ActivityContext->CurrentMesh->GetVertex(CornerVid);
	TangentOut = UE::Geometry::Normalized(CornerPosition - AdjacentPoints[0]);
	TangentOut += UE::Geometry::Normalized(AdjacentPoints[1] - CornerPosition);
	UE::Geometry::Normalize(TangentOut);
}

bool PolyEditInsertEdgeActivityLocals::GetSharedBoundary(const FGroupTopology& Topology,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& StartPoint, int32 StartTopologyID, bool bStartIsCorner,
	const FGroupEdgeInserter::FGroupEdgeSplitPoint& EndPoint, int32 EndTopologyID, bool bEndIsCorner,
	int32& GroupIDOut, int32& BoundaryIndexOut)
{
	// The start and endpoints could be on the same boundary of multiple groups at
	// the same time, and sometimes we won't be able to resolve the ambiguity
	// (one example is a sphere split into two equal groups, but could even happen
	// with more than two groups when endpoints are corners).
	// Sometimes there are things we can do to eliminate some contenders- the best
	// approach is probably trying to do a plane cut for all of the options and 
	// removing those that fail. However, it's worth noting that such issues won't
	// arise in the standard application of this tool for low-poly modeling, where
	// groups are planar, so it's not worth the bother.
	// Instead, we'll just take one of the results arbitrarily, though we will try to
	// take one that has a single boundary (this will prefer a cylinder cap over
	// a cylinder side).
	// TODO: The code would be simpler if we didn't even want to do that filtering- we'd
	// just return the first result we found. Should we consider doing that?

	GroupIDOut = FDynamicMesh3::InvalidID;
	BoundaryIndexOut = FDynamicMesh3::InvalidID;

	TArray<TPair<int32, int32>> CandidateGroupIDsAndBoundaryIndices;
	if (bStartIsCorner)
	{
		// Go through all neighboring groups and their boundaries to find a shared one.
		const FGroupTopology::FCorner& StartCorner = Topology.Corners[StartTopologyID];
		for (int32 GroupID : StartCorner.NeighbourGroupIDs)
		{
			const FGroupTopology::FGroup* Group = Topology.FindGroupByID(GroupID);
			for (int32 i = 0; i < Group->Boundaries.Num(); ++i)
			{
				const FGroupTopology::FGroupBoundary& Boundary = Group->Boundaries[i];
				if (DoesBoundaryContainPoint(Topology, Boundary, EndTopologyID, bEndIsCorner)
					&& DoesBoundaryContainPoint(Topology, Boundary, StartTopologyID, bStartIsCorner))
				{
					CandidateGroupIDsAndBoundaryIndices.Emplace(GroupID, i);
					break; // Can't share more than one boundary in the same group
				}
			}
		}
	}
	else
	{
		// Start is on an edge, so there are fewer boundaries to look through.
		const FGroupTopology::FGroupEdge& GroupEdge = Topology.Edges[StartTopologyID];
		const FGroupTopology::FGroup* Group = Topology.FindGroupByID(GroupEdge.Groups.A);
		for (int32 i = 0; i < Group->Boundaries.Num(); ++i)
		{
			const FGroupTopology::FGroupBoundary& Boundary = Group->Boundaries[i];

			if (DoesBoundaryContainPoint(Topology, Boundary, EndTopologyID, bEndIsCorner)
				&& DoesBoundaryContainPoint(Topology, Boundary, StartTopologyID, bStartIsCorner))
			{
				CandidateGroupIDsAndBoundaryIndices.Emplace(GroupEdge.Groups.A, i);
				break;
			}
		}
		if (GroupEdge.Groups.B != FDynamicMesh3::InvalidID)
		{
			Group = Topology.FindGroupByID(GroupEdge.Groups.B);
			for (int32 i = 0; i < Group->Boundaries.Num(); ++i)
			{
				const FGroupTopology::FGroupBoundary& Boundary = Group->Boundaries[i];

				if (DoesBoundaryContainPoint(Topology, Boundary, EndTopologyID, bEndIsCorner)
					&& DoesBoundaryContainPoint(Topology, Boundary, StartTopologyID, bStartIsCorner))
				{
					CandidateGroupIDsAndBoundaryIndices.Emplace(GroupEdge.Groups.B, i);
					break;
				}
			}
		}
	}

	if (CandidateGroupIDsAndBoundaryIndices.Num() == 0)
	{
		return false;
	}

	// Prefer a result that has a single boundary if there are multiple.
	if (CandidateGroupIDsAndBoundaryIndices.Num() > 1)
	{
		for (const TPair<int32, int32>& GroupIDBoundaryIdxPair : CandidateGroupIDsAndBoundaryIndices)
		{
			if (Topology.FindGroupByID(GroupIDBoundaryIdxPair.Key)->Boundaries.Num() == 1)
			{
				GroupIDOut = GroupIDBoundaryIdxPair.Key;
				BoundaryIndexOut = 0;
				return true;
			}
		}
	}

	GroupIDOut = CandidateGroupIDsAndBoundaryIndices[0].Key;
	BoundaryIndexOut = CandidateGroupIDsAndBoundaryIndices[0].Value;
	return true;
}

bool PolyEditInsertEdgeActivityLocals::DoesBoundaryContainPoint(const FGroupTopology& Topology,
	const FGroupTopology::FGroupBoundary& Boundary, int32 PointTopologyID, bool bPointIsCorner)
{
	for (int32 GroupEdgeID : Boundary.GroupEdges)
	{
		if (!bPointIsCorner && GroupEdgeID == PointTopologyID)
		{
			return true;
		}

		const FGroupTopology::FGroupEdge& GroupEdge = Topology.Edges[GroupEdgeID];
		if (bPointIsCorner && (GroupEdge.EndpointCorners.A == PointTopologyID
			|| GroupEdge.EndpointCorners.B == PointTopologyID))
		{
			return true;
		}
	}
	return false;
}

void FGroupEdgeInsertionFirstPointChange::Revert(UObject* Object)
{
	UPolyEditInsertEdgeActivity* Activity = Cast<UPolyEditInsertEdgeActivity>(Object);

	check(Activity->ToolState == UPolyEditInsertEdgeActivity::EState::GettingEnd
		|| Activity->ToolState == UPolyEditInsertEdgeActivity::EState::WaitingForInsertComplete);
	Activity->ToolState = UPolyEditInsertEdgeActivity::EState::GettingStart;

	Activity->ClearPreview(true);

	bHaveDoneUndo = true;
}

#undef LOCTEXT_NAMESPACE
