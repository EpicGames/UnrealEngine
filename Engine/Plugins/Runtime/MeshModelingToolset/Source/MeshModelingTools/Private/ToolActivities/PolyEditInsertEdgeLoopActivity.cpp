// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditInsertEdgeLoopActivity.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "CuttingOps/EdgeLoopInsertionOp.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h"
#include "Operations/GroupEdgeInserter.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditInsertEdgeLoopActivity)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPolyEditInsertEdgeLoopActivity"

TUniquePtr<FDynamicMeshOperator> UPolyEditInsertEdgeLoopActivity::MakeNewOperator()
{
	TUniquePtr<FEdgeLoopInsertionOp> Op = MakeUnique<FEdgeLoopInsertionOp>();

	Op->OriginalMesh = ComputeStartMesh;
	Op->OriginalTopology = ComputeStartTopology;
	Op->SetTransform(TargetTransform);

	if (Settings->InsertionMode == EEdgeLoopInsertionMode::PlaneCut)
	{
		Op->Mode = FGroupEdgeInserter::EInsertionMode::PlaneCut;
	}
	else
	{
		Op->Mode = FGroupEdgeInserter::EInsertionMode::Retriangulate;
	}

	Op->VertexTolerance = Settings->VertexTolerance;

	if (!ensure(InputGroupEdgeID >= 0))
	{
		return Op;
	}
	Op->GroupEdgeID = InputGroupEdgeID;

	Op->StartCornerID = Settings->bFlipOffsetDirection ?
		Op->OriginalTopology->Edges[Op->GroupEdgeID].EndpointCorners.B
		: Op->OriginalTopology->Edges[Op->GroupEdgeID].EndpointCorners.A;

	// Set up the inputs
	if (Settings->PositionMode == EEdgeLoopPositioningMode::Even)
	{
		int32 NumLoops = Settings->NumLoops;
		for (int32 i = 0; i < NumLoops; ++i)
		{
			Op->InputLengths.Add((i + 1.0) / (NumLoops + 1.0));
		}
	}
	else if (Settings->bInteractive)
	{
		Op->InputLengths.Add(InteractiveInputLength);
	}
	else if (Settings->PositionMode == EEdgeLoopPositioningMode::ProportionOffset)
	{
		Op->InputLengths.Add(Settings->ProportionOffset);
	}
	else
	{
		Op->InputLengths.Add(Settings->DistanceOffset);
	}

	Op->bInputsAreProportions = (Settings->PositionMode == EEdgeLoopPositioningMode::Even
		|| Settings->PositionMode == EEdgeLoopPositioningMode::ProportionOffset);

	return Op;
}

void UPolyEditInsertEdgeLoopActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	// Set up properties
	Settings = NewObject<UEdgeLoopInsertionProperties>(this);
	Settings->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(Settings);
	SetToolPropertySourceEnabled(Settings, false);
	Settings->GetOnModified().AddUObject(this, &UPolyEditInsertEdgeLoopActivity::OnPropertyModified);

	// Draws the new group edges that are added
	PreviewEdgeRenderer.LineColor = FLinearColor::Green;
	PreviewEdgeRenderer.LineThickness = 2.0;
	PreviewEdgeRenderer.bDepthTested = false;

	// Highlights non-quad groups that stop the loop;
	ProblemTopologyRenderer.LineColor = FLinearColor::Red;
	ProblemTopologyRenderer.LineThickness = 3.0;
	ProblemTopologyRenderer.DepthBias = 1.0;
	ProblemTopologyRenderer.bDepthTested = false;

	TopologySelectorSettings.bEnableEdgeHits = true;
	TopologySelectorSettings.bEnableFaceHits = false;
	TopologySelectorSettings.bEnableCornerHits = false;

	// Set up our input routing
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickBehavior, this);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior, this);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();

	// TODO: When the deprecated function GetTopologySelector is removed from UPolygonSelectionMechanic, we can remove the UMeshTopologySelectionMechanic:: prefix here
	TopologySelector = ActivityContext->SelectionMechanic->UMeshTopologySelectionMechanic::GetTopologySelector();

	ActivityContext->OnUndoRedo.AddWeakLambda(this, [this](bool bGroupTopologyModified)
	{
		UpdateComputeInputs();
		ClearPreview();
	});
}

void UPolyEditInsertEdgeLoopActivity::UpdateComputeInputs()
{
	ComputeStartMesh = MakeShared<FDynamicMesh3>(*ActivityContext->CurrentMesh);

	TSharedPtr<FGroupTopology> NonConstTopology = MakeShared<FGroupTopology>(*ActivityContext->CurrentTopology);
	NonConstTopology->RetargetOnClonedMesh(ComputeStartMesh.Get());

	ComputeStartTopology = NonConstTopology;
}

void UPolyEditInsertEdgeLoopActivity::Shutdown(EToolShutdownType ShutdownType)
{
	if (bIsRunning)
	{
		End(ShutdownType);
	}

	Settings->SaveProperties(ParentTool.Get());
	Settings->GetOnModified().Clear();

	TopologySelector.Reset();
	Settings = nullptr;

	ActivityContext->OnUndoRedo.RemoveAll(this);
	ActivityContext = nullptr;

	Super::Shutdown(ShutdownType);
}

EToolActivityStartResult UPolyEditInsertEdgeLoopActivity::Start()
{
	ParentTool->GetToolManager()->DisplayMessage(
		LOCTEXT("InsertEdgeLoopActivityDescription", "Click an edge to insert an edge loop passing across "
			"that edge. Edge loops follow a sequence of quad-like polygroups."),
		EToolMessageLevel::UserNotification);

	SetToolPropertySourceEnabled(Settings, true);

	// We don't use selection, so clear it if necessary (have to issue an undo/redo event)
	if (!ActivityContext->SelectionMechanic->GetActiveSelection().IsEmpty())
	{
		ActivityContext->SelectionMechanic->BeginChange();
		ActivityContext->SelectionMechanic->ClearSelection();
		ParentTool->GetToolManager()->EmitObjectChange(ActivityContext->SelectionMechanic,
			ActivityContext->SelectionMechanic->EndChange(), LOCTEXT("ClearSelection", "Clear Selection"));
	}

	TargetTransform = ActivityContext->Preview->PreviewMesh->GetTransform();

	UpdateComputeInputs();
	SetupPreview();

	bWaitingForInsertionCompletion = false;
	bLastComputeSucceeded = false;
	InputGroupEdgeID = FDynamicMesh3::InvalidID;

	bIsRunning = true;

	// Emit activity start transaction
	ActivityContext->EmitActivityStart(LOCTEXT("BeginInsertEdgeLoopActivity", "Begin Insert Edge Loop"));

	return EToolActivityStartResult::Running;
}

EToolActivityEndResult UPolyEditInsertEdgeLoopActivity::End(EToolShutdownType ShutdownType)
{
	if (!bIsRunning)
	{
		return EToolActivityEndResult::ErrorDuringEnd;
	}

	SetToolPropertySourceEnabled(Settings, false);
	Settings->GetOnModified().Clear();

	ActivityContext->Preview->OnOpCompleted.RemoveAll(this);
	ActivityContext->Preview->OnMeshUpdated.RemoveAll(this);
	ClearPreview();
	ActivityContext->Preview->ClearOpFactory();

	bIsRunning = false;

	return CanAccept() ? EToolActivityEndResult::Completed : EToolActivityEndResult::Cancelled;
}

void UPolyEditInsertEdgeLoopActivity::SetupPreview()
{
	ActivityContext->Preview->ChangeOpFactory(this);

	// Whenever we get a new result from the op, we need to extract the preview edges so that
	// we can draw them if we want to, and the additional outputs we need (changed triangles and
	// topology).
	ActivityContext->Preview->OnOpCompleted.AddWeakLambda(this, [this](const FDynamicMeshOperator* UncastOp) {
		const FEdgeLoopInsertionOp* Op = static_cast<const FEdgeLoopInsertionOp*>(UncastOp);

		LatestOpTopologyResult.Reset();
		PreviewEdges.Reset();
		LatestOpChangedTids.Reset();

		// See if this compute is actually outdated, i.e. we changed the mesh
		// out from under it.
		if (Op->OriginalMesh != ComputeStartMesh)
		{
			bLastComputeSucceeded = false;
			return;
		}

		bLastComputeSucceeded = Op->bSucceeded;

		if (bLastComputeSucceeded)
		{
			Op->GetLoopEdgeLocations(PreviewEdges);
			LatestOpTopologyResult = Op->ResultTopology;
			LatestOpChangedTids = Op->ChangedTids;
		}

		// Regardless of success, extract things for highlighting any non-quads that stopped our loop.
		ProblemTopologyEdges.Reset();
		ProblemTopologyVerts.Reset();
		for (int32 GroupEdgeID : Op->ProblemGroupEdgeIDs)
		{
			for (int32 Eid : ActivityContext->CurrentTopology->GetGroupEdgeEdges(GroupEdgeID))
			{
				TPair<FVector3d, FVector3d> Endpoints;
				ActivityContext->CurrentMesh->GetEdgeV(Eid, Endpoints.Key, Endpoints.Value);
				ProblemTopologyEdges.Add(MoveTemp(Endpoints));
			}
			FGroupTopology::FGroupEdge& GroupEdge = ActivityContext->CurrentTopology->Edges[GroupEdgeID];
			if (GroupEdge.EndpointCorners.A != FDynamicMesh3::InvalidID)
			{
				ProblemTopologyVerts.AddUnique(ActivityContext->CurrentMesh->GetVertex(
					ActivityContext->CurrentTopology->Corners[GroupEdge.EndpointCorners.A].VertexID));
				ProblemTopologyVerts.AddUnique(ActivityContext->CurrentMesh->GetVertex(
					ActivityContext->CurrentTopology->Corners[GroupEdge.EndpointCorners.B].VertexID));
			}
		}
		});

	// In case of failure, we want to hide the broken preview, since we wouldn't accept it on
	// a click. Note that this can't be fired OnOpCompleted because the preview is updated
	// with the op result after that callback, which would undo the reset. The preview edge
	// extraction can't be lumped in here because it needs the op rather than the preview object.
	ActivityContext->Preview->OnMeshUpdated.AddWeakLambda(this, [this](UMeshOpPreviewWithBackgroundCompute*) {
		if (!bLastComputeSucceeded)
		{
			ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());
		}
		});
}

bool UPolyEditInsertEdgeLoopActivity::CanStart() const
{
	return true;
}

void UPolyEditInsertEdgeLoopActivity::Tick(float DeltaTime)
{
	// See if we clicked and are waiting for the result to be computed and put in.
	if (bWaitingForInsertionCompletion && ActivityContext->Preview->HaveValidResult())
	{
		if (bLastComputeSucceeded)
		{
			FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
			ChangeTracker.BeginChange();
			ChangeTracker.SaveTriangles(*LatestOpChangedTids, true /*bSaveVertices*/);

			// Update current mesh
			ActivityContext->CurrentMesh->Copy(*ActivityContext->Preview->PreviewMesh->GetMesh(), 
				true, true, true, true);

			*ActivityContext->CurrentTopology = *LatestOpTopologyResult;
			ActivityContext->CurrentTopology->RetargetOnClonedMesh(ActivityContext->CurrentMesh.Get());

			UpdateComputeInputs();

			// Emit transaction
			FGroupTopologySelection EmptySelection;
			ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("EdgeLoopInsertionTransactionName", "Edge Loop Insertion"),
				ChangeTracker.EndChange(), EmptySelection);
		}

		PreviewEdges.Reset();
		ProblemTopologyEdges.Reset();
		ProblemTopologyVerts.Reset();

		bWaitingForInsertionCompletion = false;
	}
}

void UPolyEditInsertEdgeLoopActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	ParentTool->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	// Draw the preview edges
	PreviewEdgeRenderer.BeginFrame(RenderAPI, CameraState);
	PreviewEdgeRenderer.SetTransform(TargetTransform);
	for (TPair<FVector3d, FVector3d>& EdgeVerts : PreviewEdges)
	{
		PreviewEdgeRenderer.DrawLine(EdgeVerts.Key, EdgeVerts.Value);
	}
	PreviewEdgeRenderer.EndFrame();

	if (Settings->bHighlightProblemGroups)
	{
		// Highlight any non-quad groups that stopped the loop.
		ProblemTopologyRenderer.BeginFrame(RenderAPI, CameraState);
		ProblemTopologyRenderer.SetTransform(TargetTransform);
		for (TPair<FVector3d, FVector3d>& EdgeVerts : ProblemTopologyEdges)
		{
			ProblemTopologyRenderer.DrawLine(EdgeVerts.Key, EdgeVerts.Value);
		}
		for (FVector3d& Vert : ProblemTopologyVerts)
		{
			ProblemTopologyRenderer.DrawViewFacingX(Vert, ProblemVertTickWidth);
		}
		ProblemTopologyRenderer.EndFrame();
	}
}

bool UPolyEditInsertEdgeLoopActivity::CanAccept() const
{
	return !bWaitingForInsertionCompletion;
}

void UPolyEditInsertEdgeLoopActivity::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	ClearPreview();
}

FInputRayHit UPolyEditInsertEdgeLoopActivity::HitTest(const FRay& WorldRay)
{
	FInputRayHit Hit;

	// See if we hit an edge
	FRay3d LocalRay((FVector3d)TargetTransform.InverseTransformPosition(WorldRay.Origin),
		(FVector3d)TargetTransform.InverseTransformVector(WorldRay.Direction), false);
	FGroupTopologySelection Selection;
	FVector3d Position, Normal;
	TopologySelectorSettings.bHitBackFaces = ActivityContext->SelectionMechanic->Properties->bHitBackFaces;
	if (TopologySelector->FindSelectedElement(
		TopologySelectorSettings, LocalRay, Selection, Position, Normal))
	{
		// TODO: We could check here that the edge has some quad-like neighbor. For now we
		// just check that the edge isn't a loop unto itself (in which case the neighbor groups
		// are definitely not quad-like).
		int32 GroupEdgeID = Selection.GetASelectedEdgeID();
		const FGroupTopology::FGroupEdge& GroupEdge = ActivityContext->CurrentTopology->Edges[GroupEdgeID];
		if (GroupEdge.EndpointCorners.A != FDynamicMesh3::InvalidID)
		{
			Hit = FInputRayHit(LocalRay.GetParameter(Position));
		}
	}

	return Hit;
}

bool UPolyEditInsertEdgeLoopActivity::UpdateHoveredItem(const FRay& WorldRay)
{
	// Check that we hit an edge
	FRay3d LocalRay((FVector3d)TargetTransform.InverseTransformPosition(WorldRay.Origin),
		(FVector3d)TargetTransform.InverseTransformVector(WorldRay.Direction), false);

	FGroupTopologySelection Selection;
	FVector3d Position, Normal;
	int32 EdgeSegmentID;
	TopologySelectorSettings.bHitBackFaces = ActivityContext->SelectionMechanic->Properties->bHitBackFaces;
	if (!TopologySelector->FindSelectedElement(
		TopologySelectorSettings, LocalRay, Selection, Position, Normal, &EdgeSegmentID))
	{
		ClearPreview();
		return false; // Didn't hit anything
	}

	// Check that the edge has endpoints
	int32 GroupEdgeID = Selection.GetASelectedEdgeID();
	FGroupTopology::FGroupEdge GroupEdge = ActivityContext->CurrentTopology->Edges[GroupEdgeID];
	if (GroupEdge.EndpointCorners.A == FDynamicMesh3::InvalidID)
	{
		ClearPreview();
		return false; // Edge definitely doesn't have quad-like neighbors
	}

	if (Settings->PositionMode == EEdgeLoopPositioningMode::Even)
	{
		// In even mode and non-interactive mode, all that matters is the group edge 
		// that we're hovering, not where our pointer is exactly.
		ConditionallyUpdatePreview(GroupEdgeID);
		return true;
	}
	if (!Settings->bInteractive)
	{
		// Don't try to insert a loop when our inputs don't make sense.
		double TotalLength = ActivityContext->CurrentTopology->GetEdgeArcLength(GroupEdgeID);
		if (Settings->PositionMode == EEdgeLoopPositioningMode::DistanceOffset)
		{
			if (Settings->DistanceOffset > TotalLength || Settings->DistanceOffset <= Settings->VertexTolerance)
			{
				ClearPreview();
				return false;
			}
		}
		else if (Settings->PositionMode == EEdgeLoopPositioningMode::ProportionOffset)
		{
			if (abs(Settings->ProportionOffset * TotalLength - TotalLength) <= Settings->VertexTolerance)
			{
				ClearPreview();
				return false;
			}
		}

		ConditionallyUpdatePreview(GroupEdgeID);
		return true;
	}

	// Otherwise, we need to figure out where along the edge we are hovering.
	double NewInputLength = 0;
	int32 StartVid = GroupEdge.Span.Vertices[EdgeSegmentID];
	int32 EndVid = GroupEdge.Span.Vertices[EdgeSegmentID + 1];
	FVector3d StartVert = ActivityContext->CurrentMesh->GetVertex(StartVid);
	FVector3d EndVert = ActivityContext->CurrentMesh->GetVertex(EndVid);

	FRay EdgeRay((FVector)StartVert, (FVector)(EndVert - StartVert), false);

	double DistDownEdge = EdgeRay.GetParameter((FVector)Position);

	TArray<double> PerVertexLengths;
	double TotalLength = ActivityContext->CurrentTopology->GetEdgeArcLength(GroupEdgeID, &PerVertexLengths);

	NewInputLength = PerVertexLengths[EdgeSegmentID] + DistDownEdge;
	if (Settings->bFlipOffsetDirection)
	{
		// If we flipped start corner, we should be measuring from the opposite direction
		NewInputLength = TotalLength - NewInputLength;
	}
	// We avoid trying to insert loops that are guaranteed to follow existing group edges.
	// Distance offset with total length may work if the group widens on the other side.
	// Though it's worth noting that this filter as a whole is assuming straight group edges...
	if (NewInputLength <= Settings->VertexTolerance ||
		(Settings->PositionMode == EEdgeLoopPositioningMode::ProportionOffset
			&& abs(NewInputLength - TotalLength) <= Settings->VertexTolerance))
	{
		ClearPreview();
		return false;
	}
	if (Settings->PositionMode == EEdgeLoopPositioningMode::ProportionOffset)
	{
		NewInputLength /= TotalLength;
	}
	ConditionallyUpdatePreview(GroupEdgeID, &NewInputLength);
	return true;
}

void UPolyEditInsertEdgeLoopActivity::ClearPreview()
{
	ActivityContext->Preview->CancelCompute();
	ActivityContext->Preview->PreviewMesh->UpdatePreview(ActivityContext->CurrentMesh.Get());
	InputGroupEdgeID = FDynamicMesh3::InvalidID;
	PreviewEdges.Reset();
	ProblemTopologyEdges.Reset();
	ProblemTopologyVerts.Reset();
}

/** 
 * Update the preview unless we've already computed one with the same parameters (such as when using "even" or non-interactive
 * parameter setting) 
 */
void UPolyEditInsertEdgeLoopActivity::ConditionallyUpdatePreview(int32 NewGroupID, double* NewInteractiveInputLength)
{
	if (InputGroupEdgeID != NewGroupID
		|| (NewInteractiveInputLength && Settings->PositionMode != EEdgeLoopPositioningMode::Even
			&& *NewInteractiveInputLength != InteractiveInputLength))
	{
		InputGroupEdgeID = NewGroupID;
		if (NewInteractiveInputLength)
		{
			InteractiveInputLength = *NewInteractiveInputLength;
		}
		PreviewEdges.Reset();
		ActivityContext->Preview->InvalidateResult();
	}
}

FInputRayHit UPolyEditInsertEdgeLoopActivity::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	// Note that the check for bIsRunning is important here and other input handlers since the
	// hover/click behaviors are always in the behavior list while the tool is running (we don't
	// currently have a way to add/remove at will).
	if (bWaitingForInsertionCompletion || !bIsRunning)
	{
		return FInputRayHit();
	}

	return HitTest(PressPos.WorldRay);
}

bool UPolyEditInsertEdgeLoopActivity::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (bWaitingForInsertionCompletion || !bIsRunning)
	{
		return false;
	}

	return UpdateHoveredItem(DevicePos.WorldRay);
}

void UPolyEditInsertEdgeLoopActivity::OnEndHover()
{
	// Note that OnEndHover actually gets called before MouseDown, and therefore before OnClicked
	// (which gets called on MouseUp). Currently, this means that we don't actually end up using
	// an existing hover computation for our click even if MouseUp happened in the same place- we just
	// always redo the hover in OnClicked and immediately say that we want to accept it.
	// 
	// If we ever want to change things to do all this a bit more properly, we would need to make a
	// couple changes: 1. We need to use a ClickDrag behavior instead of SingleClick. 2. OnEndHover
	// should set a boolean to request a clear to happen in Tick(). 3. In the click behavior, we
	// cancel any requested clear on mouse down, and keep updating hover during drag. On mouse up
	// we accept whatever hover we have.

	if (bIsRunning && !bWaitingForInsertionCompletion)
	{
		ClearPreview();
	}
}

FInputRayHit UPolyEditInsertEdgeLoopActivity::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Hit;
	if (bWaitingForInsertionCompletion || !bIsRunning)
	{
		return Hit;
	}

	return HitTest(ClickPos.WorldRay);
}

void UPolyEditInsertEdgeLoopActivity::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (bWaitingForInsertionCompletion)
	{
		return;
	}

	if (UpdateHoveredItem(ClickPos.WorldRay))
	{
		bWaitingForInsertionCompletion = true;
	}

}

#undef LOCTEXT_NAMESPACE
