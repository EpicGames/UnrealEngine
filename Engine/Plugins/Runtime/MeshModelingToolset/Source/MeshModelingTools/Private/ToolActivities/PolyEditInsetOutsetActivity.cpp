// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditInsetOutsetActivity.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "Drawing/PolyEditPreviewMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "EditMeshPolygonsTool.h"
#include "InteractiveToolManager.h"
#include "Mechanics/SpatialCurveDistanceMechanic.h"
#include "MeshBoundaryLoops.h"
#include "MeshOpPreviewHelpers.h"
#include "Operations/InsetMeshRegion.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolActivities/PolyEditActivityUtil.h"
#include "ToolSceneQueriesUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditInsetOutsetActivity)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPolyEditInsetOutsetActivity"

void UPolyEditInsetOutsetActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	Settings = NewObject<UPolyEditInsetOutsetProperties>();
	Settings->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(Settings);
	SetToolPropertySourceEnabled(Settings, false);

	// Register ourselves to receive clicks and hover
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();
}

void UPolyEditInsetOutsetActivity::Shutdown(EToolShutdownType ShutdownType)
{
	Clear();
	Settings->SaveProperties(ParentTool.Get());

	Settings = nullptr;
	ParentTool = nullptr;
	ActivityContext = nullptr;
}

bool UPolyEditInsetOutsetActivity::CanStart() const
{
	if (!ActivityContext)
	{
		return false;
	}
	const FGroupTopologySelection& Selection = ActivityContext->SelectionMechanic->GetActiveSelection();
	return !Selection.SelectedGroupIDs.IsEmpty();
}

EToolActivityStartResult UPolyEditInsetOutsetActivity::Start()
{
	if (!CanStart())
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("InsetOutsetNoSelectionMesssage", "Cannot inset or outset without face selection."),
			EToolMessageLevel::UserWarning);
		return EToolActivityStartResult::FailedStart;
	}

	Clear();
	if (!BeginInset())
	{
		return EToolActivityStartResult::FailedStart;
	}
	bIsRunning = true;

	ActivityContext->EmitActivityStart(LOCTEXT("BeginInsetOutsetActivity", "Begin Inset/Outset"));

	return EToolActivityStartResult::Running;
}

bool UPolyEditInsetOutsetActivity::CanAccept() const
{
	return true;
}

EToolActivityEndResult UPolyEditInsetOutsetActivity::End(EToolShutdownType ShutdownType)
{
	if (!bIsRunning)
	{
		Clear();
		return EToolActivityEndResult::ErrorDuringEnd;
	}

	if (ShutdownType == EToolShutdownType::Cancel)
	{
		Clear();
		bIsRunning = false;
		return EToolActivityEndResult::Cancelled;
	}
	else
	{
		ApplyInset();
		Clear();
		bIsRunning = false;
		return EToolActivityEndResult::Completed;
	}
}

bool UPolyEditInsetOutsetActivity::BeginInset()
{
	const FGroupTopologySelection& ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	TArray<int32> ActiveTriangleSelection;
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	FTransform3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());

	EditPreview = PolyEditActivityUtil::CreatePolyEditPreviewMesh(*ParentTool, *ActivityContext);
	FTransform3d WorldTranslation, WorldRotateScale;
	EditPreview->ApplyTranslationToPreview(WorldTransform, WorldTranslation, WorldRotateScale);
	EditPreview->InitializeInsetType(ActivityContext->CurrentMesh.Get(), ActiveTriangleSelection, &WorldRotateScale);

	// Hide the selected triangles (that are being replaced by the inset/outset portion)
	ActivityContext->Preview->PreviewMesh->SetSecondaryBuffersVisibility(false);

	// make infinite-extent hit-test mesh
	FDynamicMesh3 InsetHitTargetMesh;
	EditPreview->MakeInsetTypeTargetMesh(InsetHitTargetMesh);

	CurveDistMechanic = NewObject<USpatialCurveDistanceMechanic>(this);
	CurveDistMechanic->Setup(ParentTool.Get());
	CurveDistMechanic->WorldPointSnapFunc = [this](const FVector3d& WorldPos, FVector3d& SnapPos)
	{
		return ToolSceneQueriesUtil::FindWorldGridSnapPoint(ParentTool.Get(), WorldPos, SnapPos);
	};
	CurveDistMechanic->CurrentDistance = 1.0f;  // initialize to something non-zero...prob should be based on polygon bounds maybe?

	FMeshBoundaryLoops Loops(&InsetHitTargetMesh);
	if (Loops.Loops.Num() == 0)
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("InsetOutsetNoBorderMesssage", "Cannot inset or outset when selection has no border."),
			EToolMessageLevel::UserWarning);
		return false;
	}
	TArray<FVector3d> LoopVertices;
	Loops.Loops[0].GetVertices(LoopVertices);
	CurveDistMechanic->InitializePolyLoop(LoopVertices, FTransformSRT3d::Identity());

	SetToolPropertySourceEnabled(Settings, true);

	float BoundsMaxDim = ActivityContext->CurrentMesh->GetBounds().MaxDim();
	if (BoundsMaxDim > 0)
	{
		UVScaleFactor = 1.0 / BoundsMaxDim;
	}

	bPreviewUpdatePending = true;

	return true;
}

void UPolyEditInsetOutsetActivity::ApplyInset()
{
	check(CurveDistMechanic != nullptr && EditPreview != nullptr);

	const FGroupTopologySelection& ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	TArray<int32> ActiveTriangleSelection;
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	FInsetMeshRegion Inset(ActivityContext->CurrentMesh.Get());
	Inset.UVScaleFactor = UVScaleFactor;
	Inset.Triangles = ActiveTriangleSelection;
	Inset.InsetDistance = (Settings->bOutset) ? -CurveDistMechanic->CurrentDistance
		: CurveDistMechanic->CurrentDistance;
	Inset.bReproject = (Settings->bOutset) ? false : Settings->bReproject;
	Inset.Softness = Settings->Softness;
	Inset.bSolveRegionInteriors = !Settings->bBoundaryOnly;
	Inset.AreaCorrection = Settings->AreaScale;

	Inset.ChangeTracker = MakeUnique<FDynamicMeshChangeTracker>(ActivityContext->CurrentMesh.Get());
	Inset.ChangeTracker->BeginChange();
	Inset.Apply();

	FMeshNormals::QuickComputeVertexNormalsForTriangles(*ActivityContext->CurrentMesh, Inset.AllModifiedTriangles);

	// Emit undo (also updates relevant structures)
	ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshInsetOutsetChange", "Inset/Outset"),
		Inset.ChangeTracker->EndChange(), ActiveSelection);
}

void UPolyEditInsetOutsetActivity::Clear()
{
	if (EditPreview != nullptr)
	{
		EditPreview->Disconnect();
		EditPreview = nullptr;
	}

	ActivityContext->Preview->PreviewMesh->SetSecondaryBuffersVisibility(true);

	CurveDistMechanic = nullptr;
	SetToolPropertySourceEnabled(Settings, false);
}

void UPolyEditInsetOutsetActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (CurveDistMechanic != nullptr)
	{
		CurveDistMechanic->Render(RenderAPI);
	}
}

void UPolyEditInsetOutsetActivity::Tick(float DeltaTime)
{
	if (EditPreview && bPreviewUpdatePending)
	{
		double Sign = Settings->bOutset ? -1.0 : 1.0;
		bool bReproject = (Settings->bOutset) ? false : Settings->bReproject;
		double Softness = Settings->Softness;
		bool bBoundaryOnly = Settings->bBoundaryOnly;
		double AreaCorrection = Settings->AreaScale;
		EditPreview->UpdateInsetType(Sign * CurveDistMechanic->CurrentDistance,
			bReproject, Softness, AreaCorrection, bBoundaryOnly);

		bPreviewUpdatePending = false;
	}
}

FInputRayHit UPolyEditInsetOutsetActivity::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning;
	return OutHit;
}

void UPolyEditInsetOutsetActivity::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (bIsRunning)
	{
		ApplyInset();

		// End activity
		Clear();
		bIsRunning = false;
		Cast<IToolActivityHost>(ParentTool)->NotifyActivitySelfEnded(this);
	}
}

FInputRayHit UPolyEditInsetOutsetActivity::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning;
	return OutHit;
}

bool UPolyEditInsetOutsetActivity::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	CurveDistMechanic->UpdateCurrentDistance(DevicePos.WorldRay);
	bPreviewUpdatePending = true;
	return bIsRunning;
}

#undef LOCTEXT_NAMESPACE

