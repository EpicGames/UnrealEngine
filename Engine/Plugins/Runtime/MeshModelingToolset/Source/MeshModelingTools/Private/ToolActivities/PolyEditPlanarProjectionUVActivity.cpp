// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolActivities/PolyEditPlanarProjectionUVActivity.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ContextObjectStore.h"
#include "Drawing/PolyEditPreviewMesh.h"
#include "InteractiveToolManager.h"
#include "Mechanics/CollectSurfacePathMechanic.h"
#include "MeshOpPreviewHelpers.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ToolActivities/PolyEditActivityContext.h"
#include "ToolSceneQueriesUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditPlanarProjectionUVActivity)

#define LOCTEXT_NAMESPACE "UPolyEditPlanarProjectionUVActivity"

using namespace UE::Geometry;

void UPolyEditPlanarProjectionUVActivity::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	SetUVProperties = NewObject<UPolyEditSetUVProperties>();
	SetUVProperties->RestoreProperties(ParentTool.Get());
	AddToolPropertySource(SetUVProperties);
	SetToolPropertySourceEnabled(SetUVProperties, false);

	// Register ourselves to receive clicks and hover
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	ActivityContext = ParentTool->GetToolManager()->GetContextObjectStore()->FindContext<UPolyEditActivityContext>();
}

void UPolyEditPlanarProjectionUVActivity::Shutdown(EToolShutdownType ShutdownType)
{
	Clear();
	SetUVProperties->SaveProperties(ParentTool.Get());

	SetUVProperties = nullptr;
	ParentTool = nullptr;
	ActivityContext = nullptr;
}

bool UPolyEditPlanarProjectionUVActivity::CanStart() const
{
	if (!ActivityContext)
	{
		return false;
	}
	const FGroupTopologySelection& Selection = ActivityContext->SelectionMechanic->GetActiveSelection();
	return !Selection.SelectedGroupIDs.IsEmpty();
}

EToolActivityStartResult UPolyEditPlanarProjectionUVActivity::Start()
{
	if (!CanStart())
	{
		ParentTool->GetToolManager()->DisplayMessage(
			LOCTEXT("OnSetUVsFailedMesssage", "Cannot set UVs without face selection."), 
			EToolMessageLevel::UserWarning);
		return EToolActivityStartResult::FailedStart;
	}

	Clear();
	BeginSetUVs();
	bIsRunning = true;

	ParentTool->GetToolManager()->DisplayMessage(
		LOCTEXT("OnBeginSetUVsMessage", "Click on the face to Set UVs"), EToolMessageLevel::UserMessage);

	ActivityContext->EmitActivityStart(LOCTEXT("BeginSetUVsActivity", "Begin Set UVs"));

	return EToolActivityStartResult::Running;
}

bool UPolyEditPlanarProjectionUVActivity::CanAccept() const
{
	return false;
}

EToolActivityEndResult UPolyEditPlanarProjectionUVActivity::End(EToolShutdownType)
{
	Clear();
	EToolActivityEndResult ToReturn = bIsRunning ? EToolActivityEndResult::Cancelled 
		: EToolActivityEndResult::ErrorDuringEnd;
	bIsRunning = false;
	return ToReturn;
}

void UPolyEditPlanarProjectionUVActivity::BeginSetUVs()
{
	using namespace UE::Geometry::PolyEditActivityUtil;

	TArray<int32> ActiveTriangleSelection;
	const FGroupTopologySelection& ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	FTransform3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());

	EditPreview = PolyEditActivityUtil::CreatePolyEditPreviewMesh(*ParentTool, *ActivityContext);
	FTransform3d WorldTranslation, WorldRotateScale;
	EditPreview->ApplyTranslationToPreview(WorldTransform, WorldTranslation, WorldRotateScale);
	EditPreview->InitializeStaticType(ActivityContext->CurrentMesh.Get(), ActiveTriangleSelection, &WorldRotateScale);

	UpdatePolyEditPreviewMaterials(*ParentTool, *ActivityContext, *EditPreview, (SetUVProperties->bShowMaterial) ?
		EPreviewMaterialType::SourceMaterials : EPreviewMaterialType::UVMaterial);
	CurrentPreviewMaterial = SetUVProperties->bShowMaterial ?
		EPreviewMaterialType::SourceMaterials : EPreviewMaterialType::UVMaterial;

	FDynamicMesh3 StaticHitTargetMesh;
	EditPreview->MakeInsetTypeTargetMesh(StaticHitTargetMesh);

	// Hide the selected triangles (that are being replaced by the portion we're editing)
	ActivityContext->Preview->PreviewMesh->SetSecondaryBuffersVisibility(false);

	SurfacePathMechanic = NewObject<UCollectSurfacePathMechanic>(this);
	SurfacePathMechanic->Setup(ParentTool.Get());
	SurfacePathMechanic->InitializeMeshSurface(MoveTemp(StaticHitTargetMesh));
	SurfacePathMechanic->SetFixedNumPointsMode(2);
	SurfacePathMechanic->bSnapToTargetMeshVertices = true;
	double SnapTol = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	SurfacePathMechanic->SpatialSnapPointsFunc = [this, SnapTol](FVector3d Position1, FVector3d Position2)
	{
		return ToolSceneQueriesUtil::PointSnapQuery(this->CameraState, Position1, Position2, SnapTol);
	};

	ActiveSelectionBounds = FAxisAlignedBox3d::Empty();
	for (int tid : ActiveTriangleSelection)
	{
		ActiveSelectionBounds.Contain(ActivityContext->CurrentMesh->GetTriBounds(tid));
	}

	SetToolPropertySourceEnabled(SetUVProperties, true);
}

void UPolyEditPlanarProjectionUVActivity::UpdateSetUVS()
{
	// align projection frame to line user is drawing out from plane origin
	FFrame3d WorldPlanarFrame = SurfacePathMechanic->PreviewPathPoint;
	FFrame3d LocalPlanarFrame;
	double UVScale = 1.0 / ActiveSelectionBounds.MaxDim(); // Note: EditPreview has baked scale, so UVScale is already in world
	FTransformSRT3d WorldTransform = EditPreview->GetTransform();
	if (SurfacePathMechanic->HitPath.Num() == 1)
	{
		SurfacePathMechanic->InitializePlaneSurface(WorldPlanarFrame);

		FVector3d Delta = WorldPlanarFrame.Origin - SurfacePathMechanic->HitPath[0].Origin;
		double Dist = UE::Geometry::Normalize(Delta);
		UVScale *= FMathd::Lerp(1.0, 25.0, Dist / ActiveSelectionBounds.MaxDim());
		WorldPlanarFrame = SurfacePathMechanic->HitPath[0];
		FVector3d WorldNormal = WorldPlanarFrame.Z();
		FVector3d LocalNormal = WorldTransform.InverseTransformNormal(WorldNormal);
		LocalPlanarFrame = FFrame3d(WorldTransform.InverseTransformPosition(WorldPlanarFrame.Origin), LocalNormal);
		LocalPlanarFrame.ConstrainedAlignAxis(0, WorldTransform.InverseTransformVector(Delta), LocalPlanarFrame.Z());
	}
	else
	{
		FVector3d WorldNormal = WorldPlanarFrame.Z();
		FVector3d LocalNormal = WorldTransform.InverseTransformNormal(WorldNormal);
		LocalPlanarFrame = FFrame3d(WorldTransform.InverseTransformPosition(WorldPlanarFrame.Origin), LocalNormal);
	}

	EditPreview->UpdateStaticType([&](FDynamicMesh3& Mesh)
		{
			FDynamicMeshEditor Editor(&Mesh);
			TArray<int32> AllTriangles;
			for (int32 tid : Mesh.TriangleIndicesItr())
			{
				AllTriangles.Add(tid);
			}
			Editor.SetTriangleUVsFromProjection(AllTriangles, LocalPlanarFrame, UVScale, FVector2f::Zero(), false);
		}, false);

}

void UPolyEditPlanarProjectionUVActivity::ApplySetUVs()
{
	const FGroupTopologySelection& ActiveSelection = ActivityContext->SelectionMechanic->GetActiveSelection();
	TArray<int32> ActiveTriangleSelection;
	ActivityContext->CurrentTopology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);

	// Get world-position data about line user drew
	FFrame3d WorldPlanarFrame = SurfacePathMechanic->HitPath[0];
	double WorldUVScale = 1.0 / ActiveSelectionBounds.MaxDim();
	FVector3d WorldDelta = SurfacePathMechanic->HitPath[1].Origin - WorldPlanarFrame.Origin;
	double WorldDist = UE::Geometry::Normalize(WorldDelta);
	WorldUVScale *= FMathd::Lerp(1.0, 25.0, WorldDist / ActiveSelectionBounds.MaxDim());
	FVector3d WorldNormal = WorldPlanarFrame.Z();
	
	// Create a local frame w/ matching normal, aligned to the user's line
	FTransformSRT3d WorldTransform(ActivityContext->Preview->PreviewMesh->GetTransform());
	FVector3d LocalNormal = WorldTransform.InverseTransformNormal(WorldNormal);
	FFrame3d LocalPlanarFrame(WorldTransform.InverseTransformPosition(WorldPlanarFrame.Origin), LocalNormal);
	LocalPlanarFrame.ConstrainedAlignAxis(0, WorldTransform.InverseTransformVector(WorldDelta), LocalPlanarFrame.Z());
	// Use a reference point to get a uniform scale factor based on the world scale. Note it will not match exactly if the WorldTransform has non-uniform scale.
	FVector3d ScalePt = WorldPlanarFrame.Origin + WorldUVScale * (WorldPlanarFrame.X() + WorldPlanarFrame.Y()) * FMathd::InvSqrt2;
	ScalePt = WorldTransform.InverseTransformPosition(ScalePt);
	double LocalUVScale = Distance(ScalePt, LocalPlanarFrame.Origin);

	// track changes
	FDynamicMeshChangeTracker ChangeTracker(ActivityContext->CurrentMesh.Get());
	ChangeTracker.BeginChange();
	ChangeTracker.SaveTriangles(ActiveTriangleSelection, true);
	FDynamicMeshEditor Editor(ActivityContext->CurrentMesh.Get());
	Editor.SetTriangleUVsFromProjection(ActiveTriangleSelection, LocalPlanarFrame, LocalUVScale, FVector2f::Zero(), false, 0);

	// Emit undo (also updates relevant structures). We didn't change the mesh topology here but for now we use
	// the same route as everything else. See :HandlePositionOnlyMeshChanges
	ActivityContext->EmitCurrentMeshChangeAndUpdate(LOCTEXT("PolyMeshSetUVsChange", "Set UVs"),
		ChangeTracker.EndChange(), ActiveSelection);

	// End activity
	Clear();
	bIsRunning = false;
	Cast<IToolActivityHost>(ParentTool)->NotifyActivitySelfEnded(this);
}

void UPolyEditPlanarProjectionUVActivity::Clear()
{
	if (EditPreview != nullptr)
	{
		EditPreview->Disconnect();
		EditPreview = nullptr;
	}

	ActivityContext->Preview->PreviewMesh->SetSecondaryBuffersVisibility(true);

	SurfacePathMechanic = nullptr;
	SetToolPropertySourceEnabled(SetUVProperties, false);
}

void UPolyEditPlanarProjectionUVActivity::Render(IToolsContextRenderAPI* RenderAPI)
{
	ParentTool->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (SurfacePathMechanic != nullptr)
	{
		SurfacePathMechanic->Render(RenderAPI);
	}
}

void UPolyEditPlanarProjectionUVActivity::Tick(float DeltaTime)
{
	using namespace UE::Geometry::PolyEditActivityUtil;

	EPreviewMaterialType WantMaterial = (SetUVProperties->bShowMaterial) ? EPreviewMaterialType::SourceMaterials : EPreviewMaterialType::UVMaterial;
	if (CurrentPreviewMaterial != WantMaterial)
	{
		UpdatePolyEditPreviewMaterials(*ParentTool, *ActivityContext, *EditPreview, WantMaterial);
		CurrentPreviewMaterial = WantMaterial;
	}

	if (bPreviewUpdatePending)
	{
		UpdateSetUVS();
		bPreviewUpdatePending = false;
	}
}

FInputRayHit UPolyEditPlanarProjectionUVActivity::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning;
	return OutHit;
}

void UPolyEditPlanarProjectionUVActivity::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (bIsRunning && SurfacePathMechanic->TryAddPointFromRay((FRay3d)ClickPos.WorldRay))
	{
		if (SurfacePathMechanic->IsDone())
		{
			ApplySetUVs();
		}
	}
}

FInputRayHit UPolyEditPlanarProjectionUVActivity::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit OutHit;
	OutHit.bHit = bIsRunning;
	return OutHit;
}

bool UPolyEditPlanarProjectionUVActivity::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	SurfacePathMechanic->UpdatePreviewPoint((FRay3d)DevicePos.WorldRay);
	bPreviewUpdatePending = true;
	return bIsRunning;
}

#undef LOCTEXT_NAMESPACE

