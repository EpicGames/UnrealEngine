// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVLayoutTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"

#include "DynamicMesh/DynamicMesh3.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "ParameterizationOps/UVLayoutOp.h"
#include "Properties/UVLayoutProperties.h"

#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"

#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(UVLayoutTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVLayoutTool"

/*
 * ToolBuilder
 */

UMultiSelectionMeshEditingTool* UUVLayoutToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UUVLayoutTool>(SceneState.ToolManager);
}

bool UUVLayoutToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMultiSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return ToolBuilderUtil::ComponentTypeCouldHaveUVs(Component); }) > 0;
}

const FToolTargetTypeRequirements& UUVLayoutToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

/*
 * Tool
 */

UUVLayoutTool::UUVLayoutTool()
{
}

void UUVLayoutTool::Setup()
{
	UInteractiveTool::Setup();

	// hide input StaticMeshComponent
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::HideSourceObject(Targets[ComponentIdx]);
	}

	// if we only have one object, add ability to set UV channel
	if (Targets.Num() == 1)
	{
		UVChannelProperties = NewObject<UMeshUVChannelProperties>(this);
		UVChannelProperties->RestoreProperties(this);
		UVChannelProperties->Initialize(UE::ToolTarget::GetMeshDescription(Targets[0]), false);
		UVChannelProperties->ValidateSelection(true);
		AddToolPropertySource(UVChannelProperties);
		UVChannelProperties->WatchProperty(UVChannelProperties->UVChannel, [this](const FString& NewValue)
		{
			MaterialSettings->UpdateUVChannels(UVChannelProperties->UVChannelNamesList.IndexOfByKey(UVChannelProperties->UVChannel),
			                                   UVChannelProperties->UVChannelNamesList);
			UpdateVisualization();
		});
	}

	BasicProperties = NewObject<UUVLayoutProperties>(this);
	BasicProperties->RestoreProperties(this);
	BasicProperties->bUDIMCVAREnabled = false;
	BasicProperties->bEnableUDIMLayout = false;
	AddToolPropertySource(BasicProperties);

	MaterialSettings = NewObject<UExistingMeshMaterialProperties>(this);
	MaterialSettings->RestoreProperties(this);
	AddToolPropertySource(MaterialSettings);

	// if we only have one object, add optional UV layout view
	if (Targets.Num() == 1)
	{
		UVLayoutView = NewObject<UUVLayoutPreview>(this);
		UVLayoutView->CreateInWorld(GetTargetWorld());

		const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
		UVLayoutView->SetSourceMaterials(MaterialSet);

		const AActor* Actor = UE::ToolTarget::GetTargetActor(Targets[0]);
		UVLayoutView->SetSourceWorldPosition(
			Actor->GetTransform(),
			Actor->GetComponentsBoundingBox());

		UVLayoutView->Settings->RestoreProperties(this);
		AddToolPropertySource(UVLayoutView->Settings);
	}

	UpdateVisualization();

	SetToolDisplayName(LOCTEXT("ToolName", "UV Layout"));
	GetToolManager()->DisplayMessage(LOCTEXT("OnStartUVLayoutTool", "Transform/Rotate/Scale existing UV Charts using various strategies"),
		EToolMessageLevel::UserNotification);
}


void UUVLayoutTool::UpdateNumPreviews()
{
	const int32 CurrentNumPreview = Previews.Num();
	const int32 TargetNumPreview = Targets.Num();
	if (TargetNumPreview < CurrentNumPreview)
	{
		for (int32 PreviewIdx = CurrentNumPreview - 1; PreviewIdx >= TargetNumPreview; PreviewIdx--)
		{
			Previews[PreviewIdx]->Cancel();
		}
		Previews.SetNum(TargetNumPreview);
		OriginalDynamicMeshes.SetNum(TargetNumPreview);
		Factories.SetNum(TargetNumPreview);
	}
	else
	{
		OriginalDynamicMeshes.SetNum(TargetNumPreview);
		Factories.SetNum(TargetNumPreview);
		for (int32 PreviewIdx = CurrentNumPreview; PreviewIdx < TargetNumPreview; PreviewIdx++)
		{
			OriginalDynamicMeshes[PreviewIdx] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
			*OriginalDynamicMeshes[PreviewIdx] = UE::ToolTarget::GetDynamicMeshCopy(Targets[PreviewIdx]);

			Factories[PreviewIdx]= NewObject<UUVLayoutOperatorFactory>();
			Factories[PreviewIdx]->OriginalMesh = OriginalDynamicMeshes[PreviewIdx];
			Factories[PreviewIdx]->Settings = BasicProperties;
			Factories[PreviewIdx]->TargetTransform = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[PreviewIdx]);
			Factories[PreviewIdx]->GetSelectedUVChannel = [this]() { return GetSelectedUVChannel(); };

			UMeshOpPreviewWithBackgroundCompute* Preview = Previews.Add_GetRef(NewObject<UMeshOpPreviewWithBackgroundCompute>(Factories[PreviewIdx], "Preview"));
			Preview->Setup(GetTargetWorld(), Factories[PreviewIdx]);
			ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Targets[PreviewIdx]); 

			const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[PreviewIdx]);
			Preview->ConfigureMaterials(MaterialSet.Materials,
				ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
			);
			Preview->PreviewMesh->UpdatePreview(OriginalDynamicMeshes[PreviewIdx].Get());
			Preview->PreviewMesh->SetTransform((FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[PreviewIdx]));

			Preview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute)
			{
				OnPreviewMeshUpdated(Compute);
			});

			Preview->SetVisibility(true);
		}
	}
}


void UUVLayoutTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (UVLayoutView)
	{
		UVLayoutView->Settings->SaveProperties(this);
		UVLayoutView->Disconnect();
	}

	BasicProperties->SaveProperties(this);
	MaterialSettings->SaveProperties(this);

	// Restore (unhide) the source meshes
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	TArray<FDynamicMeshOpResult> Results;
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Results.Emplace(Preview->Shutdown());
	}
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Results);
	}
	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		Factories[TargetIndex] = nullptr;
	}
}

int32 UUVLayoutTool::GetSelectedUVChannel() const
{
	return UVChannelProperties ? UVChannelProperties->GetSelectedChannelIndex(true) : 0;
}


void UUVLayoutTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (UVLayoutView)
	{
		UVLayoutView->Render(RenderAPI);
	}

}

void UUVLayoutTool::OnTick(float DeltaTime)
{
	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->Tick(DeltaTime);
	}

	if (UVLayoutView)
	{
		UVLayoutView->OnTick(DeltaTime);
	}


}



void UUVLayoutTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == BasicProperties || PropertySet == UVChannelProperties)
	{
		UpdateNumPreviews();
		for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
		{
			Preview->InvalidateResult();
		}
	}
	else if (PropertySet == MaterialSettings)
	{
		// if we don't know what changed, or we know checker density changed, update checker material
		UpdatePreviewMaterial();
	}
}


void UUVLayoutTool::OnPreviewMeshUpdated(UMeshOpPreviewWithBackgroundCompute* Compute)
{
	if (UVLayoutView)
	{
		FDynamicMesh3 ResultMesh;
		if (Compute->GetCurrentResultCopy(ResultMesh, false) == false)
		{
			return;
		}
		UVLayoutView->UpdateUVMesh(&ResultMesh, GetSelectedUVChannel());
	}

}

void UUVLayoutTool::UpdatePreviewMaterial()
{
	MaterialSettings->UpdateMaterials();
	UpdateNumPreviews();
	for (int PreviewIdx = 0; PreviewIdx < Previews.Num(); PreviewIdx++)
	{
		UMeshOpPreviewWithBackgroundCompute* Preview = Previews[PreviewIdx];
		Preview->OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
	}
}

void UUVLayoutTool::UpdateVisualization()
{
	UpdatePreviewMaterial();

	for (UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		Preview->InvalidateResult();
	}
}

bool UUVLayoutTool::CanAccept() const
{
	for (const UMeshOpPreviewWithBackgroundCompute* Preview : Previews)
	{
		if (!Preview->HaveValidResult())
		{
			return false;
		}
	}
	return Super::CanAccept();
}


void UUVLayoutTool::GenerateAsset(const TArray<FDynamicMeshOpResult>& Results)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("UVLayoutToolTransactionName", "UV Layout Tool"));

	check(Results.Num() == Targets.Num());
	
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		const FDynamicMesh3* DynamicMesh = Results[ComponentIdx].Mesh.Get();
		check(DynamicMesh != nullptr);
		UE::ToolTarget::CommitDynamicMeshUVUpdate(Targets[ComponentIdx], DynamicMesh);
	}

	GetToolManager()->EndUndoTransaction();
}




#undef LOCTEXT_NAMESPACE

