// Copyright Epic Games, Inc. All Rights Reserved.

#include "UVProjectionTool.h"
#include "InteractiveToolManager.h"
#include "ToolTargetManager.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "BaseBehaviors/SingleClickBehavior.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selections/GeometrySelectionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "MeshRegionBoundaryLoops.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Operations/MeshConvexHull.h"
#include "MinVolumeBox3.h"
#include "ToolTargetManager.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "Selection/GeometrySelectionVisualization.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(UVProjectionTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UUVProjectionTool"

/*
 * ToolBuilder
 */

USingleTargetWithSelectionTool* UUVProjectionToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UUVProjectionTool>(SceneState.ToolManager);
}

bool UUVProjectionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return USingleTargetWithSelectionToolBuilder::CanBuildTool(SceneState) && 
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return ToolBuilderUtil::ComponentTypeCouldHaveUVs(Component); }) > 0;
}

/*
 * Propert Sets
 */

void UUVProjectionToolEditActions::PostAction(EUVProjectionToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}




/*
 * Tool
 */

void UUVProjectionTool::Setup()
{
	UInteractiveTool::Setup();

	UE::ToolTarget::HideSourceObject(Target);

	BasicProperties = NewObject<UUVProjectionToolProperties>(this);
	BasicProperties->RestoreProperties(this);
	BasicProperties->GetOnModified().AddLambda([this](UObject*, FProperty*)	{ Preview->InvalidateResult(); });

	// initialize input mesh
	InitializeMesh();

	// initialize the PreviewMesh+BackgroundCompute object
	UpdateNumPreviews();

	UVChannelProperties = NewObject<UMeshUVChannelProperties>(this);
	UVChannelProperties->RestoreProperties(this);
	UVChannelProperties->Initialize(InputMesh.Get(), false);
	UVChannelProperties->ValidateSelection(true);
	UVChannelProperties->WatchProperty(UVChannelProperties->UVChannel, [this](const FString& NewValue)
	{
		MaterialSettings->UpdateUVChannels(UVChannelProperties->UVChannelNamesList.IndexOfByKey(UVChannelProperties->UVChannel),
		                                   UVChannelProperties->UVChannelNamesList);
		Preview->InvalidateResult();
		OnMaterialSettingsChanged();
	});

	MaterialSettings = NewObject<UExistingMeshMaterialProperties>(this);
	MaterialSettings->RestoreProperties(this);
	MaterialSettings->GetOnModified().AddLambda([this](UObject*, FProperty*)
	{
		OnMaterialSettingsChanged();
	});

	EditActions = NewObject<UUVProjectionToolEditActions>(this);
	EditActions->Initialize(this);

	AddToolPropertySource(UVChannelProperties);
	AddToolPropertySource(BasicProperties);
	AddToolPropertySource(EditActions);
	AddToolPropertySource(MaterialSettings);
	
	OnMaterialSettingsChanged();

	// set up visualizers
	ProjectionShapeVisualizer.LineColor = FLinearColor::Red;
	ProjectionShapeVisualizer.LineThickness = 2.0;

	// initialize Gizmo
	FTransform3d GizmoPositionWorld(WorldBounds.Center());
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->SetTransform((FTransform)GizmoPositionWorld);
	TransformProxy->OnTransformChanged.AddUObject(this, &UUVProjectionTool::TransformChanged);

	TransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(
		GetToolManager(), ETransformGizmoSubElements::StandardTranslateRotate, this);
	TransformGizmo->SetActiveTarget(TransformProxy, GetToolManager());

	InitialDimensions = BasicProperties->Dimensions;
	bInitialProportionalDimensions = BasicProperties->bProportionalDimensions;
	InitialTransform = TransformGizmo->GetGizmoTransform();
	
	ApplyInitializationMode();

	// Some lambdas for proportional dimension changes
	auto OnProportionalDimensionsChanged = [this](bool bNewValue)
	{
		if (bNewValue) { CachedDimensions = BasicProperties->Dimensions; }
		bTransformModified = true;
	};

	auto ApplyProportionalDimensions = [this](FVector& NewVector, FVector& CachedVector)
	{
		// Determines which component of the vector is being changed by looking at which component is
		// most different from the previous values
		const FVector Difference = NewVector - CachedVector;
		const int32 DifferenceMaxElementIndex = MaxAbsElementIndex(Difference);

		// Readability aliases
		const double& NewComponent = NewVector[DifferenceMaxElementIndex];
		const double& ChangedComponent = CachedVector[DifferenceMaxElementIndex];

		// If the component we are changing is 0 (or very close to it), then just set all components to new component,
		// effectively setting the vector's direction to (1, 1, 1) and scaling according to the new value
		const double ScaleFactor = NewComponent / ChangedComponent;
		NewVector = FMath::Abs(ChangedComponent) < UE_KINDA_SMALL_NUMBER ? FVector(NewComponent) : CachedVector * ScaleFactor;
		CachedVector = NewVector;
	};

	// start watching for dimensions changes
	DimensionsWatcher = BasicProperties->WatchProperty(BasicProperties->Dimensions, [this, &ApplyProportionalDimensions](FVector)
	{
		if (BasicProperties->bProportionalDimensions)
		{
			ApplyProportionalDimensions(BasicProperties->Dimensions, CachedDimensions);
			BasicProperties->SilentUpdateWatcherAtIndex(DimensionsWatcher);
		}

		bTransformModified = true;
	});
	DimensionsModeWatcher = BasicProperties->WatchProperty(BasicProperties->bProportionalDimensions, OnProportionalDimensionsChanged);
	BasicProperties->WatchProperty(BasicProperties->Initialization, [this](EUVProjectionToolInitializationMode NewMode) { OnInitializationModeChanged(); });
	BasicProperties->SilentUpdateWatched();

	OnProportionalDimensionsChanged(true);	// Initialize CachedDimensions

	// Allow the user to change the Initialization mode and have it affect the current run of the tool, as long as they do it before modifying the transform
	bTransformModified = false;


	// click to set plane behavior
	SetPlaneCtrlClickBehaviorTarget = MakeUnique<FSelectClickedAction>();
	SetPlaneCtrlClickBehaviorTarget->SnapManager = USceneSnappingManager::Find(GetToolManager());
	SetPlaneCtrlClickBehaviorTarget->OnClickedPositionFunc = [this](const FHitResult& Hit)
	{
		UpdatePlaneFromClick(FVector3d(Hit.ImpactPoint), FVector3d(Hit.ImpactNormal), SetPlaneCtrlClickBehaviorTarget->bShiftModifierToggle);
	};
	SetPlaneCtrlClickBehaviorTarget->InvisibleComponentsToHitTest.Add(UE::ToolTarget::GetTargetComponent(Target));
	ClickToSetPlaneBehavior = NewObject<USingleClickInputBehavior>();
	ClickToSetPlaneBehavior->ModifierCheckFunc = FInputDeviceState::IsCtrlKeyDown;
	ClickToSetPlaneBehavior->Modifiers.RegisterModifier(FSelectClickedAction::ShiftModifier, FInputDeviceState::IsShiftKeyDown);
	ClickToSetPlaneBehavior->Initialize(SetPlaneCtrlClickBehaviorTarget.Get());
	AddInputBehavior(ClickToSetPlaneBehavior);

	if (HasGeometrySelection())
	{
		GeometrySelectionVizProperties = NewObject<UGeometrySelectionVisualizationProperties>(this);
		GeometrySelectionVizProperties->RestoreProperties(this);
		AddToolPropertySource(GeometrySelectionVizProperties);
		GeometrySelectionVizProperties->Initialize(this);
		GeometrySelectionVizProperties->bEnableShowTriangleROIBorder = true;
		GeometrySelectionVizProperties->SelectionElementType = static_cast<EGeometrySelectionElementType>(GeometrySelection.ElementType);
		GeometrySelectionVizProperties->SelectionTopologyType = static_cast<EGeometrySelectionTopologyType>(GeometrySelection.TopologyType);

		GeometrySelectionViz = NewObject<UPreviewGeometry>(this);
		GeometrySelectionViz->CreateInWorld(GetTargetWorld(), WorldTransform);
		UE::Geometry::InitializeGeometrySelectionVisualization(
			GeometrySelectionViz,
			GeometrySelectionVizProperties,
			*InputMesh,
			GeometrySelection,
			nullptr,
			nullptr,
			TriangleROI.Get());
	}

	// probably already done
	Preview->InvalidateResult();

	SetToolDisplayName(LOCTEXT("ToolName", "UV Projection"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("UVProjectionToolDescription", "Generate UVs for a Mesh by projecting onto simple geometric shapes. Ctrl+click to reposition shape. Shift+Ctrl+click to reposition shape without reorienting. Face selections can be made in the PolyEdit and TriEdit Tools. "),
		EToolMessageLevel::UserNotification);
}


void UUVProjectionTool::UpdateNumPreviews()
{
	OperatorFactory = NewObject<UUVProjectionOperatorFactory>();
	OperatorFactory->Tool = this;

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(OperatorFactory);
	Preview->Setup(GetTargetWorld(), OperatorFactory);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target); 
	Preview->OnMeshUpdated.AddUObject(this, &UUVProjectionTool::OnMeshUpdated);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target, false);
	Preview->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);
	Preview->PreviewMesh->UpdatePreview(InputMesh.Get());
	Preview->PreviewMesh->SetTransform((FTransform)WorldTransform);

	Preview->SetVisibility(true);

	EdgeRenderer = NewObject<UPreviewGeometry>(this);
	EdgeRenderer->CreateInWorld(GetTargetWorld(), (FTransform)WorldTransform);
}


void UUVProjectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	UVChannelProperties->SaveProperties(this);
	BasicProperties->SavedDimensions = BasicProperties->Dimensions;
	BasicProperties->bSavedProportionalDimensions = BasicProperties->bProportionalDimensions;
	BasicProperties->SavedTransform = TransformGizmo->GetGizmoTransform();
	BasicProperties->SaveProperties(this);
	MaterialSettings->SaveProperties(this);

	EdgeRenderer->Disconnect();

	// Restore (unhide) the source meshes
	UE::ToolTarget::ShowSourceObject(Target);

	FDynamicMeshOpResult Result = Preview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("UVProjectionToolTransactionName", "Project UVs"));
		FDynamicMesh3* NewDynamicMesh = Result.Mesh.Get();
		if (ensure(NewDynamicMesh))
		{
			UE::ToolTarget::CommitDynamicMeshUVUpdate(Target, NewDynamicMesh);
		}

		if (HasGeometrySelection())
		{
			UE::Geometry::SetToolOutputGeometrySelectionForTarget(this, Target, GetGeometrySelection());
		}

		GetToolManager()->EndUndoTransaction();
	}

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	TransformGizmo = nullptr;
	TransformProxy = nullptr;

	Super::OnShutdown(ShutdownType);
}


void UUVProjectionTool::RequestAction(EUVProjectionToolActions ActionType)
{
	if (bHavePendingAction)
	{
		return;
	}
	PendingAction = ActionType;
	bHavePendingAction = true;
}



TUniquePtr<FDynamicMeshOperator> UUVProjectionOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FUVProjectionOp> Op = MakeUnique<FUVProjectionOp>();
	Op->ProjectionMethod = Tool->BasicProperties->ProjectionType;

	Op->MeshToProjectionSpace = Tool->WorldTransform;
	Op->ProjectionBox = Tool->GetProjectionBox();
	Op->CylinderSplitAngle = Tool->BasicProperties->CylinderSplitAngle;
	Op->BlendWeight = Tool->BasicProperties->ExpMapNormalBlending;
	Op->SmoothingRounds = Tool->BasicProperties->ExpMapSmoothingSteps;
	Op->SmoothingAlpha = Tool->BasicProperties->ExpMapSmoothingAlpha;

	Op->UVRotationAngleDeg = Tool->BasicProperties->Rotation;
	Op->UVScale = (FVector2f)Tool->BasicProperties->Scale;
	Op->UVTranslate = (FVector2f)Tool->BasicProperties->Translation;

	Op->OriginalMesh = Tool->InputMesh;
	Op->TriangleROI = Tool->TriangleROI;
	Op->UseUVLayer = Tool->UVChannelProperties->GetSelectedChannelIndex(true);
		
	Op->SetResultTransform(Tool->WorldTransform);

	return Op;
}




void UUVProjectionTool::InitializeMesh()
{
	InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*InputMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
	WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);

	// initialize triangle ROI if one exists
	TriangleROI = MakeShared<TArray<int32>, ESPMode::ThreadSafe>();
	if (HasGeometrySelection())
	{
		const FGeometrySelection& InputSelection = GetGeometrySelection();
		UE::Geometry::EnumerateSelectionTriangles(InputSelection, *InputMesh, 
			[&](int32 TriangleID) { TriangleROI->Add(TriangleID); });
		TriangleROISet.Append(*TriangleROI);
	}
	VertexROI = MakeShared<TArray<int32>, ESPMode::ThreadSafe>();
	UE::Geometry::TriangleToVertexIDs(InputMesh.Get(), *TriangleROI, *VertexROI);

	InputMeshROISpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(InputMesh.Get(), false);
	if (TriangleROI->Num() > 0)
	{
		InputMeshROISpatial->Build(*TriangleROI);
	}
	else
	{
		InputMeshROISpatial->Build();
	}

	WorldBounds = (VertexROI->Num() > 0) ?
		FAxisAlignedBox3d::MakeBoundsFromIndices(*VertexROI, [this](int32 Index) { return WorldTransform.TransformPosition(InputMesh->GetVertex(Index)); })
		: FAxisAlignedBox3d::MakeBoundsFromIndices(InputMesh->VertexIndicesItr(), [this](int32 Index) { return WorldTransform.TransformPosition(InputMesh->GetVertex(Index)); });

	BasicProperties->Dimensions = (FVector)WorldBounds.Diagonal();
}



FOrientedBox3d UUVProjectionTool::GetProjectionBox() const
{
	const FFrame3d BoxFrame(TransformProxy->GetTransform());
	const FVector3d BoxDimensions = 0.5 * BasicProperties->Dimensions;
	return FOrientedBox3d(BoxFrame, BoxDimensions);
}



void UUVProjectionTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	ProjectionShapeVisualizer.bDepthTested = false;
	ProjectionShapeVisualizer.BeginFrame(RenderAPI, CameraState);

	UE::Geometry::FOrientedBox3d PrimBox = GetProjectionBox();
	FVector MinCorner = (FVector)-PrimBox.Extents;
	FVector MaxCorner = (FVector)PrimBox.Extents;
	float Width = MaxCorner.X - MinCorner.X;
	float Depth = MaxCorner.Y - MinCorner.Y;
	float Height = MaxCorner.Z - MinCorner.Z;

	FTransform UseTransform = PrimBox.Frame.ToFTransform();
	if (BasicProperties->ProjectionType == EUVProjectionMethod::Cylinder)
	{
		UseTransform.SetScale3D(FVector(Width, Depth, 1.0f));
	}
	ProjectionShapeVisualizer.SetTransform(UseTransform);

	switch (BasicProperties->ProjectionType)
	{
	case EUVProjectionMethod::Box:
		ProjectionShapeVisualizer.DrawWireBox(FBox(MinCorner, MaxCorner));
		break;
	case EUVProjectionMethod::Cylinder:
		ProjectionShapeVisualizer.DrawWireCylinder(FVector(0, 0, -Height*0.5f), FVector(0, 0, 1), 0.5f, Height, 16);
		break;
	case EUVProjectionMethod::Plane:
		ProjectionShapeVisualizer.DrawSquare(FVector(0, 0, 0), FVector(Width, 0, 0), FVector(0, Depth, 0));
		break;
	case EUVProjectionMethod::ExpMap:
		ProjectionShapeVisualizer.DrawSquare(FVector(0, 0, 0), FVector(Width, 0, 0), FVector(0, Depth, 0));
		break;
	}
	
	ProjectionShapeVisualizer.EndFrame();
}

void UUVProjectionTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == BasicProperties && BasicProperties->bProportionalDimensions)
	{
		// We are silencing all watchers if Property corresponds to an FVector UProperty because this indicates that
		// the "Reset to Default" button was pressed. If individual components are modified instead, then Property will
		// not be castable to an FStructProperty
		
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if (StructProperty != nullptr && StructProperty->Struct->GetName() == FString("Vector"))
		{
			CachedDimensions = BasicProperties->Dimensions;
			BasicProperties->SilentUpdateWatcherAtIndex(DimensionsWatcher);
		}
	}
}

void UUVProjectionTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (bHavePendingAction)
	{
		ApplyAction(PendingAction);
		bHavePendingAction = false;
		PendingAction = EUVProjectionToolActions::NoAction;
	}

	Preview->Tick(DeltaTime);
}



void UUVProjectionTool::OnMaterialSettingsChanged()
{
	MaterialSettings->UpdateMaterials();

	UMaterialInterface* OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
	if (OverrideMaterial != nullptr)
	{
		Preview->OverrideMaterial = OverrideMaterial;
	}
	else
	{
		Preview->OverrideMaterial = nullptr;
	}
}


void UUVProjectionTool::OnMeshUpdated(UMeshOpPreviewWithBackgroundCompute* PreviewCompute)
{
	const FColor UVSeamColor(15, 240, 15);
	const float UVSeamThickness = 2.0f;
	const float UVSeamDepthBias = 0.1f * (float)(WorldBounds.DiagonalLength() * 0.01);

	const FDynamicMesh3* UseMesh = Preview->PreviewMesh->GetMesh();
	const FDynamicMeshUVOverlay* UVOverlay = UseMesh->Attributes()->GetUVLayer(UVChannelProperties->GetSelectedChannelIndex(true));
	auto AppendSeamEdge = [UseMesh, UVSeamColor, UVSeamThickness, UVSeamDepthBias](int32 eid, TArray<FRenderableLine>& LinesOut) {
		FVector3d A, B;
		UseMesh->GetEdgeV(eid, A, B);
		LinesOut.Add(FRenderableLine((FVector)A, (FVector)B, UVSeamColor, UVSeamThickness, UVSeamDepthBias));
	};
	if (TriangleROI->Num() > 0)
	{
		EdgeRenderer->CreateOrUpdateLineSet(TEXT("UVSeams"), UseMesh->MaxEdgeID(), [&](int32 eid, TArray<FRenderableLine>& LinesOut) {
			FIndex2i EdgeT = UseMesh->GetEdgeT(eid);
			if (UseMesh->IsEdge(eid) && TriangleROISet.Contains(EdgeT.A) && TriangleROISet.Contains(EdgeT.B) && UVOverlay->IsSeamEdge(eid))
			{
				AppendSeamEdge(eid, LinesOut);
			}
		}, 1);
	}
	else
	{
		EdgeRenderer->CreateOrUpdateLineSet(TEXT("UVSeams"), UseMesh->MaxEdgeID(), [&](int32 eid, TArray<FRenderableLine>& LinesOut) {
			if (UseMesh->IsEdge(eid) && UVOverlay->IsSeamEdge(eid))
			{
				AppendSeamEdge(eid, LinesOut);
			}
		}, 1);
	}
}


void UUVProjectionTool::UpdatePlaneFromClick(const FVector3d& Position, const FVector3d& Normal, bool bTransitionOnly)
{
	FFrame3d CurrentFrame(TransformProxy->GetTransform());
	CurrentFrame.Origin = Position;
	if (bTransitionOnly == false)
	{
		CurrentFrame.AlignAxis(2, Normal);
	}
	TransformGizmo->SetNewGizmoTransform(CurrentFrame.ToFTransform());
	Preview->InvalidateResult();
	bTransformModified = true;
}


void UUVProjectionTool::TransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	Preview->InvalidateResult();
	bTransformModified = true;
}


bool UUVProjectionTool::CanAccept() const
{
	if (!Preview->HaveValidResult())
	{
		return false;
	}
	return Super::CanAccept();
}



void UUVProjectionTool::OnInitializationModeChanged()
{
	bool bWasTransformModified = bTransformModified;
	if (!bTransformModified)
	{
		ApplyInitializationMode();
		bTransformModified = bWasTransformModified;
	}
}


void UUVProjectionTool::ApplyInitializationMode()
{

	switch (BasicProperties->Initialization)
	{
	case EUVProjectionToolInitializationMode::Default:
		BasicProperties->Dimensions = InitialDimensions;
		TransformGizmo->SetNewGizmoTransform(InitialTransform);
		break;

	case EUVProjectionToolInitializationMode::UsePrevious:
		{
			bool bHavePrevious = (BasicProperties->SavedDimensions != FVector::ZeroVector);
			BasicProperties->Dimensions = (bHavePrevious) ? BasicProperties->SavedDimensions : InitialDimensions;
			TransformGizmo->SetNewGizmoTransform( (bHavePrevious) ? BasicProperties->SavedTransform : InitialTransform);
		}
		break;

	case EUVProjectionToolInitializationMode::AutoFit:
		ApplyAction_AutoFit(false);
		break;

	case EUVProjectionToolInitializationMode::AutoFitAlign:
		ApplyAction_AutoFit(true);
		break;
	}
}



void UUVProjectionTool::ApplyAction(EUVProjectionToolActions ActionType)
{
	switch (ActionType)
	{
		case EUVProjectionToolActions::AutoFit:
			ApplyAction_AutoFit(false);
			break;
		case EUVProjectionToolActions::AutoFitAlign:
			ApplyAction_AutoFit(true);
			break;
		case EUVProjectionToolActions::Reset:
			ApplyAction_Reset();
			break;
		case EUVProjectionToolActions::NoAction:
			break;
	}
}


void UUVProjectionTool::ApplyAction_AutoFit(bool bAlign)
{
	// get current transform
	FFrame3d AlignFrame(TransformGizmo->GetGizmoTransform());

	TArray<FVector3d> Points;
	if (VertexROI->Num() > 0)
	{
		CollectVertexPositions(*InputMesh, *VertexROI, Points);
	}
	else
	{
		CollectVertexPositions(*InputMesh, InputMesh->VertexIndicesItr(), Points);
	}

	// auto-compute orientation if desired
	if (bAlign)
	{
		if (BasicProperties->ProjectionType == EUVProjectionMethod::Plane || BasicProperties->ProjectionType == EUVProjectionMethod::Box || BasicProperties->ProjectionType == EUVProjectionMethod::Cylinder)
		{
			// compute min-volume box
			UE::Geometry::FOrientedBox3d MinBoundingBox;
			FMinVolumeBox3d MinBoxCalc;
			bool bMinBoxOK = MinBoxCalc.SolveSubsample(Points.Num(), 1000,
				[&](int32 Index) { return WorldTransform.TransformPosition(Points[Index]); }, false, nullptr);
			if (bMinBoxOK && MinBoxCalc.IsSolutionAvailable())
			{
				MinBoxCalc.GetResult(MinBoundingBox);
			}
			else
			{
				MinBoundingBox = UE::Geometry::FOrientedBox3d(WorldBounds);
			}

			AlignFrame = MinBoundingBox.Frame;
		}
		else if (BasicProperties->ProjectionType == EUVProjectionMethod::ExpMap)
		{
			int32 VertexID;
			if (TriangleROI->Num() > 0)
			{
				FDynamicMeshUVEditor::EstimateGeodesicCenterFrameVertex(*InputMesh, *TriangleROI, AlignFrame, VertexID);
				AlignFrame.Transform(WorldTransform);
			}
			else
			{
				FDynamicMeshUVEditor::EstimateGeodesicCenterFrameVertex(*InputMesh, AlignFrame, VertexID);
				AlignFrame.Transform(WorldTransform);
			}
		}
	}

	// fit bounds to current frame
	FAxisAlignedBox3d FitBounds = FAxisAlignedBox3d::MakeBoundsFromIndices(Points.Num(), [this, &Points, &AlignFrame](int32 Index) 
	{ 
		return AlignFrame.ToFramePoint(WorldTransform.TransformPosition(Points[Index])); }
	);
	if (BasicProperties->ProjectionType == EUVProjectionMethod::Plane || BasicProperties->ProjectionType == EUVProjectionMethod::Box || BasicProperties->ProjectionType == EUVProjectionMethod::Cylinder)
	{
		FVector3d ShiftCenter = AlignFrame.FromFramePoint(FitBounds.Center());
		AlignFrame.Origin = ShiftCenter;
	}
	BasicProperties->Dimensions = 2.0f * (FVector)FitBounds.Extents();
	if (DimensionsWatcher >= 0)
	{
		BasicProperties->SilentUpdateWatcherAtIndex(DimensionsWatcher);
	}
	if (DimensionsModeWatcher >= 0)
	{
		BasicProperties->SilentUpdateWatcherAtIndex(DimensionsModeWatcher);
	}

	// update Gizmo
	TransformGizmo->SetNewGizmoTransform(AlignFrame.ToFTransform());
}



void UUVProjectionTool::ApplyAction_Reset()
{
	bool bWasTransformModified = bTransformModified;
	ApplyInitializationMode();
	bTransformModified = bWasTransformModified;
}


#undef LOCTEXT_NAMESPACE

