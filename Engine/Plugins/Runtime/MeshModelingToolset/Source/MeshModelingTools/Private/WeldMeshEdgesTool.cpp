// Copyright Epic Games, Inc. All Rights Reserved.

#include "WeldMeshEdgesTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "PreviewMesh.h"
#include "Drawing/MeshElementsVisualizer.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshEditor.h"
#include "Operations/MeshResolveTJunctions.h"

#include "Math/UnrealMathUtility.h"
#include "Selections/GeometrySelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeldMeshEdgesTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UWeldMeshEdgesTool"

/*
 * ToolBuilder
 */

USingleTargetWithSelectionTool* UWeldMeshEdgesToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UWeldMeshEdgesTool>(SceneState.ToolManager);
}




class FWeldMeshEdgesOp : public  FDynamicMeshOperator
{

public:
	enum class EWeldAttributeMode : uint8
	{
		/** Do not weld attributes*/
		None,
		/** Apply attribute merging on the mesh weld */
		OnWeldedMeshEdgesOnly,
		/** Apply attribute merging to all split attributes */
		OnFullMesh
	};

	// parameters set by the tool
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> SourceMesh;
	TSet<int32> SelectedEdges;
	double Tolerance;
	bool bOnlyUnique;
	bool bResolveTJunctions;
	EWeldAttributeMode WeldAttributeMode;
	float SplitNormalThreshold;
	float SplitTangentsThreshold;
	float SplitUVThreshold;
	float SplitColorThreshold; 
	bool bSplitBowties;

	int32 InitialNumBoundaryEdges = 0;
	int32 FinalNumBoundaryEdges = 0;

public:
	FWeldMeshEdgesOp() : FDynamicMeshOperator() {}

	virtual void CalculateResult(FProgressCancel* Progress) override
	{

		auto SetAttributeWeldOptions = [this](FSplitAttributeWelder& SplitAttributeWelder)
		{
			constexpr float DegToRad = static_cast<float>(UE_PI / 180.f);

			SplitAttributeWelder.UVDistSqrdThreshold = SplitUVThreshold * SplitUVThreshold;
			SplitAttributeWelder.NormalVecDotThreshold = FMath::Abs(1.f - FMath::Cos( DegToRad * SplitNormalThreshold));
			SplitAttributeWelder.TangentVecDotThreshold = FMath::Abs(1.f - FMath::Cos( DegToRad * SplitTangentsThreshold));
			SplitAttributeWelder.ColorDistSqrdThreshold = SplitColorThreshold * SplitColorThreshold;
		};

		if ((Progress && Progress->Cancelled()) || !SourceMesh)
		{
			return;
		}

		if (SourceMesh->TriangleCount() == 0 || SourceMesh->VertexCount() == 0)
		{
			return;
		}

		ResultMesh->Copy(*SourceMesh, true, true, true, true);

		if (bSplitBowties)
		{
			FDynamicMeshEditor Editor(ResultMesh.Get());
			FDynamicMeshEditResult EditResult;
			if (SelectedEdges.IsEmpty())
			{
				Editor.SplitBowties(EditResult);
			}
			else
			{
				// Note split bowties should never introduce new edges, just reconnects triangles to a new vertex, so edge IDs should be stable here
				for (int32 EID : SelectedEdges)
				{
					FIndex2i EdgeV = ResultMesh->GetEdgeV(EID);
					for (int32 SubIdx = 0; SubIdx < 2; ++SubIdx)
					{
						Editor.SplitBowties(EdgeV[SubIdx], EditResult);
					}
				}
			}
		}

		// If we had bowties or broken normals on input, we will probably still have them on output...
		bool bWasCleanMesh = ResultMesh->CheckValidity(FDynamicMesh3::FValidityOptions(), EValidityCheckFailMode::ReturnOnly);

		InitialNumBoundaryEdges = 0;
		for (int eid : SourceMesh->BoundaryEdgeIndicesItr())
		{
			InitialNumBoundaryEdges++;
		}
		FinalNumBoundaryEdges = InitialNumBoundaryEdges;

		FMergeCoincidentMeshEdges Merger(ResultMesh.Get());
		if (!SelectedEdges.IsEmpty())
		{
			Merger.EdgesToMerge = &SelectedEdges;	
		}
		Merger.MergeVertexTolerance = Tolerance;
		Merger.MergeSearchTolerance = 2 * Merger.MergeVertexTolerance;
		Merger.OnlyUniquePairs = bOnlyUnique;
		Merger.bWeldAttrsOnMergedEdges = (WeldAttributeMode == EWeldAttributeMode::OnWeldedMeshEdgesOnly);
		SetAttributeWeldOptions(Merger.SplitAttributeWelder);

		bool bOK = Merger.Apply();
		FinalNumBoundaryEdges = Merger.FinalNumBoundaryEdges;

		if (bOK && Merger.FinalNumBoundaryEdges > 0 && bResolveTJunctions)
		{
			FMeshResolveTJunctions Resolver(ResultMesh.Get());
			Resolver.DistanceTolerance = 2 * Tolerance;
			bool bResolveOK = Resolver.Apply();

			if (bResolveOK && Resolver.NumSplitEdges > 0)
			{
				FMergeCoincidentMeshEdges SecondPassMerger(ResultMesh.Get());
				SecondPassMerger.MergeVertexTolerance = Tolerance;
				SecondPassMerger.MergeSearchTolerance = 2 * SecondPassMerger.MergeVertexTolerance;
				SecondPassMerger.OnlyUniquePairs = bOnlyUnique;
				SecondPassMerger.bWeldAttrsOnMergedEdges = (WeldAttributeMode == EWeldAttributeMode::OnWeldedMeshEdgesOnly);
				SetAttributeWeldOptions(SecondPassMerger.SplitAttributeWelder);

				bOK = SecondPassMerger.Apply();
				FinalNumBoundaryEdges = SecondPassMerger.FinalNumBoundaryEdges;
			}
		}

		if (WeldAttributeMode == EWeldAttributeMode::OnFullMesh)
		{
			FSplitAttributeWelder AttributeWelder;
			SetAttributeWeldOptions(AttributeWelder);
			AttributeWelder.WeldSplitElements(*ResultMesh.Get());
		}

		FDynamicMesh3::FValidityOptions UseValidityCheck = (bWasCleanMesh) ?
			FDynamicMesh3::FValidityOptions() : FDynamicMesh3::FValidityOptions::Permissive();
		if (bOK == false)
		{
			ResultMesh->Copy(*SourceMesh, true, true, true, true);
			FinalNumBoundaryEdges = InitialNumBoundaryEdges;
		}
		else if (ResultMesh->CheckValidity(UseValidityCheck, EValidityCheckFailMode::ReturnOnly) == false)
		{
			ResultMesh->Copy(*SourceMesh, true, true, true, true);
			FinalNumBoundaryEdges = InitialNumBoundaryEdges;
		}
	}

	void SetTransform(const FTransformSRT3d& Transform)
	{
		ResultTransform = Transform;
	}
};



TUniquePtr<FDynamicMeshOperator> UWeldMeshEdgesOperatorFactory::MakeNewOperator()
{
	check(WeldMeshEdgesTool);
	TUniquePtr<FWeldMeshEdgesOp> MeshOp = MakeUnique<FWeldMeshEdgesOp>();
	WeldMeshEdgesTool->UpdateOpParameters(*MeshOp);
	return MeshOp;
}




/*
 * Tool
 */


UWeldMeshEdgesTool::UWeldMeshEdgesTool()
{
	SetToolDisplayName(LOCTEXT("WeldMeshEdgesToolName", "Weld Edges"));
}



void UWeldMeshEdgesTool::Setup()
{
	UInteractiveTool::Setup();

	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	SourceMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams));

	// initialize selection if exists
	if (HasGeometrySelection())
	{
		const FGeometrySelection& InputSelection = GetGeometrySelection();

		UE::Geometry::EnumerateSelectionEdges(InputSelection, *SourceMesh,
			[&](const int32 EdgeID) { SelectedEdges.Add(EdgeID); });
	}

	FTransform MeshTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);

	OperatorFactory = NewObject<UWeldMeshEdgesOperatorFactory>(this);
	OperatorFactory->WeldMeshEdgesTool = this;

	PreviewCompute = NewObject<UMeshOpPreviewWithBackgroundCompute>(OperatorFactory);
	PreviewCompute->Setup(GetTargetWorld(), OperatorFactory);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewCompute->PreviewMesh, Target); 

	// Give the preview something to display
	PreviewCompute->PreviewMesh->SetTransform(MeshTransform);
	PreviewCompute->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::ExternallyProvided);
	PreviewCompute->PreviewMesh->UpdatePreview(SourceMesh.Get());

	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewCompute->ConfigureMaterials(MaterialSet.Materials,
									   ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()) );

	PreviewCompute->SetVisibility(true);
	UE::ToolTarget::HideSourceObject(Target);


	Settings = NewObject<UWeldMeshEdgesToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	Settings->WatchProperty(Settings->Tolerance, [this](float) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->bOnlyUnique, [this](bool) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->bResolveTJunctions, [this](bool) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->bSplitBowties, [this](bool) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->AttrWeldingMode, [this](EWeldMeshEdgesAttributeUIMode) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->SplitUVThreshold, [this](float) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->SplitColorThreshold, [this](float) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->SplitNormalThreshold, [this](float) { PreviewCompute->InvalidateResult(); });
	Settings->WatchProperty(Settings->SplitTangentsThreshold, [this](float) { PreviewCompute->InvalidateResult(); });

	// create mesh display
	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(PreviewCompute->PreviewMesh->GetWorld(), PreviewCompute->PreviewMesh->GetTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowUVSeams = false;
		MeshElementsDisplay->Settings->bShowNormalSeams = false;
		MeshElementsDisplay->Settings->bShowColorSeams = false;
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("WeldEdges"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		PreviewCompute->ProcessCurrentMesh(ProcessFunc);
	});


	PreviewCompute->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute)
	{
		Compute->ProcessCurrentMesh([&](const FDynamicMesh3& ReadMesh)
		{
			MeshElementsDisplay->NotifyMeshChanged();
		});
	});

	PreviewCompute->OnOpCompleted.AddLambda([this](const FDynamicMeshOperator* Op)
	{
		Settings->InitialEdges = ((FWeldMeshEdgesOp*)(Op))->InitialNumBoundaryEdges;
		Settings->RemainingEdges = ((FWeldMeshEdgesOp*)(Op))->FinalNumBoundaryEdges;
	});

	PreviewCompute->InvalidateResult();


	SetToolDisplayName(LOCTEXT("ToolName", "Weld Edges"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("WeldMeshEdgesToolDescription", "Weld overlapping/identical border edges of the selected Mesh, by merging the vertices."),
		EToolMessageLevel::UserNotification);

}


void UWeldMeshEdgesTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Settings->SaveProperties(this);
	UE::ToolTarget::ShowSourceObject(Target);

	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, TEXT("WeldEdges"));
	}
	MeshElementsDisplay->Disconnect();

	if (PreviewCompute)
	{
		FDynamicMeshOpResult Result = PreviewCompute->Shutdown();
		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("WeldMeshEdgesToolTransactionName", "Weld Edges"));
			FDynamicMesh3* DynamicMeshResult = Result.Mesh.Get();
			if (ensure(DynamicMeshResult != nullptr))
			{
				// todo: have not actually modified topology here, but groups-only update is not supported yet
				UE::ToolTarget::CommitDynamicMeshUpdate(Target, *DynamicMeshResult, true);
			}
			GetToolManager()->EndUndoTransaction();
		}
	}
}



bool UWeldMeshEdgesTool::CanAccept() const
{
	return Super::CanAccept() && (PreviewCompute == nullptr || PreviewCompute->HaveValidResult());
}



void UWeldMeshEdgesTool::OnTick(float DeltaTime)
{
	PreviewCompute->Tick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);
}



void UWeldMeshEdgesTool::UpdateOpParameters(FWeldMeshEdgesOp& Op) const
{
	auto GetAttrMergeMode = [](const EWeldMeshEdgesAttributeUIMode& m)
	{
		switch (m)
		{
			case EWeldMeshEdgesAttributeUIMode::None :  return FWeldMeshEdgesOp::EWeldAttributeMode::None;
			case EWeldMeshEdgesAttributeUIMode::OnWeldedMeshEdgesOnly: return FWeldMeshEdgesOp::EWeldAttributeMode::OnWeldedMeshEdgesOnly;
			case EWeldMeshEdgesAttributeUIMode::OnFullMesh : return FWeldMeshEdgesOp::EWeldAttributeMode::OnFullMesh;
			default : return FWeldMeshEdgesOp::EWeldAttributeMode::None;
		}
	};

	Op.bOnlyUnique = Settings->bOnlyUnique;
	Op.bResolveTJunctions = Settings->bResolveTJunctions;
	Op.bSplitBowties = Settings->bSplitBowties;
	Op.Tolerance = Settings->Tolerance;

	Op.WeldAttributeMode = GetAttrMergeMode(Settings->AttrWeldingMode);
	Op.SplitUVThreshold = Settings->SplitUVThreshold;
	Op.SplitColorThreshold = Settings->SplitColorThreshold;
	Op.SplitNormalThreshold = Settings->SplitNormalThreshold;
	Op.SplitTangentsThreshold = Settings->SplitTangentsThreshold;
	
	Op.SourceMesh = SourceMesh;
	Op.SelectedEdges = SelectedEdges;

	FTransform LocalToWorld = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	Op.SetTransform(LocalToWorld);
}




#undef LOCTEXT_NAMESPACE

