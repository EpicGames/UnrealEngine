// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleClickTool.h"
#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "PreviewMesh.h"
#include "Properties/MeshMaterialProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "UObject/NoExportTypes.h"
#include "InteractiveToolQueryInterfaces.h"

#include "AddPrimitiveTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class UCombinedTransformGizmo;
class UDragAlignmentMechanic;

/**
 * Builder
 */
UCLASS(MinimalAPI)
class UAddPrimitiveToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	enum class EMakeMeshShapeType : uint32
	{
		Box,
		Cylinder,
		Cone,
		Arrow,
		Rectangle,
		Disc,
		Torus,
		Sphere,
		Stairs,
		Capsule
	};

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	EMakeMeshShapeType ShapeType{EMakeMeshShapeType::Box};
};

/** Placement Target Types */
UENUM()
enum class EMakeMeshPlacementType : uint8
{
	GroundPlane = 0,
	OnScene     = 1,
	AtOrigin	= 2
};

/** Placement Pivot Location */
UENUM()
enum class EMakeMeshPivotLocation : uint8
{
	Base,
	Centered,
	Top
};

/** PolyGroup mode for shape */
UENUM()
enum class EMakeMeshPolygroupMode : uint8
{
	/** One PolyGroup for the entire shape */
	PerShape,
	/** One PolyGroup for each geometric face */
	PerFace,
	/** One PolyGroup for each quad/triangle */
	PerQuad
};

UCLASS(MinimalAPI)
class UProceduralShapeToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** How PolyGroups are assigned to shape primitives. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (ProceduralShapeSetting, DisplayName = "PolyGroup Mode"))
	EMakeMeshPolygroupMode PolygroupMode = EMakeMeshPolygroupMode::PerFace;

	/** How the shape is placed in the scene. */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (DisplayName = "Target Position"))
	EMakeMeshPlacementType TargetSurface = EMakeMeshPlacementType::OnScene;

	/** Location of pivot within the shape */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (ProceduralShapeSetting))
	EMakeMeshPivotLocation PivotLocation = EMakeMeshPivotLocation::Base;

	/** Initial rotation of the shape around its up axis, before placement. After placement, use the gizmo to control rotation. */
	UPROPERTY(EditAnywhere, Category = Positioning, DisplayName = "Initial Rotation", meta = (UIMin = "0.0", UIMax = "360.0", EditCondition = "!bShowGizmoOptions", HideEditConditionToggle))
	float Rotation = 0.0;

	/** If true, aligns the shape along the normal of the surface it is placed on. */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (EditCondition = "TargetSurface == EMakeMeshPlacementType::OnScene"))
	bool bAlignToNormal = true;

	/** Show a gizmo to allow the mesh to be repositioned after the initial placement click. */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (EditCondition = "bShowGizmoOptions", EditConditionHides, HideEditConditionToggle))
	bool bShowGizmo = true;

	//~ Not user visible- used to hide the bShowGizmo option when not yet placed mesh.
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowGizmoOptions = false;
};

UCLASS(MinimalAPI)
class UProceduralBoxToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Width of the box */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Width = 100.f;

	/** Depth of the box */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Depth = 100.f;

	/** Height of the box */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Height = 100.f;

	/** Number of subdivisions along the width */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int WidthSubdivisions = 1;

	/** Number of subdivisions along the depth */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int DepthSubdivisions = 1;

	/** Number of subdivisions along the height */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int HeightSubdivisions = 1;
};

UENUM()
enum class EProceduralRectType
{
	/** Create a rectangle */
	Rectangle,
	/** Create a rounded rectangle */
	RoundedRectangle
};

UCLASS(MinimalAPI)
class UProceduralRectangleToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Type of rectangle */
	UPROPERTY(EditAnywhere, Category = Shape)
	EProceduralRectType RectangleType = EProceduralRectType::Rectangle;

	/** Width of the rectangle */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Width = 100.f;

	/** Depth of the rectangle */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Depth = 100.f;

	/** Number of subdivisions along the width */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int WidthSubdivisions = 1;

	/** Number of subdivisions along the depth */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int DepthSubdivisions = 1;

	/** Whether to preserve the overall Width and Depth for a Rounded Rectangle, or to allow the rounded corners to extend outside those dimensions. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (EditCondition = "RectangleType == EProceduralRectType::RoundedRectangle"))
	bool bMaintainDimension = true;

	/** Radius of rounded corners. This is only available for Rounded Rectangles. */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting,
			EditCondition = "RectangleType == EProceduralRectType::RoundedRectangle"))
	float CornerRadius = 25.f;

	/** Number of radial slices for each rounded corner. This is only available for Rounded Rectangles. */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "3", UIMax = "128", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting,
			EditCondition = "RectangleType == EProceduralRectType::RoundedRectangle"))
	int CornerSlices = 16;
};

UENUM()
enum class EProceduralDiscType
{
	/** Create a disc */
	Disc,
	/** Create a disc with a hole */
	PuncturedDisc
};

UCLASS(MinimalAPI)
class UProceduralDiscToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Type of disc */
	UPROPERTY(EditAnywhere, Category = Shape)
	EProceduralDiscType DiscType = EProceduralDiscType::Disc;

	/** Radius of the disc */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Radius = 50.f;

	/** Number of radial slices for the disc */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "128", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int RadialSlices = 16;

	/** Number of radial subdivisions for each radial slice of the disc */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int RadialSubdivisions = 1;

	/** Radius of the hole in the disc. This is only available for Punctured Discs. */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting,
			EditCondition = "DiscType == EProceduralDiscType::PuncturedDisc"))
	float HoleRadius = 25.f;
};

UCLASS(MinimalAPI)
class UProceduralTorusToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Major radius of the torus, measured from the torus center to the center of the torus tube */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float MajorRadius = 50.f;

	/** Minor radius of the torus, measured from the center ot the torus tube to the tube surface */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float MinorRadius = 25.f;

	/** Number of radial slices along the torus tube */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "128", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int MajorSlices = 16;

	/** Number of radial slices around the torus tube */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "128", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int MinorSlices = 16;
};

UCLASS(MinimalAPI)
class UProceduralCylinderToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:

	/** Radius of the cylinder */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Radius = 50.f;

	/** Height of the cylinder */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Height = 200.f;

	/** Number of radial slices for the cylinder */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "128", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int RadialSlices = 16;

	/** Number of subdivisions along the height of the cylinder */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int HeightSubdivisions = 1;
};

UCLASS(MinimalAPI)
class UProceduralConeToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Radius of the cone */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Radius = 50.f;

	/** Height of the cone */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Height = 200.f;

	/** Number of radial slices for the cone */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "128", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int RadialSlices = 16;

	/** Number of subdivisions along the height of the cone */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int HeightSubdivisions = 1;
};

UCLASS(MinimalAPI)
class UProceduralArrowToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Radius of the arrow shaft */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float ShaftRadius = 20.f;

	/** Height of arrow shaft */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float ShaftHeight = 200.f;

	/** Radius of the arrow head base */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float HeadRadius = 60.f;

	/** Height of arrow head */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float HeadHeight = 120.f;

	/** Number of radial slices for the arrow */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "100", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int RadialSlices = 16;

	/** Number of subdivisions along each part of the arrow, i.e. shaft, head base, head cone */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting))
	int HeightSubdivisions = 1;
};

UENUM()
enum class EProceduralSphereType
{
	/** Create a Sphere with Lat Long parameterization */
	LatLong,
	/** Create a Sphere with Box parameterization */
	Box
};

UCLASS(MinimalAPI)
class UProceduralSphereToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Radius of the sphere */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Radius = 50.f;

	/** Type of subdivision for the sphere */
	UPROPERTY(EditAnywhere, Category = Shape)
	EProceduralSphereType SubdivisionType = EProceduralSphereType::Box;

	/** Number of subdivisions for each side of the sphere. This is only available for spheres with Box subdivision. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "500", ProceduralShapeSetting,
		EditCondition = "SubdivisionType == EProceduralSphereType::Box"))
	int Subdivisions = 16;

	/** Number of horizontal slices of the sphere. This is only available for spheres with Lat Long subdivision. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "100", ClampMin = "4", ClampMax = "500", ProceduralShapeSetting,
		EditCondition = "SubdivisionType == EProceduralSphereType::LatLong"))
	int HorizontalSlices = 16;

	/** Number of vertical slices of the sphere. This is only available for spheres with Lat Long subdivision. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "100", ClampMin = "4", ClampMax = "500", ProceduralShapeSetting,
		EditCondition = "SubdivisionType == EProceduralSphereType::LatLong"))
	int VerticalSlices = 16;
};

UCLASS(MinimalAPI)
class UProceduralCapsuleToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Radius of the capsule */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float Radius = 25.f;

	/** Length of cylindrical section of the capsule */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float CylinderLength = 50.f;

	/** Number of slices of the hemispherical end caps. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "2", UIMax = "100", ClampMin = "2", ClampMax = "500", ProceduralShapeSetting))
	int HemisphereSlices = 8;

	/** Number of radial slices of the cylindrical section. */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "100", ClampMin = "3", ClampMax = "500", ProceduralShapeSetting))
	int CylinderSlices = 16;

	/** Number of lengthwise subdivisions along cylindrical section */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "500", ProceduralShapeSetting))
	int CylinderSubdivisions = 1;
};

UENUM()
enum class EProceduralStairsType
{
	/** Create a linear staircase */
	Linear,
	/** Create a floating staircase */
	Floating,
	/** Create a curved staircase */
	Curved,
	/** Create a spiral staircase */
	Spiral
};

UCLASS(MinimalAPI)
class UProceduralStairsToolProperties : public UProceduralShapeToolProperties
{
	GENERATED_BODY()

public:
	/** Type of staircase */
	UPROPERTY(EditAnywhere, Category = Shape)
	EProceduralStairsType StairsType = EProceduralStairsType::Linear;

	/** Number of steps */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (DisplayName = "Number of Steps", UIMin = "2", UIMax = "100", ClampMin = "2", ClampMax = "1000000", ProceduralShapeSetting))
	int NumSteps = 8;

	/** Width of each step */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float StepWidth = 150.0f;

	/** Height of each step */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting))
	float StepHeight = 20.0f;

	/** Depth of each step of linear stairs */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting,
			EditCondition =	"StairsType == EProceduralStairsType::Linear || StairsType == EProceduralStairsType::Floating"))
	float StepDepth = 30.0f;

	/** Angular length of curved stairs. Positive values are for clockwise and negative values are for counterclockwise. */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "-360.0", UIMax = "360.0", ClampMin = "-360.0", ClampMax = "360.0", ProceduralShapeSetting,
			EditCondition =	"StairsType == EProceduralStairsType::Curved"))
	float CurveAngle = 90.0f;

	/** Angular length of spiral stairs. Positive values are for clockwise and negative values are for counterclockwise. */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "-720.0", UIMax = "720.0", ClampMin = "-360000.0", ClampMax = "360000.0", ProceduralShapeSetting,
			EditCondition =	"StairsType == EProceduralStairsType::Spiral"))
	float SpiralAngle = 90.0f;

	/** Inner radius of curved and spiral stairs */
	UPROPERTY(EditAnywhere, Category = Shape,
		meta = (UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.0001", ClampMax = "1000000.0", ProceduralShapeSetting,
			EditCondition = "StairsType == EProceduralStairsType::Curved || StairsType == EProceduralStairsType::Spiral"))
	float InnerRadius = 150.0f;
};


/**
 * Base tool to create primitives
 */
UCLASS(MinimalAPI)
class UAddPrimitiveTool : public USingleClickTool, public IHoverBehaviorTarget, public IInteractiveToolCameraFocusAPI
{
	GENERATED_BODY()

public:
	UE_API explicit UAddPrimitiveTool(const FObjectInitializer&);

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// USingleClickTool
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget interface
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual bool SupportsWorldSpaceFocusBox() override;
	UE_API virtual FBox GetWorldSpaceFocusBox() override;
	UE_API virtual bool SupportsWorldSpaceFocusPoint() override;
	UE_API virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) override;


protected:
	virtual EMakeMeshPolygroupMode GetDefaultPolygroupMode() const { return EMakeMeshPolygroupMode::PerQuad; }

	enum class EState
	{
		PlacingPrimitive,
		AdjustingSettings
	};

	EState CurrentState = EState::PlacingPrimitive;
	UE_API void SetState(EState NewState);

	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const {}
	virtual UProceduralShapeToolProperties* CreateShapeSettings(){return nullptr;}

	UE_API virtual void GenerateAsset();

	/** Property set for type of output object (StaticMesh, Volume, etc) */
	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<UProceduralShapeToolProperties> ShapeSettings;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> Gizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UPROPERTY()
	FString AssetName = TEXT("GeneratedAsset");

	UWorld* TargetWorld;

	UE_API void UpdatePreviewPosition(const FInputDeviceRay& ClickPos);
	UE::Geometry::FFrame3d ShapeFrame;

	UE_API void UpdatePreviewMesh() const;

	UE_API void UpdateTargetSurface();

	// @return true if the primitive needs to be centered in the XY plane when placed.
	virtual bool ShouldCenterXY() const
	{
		// Most primitives are already XY centered, and re-centering them only introduces issues at very low samplings where the bounds center is offset from the intended center.
		return false;
	}

	// Used to make the initial placement of the mesh undoable
	class FStateChange : public FToolCommandChange
	{
	public:
		FStateChange(const FTransform& MeshTransformIn)
			: MeshTransform(MeshTransformIn)
		{
		}

		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;
		virtual FString ToString() const override
		{
			return TEXT("UAddPrimitiveTool::FStateChange");
		}

	protected:
		FTransform MeshTransform;
	};
};


UCLASS()
class UAddBoxPrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddBoxPrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddCylinderPrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddCylinderPrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddCapsulePrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddCapsulePrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddConePrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddConePrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddRectanglePrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddRectanglePrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddDiscPrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddDiscPrimitiveTool(const FObjectInitializer& ObjectInitializer);
	virtual void Setup() override;
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddTorusPrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddTorusPrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddArrowPrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddArrowPrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddSpherePrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddSpherePrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual EMakeMeshPolygroupMode GetDefaultPolygroupMode() const override 
	{ 
		return EMakeMeshPolygroupMode::PerFace; 
	}
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
};

UCLASS()
class UAddStairsPrimitiveTool : public UAddPrimitiveTool
{
	GENERATED_BODY()
public:
	explicit UAddStairsPrimitiveTool(const FObjectInitializer& ObjectInitializer);
protected:
	virtual void GenerateMesh(FDynamicMesh3* OutMesh) const override;
	virtual bool ShouldCenterXY() const override
	{
		return true;
	}
};


#undef UE_API
