// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/BaseCreateFromSelectedTool.h"
#include "CompositionOps/BooleanMeshesOp.h"
#include "Drawing/LineSetComponent.h"
#include "UObject/NoExportTypes.h"

#include "CSGMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

// Forward declarations
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);


/**
 * Standard properties of the CSG operation
 */
UCLASS(MinimalAPI)
class UCSGMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Type of Boolean operation */
	UPROPERTY(EditAnywhere, Category = Boolean)
	ECSGOperation Operation = ECSGOperation::DifferenceAB;

	/** Try to fill holes created by the Boolean operation, e.g. due to numerical errors */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay)
	bool bTryFixHoles = false;

	/** Try to collapse extra edges created by the Boolean operation */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay)
	bool bTryCollapseEdges = true;

	/** Threshold to determine whether a triangle in one mesh is inside or outside of the other */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float WindingThreshold = 0.5;

	/** Show boundary edges created by the Boolean operation, which might happen due to numerical errors */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowNewBoundaries = true;

	/** Show a translucent version of the subtracted mesh, to help visualize geometry that is being removed */
	UPROPERTY(EditAnywhere, Category = Display, meta = (EditCondition = "Operation == ECSGOperation::DifferenceAB || Operation == ECSGOperation::DifferenceBA"))
	bool bShowSubtractedMesh = true;
	
	/** Opacity of the translucent subtracted mesh */
	UPROPERTY(EditAnywhere, Category = Display, meta = (DisplayName = "Opacity Subtracted Mesh", ClampMin = "0", ClampMax = "1",
		EditCondition = "bShowSubtractedMesh && Operation == ECSGOperation::DifferenceAB || bShowSubtractedMesh && Operation == ECSGOperation::DifferenceBA"))
	float SubtractedMeshOpacity = 0.2f;
	
	/** Color of the translucent subtracted mesh */
	UPROPERTY(EditAnywhere, Category = Display, meta = (DisplayName = "Color Subtracted Mesh", HideAlphaChannel, ClampMin = "0", ClampMax = "1",
		EditCondition = "bShowSubtractedMesh && Operation == ECSGOperation::DifferenceAB || bShowSubtractedMesh && Operation == ECSGOperation::DifferenceBA"))
	FLinearColor SubtractedMeshColor = FLinearColor::Black;

	/** If true, only the first mesh will keep its material assignments, and all other faces will have the first material assigned */
	UPROPERTY(EditAnywhere, Category = Materials)
	bool bUseFirstMeshMaterials = false;
};


/**
 * Properties of the trim mode
 */
UCLASS(MinimalAPI)
class UTrimMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Which object to trim */
	UPROPERTY(EditAnywhere, Category = Operation)
	ETrimOperation WhichMesh = ETrimOperation::TrimA;

	/** Whether to remove the surface inside or outside of the trimming geometry */
	UPROPERTY(EditAnywhere, Category = Operation)
	ETrimSide TrimSide = ETrimSide::RemoveInside;

	/** Threshold to determine whether a triangle in one mesh is inside or outside of the other */
	UPROPERTY(EditAnywhere, Category = Operation, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float WindingThreshold = 0.5;

	/** Whether to show a translucent version of the trimming mesh, to help visualize what is being cut */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowTrimmingMesh = true;

	/** Opacity of translucent version of the trimming mesh */
	UPROPERTY(EditAnywhere, Category = Display, meta = (EditCondition = "bShowTrimmingMesh", ClampMin = "0", ClampMax = "1"))
	float OpacityOfTrimmingMesh = .2f;

	/** Color of translucent version of the trimming mesh */
	UPROPERTY(EditAnywhere, Category = Display, meta = (EditCondition = "bShowTrimmingMesh"), AdvancedDisplay)
	FLinearColor ColorOfTrimmingMesh = FLinearColor::Black;

};


/**
 * Simple Mesh Plane Cutting Tool
 */
UCLASS(MinimalAPI)
class UCSGMeshesTool : public UBaseCreateFromSelectedTool
{
	GENERATED_BODY()

public:

	UCSGMeshesTool() {}

	UE_API void EnableTrimMode();

protected:

	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh = true) override;

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;
	UE_API virtual void SetPreviewCallbacks() override;

	UE_API virtual FString GetCreatedAssetName() const;
	UE_API virtual FText GetActionName() const;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UE_API void UpdateVisualization();

	UPROPERTY()
	TObjectPtr<UCSGMeshesToolProperties> CSGProperties;

	UPROPERTY()
	TObjectPtr<UTrimMeshesToolProperties> TrimProperties;

	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> OriginalDynamicMeshes;

	UPROPERTY()
	TArray<TObjectPtr<UPreviewMesh>> OriginalMeshPreviews;

	// Material used to show the otherwise-invisible cutting/trimming mesh
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> PreviewsGhostMaterial;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnLineSet;

	// for visualization of any errors in the currently-previewed CSG operation
	TArray<int> CreatedBoundaryEdges;

	bool bTrimMode = false;

	UE_API virtual int32 GetHiddenGizmoIndex() const;

	// Update visibility of ghostly preview meshes (used to show trimming or subtracting surface)
	UE_API void UpdatePreviewsVisibility();

	// update the material of ghostly preview meshes (used to show trimming or subtracting surface)
	UE_API void UpdatePreviewsMaterial();

	virtual bool KeepCollisionFrom(int32 TargetIdx) const override
	{
		if (bTrimMode)
		{
			return static_cast<int32>(TrimProperties->WhichMesh) == TargetIdx;
		}
		else if (CSGProperties->Operation == ECSGOperation::DifferenceAB)
		{
			return TargetIdx == 0;
		}
		else if (CSGProperties->Operation == ECSGOperation::DifferenceBA)
		{
			return TargetIdx == 1;
		}
		return true;
	}
};


UCLASS(MinimalAPI)
class UCSGMeshesToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:

	bool bTrimMode = false;

	virtual TOptional<int32> MaxComponentsSupported() const override
	{
		return TOptional<int32>(2);
	}

	virtual int32 MinComponentsSupported() const override
	{
		return 2;
	}

	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		UCSGMeshesTool* Tool = NewObject<UCSGMeshesTool>(SceneState.ToolManager);
		if (bTrimMode)
		{
			Tool->EnableTrimMode();
		}
		return Tool;
	}
};

#undef UE_API
