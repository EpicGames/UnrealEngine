// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolExclusiveToolAPI
#include "MeshOpPreviewHelpers.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "BaseTools/BaseCreateFromSelectedTool.h"
#include "PropertySets/OnAcceptProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "CombineMeshesTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

// Forward declarations
struct FMeshDescription;

/**
 *
 */
UCLASS(MinimalAPI)
class UCombineMeshesToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	bool bIsDuplicateTool = false;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;

};


/**
 * Common properties
 */
UCLASS(MinimalAPI)
class UCombineMeshesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(meta = (TransientToolProperty))
	bool bIsDuplicateMode = false;

	/** Defines the object the tool output is written to. */
	UPROPERTY(EditAnywhere, Category = OutputObject, meta = (DisplayName = "Write To",
		EditCondition = "bIsDuplicateMode == false", EditConditionHides, HideEditConditionToggle))
	EBaseCreateFromSelectedTargetType OutputWriteTo = EBaseCreateFromSelectedTargetType::NewObject;

	/** Base name of the newly generated object to which the output is written to. */
	UPROPERTY(EditAnywhere, Category = OutputObject, meta = (TransientToolProperty, DisplayName = "Name",
		EditCondition = "bIsDuplicateMode || OutputWriteTo == EBaseCreateFromSelectedTargetType::NewObject", EditConditionHides, NoResetToDefault))
	FString OutputNewName;

	/** Name of the existing object to which the output is written to. */
	UPROPERTY(VisibleAnywhere, Category = OutputObject, meta = (TransientToolProperty, DisplayName = "Name",
		EditCondition = "bIsDuplicateMode == false && OutputWriteTo != EBaseCreateFromSelectedTargetType::NewObject", EditConditionHides, NoResetToDefault))
	FString OutputExistingName;
};


/**
 * Simple tool to combine multiple meshes into a single mesh asset
 */
UCLASS(MinimalAPI)
class UCombineMeshesTool : public UMultiSelectionMeshEditingTool, 
	// Disallow auto-accept switch-away because it's easy to accidentally make an extra asset in duplicate mode,
	// and it's not great in combine mode either.
	public IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void SetDuplicateMode(bool bDuplicateMode);

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

protected:

	UPROPERTY()
	TObjectPtr<UCombineMeshesToolProperties> BasicProperties;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<UOnAcceptHandleSourcesPropertiesBase> HandleSourceProperties;

	bool bDuplicateMode;

	UE_API void CreateNewAsset();
	UE_API void UpdateExistingAsset();

	UE_API void BuildCombinedMaterialSet(TArray<UMaterialInterface*>& NewMaterialsOut, TArray<TArray<int32>>& MaterialIDRemapsOut);
};

#undef UE_API
