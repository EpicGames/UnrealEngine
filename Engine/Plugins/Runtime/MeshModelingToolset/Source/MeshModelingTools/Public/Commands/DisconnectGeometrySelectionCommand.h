// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selection/SelectionEditInteractiveCommand.h"
#include "DisconnectGeometrySelectionCommand.generated.h"

#define UE_API MESHMODELINGTOOLS_API


/**
 * UDisconnectGeometrySelectionCommand disconnects the geometric elements identified by the Selection.
 * Currently only supports mesh selections (Triangle and Polygroup types)
 * Disconnects selected faces, or faces connected to selected edges, or faces connected to selected vertices.
 */
UCLASS(MinimalAPI)
class UDisconnectGeometrySelectionCommand : public UGeometrySelectionEditCommand
{
	GENERATED_BODY()
public:

	UE_API virtual FText GetCommandShortString() const override;

	UE_API virtual bool CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments) override;
	UE_API virtual void ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments, UInteractiveCommandResult** Result) override;
};

#undef UE_API
