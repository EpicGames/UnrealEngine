// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selection/SelectionEditInteractiveCommand.h"
#include "ModifyGeometrySelectionCommand.generated.h"

#define UE_API MESHMODELINGTOOLS_API


/**
 * UModifyGeometrySelectionCommand updates/edits the current selection in various ways.
 * Default operation is to Select All.
 * Subclasses below can be used in situations where specific per-modification types are needed.
 */
UCLASS(MinimalAPI)
class UModifyGeometrySelectionCommand : public UGeometrySelectionEditCommand
{
	GENERATED_BODY()
public:

	enum class EModificationType
	{
		SelectAll = 0,
		ExpandToConnected = 1,

		Invert = 10,
		InvertConnected = 11,

		Expand = 20,
		Contract = 21
	};
	virtual EModificationType GetModificationType() const { return EModificationType::SelectAll; }

	virtual bool AllowEmptySelection() const override { return GetModificationType() == EModificationType::SelectAll || GetModificationType() == EModificationType::Invert; }

	virtual bool IsModifySelectionCommand() const override { return true; }
	UE_API virtual FText GetCommandShortString() const override;

	UE_API virtual bool CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments) override;
	UE_API virtual void ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments, UInteractiveCommandResult** Result) override;
};


/**
 * Command to Invert the current Selection
 */
UCLASS(MinimalAPI)
class UModifyGeometrySelectionCommand_Invert : public UModifyGeometrySelectionCommand
{
	GENERATED_BODY()
public:
	virtual EModificationType GetModificationType() const override { return EModificationType::Invert; }

};

/**
 * Command to Expand the current Selection to all connected geometry
 */
UCLASS(MinimalAPI)
class UModifyGeometrySelectionCommand_ExpandToConnected : public UModifyGeometrySelectionCommand
{
	GENERATED_BODY()
public:
	virtual EModificationType GetModificationType() const override { return EModificationType::ExpandToConnected; }

};

/**
 * Command to Invert the current Selection, only considering connected geometry
 */
UCLASS(MinimalAPI)
class UModifyGeometrySelectionCommand_InvertConnected : public UModifyGeometrySelectionCommand
{
	GENERATED_BODY()
public:
	virtual EModificationType GetModificationType() const override { return EModificationType::InvertConnected; }
};

/**
 * Command to Expand the current Selection by a one-ring
 */
UCLASS(MinimalAPI)
class UModifyGeometrySelectionCommand_Expand : public UModifyGeometrySelectionCommand
{
	GENERATED_BODY()
public:
	virtual EModificationType GetModificationType() const override { return EModificationType::Expand; }
};

/**
 * Command to Contract the current Selection by a one-ring
 */
UCLASS(MinimalAPI)
class UModifyGeometrySelectionCommand_Contract : public UModifyGeometrySelectionCommand
{
	GENERATED_BODY()
public:
	virtual EModificationType GetModificationType() const override { return EModificationType::Contract; }
};

#undef UE_API
