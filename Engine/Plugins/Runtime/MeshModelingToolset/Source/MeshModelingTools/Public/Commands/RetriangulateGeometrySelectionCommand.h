// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selection/SelectionEditInteractiveCommand.h"
#include "RetriangulateGeometrySelectionCommand.generated.h"

#define UE_API MESHMODELINGTOOLS_API


/**
 * URetriangulateGeometrySelectionCommand 
 */
UCLASS(MinimalAPI)
class URetriangulateGeometrySelectionCommand : public UGeometrySelectionEditCommand
{
	GENERATED_BODY()
public:

	virtual bool AllowEmptySelection() const override { return true; }
	UE_API virtual FText GetCommandShortString() const override;

	UE_API virtual bool CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments) override;
	UE_API virtual void ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments, UInteractiveCommandResult** Result) override;
};

#undef UE_API
