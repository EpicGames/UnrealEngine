// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "PreviewMesh.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "ConvertToPolygonsTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

// predeclaration
class UConvertToPolygonsTool;
class FConvertToPolygonsOp;
class UPreviewGeometry;
namespace UE::Geometry
{
	class FDynamicMesh3;
	struct FDynamicSubmesh3;
	class ITriangleGroupSetAdapter;
}

/**
 *
 */
UCLASS(MinimalAPI)
class UConvertToPolygonsToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState & SceneState) const override;
	virtual bool RequiresInputSelection() const override { return false; }
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};




UENUM()
enum class EConvertToPolygonsMode
{
	/** Convert based on Angle Tolerance between Face Normals */
	FaceNormalDeviation = 0 UMETA(DisplayName = "Face Normal Deviation"),
	/** Create Polygroups by merging triangle pairs into Quads */
	FindPolygons = 1 UMETA(DisplayName = "Find Quads"),
	/** Create PolyGroups based on Material IDs */
	FromMaterialIDs = 7 UMETA(DisplayName = "From Material IDs"),
	/** Create PolyGroups based on UV Islands */
	FromUVIslands = 2 UMETA(DisplayName = "From UV Islands"),
	/** Create PolyGroups based on Hard Normal Seams */
	FromNormalSeams = 3 UMETA(DisplayName = "From Hard Normal Seams"),
	/** Create Polygroups based on Connected Triangles */
	FromConnectedTris = 4 UMETA(DisplayName = "From Connected Tris"),
	/** Create Polygroups centered on well-spaced sample points, approximating a surface Voronoi diagram */
	FromFurthestPointSampling = 5 UMETA(DisplayName = "Furthest Point Sampling"),
	/** Copy from existing Polygroup Layer */
	CopyFromLayer = 6 UMETA(DisplayName = "Copy From Layer")
};

UENUM()
enum class EPerTriangleAttributeType
{
	/** Create per-triangle integer attribute. */
	Integer = 0 UMETA(DisplayName = "Int32"),
	/** Create per-triangle label attribute. */
	Label = 1 UMETA(DisplayName = "FName")
};



UCLASS(MinimalAPI)
class UConvertToPolygonsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Strategy to use to group triangles */
	UPROPERTY(EditAnywhere, Category = PolyGroups)
	EConvertToPolygonsMode ConversionMode = EConvertToPolygonsMode::FaceNormalDeviation;

	/** Tolerance for planarity */
	UPROPERTY(EditAnywhere, Category = NormalDeviation, meta = (UIMin = "0.001", UIMax = "60.0", ClampMin = "0.0", ClampMax = "90.0", EditCondition = "ConversionMode == EConvertToPolygonsMode::FaceNormalDeviation", EditConditionHides))
	float AngleTolerance = 0.1f;

	/** Whether to compute Face Normal Deviation based on the overall PolyGroup's average normal, or to only consider the normals of the individual triangles */
	UPROPERTY(EditAnywhere, Category = NormalDeviation, meta = (EditCondition = "ConversionMode == EConvertToPolygonsMode::FaceNormalDeviation", EditConditionHides))
	bool bUseAverageGroupNormal = true;

	/** Furthest-Point Sample count, approximately this number of polygroups will be generated */
	UPROPERTY(EditAnywhere, Category = FurthestPoint, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "10000", EditCondition = "ConversionMode == EConvertToPolygonsMode::FromFurthestPointSampling", EditConditionHides))
	int32 NumPoints = 100;

	/** If enabled, then furthest-point sampling happens with respect to existing Polygroups, ie the existing groups are further subdivided */
	UPROPERTY(EditAnywhere, Category = FurthestPoint, meta = (EditCondition = "ConversionMode == EConvertToPolygonsMode::FromFurthestPointSampling", EditConditionHides))
	bool bSplitExisting = false;

	/** If true, region-growing in Sampling modes will be controlled by face normals, resulting in regions with borders that are more-aligned with curvature ridges */
	UPROPERTY(EditAnywhere, Category = FurthestPoint, meta = (EditCondition = "ConversionMode == EConvertToPolygonsMode::FromFurthestPointSampling", EditConditionHides))
	bool bNormalWeighted = true;

	/** This parameter modulates the effect of normal weighting during region-growing */
	UPROPERTY(EditAnywhere, Category = FurthestPoint, meta = (UIMin = "0.1", UIMax = "2.0", ClampMin = "0.01", ClampMax = "100.0", EditCondition = "ConversionMode == EConvertToPolygonsMode::FromFurthestPointSampling", EditConditionHides))
	float NormalWeighting = 1.0f;


	/** Bias for Quads that are adjacent to already-discovered Quads. Set to 0 to disable.  */
	UPROPERTY(EditAnywhere, Category = FindQuads, meta = (UIMin = 0, UIMax = 5, EditCondition = "ConversionMode == EConvertToPolygonsMode::FindPolygons", EditConditionHides))
	float QuadAdjacencyWeight = 1.0;

	/** Set to values below 1 to ignore less-likely triangle pairings */
	UPROPERTY(EditAnywhere, Category = FindQuads, meta = (AdvancedDisplay, UIMin = 0, UIMax = 1, EditCondition = "ConversionMode == EConvertToPolygonsMode::FindPolygons", EditConditionHides))
	float QuadMetricClamp = 1.0;

	/** Iteratively repeat quad-searching in uncertain areas, to try to slightly improve results */
	UPROPERTY(EditAnywhere, Category = FindQuads, meta = (AdvancedDisplay, UIMin = 1, UIMax = 5, EditCondition = "ConversionMode == EConvertToPolygonsMode::FindPolygons", EditConditionHides))
	int QuadSearchRounds = 1;

	/** If true, polygroup borders will not cross existing UV seams */
	UPROPERTY(EditAnywhere, Category = Topology, meta = (EditCondition = "ConversionMode == EConvertToPolygonsMode::FaceNormalDeviation || ConversionMode == EConvertToPolygonsMode::FindPolygons", EditConditionHides))
	bool bRespectUVSeams = false;

	/** If true, polygroup borders will not cross existing hard normal seams */
	UPROPERTY(EditAnywhere, Category = Topology, meta = (EditCondition = "ConversionMode == EConvertToPolygonsMode::FaceNormalDeviation || ConversionMode == EConvertToPolygonsMode::FindPolygons", EditConditionHides))
	bool bRespectHardNormals = false;


	/** Minimum number of triangles to include in a group. Any group containing fewer triangles will be merged with an adjacent group (if possible). */
	UPROPERTY(EditAnywhere, Category = Filtering, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "10000", EditCondition = "ConversionMode != EConvertToPolygonsMode::CopyFromLayer"))
	int32 MinGroupSize = 2;
	
	/** Display each group with a different auto-generated color */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowGroupColors = true;

	/** If true, normals are recomputed per-group, with hard edges at group boundaries */
	UPROPERTY(EditAnywhere, Category = Output, meta = (EditCondition = "ConversionMode != EConvertToPolygonsMode::CopyFromLayer"))
	bool bCalculateNormals = false;

	/** Select PolyGroup layer to use. */
	UPROPERTY(EditAnywhere, Category = Output, meta = (DisplayName = "Output Layer", GetOptions = GetGroupOptionsList, NoResetToDefault))
	FName GroupLayer = "Default";

	// Provides set of available group layers
	UFUNCTION()
	TArray<FString> GetGroupOptionsList() { return OptionsList; }

	// internal list used to implement above
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> OptionsList;

	UPROPERTY(meta = (TransientToolProperty))
	bool bShowNewLayerName = false;

	/** Name of the new Group Layer */
	UPROPERTY(EditAnywhere, Category = Output, meta = (TransientToolProperty, DisplayName = "New Layer Name",
		EditCondition = "bShowNewLayerName", HideEditConditionToggle, NoResetToDefault))
	FString NewLayerName = TEXT("polygroups");

	/** Type of the new Group Layer */
	UPROPERTY(EditAnywhere, Category = Output, meta = (TransientToolProperty,
		EditCondition = "bShowNewLayerName", HideEditConditionToggle, NoResetToDefault))
	EPerTriangleAttributeType LayerType = EPerTriangleAttributeType::Integer;
};




UCLASS(MinimalAPI)
class UConvertToPolygonsOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UConvertToPolygonsTool> ConvertToPolygonsTool;  // back pointer
};

/**
 *
 */
UCLASS(MinimalAPI)
class UConvertToPolygonsTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()

public:
	UE_API UConvertToPolygonsTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	// update parameters in ConvertToPolygonsOp based on current Settings
	UE_API void UpdateOpParameters(FConvertToPolygonsOp& ConvertToPolygonsOp) const;

protected:
	
	UE_API void OnSettingsModified();

protected:
	UPROPERTY()
	TObjectPtr<UConvertToPolygonsToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> CopyFromLayerProperties = nullptr;


	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> PreviewCompute = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry = nullptr;

	// If a selection was provided (bUsingSelection = true), UnmodifiedAreaPreviewMesh is used to render the unmodified (non-selected) part of the input mesh,
	// as the PreviewCompute input mesh will be limited to the input selected area
	UPROPERTY()
	TObjectPtr<UPreviewMesh> UnmodifiedAreaPreviewMesh = nullptr;

protected:
	// copy of the input mesh (possibly does not need to be SharedPtr anymore now that FSharedConstDynamicMesh3 is used below, but then needs PimplPtr...)
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalDynamicMesh;

	// If there is an active selection, bUsingSelection will be true and the ROI and Submesh will be initialized
	bool bUsingSelection = false;
	TSharedPtr<TSet<int32>> SelectionTriangleROI;
	TSharedPtr<UE::Geometry::FDynamicSubmesh3, ESPMode::ThreadSafe> OriginalSubmesh;

	// The mesh passed to the Compute operator to base result on, will either be OriginalDynamicMesh or the submesh from OriginalSubmesh if using a selection
	TSharedPtr<UE::Geometry::FSharedConstDynamicMesh3> ComputeOperatorSharedMesh;

	// current set of detected polygroup edges, relative to ComputeOperatorSharedMesh
	TArray<int> PolygonEdges;
	
	UE_API void UpdateVisualization();

	// current input group set used in CopyFromLayer mode, relative to ComputeOperatorSharedMesh
	TSharedPtr<UE::Geometry::ITriangleGroupSetAdapter, ESPMode::ThreadSafe> ActiveFromGroupSetAdapter;
	UE_API void OnSelectedFromGroupLayerChanged();
	UE_API void UpdateFromGroupLayer();
	
	// updates the group layer data to the mesh 
	void DoUpdateMeshAttribute(const FDynamicMesh3& InMesh) const;
};

#undef UE_API
