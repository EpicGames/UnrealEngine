// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "Drawing/LineSetComponent.h"
#include "BaseTools/BaseCreateFromSelectedTool.h"

#include "CutMeshWithMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

// Forward declarations
class UPreviewMesh;
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);


/**
 * Standard properties of the CutMeshWithMesh operation
 */
UCLASS(MinimalAPI)
class UCutMeshWithMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Try to fill holes created by the Boolean operation, e.g. due to numerical errors */
	UPROPERTY(EditAnywhere, Category = Boolean)
	bool bTryFixHoles = false;

	/** Try to collapse extra edges created by the Boolean operation */
	UPROPERTY(EditAnywhere, Category = Boolean)
	bool bTryCollapseEdges = true;

	/** Threshold to determine whether a triangle in one mesh is inside or outside of the other */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float WindingThreshold = 0.5;

	/** Show boundary edges created by the Boolean operation, which might happen due to numerical errors */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowNewBoundaries = true;

	/** If true, only the first mesh will keep its material assignments, and all other faces will have the first material assigned */
	UPROPERTY(EditAnywhere, Category = Materials)
	bool bUseFirstMeshMaterials = false;
};


/**
 * UCutMeshWithMeshTool cuts an input mesh into two pieces based on a second input mesh.
 * Essentially this just both a Boolean Subtract and a Boolean Intersection. However
 * doing those as two separate operations involves quite a few steps, so this Tool
 * does it in a single step and with some improved efficiency.
 */
UCLASS(MinimalAPI)
class UCutMeshWithMeshTool : public UBaseCreateFromSelectedTool
{
	GENERATED_BODY()

public:

	UCutMeshWithMeshTool() {}

protected:

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh = true) override;

	UE_API virtual void SetupProperties() override;
	UE_API virtual void SaveProperties() override;
	UE_API virtual void SetPreviewCallbacks() override;

	UE_API virtual FString GetCreatedAssetName() const override;
	UE_API virtual FText GetActionName() const override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API void UpdateVisualization();

	UPROPERTY()
	TObjectPtr<UCutMeshWithMeshToolProperties> CutProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> IntersectPreviewMesh;

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalTargetMesh;
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalCuttingMesh;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnLineSet;

	// for visualization of any errors in the currently-previewed CSG operation
	TArray<int> CreatedSubtractBoundaryEdges;
	TArray<int> CreatedIntersectBoundaryEdges;

	FDynamicMesh3 IntersectionMesh;
};




UCLASS(MinimalAPI)
class UCutMeshWithMeshToolBuilder : public UBaseCreateFromSelectedToolBuilder
{
	GENERATED_BODY()

public:
	virtual TOptional<int32> MaxComponentsSupported() const override
	{
		return TOptional<int32>(2);
	}

	virtual int32 MinComponentsSupported() const override
	{
		return 2;
	}

	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override
	{
		return NewObject<UCutMeshWithMeshTool>(SceneState.ToolManager);
	}
};



#undef UE_API
