// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "ToolDataVisualizer.h"
#include "Transforms/QuickAxisTranslater.h"
#include "Transforms/QuickAxisRotator.h"
#include "Changes/MeshVertexChange.h"
#include "GroupTopology.h"
#include "Spatial/GeometrySet3.h"
#include "Selection/GroupTopologySelector.h"
#include "Operations/GroupTopologyDeformer.h"
#include "Solvers/MeshLaplacian.h"
#include "TransactionUtil.h"

#include "DeformMeshPolygonsTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class FMeshVertexChangeBuilder;
class FGroupTopologyLaplacianDeformer;
struct FDeformerVertexConstraintData;

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UDeformMeshPolygonsToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:

	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

/** Deformation strategies */
UENUM()
enum class EGroupTopologyDeformationStrategy : uint8
{
	/** Deforms the mesh using linear deformation. */
	Linear UMETA(DisplayName = "Linear"),

	/** Deforms the mesh using Laplacian deformation. */
	Laplacian UMETA(DisplayName = "Smooth")
};

/** Laplacian weight schemes determine how we will look at the curvature at a given vertex in relation to its neighborhood*/
UENUM()
enum class EWeightScheme 
{
	Uniform				UMETA(DisplayName = "Uniform"),
	Umbrella			UMETA(DisplayName = "Umbrella"),
	Valence				UMETA(DisplayName = "Valence"),
	MeanValue			UMETA(DisplayName = "MeanValue"),
	Cotangent			UMETA(DisplayName = "Cotangent"),
	ClampedCotangent	UMETA(DisplayName = "ClampedCotangent"),
	IDTCotangent        UMETA(DisplayName = "IDTCotangent")
};

/** The ELaplacianWeightScheme enum is the same..*/
static ELaplacianWeightScheme ConvertToLaplacianWeightScheme(const EWeightScheme WeightScheme)
{
	return static_cast<ELaplacianWeightScheme>(WeightScheme);
}


/** Modes for quick transformer */
UENUM()
enum class EQuickTransformerMode : uint8
{
	/** Translation along axes */
	AxisTranslation = 0 UMETA(DisplayName = "Translate"),

	/** Rotation around axes */
	AxisRotation = 1 UMETA(DisplayName = "Rotate"),
};




UCLASS(MinimalAPI)
class UDeformMeshPolygonsTransformProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API UDeformMeshPolygonsTransformProperties();

	//Options

	/** Type of deformation used. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Deformation"))
	EGroupTopologyDeformationStrategy DeformationStrategy;

	/** Type of transformation used. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Transformation"))
	EQuickTransformerMode TransformMode;

	/** Allow for faces (PolyGroups) to be selected. */
	UPROPERTY(EditAnywhere, Category = Selection)
	bool bSelectFaces;

	/** Allow for edges to be selected. */
	UPROPERTY(EditAnywhere, Category = Selection)
	bool bSelectEdges;

	/** Allow for vertices to be selected. */
	UPROPERTY(EditAnywhere, Category = Selection)
	bool bSelectVertices;

	/** If true, overlays preview with wireframe. */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowWireframe;

	//Laplacian Deformation Options, currently not exposed.

	UPROPERTY() 
	EWeightScheme SelectedWeightScheme = EWeightScheme::IDTCotangent;

	UPROPERTY() 
	double HandleWeight = 1000.0;

	UPROPERTY()
	bool bPostFixHandles = false;
/**
	// How to add a weight curve
	UPROPERTY(EditAnywhere, Category = LaplacianOptions, meta = (EditCondition="DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian", DisplayName = "Localize Deformation", ToolTip = "When enabled, only the vertices in the polygroups immediately adjacent to the selected group will be affected by the deformation.\nWhen disabled, the deformer will solve for the curvature of the entire mesh (slower)"))
	bool bLocalizeDeformation{ true};

	UPROPERTY(EditAnywhere, Category = LaplacianOptions, meta = (EditCondition="DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian", DisplayName = "Apply Weight Attenuation Curve", ToolTip = "When enabled, the curve below will be used to calculate the weight at a given vertex based on distance from the handles"))
	bool bApplyAttenuationCurve{ false };
	
	FRichCurve DefaultFalloffCurve;

	UPROPERTY(EditAnywhere,  Category = LaplacianOptions, meta = ( EditCondition="DeformationStrategy == EGroupTopologyDeformationStrategy::Laplacian && bApplyAttenuationCurve", DisplayName = "Distance-Weight Attenuation Curve",UIMin = "1.0", UIMax = "1.0", ClampMin = "1.0", ClampMax = "1.0", ToolTip = "This curve determines the weight attenuation over the distance of the mesh.\nThe selected polygroup handle is t=0.0, and t=1.0 is roughly the farthest vertices from the handles.\nThe value of the curve at each time interval represents the weight of the vertices at that distance from the selection."))
	FRuntimeFloatCurve WeightAttenuationCurve;
*/ 
};


/**
 *
 */
UCLASS(MinimalAPI)
class UDeformMeshPolygonsTool : public UMeshSurfacePointTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API UDeformMeshPolygonsTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual void OnCancelDrag() override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

public:
	UE_API virtual void NextTransformTypeAction();

	//
	float VisualAngleSnapThreshold = 0.5;

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;

	UPROPERTY()
	TObjectPtr<UDeformMeshPolygonsTransformProperties> TransformProps;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged();
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	
	// camera state at last render
	FViewCameraState CameraState;
	FTransform3d WorldTransform;

	FToolDataVisualizer PolyEdgesRenderer;

	// True for the duration of UI click+drag
	bool bInDrag;

	FPlane ActiveDragPlane;
	FVector StartHitPosWorld;
	FVector StartHitNormalWorld;
	FVector LastHitPosWorld;
	FVector LastBrushPosLocal;
	FVector StartBrushPosLocal;

	UE::Geometry::FFrame3d ActiveSurfaceFrame;
	UE_API UE::Geometry::FQuickTransformer* GetActiveQuickTransformer();
	UE_API void UpdateActiveSurfaceFrame(FGroupTopologySelection& Selection);
	UE_API void UpdateQuickTransformer();

	FRay UpdateRay;
	bool bUpdatePending = false;
	UE_API void ComputeUpdate();

	FVector3d LastMoveDelta;
	UE::Geometry::FQuickAxisTranslater QuickAxisTranslater;
	UE_API void ComputeUpdate_Translate();

	UE::Geometry::FQuickAxisRotator QuickAxisRotator;
	FVector3d RotationStartPointWorld;
	UE::Geometry::FFrame3d RotationStartFrame;
	UE_API void ComputeUpdate_Rotate();

	FGroupTopology Topology;
	UE_API void PrecomputeTopology();

	FGroupTopologySelector TopoSelector;
	UE_API FGroupTopologySelector::FSelectionSettings GetTopoSelectorSettings();
	
	//
	// data for current drag
	//

	FGroupTopologySelection HilightSelection;
	FToolDataVisualizer HilightRenderer;

	UE::Geometry::FDynamicMeshAABBTree3 MeshSpatial;
	UE_API UE::Geometry::FDynamicMeshAABBTree3& GetSpatial();

	FMeshVertexChangeBuilder* ActiveVertexChange;

	EGroupTopologyDeformationStrategy DeformationStrategy;

	// The two deformer type options.
	UE::Geometry::FGroupTopologyDeformer LinearDeformer;
	TPimplPtr<FGroupTopologyLaplacianDeformer> LaplacianDeformer;



	// This is true when the spatial index needs to reflect a modification
	bool bSpatialDirty; 

	UE_API void BeginChange();
	UE_API void EndChange();
	UE_API void UpdateChangeFromROI(bool bFinal);

private:
	UE::TransactionUtil::FLongTransactionTracker LongTransactions;
};

#undef UE_API
