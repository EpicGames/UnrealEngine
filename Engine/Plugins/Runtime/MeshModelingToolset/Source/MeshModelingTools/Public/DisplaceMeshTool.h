// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/Texture2D.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "SingleSelectionTool.h"
#include "Spatial/SampledScalarField2.h"
#include "UObject/NoExportTypes.h"
#include "WeightMapUtil.h"
#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "DisplaceMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

struct FMeshDescription;
class UCurveFloat;
class UDynamicMeshComponent;
class UMeshStatisticsProperties;

UENUM()
enum class EDisplaceMeshToolDisplaceType : uint8
{
	/** Offset a set distance in the normal direction. */
	Constant UMETA(DisplayName = "Constant"),

	/** Offset in the normal direction using the first channel of a 2D texture. */
	DisplacementMap UMETA(DisplayName = "Texture2D Map"),

	/** Offset vertices randomly. */
	RandomNoise UMETA(DisplayName = "Random Noise"),

	/** Offset in the normal direction weighted by Perlin noise. 
	    We use the following formula to compute the weighting for each vertex:
			w = PerlinNoise3D(f * (X + r))
		Where f is a frequency parameter, X is the vertex position, and r is a randomly-generated offset (using the Seed property).
		Note the range of 3D Perlin noise is [-sqrt(3/4), sqrt(3/4)].
	*/
	PerlinNoise UMETA(DisplayName = "Perlin Noise"),

	/** Move vertices in spatial sine wave pattern */
	SineWave UMETA(DisplayName = "Sine Wave"),
};

UENUM() 
enum class EDisplaceMeshToolSubdivisionType : uint8  
{
	/** Subdivide the mesh using loop-style subdivision. */
	Flat UMETA(DisplayName = "Flat"),
	
	/** Subdivide the mesh using PN triangles which replace each original flat triangle by a curved shape that is 
		retriangulated into a number of small subtriangles. The geometry of a PN triangle is defined as one cubic Bezier 
		patch using control points. The patch matches the point and normal information at the vertices of the original 
		flat triangle.*/
	PNTriangles UMETA(DisplayName = "PN Triangles")
};

UENUM()
enum class EDisplaceMeshToolTriangleSelectionType : uint8
{	
	/** Tessellate the whole mesh */
	None UMETA(DisplayName = "None"),

	/** Tessellate only triangles assigned to the chosen material */
	Material UMETA(DisplayName = "Material")
};

UENUM() 
enum class EDisplaceMeshToolChannelType : uint8  
{
	Red,
	Green,
	Blue,
	Alpha
};

/** The basic set of properties shared by (more or less) all DisplacementTypes. */
UCLASS(MinimalAPI)
class UDisplaceMeshCommonProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Displacement type */
	UPROPERTY(EditAnywhere, Category = Options)
	EDisplaceMeshToolDisplaceType DisplacementType = EDisplaceMeshToolDisplaceType::PerlinNoise;

	/** Displacement intensity */
	UPROPERTY(EditAnywhere, Category = Options,
		meta = (UIMin = "-100.0", UIMax = "100.0", ClampMin = "-10000.0", ClampMax = "100000.0"))
	float DisplaceIntensity = 10.0f;

	/** Seed for randomization */
	UPROPERTY(EditAnywhere, Category = Options,
		meta = (EditCondition = "DisplacementType == EDisplaceMeshToolDisplaceType::RandomNoise || DisplacementType == EDisplaceMeshToolDisplaceType::PerlinNoise", Delta = 1, LinearDeltaSensitivity = 50))
	int RandomSeed = 31337;

	/** Type of the  mesh subdivision. */
	UPROPERTY(EditAnywhere, Category = Options)
	EDisplaceMeshToolSubdivisionType SubdivisionType = EDisplaceMeshToolSubdivisionType::PNTriangles;

	/** Number of times to subdivide the mesh before displacing it. */
	UPROPERTY(EditAnywhere, Category = Options,
		meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "100"))
	int Subdivisions = 4;

	/** Select vertex weight map. If configured, the weight map value will be sampled to modulate displacement intensity. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (GetOptions = GetWeightMapsFunc))
	FName WeightMap;

	UFUNCTION()
	UE_API TArray<FString> GetWeightMapsFunc();

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> WeightMapsList;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bInvertWeightMap = false;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowWireframe = false;

	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (TransientToolProperty) )
	bool bDisableSizeWarning = false;
};

/** PropertySet for properties affecting the selective tessellation. */
UCLASS(MinimalAPI)
class USelectiveTessellationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
			
public:

	/** Optionally, restrict tessellation to a subset of the triangles. */
	UPROPERTY(EditAnywhere, Category = SelectiveTessellationOptions)
	EDisplaceMeshToolTriangleSelectionType SelectionType = EDisplaceMeshToolTriangleSelectionType::None;

	/** Name of the selected material. */
	UPROPERTY(EditAnywhere, Category = SelectiveTessellationOptions, meta = (EditCondition = "SelectionType == EDisplaceMeshToolTriangleSelectionType::Material", EditConditionHides, GetOptions = GetMaterialIDsFunc))
	FName ActiveMaterial;

	UFUNCTION()
	TArray<FString> GetMaterialIDsFunc() { return MaterialIDList; }

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> MaterialIDList;	
};


/** PropertySet for properties affecting the Image Map displacement type. */
UCLASS(MinimalAPI)
class UDisplaceMeshTextureMapProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
			
public:
	/** The texture to sample for displacement. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions)
	TObjectPtr<UTexture2D> DisplacementMap = nullptr;

	/** Channel in the displacement map to use. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions)
	EDisplaceMeshToolChannelType Channel = EDisplaceMeshToolChannelType::Red;

	/** The value in the texture map that corresponds to no displacement. For instance, if set to 0, then all
	 displacement will be positive. If set to 0.5, displacement below 0.5 will be negative, and above will be
	 positive. Default is for 128/255 to be no displacement. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float DisplacementMapBaseValue = 128.0f/255;

	/** When sampling from the texture map, how to scale the mesh UV's in the x and y directions. For a mesh with
	UV's in the range 0 to 1, setting a scale above 1 will result in tiling the texture map, and scaling below
	1 will result in using only part of the texture map.*/
	UPROPERTY(EditAnywhere, Category = TextureMapOptions, meta = (Delta = 0.01, LinearDeltaSensitivity = 1))
	FVector2D UVScale = FVector2D(1,1);

	/** When sampling from the texture map, how to offset the mesh UV's. This will result in offsetting the
	tiling of the texture map across the mesh. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions, meta = (UIMin = "-1", UIMax = "1"))
	FVector2D UVOffset = FVector2D(0, 0);

	/** When true, applies a function to remap the values in the displacement map, which can be used 
	 for contrast adjustment. The texture map values are converted to the range [0,1] before applying
	 the remapping. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions)
	bool bApplyAdjustmentCurve = false;

	/** This curve is queried in the range [0,1] to adjust contrast of the displacement map. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions, meta = (EditCondition = "bApplyAdjustmentCurve", EditConditionHides))
	TObjectPtr<UCurveFloat> AdjustmentCurve = nullptr;

	/** Recalculate normals from displaced mesh. Disable this if you are applying Displacements that are paired with an existing Normal Map in your Material. */
	UPROPERTY(EditAnywhere, Category = TextureMapOptions)
	bool bRecalcNormals = true;
};



/** Properties for a directional filter. Allows for displacement to be applied only to vertices whose normals point in a given direction */
UCLASS(MinimalAPI)
class UDisplaceMeshDirectionalFilterProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Whether the directional filter is active. */
	UPROPERTY(EditAnywhere, Category = DirectionalFilterOptions)
	bool bEnableFilter = false;

	/** Unit vector representing the direction to filter along. */
	UPROPERTY(EditAnywhere, Category = DirectionalFilterOptions, 
		meta = (EditCondition = "bEnableFilter == true", Delta = 0.5, LinearDeltaSensitivity = 1))
	FVector FilterDirection = {0.0f, 0.0f, 1.0f};

	/** Scalar value determining how close to the filter direction the vertex normals must be in order to be displaced.
		0: Only normals pointing exactly in the filter direction are displaced.
		0.5: Normals forming angle up to 90 from the filter direction are displaced.
		1.0: All vertices are displaced.
	*/
	UPROPERTY(EditAnywhere, Category = DirectionalFilterOptions,
		meta = (EditCondition = "bEnableFilter == true", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float FilterWidth = 0.25f;

};

/** Per-layer properties for Perlin noise. Each layer has independent Frequency and Intensity. */
USTRUCT()
struct FPerlinLayerProperties
{
	GENERATED_BODY()

	FPerlinLayerProperties() = default;

	FPerlinLayerProperties(float FrequencyIn, float IntensityIn) : 
		Frequency(FrequencyIn), 
		Intensity(IntensityIn) 
    {}

	/** Frequency of Perlin noise layer */
	UPROPERTY(EditAnywhere, Category = PerlinNoiseOptions,
			meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0"))
	float Frequency = 0.1f;

	/** Intensity/amplitude of Perlin noise layer */
	UPROPERTY(EditAnywhere, Category = PerlinNoiseOptions,
		meta = (UIMin = "-10.0", UIMax = "10.0", ClampMin = "-100.0", ClampMax = "100.0"))
	float Intensity = 1.0f;
};

/** PropertySet for properties affecting the Perlin Noise displacement type. */
UCLASS(MinimalAPI)
class UDisplaceMeshPerlinNoiseProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = PerlinNoiseOptions)
	TArray<FPerlinLayerProperties> PerlinLayerProperties = 
		{ /*{Frequency, Intensity}*/ {0.05f, 1.0f}, {0.25f, 0.5f}, {0.5f, 0.2f}, {1.0f, 0.1f} };
};


/** PropertySet for Sine wave displacement */
UCLASS(MinimalAPI)
class UDisplaceMeshSineWaveProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Sine wave displacement frequency */
	UPROPERTY(EditAnywhere, Category = SineWaveOptions,
		meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0"))
	float SineWaveFrequency = 0.1f;

	/** Sine wave phase shift */
	UPROPERTY(EditAnywhere, Category = SineWaveOptions,
		meta = (UIMin = "0.0", UIMax = "6.28318531", ClampMin = "0.0", ClampMax = "6.28318531"))
	float SineWavePhaseShift = 0.0f;

	/** Unit vector representing the direction of wave displacement. */
	UPROPERTY(EditAnywhere, Category = SineWaveOptions, meta = (Delta = 0.5, LinearDeltaSensitivity = 1))
	FVector SineWaveDirection = { 0.0f, 0.0f, 1.0f };
};

/**
 * Builder for Simple Mesh Displacement Tool
 */
UCLASS(MinimalAPI)
class UDisplaceMeshToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override final;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * Simple Mesh Displacement Tool
 */
UCLASS(MinimalAPI)
class UDisplaceMeshTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()
public:
	UE_API void Setup() final;
	UE_API void OnShutdown(EToolShutdownType ShutdownType) final;
	UE_API void OnTick(float DeltaTime) final;

	bool HasCancel() const final { return true; }
	bool HasAccept() const final { return true; }

#if WITH_EDITOR
	UE_API void OnPropertyModified(UObject* PropertySet, FProperty* Property) final;
#endif

	/** Shared properties for all displacement modes. */
	UPROPERTY()
	TObjectPtr<UDisplaceMeshCommonProperties> CommonProperties;

	/** Properties defining the directional filter. */
	UPROPERTY()
	TObjectPtr<UDisplaceMeshDirectionalFilterProperties> DirectionalFilterProperties;

	/** Properties defining the texture map */
	UPROPERTY()
	TObjectPtr<UDisplaceMeshTextureMapProperties> TextureMapProperties;

	/** Multi-layer Perlin noise frequencies and intensities */
	UPROPERTY()
	TObjectPtr<UDisplaceMeshPerlinNoiseProperties> NoiseProperties;

	/** Sine wave parameters and direction of displacement */
	UPROPERTY()
	TObjectPtr<UDisplaceMeshSineWaveProperties> SineWaveProperties;

	/** Selective tessellation properties */
	UPROPERTY()
	TObjectPtr<USelectiveTessellationProperties> SelectiveTessellationProperties;

	/** Contrast Curve we are actively listening to */
	UPROPERTY()
	TObjectPtr<UCurveFloat> ActiveContrastCurveTarget = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshStatisticsProperties> MeshStatistics;


private:
	UE_API void StartComputation();
	UE_API void AdvanceComputation();
	UE_API void ValidateSubdivisions();

	double ComputationStartTime = 0;

	UE::Geometry::FAsyncTaskExecuterWithAbort<UE::Geometry::TModelingOpTask<UE::Geometry::FDynamicMeshOperator>>* SubdivideTask = nullptr;
	bool bNeedsSubdivided = true;
	UE::Geometry::FAsyncTaskExecuterWithAbort<UE::Geometry::TModelingOpTask<UE::Geometry::FDynamicMeshOperator>>* DisplaceTask = nullptr;
	bool bNeedsDisplaced = true;

	UE::Geometry::FDynamicMesh3 OriginalMesh;
	UE::Geometry::FDynamicMeshAABBTree3 OriginalMeshSpatial;
	TSharedPtr<UE::Geometry::FIndexedWeightMap, ESPMode::ThreadSafe> ActiveWeightMap;
	UE_API void UpdateActiveWeightMap();
	UE_API float WeightMapQuery(const FVector3d& Position, const UE::Geometry::FIndexedWeightMap& WeightMap) const;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> SubdividedMesh = nullptr;
	TSharedPtr<TArray<int>, ESPMode::ThreadSafe> VerticesToDisplace = nullptr;
	int32 NumTrianglesInSelection;
	
	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = nullptr;

	TUniquePtr<UE::Geometry::IDynamicMeshOperatorFactory> Subdivider = nullptr;
	TUniquePtr<UE::Geometry::IDynamicMeshOperatorFactory> Displacer = nullptr;

	// management of TextureMapProperties->AdjustmentCurve.OnUpdateCurve listener
	FDelegateHandle ActiveContrastCurveListenerHandle;
	UE_API void UpdateActiveContrastCurveTarget();
	UE_API void DisconnectActiveContrastCurveTarget();
};

#undef UE_API
