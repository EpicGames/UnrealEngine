// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ModelingOperators.h" //IDynamicMeshOperatorFactory
#include "InteractiveTool.h" //UInteractiveToolPropertySet
#include "InteractiveToolBuilder.h" //UInteractiveToolBuilder
#include "MeshOpPreviewHelpers.h" //FDynamicMeshOpResult
#include "Properties/MeshMaterialProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "Properties/RevolveProperties.h"

#include "DrawAndRevolveTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UCollectSurfacePathMechanic;
class UConstructionPlaneMechanic;
class UCurveControlPointsMechanic;

UCLASS(MinimalAPI)
class UDrawAndRevolveToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};

UCLASS(MinimalAPI)
class URevolveToolProperties : public URevolveProperties
{
	GENERATED_BODY()

public:

	/** Determines how end caps are created. This is not relevant if the end caps are not visible or if the path is not closed. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (DisplayAfter = "QuadSplitMode",
		EditCondition = "HeightOffsetPerDegree != 0 || RevolveDegrees != 360", ValidEnumValues = "None, CenterFan, Delaunay"))
	ERevolvePropertiesCapFillMode CapFillMode = ERevolvePropertiesCapFillMode::Delaunay;

	/** Connect the ends of an open path to the axis to add caps to the top and bottom of the revolved result.
	  * This is not relevant for paths that are already closed. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay)
	bool bClosePathToAxis = true;
	
	/** Sets the draw plane origin. The revolution axis is the X axis in the plane. */
	UPROPERTY(EditAnywhere, Category = DrawPlane, meta = (DisplayName = "Origin", EditCondition = "bAllowedToEditDrawPlane", HideEditConditionToggle,
		Delta = 5, LinearDeltaSensitivity = 1))
	FVector DrawPlaneOrigin = FVector(0, 0, 0);

	/** Sets the draw plane orientation. The revolution axis is the X axis in the plane. */
	UPROPERTY(EditAnywhere, Category = DrawPlane, meta = (DisplayName = "Orientation", EditCondition = "bAllowedToEditDrawPlane", HideEditConditionToggle, 
		UIMin = -180, UIMax = 180, ClampMin = -180000, ClampMax = 180000))
	FRotator DrawPlaneOrientation = FRotator(90, 0, 0);

	/** Enables snapping while editing the path. */
	UPROPERTY(EditAnywhere, Category = Snapping)
	bool bEnableSnapping = true;

	// Not user visible- used to disallow draw plane modification.
	UPROPERTY(meta = (TransientToolProperty))
	bool bAllowedToEditDrawPlane = true;

protected:
	virtual ERevolvePropertiesCapFillMode GetCapFillMode() const override
	{
		return CapFillMode;
	}
};

UCLASS(MinimalAPI)
class URevolveOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UDrawAndRevolveTool> RevolveTool;
};


/** Draws a profile curve and revolves it around an axis. */
UCLASS(MinimalAPI)
class UDrawAndRevolveTool : public UInteractiveTool
{
	GENERATED_BODY()

public:
	virtual void SetWorld(UWorld* World) { TargetWorld = World; }

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;
	UE_API virtual void OnPointDeletionKeyPress();

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	void SetInitialDrawFrame(UE::Geometry::FFrame3d InFrame)
	{
		InitialDrawFrame = InFrame;
	}

	UE_API bool CanGenerateAsset() const;

protected:

	UWorld* TargetWorld;

	FViewCameraState CameraState;

	// This information is replicated in the user-editable transform in the settings and in the PlaneMechanic
	// plane, but the tool turned out to be much easier to write and edit with this decoupling.
	FVector3d RevolutionAxisOrigin;
	FVector3d RevolutionAxisDirection;

	// The initial frame, used in tool setup to place the axis
	UE::Geometry::FFrame3d InitialDrawFrame;

	bool bProfileCurveComplete = false;

	UE_API void UpdateRevolutionAxis();

	UPROPERTY()
	TObjectPtr<UCurveControlPointsMechanic> ControlPointsMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	/** Property set for type of output object (StaticMesh, Volume, etc) */
	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<URevolveToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UE_API void StartPreview();

	UE_API void GenerateAsset(const FDynamicMeshOpResult& Result);

	friend class URevolveOperatorFactory;
};

#undef UE_API
