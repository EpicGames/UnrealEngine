// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "InteractiveGizmo.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"
#include "BaseTools/SingleClickTool.h"
#include "PreviewMesh.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Snapping/PointPlanarSnapSolver.h"
#include "ToolSceneQueriesUtil.h"
#include "Properties/MeshMaterialProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "Mechanics/PlaneDistanceFromHitMechanic.h"
#include "DrawPolygonTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API


class UConstructionPlaneMechanic;
class UCombinedTransformGizmo;
class UDragAlignmentMechanic;
class UTransformProxy;
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 *
 */
UCLASS(MinimalAPI)
class UDrawPolygonToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
};



/** Polygon tool draw type */
UENUM()
enum class EDrawPolygonDrawMode : uint8
{
	/** Draw a freehand polygon */
	Freehand,

	/** Draw a circle */
	Circle,

	/** Draw a square */
	Square,

	/** Draw a rectangle */
	Rectangle,

	/** Draw a rounded rectangle */
	RoundedRectangle,

	/** Draw a circle with a hole in the center */
	Ring
};


/** How the drawn polygon gets extruded */
UENUM()
enum class EDrawPolygonExtrudeMode : uint8
{
	/** Flat polygon without extrusion */
	Flat,

	/** Extrude drawn polygon to fixed height determined by the Extrude Height property */
	Fixed,

	/** Extrude drawn polygon to height set via additional mouse input after closing the polygon */
	Interactive,
};





UCLASS(MinimalAPI)
class UDrawPolygonToolStandardProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API UDrawPolygonToolStandardProperties();

	/** Type of polygon to draw in the viewport */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Polygon, meta = (DisplayName = "Draw Mode"))
	EDrawPolygonDrawMode PolygonDrawMode = EDrawPolygonDrawMode::Freehand;

	/** Allow freehand drawn polygons to self-intersect */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Polygon,
		meta = (DisplayName ="Self-Intersections", EditCondition = "PolygonDrawMode == EDrawPolygonDrawMode::Freehand"))
	bool bAllowSelfIntersections = false;

	/** Size of secondary features, e.g. the rounded corners of a rounded rectangle, as fraction of the overall shape size */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Polygon, meta = (UIMin = "0.01", UIMax = "0.99", ClampMin = "0.01", ClampMax = "0.99",
		EditCondition = "PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle || PolygonDrawMode == EDrawPolygonDrawMode::Ring"))
	float FeatureSizeRatio = .25;

	/** Number of radial subdivisions in round features, e.g. circles or rounded corners */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Polygon, meta = (UIMin = "3", UIMax = "100", ClampMin = "3", ClampMax = "10000",
		EditCondition =	"PolygonDrawMode == EDrawPolygonDrawMode::Circle || PolygonDrawMode == EDrawPolygonDrawMode::RoundedRectangle || PolygonDrawMode == EDrawPolygonDrawMode::Ring"))
	int RadialSlices = 16;

	/** Distance between the last clicked point and the current point  */
	UPROPERTY(VisibleAnywhere, NonTransactional, Category = Polygon, meta = (TransientToolProperty))
	float Distance = 0.0f;

	/** If true, shows a gizmo to manipulate the additional grid used to draw the polygon on */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Polygon)
	bool bShowGridGizmo = true;

	/** If and how the drawn polygon gets extruded */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Extrude)
	EDrawPolygonExtrudeMode ExtrudeMode = EDrawPolygonExtrudeMode::Interactive;

	/** Extrude distance when using the Fixed extrude mode */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Extrude, meta = (UIMin = "-1000", UIMax = "1000", ClampMin = "-10000", ClampMax = "10000",
		EditCondition = "ExtrudeMode == EDrawPolygonExtrudeMode::Fixed"))
	float ExtrudeHeight = 100.0f;
};

UCLASS(MinimalAPI)
class UDrawPolygonToolSnapProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Enables additional snapping controls. If false, all snapping is disabled. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping)
	bool bEnableSnapping = true;

	//~ Not user visible. Mirrors the snapping settings in the viewport and is used in EditConditions
	UPROPERTY(meta = (TransientToolProperty))
	bool bSnapToWorldGrid = false;

	/** Snap to vertices in other meshes. Requires Enable Snapping to be true. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping, meta = (EditCondition = "bEnableSnapping"))
	bool bSnapToVertices = true;

	/** Snap to edges in other meshes. Requires Enable Snapping to be true. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping, meta = (EditCondition = "bEnableSnapping"))
	bool bSnapToEdges = false;

	/** Snap to axes of the drawing grid and axes relative to the last segment. Requires grid snapping to be disabled in viewport, and Enable Snapping to be true. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping, meta = (EditCondition = "bEnableSnapping && !bSnapToWorldGrid"))
	bool bSnapToAxes = true;

	/** When snapping to axes, also try to snap to the length of an existing segment in the polygon. Requires grid snapping to be disabled in viewport, and Snap to Axes and Enable Snapping to be true. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping, meta = (EditCondition = "bEnableSnapping && !bSnapToWorldGrid && bSnapToAxes"))
	bool bSnapToLengths = true;

	/** Snap to surfaces of existing objects. Requires grid snapping to be disabled in viewport, and Enable Snapping to be true.  */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping, meta = (EditCondition = "bEnableSnapping && !bSnapToWorldGrid"))
	bool bSnapToSurfaces = false;

	/** Offset for snap point on the surface of an existing object in the direction of the surface normal. Requires grid snapping to be disabled in viewport, and Snap to Surfaces and Enable Snapping to be true. */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Snapping, meta = (DisplayName = "Surface Offset", 
		EditCondition = "bEnableSnapping && !bSnapToWorldGrid && bSnapToSurfaces"))
	float SnapToSurfacesOffset = 0.0f;
};





/**
 * This tool allows the user to draw and extrude 2D polygons
 */
UCLASS(MinimalAPI)
class UDrawPolygonTool : public UInteractiveTool, public IClickSequenceBehaviorTarget, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API UDrawPolygonTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }

	// IClickSequenceBehaviorTarget implementation

	UE_API virtual void OnBeginSequencePreview(const FInputDeviceRay& ClickPos) override;
	UE_API virtual bool CanBeginClickSequence(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnBeginClickSequence(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnNextSequencePreview(const FInputDeviceRay& ClickPos) override;
	UE_API virtual bool OnNextSequenceClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnTerminateClickSequence() override;
	UE_API virtual bool RequestAbortClickSequence() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	// polygon drawing functions
	UE_API virtual void ResetPolygon();
	UE_API virtual void UpdatePreviewVertex(const FVector3d& PreviewVertex);
	UE_API virtual void AppendVertex(const FVector3d& Vertex);
	UE_API virtual bool FindDrawPlaneHitPoint(const FInputDeviceRay& ClickPos, FVector3d& HitPosOut);
	UE_API virtual void EmitCurrentPolygon();

	UE_API virtual void BeginInteractiveExtrude();
	UE_API virtual void EndInteractiveExtrude();

	UE_API virtual void ApplyUndoPoints(const TArray<FVector3d>& ClickPointsIn, const TArray<FVector3d>& PolygonVerticesIn);

	void SetInitialDrawFrame(UE::Geometry::FFrame3d Frame)
	{
		InitialDrawFrame = Frame;
	}

	// IInteractiveToolManageGeometrySelectionAPI - enables restoration of previous selection when applicable
	UE_API virtual bool IsInputSelectionValidOnOutput() override;


protected:
	// flags used to identify modifier keys/buttons
	static constexpr int IgnoreSnappingModifier = 1;
	static constexpr int AngleSnapModifier = 2;

	/** Property set for type of output object (StaticMesh, Volume, etc) */
	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	/** Properties that control polygon generation exposed to user via details view */
	UPROPERTY()
	TObjectPtr<UDrawPolygonToolStandardProperties> PolygonProperties;

	UPROPERTY()
	TObjectPtr<UDrawPolygonToolSnapProperties> SnapProperties;

	UPROPERTY()
	TObjectPtr<UNewMeshMaterialProperties> MaterialProperties;
	
	/** Vertices of current preview polygon */
	TArray<FVector3d> PolygonVertices;

	/** Vertices of holes in current preview polygon */
	TArray<TArray<FVector3d>> PolygonHolesVertices;

	/** last vertex of polygon that is actively being updated as input device is moved */
	FVector3d PreviewVertex;

	UWorld* TargetWorld;

	FViewCameraState CameraState;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;
	
	// called on PlaneMechanic.OnPlaneChanged
	UE_API void PlaneTransformChanged();

	// whether to allow the draw plane to be updated in the UI -- returns false if there is an in-progress shape relying on the current draw plane
	UE_API bool AllowDrawPlaneUpdates();

	// polygon drawing

	UE::Geometry::FFrame3d InitialDrawFrame;

	bool bAbortActivePolygonDraw;

	bool bInFixedPolygonMode = false;
	TArray<FVector3d> FixedPolygonClickPoints;

	// can close poly if current segment intersects existing segment
	UE_API bool UpdateSelfIntersection();
	bool bHaveSelfIntersection;
	int SelfIntersectSegmentIdx;
	FVector3d SelfIntersectionPoint;

	// only used when SnapSettings.bSnapToSurfaces = true
	bool bHaveSurfaceHit;
	FVector3d SurfaceHitPoint;
	FVector3d SurfaceOffsetPoint;

	bool bIgnoreSnappingToggle = false;		// toggled by hotkey (shift)
	UE::Geometry::FPointPlanarSnapSolver SnapEngine;
	ToolSceneQueriesUtil::FSnapGeometry LastSnapGeometry;
	FVector3d LastGridSnapPoint;

	UE_API void GetPolygonParametersFromFixedPoints(const TArray<FVector3d>& FixedPoints, FVector2d& FirstReferencePt, FVector2d& BoxSize, double& YSign, double& AngleRad);
	UE_API void GenerateFixedPolygon(const TArray<FVector3d>& FixedPoints, TArray<FVector3d>& VerticesOut, TArray<TArray<FVector3d>>& HolesVerticesOut);


	// extrusion control

	bool bInInteractiveExtrude = false;
	bool bHasSavedExtrudeHeight = false;
	float SavedExtrudeHeight;

	UE_API void UpdateLivePreview();
	bool bPreviewUpdatePending;

	UPROPERTY()
	TObjectPtr<UPlaneDistanceFromHitMechanic> HeightMechanic;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	/** Generate extruded meshes.  Returns true on success. */
	UE_API bool GeneratePolygonMesh(const TArray<FVector3d>& Polygon, const TArray<TArray<FVector3d>>& PolygonHoles, FDynamicMesh3* ResultMeshOut, UE::Geometry::FFrame3d& WorldFrameOut, bool bIncludePreviewVtx, double ExtrudeDistance, bool bExtrudeSymmetric);


	// user feedback messages
	UE_API void ShowStartupMessage();
	UE_API void ShowExtrudeMessage();


	friend class FDrawPolygonStateChange;
	int32 CurrentCurveTimestamp = 1;
	UE_API void UndoCurrentOperation(const TArray<FVector3d>& ClickPointsIn, const TArray<FVector3d>& PolygonVerticesIn);
	bool CheckInCurve(int32 Timestamp) const { return CurrentCurveTimestamp == Timestamp; }
private:
	// if the drawn path is incomplete upon 'Accept' of the tool, no mesh should be created and the
	// previous mesh element selection from before entering the tool should be restored
	bool bRestoreInputSelection = true;
};



// Change event used by DrawPolygonTool to undo draw state.
// Currently does not redo.
class FDrawPolygonStateChange : public FToolCommandChange
{
public:
	bool bHaveDoneUndo = false;
	int32 CurveTimestamp = 0;
	const TArray<FVector3d> FixedVertexPoints;
	const TArray<FVector3d> PolyPoints;

	FDrawPolygonStateChange(int32 CurveTimestampIn,
							const TArray<FVector3d>& FixedVertexPointsIn,
							const TArray<FVector3d>& PolyPointsIn)
		: CurveTimestamp(CurveTimestampIn),
		FixedVertexPoints(FixedVertexPointsIn),
		PolyPoints(PolyPointsIn)
	{
	}
	virtual void Apply(UObject* Object) override {}
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual bool HasExpired(UObject* Object) const override;
	UE_API virtual FString ToString() const override;
};

#undef UE_API
