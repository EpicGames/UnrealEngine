// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "InteractiveToolBuilder.h"
#include "BaseTools/BaseBrushTool.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Components/DynamicMeshComponent.h"
#include "PreviewMesh.h"
#include "TransactionUtil.h"
#include "DynamicMeshBrushTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * UDynamicMeshBrushTool is a base class that specializes UBaseBrushTool
 * for brushing on an FDynamicMesh3. The input FPrimitiveComponentTarget is hidden
 * and a UPreviewMesh is created and shown in its place. This UPreviewMesh is
 * used for hit-testing and dynamic rendering.
 * 
 */
UCLASS(MinimalAPI, Transient)
class UDynamicMeshBrushTool : public UBaseBrushTool
{
	GENERATED_BODY()

public:
	UE_API UDynamicMeshBrushTool();

	// UInteractiveTool API

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;

	TObjectPtr<UPreviewMesh> GetPreviewMesh() { return PreviewMesh; }

protected:
	// subclasses can override these to customize behavior
	virtual void OnShutdown(EToolShutdownType ShutdownType) {}


protected:
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	// this function is called when the component inside the PreviewMesh is modified (eg via an undo/redo event)
	virtual void OnBaseMeshComponentChanged() {}	
	FDelegateHandle OnBaseMeshComponentChangedHandle;

	UE::Geometry::FAxisAlignedBox3d InputMeshBoundsLocal;

	// When true during Setup(), Rotation and Scale will be baked into the PreviewMesh.
	// The rotate/scale portion of the transform will be stored in BakedTargetTransform.
	// The translate portion of the transform will be stored in CurrentTargetTransform.
	bool bBakeRotateScale = false;
	UE::Geometry::FTransformSRT3d BakedTargetTransform;
	UE::Geometry::FTransformSRT3d CurrentTargetTransform;

	//
	// UBaseBrushTool private interface
	//
	UE_API virtual double EstimateMaximumTargetDimension() override;

	UE::TransactionUtil::FLongTransactionTracker LongTransactions;
};

#undef UE_API
