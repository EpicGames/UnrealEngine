// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "SubRegionRemesher.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/OctreeDynamicMeshComponent.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "BaseTools/BaseBrushTool.h"
#include "ToolDataVisualizer.h"
#include "Changes/ValueWatcher.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/RemeshProperties.h"
#include "TransformTypes.h"
#include "TransactionUtil.h"
#include "Sculpting/MeshSculptToolBase.h"
#include "Async/Async.h"
#include "Util/UniqueIndexSet.h"
#include "DynamicMeshSculptTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UCombinedTransformGizmo;
class UTransformProxy;
class ULocalTwoAxisPropertyEditInputBehavior;
class UMaterialInstanceDynamic;

class FMeshVertexChangeBuilder;
class UPreviewMesh;
PREDECLARE_GEOMETRY(class FDynamicMeshChangeTracker);
PREDECLARE_GEOMETRY(class FSubRegionRemesher);
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class FPersistentStampRemesher;


/** Mesh Sculpting Brush Types */
UENUM()
enum class EDynamicMeshSculptBrushType : uint8
{
	/** Move vertices parallel to the view plane  */
	Move UMETA(DisplayName = "Move"),

	/** Grab Brush, fall-off alters the influence of the grab */
	PullKelvin UMETA(DisplayName = "Kelvin Grab"),

	/** Grab Brush that may generate cusps, fall-off alters the influence of the grab */
	PullSharpKelvin UMETA(DisplayName = "Sharp Kelvin Grab"),

	/** Smooth mesh vertices  */
	Smooth UMETA(DisplayName = "Smooth"),

	/** Displace vertices along the average surface normal (Ctrl to invert) */
	Offset UMETA(DisplayName = "Sculpt (Normal)"),

	/** Displace vertices towards the camera viewpoint (Ctrl to invert) */
	SculptView UMETA(DisplayName = "Sculpt (Viewpoint)"),

	/** Displaces vertices along the average surface normal to a maximum height based on the brush size (Ctrl to invert) */
	SculptMax UMETA(DisplayName = "Sculpt Max"),

	/** Displace vertices along their vertex normals */
	Inflate UMETA(DisplayName = "Inflate"),

	/** Scale Brush will inflate or pinch radially from the center of the brush */
	ScaleKelvin UMETA(DisplayName = "Kelvin Scale"),

	/** Move vertices towards the center of the brush (Ctrl to push away)*/
	Pinch UMETA(DisplayName = "Pinch"),

	/** Twist Brush moves vertices in the plane perpendicular to the local mesh normal */
	TwistKelvin UMETA(DisplayName = "Kelvin Twist"),

	/** Move vertices towards the average plane of the brush stamp region */
	Flatten UMETA(DisplayName = "Flatten"),

	/** Move vertices towards a plane defined by the initial brush position  */
	Plane UMETA(DisplayName = "Plane (Normal)"),

	/** Move vertices towards a view-facing plane defined at the initial brush position */
	PlaneViewAligned UMETA(DisplayName = "Plane (Viewpoint)"),

	/** Move vertices towards a fixed plane in world space, positioned with a 3D gizmo */
	FixedPlane UMETA(DisplayName = "FixedPlane"),

	/** Remesh the brushed region but do not otherwise deform it */
	Resample UMETA(DisplayName = "Resample"),

	LastValue UMETA(Hidden)

};

/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UDynamicMeshSculptToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	bool bEnableRemeshing;

	UDynamicMeshSculptToolBuilder()
	{
		bEnableRemeshing = false;
	}

	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};





UCLASS(MinimalAPI)
class UDynamicMeshBrushProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayPriority = 1))
	FBrushToolRadius BrushSize;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", DisplayPriority = 3))
	float BrushFalloffAmount = 0.5f;

	/** Depth of Brush into surface along view ray or surface normal, depending on the Active Brush Type */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = "-0.5", UIMax = "0.5", ClampMin = "-1.0", ClampMax = "1.0", DisplayPriority = 5))
	float Depth = 0;

	/** Allow the Brush to hit the back-side of the mesh */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayPriority = 6))
	bool bHitBackFaces = true;
};





UCLASS(MinimalAPI)
class UDynamicMeshBrushSculptProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/* This is a dupe of the bool in the tool class.  I needed it here so it could be checked as an EditCondition */
	UPROPERTY(meta = (TransientToolProperty))
	bool bIsRemeshingEnabled = false;

	/** Primary Brush Mode */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (DisplayName = "Brush Type"))
	EDynamicMeshSculptBrushType PrimaryBrushType = EDynamicMeshSculptBrushType::Move;

	/** Strength of the Primary Brush */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "PrimaryBrushType != EDynamicMeshSculptBrushType::Pull", ModelingQuickEdit))
	float PrimaryBrushSpeed = 0.5;

	/** If true, try to preserve the shape of the UV/3D mapping. This will limit Smoothing and Remeshing in some cases. */
	UPROPERTY(EditAnywhere, Category = Sculpting)
	bool bPreserveUVFlow = false;

	/** When Freeze Target is toggled on, the Brush Target Surface will be Frozen in its current state, until toggled off. Brush strokes will be applied relative to the Target Surface, for applicable Brushes */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (EditCondition = "PrimaryBrushType == EDynamicMeshSculptBrushType::Sculpt || PrimaryBrushType == EDynamicMeshSculptBrushType::SculptMax || PrimaryBrushType == EDynamicMeshSculptBrushType::SculptView || PrimaryBrushType == EDynamicMeshSculptBrushType::Pinch || PrimaryBrushType == EDynamicMeshSculptBrushType::Resample" ))
	bool bFreezeTarget = false;

	/** Strength of Shift-to-Smooth Brushing and Smoothing Brush */
	UPROPERTY(EditAnywhere, Category = Smoothing, meta = (DisplayName = "Smoothing Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float SmoothBrushSpeed = 0.25;

	/** If enabled, Remeshing is limited during Smoothing to avoid wiping out higher-density triangle areas */
	UPROPERTY(EditAnywhere, Category = Smoothing, meta = (DisplayName = "Preserve Tri Density", EditConditionHides, HideEditConditionToggle, EditCondition = "bIsRemeshingEnabled"))
	bool bDetailPreservingSmooth = true;
};


UCLASS(MinimalAPI)
class UDynamicSculptToolActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	TWeakObjectPtr<UDynamicMeshSculptTool> ParentTool;

	void Initialize(UDynamicMeshSculptTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UFUNCTION(CallInEditor, Category = MeshEdits)
	UE_API void DiscardAttributes();
};




UCLASS(MinimalAPI)
class UBrushRemeshProperties : public URemeshProperties
{
	GENERATED_BODY()

public:
	/** Toggle remeshing on/off */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (DisplayPriority = 1))
	bool bEnableRemeshing = true;

	// Note that if you change range here, you must also update UDynamicMeshSculptTool::ConfigureRemesher!
	/** Desired size of triangles after Remeshing, relative to average initial triangle size. Larger value results in larger triangles. */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (DisplayName = "Relative Tri Size", UIMin = "-5", UIMax = "5", ClampMin = "-5", ClampMax = "5", DisplayPriority = 2))
	int TriangleSize = 0;

	/** Control the amount of simplification during sculpting. Higher values will avoid wiping out fine details on the mesh. */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (UIMin = "0", UIMax = "5", ClampMin = "0", ClampMax = "5", DisplayPriority = 3))
	int PreserveDetail = 0;

	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay)
	int Iterations = 5;
};



UCLASS(MinimalAPI)
class UFixedPlaneBrushProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY( meta = (TransientToolProperty) )
	bool bPropertySetEnabled = true;

	/** Toggle whether Work Plane Positioning Gizmo is visible */
	UPROPERTY(EditAnywhere, Category = TargetPlane, meta = (HideEditConditionToggle, EditCondition = "bPropertySetEnabled == true"))
	bool bShowGizmo = true;

	UPROPERTY(EditAnywhere, Category = TargetPlane, AdvancedDisplay, meta = (HideEditConditionToggle, EditCondition = "bPropertySetEnabled == true"))
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = TargetPlane, AdvancedDisplay, meta = (HideEditConditionToggle, EditCondition = "bPropertySetEnabled == true"))
	FQuat Rotation = FQuat::Identity;

	// Recenter the gizmo around the target position (without changing work plane), if it is "too far" (> 10 meters + max bounds dim) from that position currently
	void RecenterGizmoIfFar(FVector CenterPosition, double BoundsMaxDim, double TooFarDistance = 1000)
	{
		double DistanceTolSq = (BoundsMaxDim + TooFarDistance) * (BoundsMaxDim + TooFarDistance);
		if (FVector::DistSquared(CenterPosition, Position) > DistanceTolSq)
		{
			FVector Normal = Rotation.GetAxisZ();
			Position = CenterPosition - (CenterPosition - Position).ProjectOnToNormal(Normal);
		}
	}
};




/**
 * Dynamic Mesh Sculpt Tool Class
 */
UCLASS(MinimalAPI)
class UDynamicMeshSculptTool : public UMeshSurfacePointTool
{
	GENERATED_BODY()

public:
	using FFrame3d = UE::Geometry::FFrame3d;

	UE_API UDynamicMeshSculptTool();

	UE_API virtual void SetWorld(UWorld* World);
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;

	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual void OnCancelDrag() override;

	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	virtual void SetEnableRemeshing(bool bEnable) { bEnableRemeshing = bEnable; }
	virtual bool GetEnableRemeshing() const { return bEnableRemeshing; }

	UE_API virtual void DiscardAttributes();

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolCameraFocusAPI override to focus on brush w/ 'F' 
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

public:
	/** Properties that control brush size/etc*/
	UPROPERTY()
	TObjectPtr<UDynamicMeshBrushProperties> BrushProperties;

	/** Properties that control sculpting*/
	UPROPERTY()
	TObjectPtr<UDynamicMeshBrushSculptProperties> SculptProperties;

	UPROPERTY()
	TObjectPtr<USculptMaxBrushProperties> SculptMaxBrushProperties;
	
	UPROPERTY()
	TObjectPtr<UKelvinBrushProperties> KelvinBrushProperties;

	/** Properties that control dynamic remeshing */
	UPROPERTY()
	TObjectPtr<UBrushRemeshProperties> RemeshProperties;

	UPROPERTY()
	TObjectPtr<UFixedPlaneBrushProperties> GizmoProperties;

	UPROPERTY()
	TObjectPtr<UMeshEditingViewProperties> ViewProperties;

	UPROPERTY()
	TObjectPtr<UDynamicSculptToolActions> SculptToolActions;

public:
	UE_API virtual void IncreaseBrushRadiusAction();
	UE_API virtual void DecreaseBrushRadiusAction();
	UE_API virtual void IncreaseBrushRadiusSmallStepAction();
	UE_API virtual void DecreaseBrushRadiusSmallStepAction();

	UE_API virtual void IncreaseBrushSpeedAction();
	UE_API virtual void DecreaseBrushSpeedAction();

	UE_API virtual void NextHistoryBrushModeAction();
	UE_API virtual void PreviousHistoryBrushModeAction();

private:
	UWorld* TargetWorld;		// required to spawn UPreviewMesh/etc
	FViewCameraState CameraState;

	TWeakObjectPtr<ULocalTwoAxisPropertyEditInputBehavior> BrushEditBehavior;

	UPROPERTY()
	TObjectPtr<UBrushStampIndicator> BrushIndicator;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> BrushIndicatorMaterial;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> BrushIndicatorMesh;

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UOctreeDynamicMeshComponent> DynamicMeshComponent;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> ActiveOverrideMaterial;

	UE::Geometry::FTransformSRT3d InitialTargetTransform;
	UE::Geometry::FTransformSRT3d CurTargetTransform;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged();
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	TValueWatcher<bool> ShowWireframeWatcher;
	TValueWatcher<EMeshEditingMaterialModes> MaterialModeWatcher;
	TValueWatcher<TWeakObjectPtr<UMaterialInterface>> CustomMaterialWatcher;
	TValueWatcher<bool> FlatShadingWatcher;
	TValueWatcher<FLinearColor> ColorWatcher;
	TValueWatcher<FLinearColor> TransparentColorWatcher;
	TValueWatcher<double> OpacityWatcher;
	TValueWatcher<bool> TwoSidedWatcher;
	TValueWatcher<UTexture2D*> ImageWatcher;
	TValueWatcher<EDynamicMeshSculptBrushType> BrushTypeWatcher;
	TValueWatcher<FVector> GizmoPositionWatcher;
	TValueWatcher<FQuat> GizmoRotationWatcher;
	UE_API void UpdateMaterialMode(EMeshEditingMaterialModes NewMode);
	UE_API void UpdateFlatShadingSetting(bool bNewValue);
	UE_API void UpdateColorSetting(FLinearColor NewColor);
	UE_API void UpdateOpacitySetting(double Opacity);
	UE_API void UpdateTwoSidedSetting(bool bOn);
	UE_API void UpdateCustomMaterial(TWeakObjectPtr<UMaterialInterface> NewMaterial);
	UE_API void UpdateImageSetting(UTexture2D* NewImage);
	UE_API void UpdateBrushType(EDynamicMeshSculptBrushType BrushType);
	UE_API void UpdateGizmoFromProperties();

	UE::Geometry::FInterval1d BrushRelativeSizeRange;
	double CurrentBrushRadius;
	UE_API void CalculateBrushRadius();

	bool bEnableRemeshing;
	double InitialEdgeLength;
	UE_API void ScheduleRemeshPass();
	UE_API void ConfigureRemesher(UE::Geometry::FSubRegionRemesher& Remesher);
	UE_API void InitializeRemesherROI(UE::Geometry::FSubRegionRemesher& Remesher);

	TSharedPtr<FPersistentStampRemesher> ActiveRemesher;
	UE_API void InitializeActiveRemesher();
	UE_API void PrecomputeRemesherROI();
	UE_API void RemeshROIPass_ActiveRemesher(bool bHasPrecomputedROI);


	bool bInDrag;

	UE::Geometry::FFrame3d ActiveDragPlane;
	FVector3d LastHitPosWorld;
	FVector3d BrushStartCenterWorld;
	FVector3d BrushStartNormalWorld;
	FVector3d LastBrushPosLocal;
	FVector3d LastBrushPosWorld;
	FVector3d LastBrushPosNormalWorld;
	FVector3d LastSmoothBrushPosLocal;
	int32 LastBrushTriangleID = -1;

	TArray<int> UpdateROITriBuffer;
	UE::Geometry::FUniqueIndexSet VertexROIBuilder;
	TArray<int> VertexROI;
	UE::Geometry::FUniqueIndexSet TriangleROIBuilder;
	TSet<int> TriangleROI;
	//TSet<int> TriangleROI;
	UE_API void UpdateROI(const FVector3d& BrushPos);

	bool bRemeshPending;
	bool bNormalUpdatePending;
	
	bool bTargetDirty;
	TFuture<void> PendingTargetUpdate;

	bool bSmoothing;
	bool bInvert;
	float ActivePressure = 1.0f;

	bool bHaveRemeshed;

	bool bStampPending;
	FRay PendingStampRay;
	int StampTimestamp = 0;
	EDynamicMeshSculptBrushType LastStampType = EDynamicMeshSculptBrushType::LastValue;
	EDynamicMeshSculptBrushType PendingStampType = LastStampType;
	UE_API void ApplyStamp(const FRay& WorldRay);

	FDynamicMesh3 BrushTargetMesh;
	UE::Geometry::FDynamicMeshAABBTree3 BrushTargetMeshSpatial;
	UE::Geometry::FMeshNormals BrushTargetNormals;
	bool bCachedFreezeTarget = false;
	UE_API void UpdateTarget();
	UE_API bool GetTargetMeshNearest(const FVector3d& Position, double SearchRadius, FVector3d& TargetPosOut, FVector3d& TargetNormalOut);

	UE_API int FindHitSculptMeshTriangle(const FRay3d& LocalRay);
	UE_API int FindHitTargetMeshTriangle(const FRay3d& LocalRay);
	UE_API bool IsHitTriangleBackFacing(int32 TriangleID, const FDynamicMesh3* QueryMesh);

	UE_API bool UpdateBrushPosition(const FRay& WorldRay);
	UE_API bool UpdateBrushPositionOnActivePlane(const FRay& WorldRay);
	UE_API bool UpdateBrushPositionOnTargetMesh(const FRay& WorldRay, bool bFallbackToViewPlane);
	UE_API bool UpdateBrushPositionOnSculptMesh(const FRay& WorldRay, bool bFallbackToViewPlane);
	UE_API void AlignBrushToView();

	UE_API bool ApplySmoothBrush(const FRay& WorldRay);
	UE_API bool ApplyMoveBrush(const FRay& WorldRay);
	UE_API bool ApplyOffsetBrush(const FRay& WorldRay, bool bUseViewDirection);
	UE_API bool ApplySculptMaxBrush(const FRay& WorldRay);
	UE_API bool ApplyPinchBrush(const FRay& WorldRay);
	UE_API bool ApplyInflateBrush(const FRay& WorldRay);
	UE_API bool ApplyPlaneBrush(const FRay& WorldRay);
	UE_API bool ApplyFixedPlaneBrush(const FRay& WorldRay);
	UE_API bool ApplyFlattenBrush(const FRay& WorldRay);
	UE_API bool ApplyResampleBrush(const FRay& WorldRay);
	UE_API bool ApplyPullKelvinBrush(const FRay& WorldRay);
	UE_API bool ApplyPullSharpKelvinBrush(const FRay& WorldRay);
	UE_API bool ApplyTwistKelvinBrush(const FRay& WorldRay);
	UE_API bool ApplyScaleKelvinBrush(const FRay& WorldRay);

	double SculptMaxFixedHeight = -1.0;

	UE_API double CalculateBrushFalloff(double Distance);
	TArray<FVector3d> ROIPositionBuffer;
	UE_API void SyncMeshWithPositionBuffer(FDynamicMesh3* Mesh);

	UE::Geometry::FFrame3d ActiveFixedBrushPlane;
	UE_API UE::Geometry::FFrame3d ComputeROIBrushPlane(const FVector3d& BrushCenter, bool bIgnoreDepth, bool bViewAligned);

	TArray<int> NormalsBuffer;
	TArray<bool> NormalsVertexFlags;
	UE_API void RecalculateNormals_PerVertex(const TSet<int32>& Triangles);
	UE_API void RecalculateNormals_Overlay(const TSet<int32>& Triangles);

	bool bHaveMeshBoundaries;
	bool bHaveUVSeams;
	bool bHaveNormalSeams;
	TSet<int32> RemeshRemovedTriangles;
	TSet<int32> RemeshFinalTriangleROI;
	UE_API void PrecomputeRemeshInfo();
	UE_API void RemeshROIPass();

	FMeshVertexChangeBuilder* ActiveVertexChange = nullptr;
	UE::Geometry::FDynamicMeshChangeTracker* ActiveMeshChange = nullptr;
	UE::TransactionUtil::FLongTransactionTracker LongTransactions;
	
	UE_API void BeginChange(bool bIsVertexChange);
	UE_API void EndChange();
	UE_API void CancelChange();
	UE_API void SaveActiveROI();

	UE_API double EstimateIntialSafeTargetLength(const FDynamicMesh3& Mesh, int MinTargetTriCount);

	TArray<EDynamicMeshSculptBrushType> BrushTypeHistory;
	int BrushTypeHistoryIndex = 0;

	UE_API UPreviewMesh* MakeDefaultSphereMesh(UObject* Parent, UWorld* World, int Resolution = 32);

	//
	// support for gizmo in FixedPlane mode
	//

	// plane gizmo
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> PlaneTransformGizmo;

	UPROPERTY()
	TObjectPtr<UTransformProxy> PlaneTransformProxy;

	UE_API void PlaneTransformChanged(UTransformProxy* Proxy, FTransform Transform);

	enum class EPendingWorkPlaneUpdate
	{
		NoUpdatePending,
		MoveToHitPositionNormal,
		MoveToHitPosition,
		MoveToHitPositionViewAligned
	};
	EPendingWorkPlaneUpdate PendingWorkPlaneUpdate;
	UE_API void SetFixedSculptPlaneFromWorldPos(const FVector& Position, const FVector& Normal, EPendingWorkPlaneUpdate UpdateType);
	UE_API void UpdateFixedSculptPlanePosition(const FVector& Position);
	UE_API void UpdateFixedSculptPlaneRotation(const FQuat& Rotation);
	UE_API void UpdateFixedPlaneGizmoVisibility(bool bVisible);
};



#undef UE_API
