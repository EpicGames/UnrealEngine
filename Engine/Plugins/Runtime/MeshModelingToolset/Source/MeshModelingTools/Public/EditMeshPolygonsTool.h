// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Changes/MeshVertexChange.h" // IMeshVertexCommandChangeTarget
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h" // FDynamicMeshChange for TUniquePtr
#include "InteractiveToolActivity.h" // IToolActivityHost
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolNestedAcceptCancelAPI
#include "Internationalization/Text.h"
#include "Operations/GroupTopologyDeformer.h"
#include "BaseTools/SingleTargetWithSelectionTool.h"

#include "GeometryBase.h"

#include "EditMeshPolygonsTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_GEOMETRY(class FGroupTopology);
PREDECLARE_GEOMETRY(struct FGroupTopologySelection);
PREDECLARE_GEOMETRY(class FDynamicMeshChangeTracker);

struct FSlateBrush;
class UCombinedTransformGizmo;
class UDragAlignmentMechanic;
class UMeshOpPreviewWithBackgroundCompute; 
class FMeshVertexChangeBuilder;
class UEditMeshPolygonsTool;
class UPolyEditActivityContext;
class UPolyEditInsertEdgeActivity;
class UPolyEditInsertEdgeLoopActivity;
class UPolyEditExtrudeActivity;
class UPolyEditInsetOutsetActivity;
class UPolyEditCutFacesActivity;
class UPolyEditPlanarProjectionUVActivity;
class UPolyEditBevelEdgeActivity;
class UPolyEditExtrudeEdgeActivity;
class UPolygonSelectionMechanic;
class UTransformProxy;


/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UEditMeshPolygonsToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	bool bTriangleMode = false;

	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const override;

	virtual bool RequiresInputSelection() const override { return false; }
};


UENUM()
enum class ELocalFrameMode
{
	FromObject,
	FromGeometry
};


/** 
 * These are properties that do not get enabled/disabled based on the action 
 */
UCLASS(MinimalAPI)
class UPolyEditCommonProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowWireframe = false;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowSelectableCorners = true;

	/** When true, allows the transform gizmo to be rendered */
	UPROPERTY(EditAnywhere, Category = Gizmo)
	bool bGizmoVisible = true;

	/** Determines whether, on selection changes, the gizmo's rotation is taken from the object transform, or from the geometry
	 elements selected. Only relevant with a local coordinate system and when rotation is not locked. */
	UPROPERTY(EditAnywhere, Category = Gizmo, meta = (HideEditConditionToggle, EditCondition = "bLocalCoordSystem && !bLockRotation"))
	ELocalFrameMode LocalFrameMode = ELocalFrameMode::FromGeometry;

	/** When true, keeps rotation of gizmo constant through selection changes and manipulations 
	 (but not middle-click repositions). Only active with a local coordinate system.*/
	UPROPERTY(EditAnywhere, Category = Gizmo, meta = (HideEditConditionToggle, EditCondition = "bLocalCoordSystem"))
	bool bLockRotation = false;

	/** This gets updated internally so that properties can respond to whether the coordinate system is set to local or global */
	UPROPERTY()
	bool bLocalCoordSystem = true;
};


UENUM()
enum class EEditMeshPolygonsToolActions
{
	NoAction,
	AcceptCurrent,
	CancelCurrent,
	Extrude,
	PushPull,
	Offset,
	Inset,
	Outset,
	BevelFaces,
	InsertEdge,
	InsertEdgeLoop,
	Complete,

	PlaneCut,
	Merge,
	Delete,
	CutFaces,
	RecalculateNormals,
	FlipNormals,
	Retriangulate,
	Decompose,
	Disconnect,
	Duplicate,

	CollapseEdge,
	WeldEdges,
	WeldEdgesCentered,
	StraightenEdge,
	FillHole,
	BridgeEdges,
	ExtrudeEdges,
	BevelEdges,
	SimplifyAlongEdges,

	PlanarProjectionUV,

	SimplifyByGroups,
	RegenerateExtraCorners,

	// triangle-specific edits
	PokeSingleFace,
	SplitSingleEdge,
	FlipSingleEdge,
	CollapseSingleEdge,

	// for external use
	BevelAuto
};

UCLASS(MinimalAPI)
class UEditMeshPolygonsActionModeToolBuilder : public UEditMeshPolygonsToolBuilder
{
	GENERATED_BODY()
public:
	EEditMeshPolygonsToolActions StartupAction = EEditMeshPolygonsToolActions::Extrude;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const override;
};

UENUM()
enum class EEditMeshPolygonsToolSelectionMode
{
	Faces,
	Edges,
	Vertices,
	Loops,
	Rings,
	FacesEdgesVertices
};

UCLASS(MinimalAPI)
class UEditMeshPolygonsSelectionModeToolBuilder : public UEditMeshPolygonsToolBuilder
{
	GENERATED_BODY()
public:
	EEditMeshPolygonsToolSelectionMode SelectionMode = EEditMeshPolygonsToolSelectionMode::Faces;

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const override;
};



UCLASS(MinimalAPI)
class UEditMeshPolygonsToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UEditMeshPolygonsTool> ParentTool;

	void Initialize(UEditMeshPolygonsTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EEditMeshPolygonsToolActions Action);
};

UCLASS(MinimalAPI)
class UPolyEditTopologyProperties : public UEditMeshPolygonsToolActionPropertySet
{
	GENERATED_BODY()

public:
	/** 
	 * When true, adds extra corners at sharp group edge bends (in addition to the normal corners that
	 * are placed at junctures of three or more group edges). For instance, a single disconnected quad-like group
	 * would normally have a single group edge with no corners, since it has no neighboring groups, but this
	 * setting will allow for the generation of corners at the quad corners, which is very useful for editing.
	 * Note that the setting takes effect only after clicking Regenerate Extra Corners or performing some
	 * operation that changes the group topology.
	 */
	UPROPERTY(EditAnywhere, Category = TopologyOptions)
	bool bAddExtraCorners = true;

	UFUNCTION(CallInEditor, Category = TopologyOptions)
	void RegenerateExtraCorners() { PostAction(EEditMeshPolygonsToolActions::RegenerateExtraCorners); }

	/** 
	 * When generating extra corners, how sharp the angle needs to be to warrant an extra corner placement there. Lower values require
	 * sharper corners, so are more tolerant of curved group edges. For instance, 180 will place corners at every vertex along a group
	 * edge even if the edge is perfectly straight, and 135 will place a vertex only once the edge bends 45 degrees off the straight path
	 * (i.e. 135 degrees to the previous edge). 
	 * The setting is applied either when Regenerate Extra Corners is clicked, or after any operation that modifies topology.
	 */
	UPROPERTY(EditAnywhere, Category = TopologyOptions, meta = (ClampMin = "0", ClampMax = "180", EditCondition = "bAddExtraCorners"))
	double ExtraCornerAngleThresholdDegrees = 135;
};

/** PolyEdit Actions */
UCLASS(MinimalAPI)
class UEditMeshPolygonsToolActions : public UEditMeshPolygonsToolActionPropertySet
{
	GENERATED_BODY()
public:
	/** Extrude the current set of selected faces by moving and stitching them. */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Extrude", DisplayPriority = 1))
	void Extrude() { PostAction(EEditMeshPolygonsToolActions::Extrude); }

	/** Like Extrude/Offset, but performed in a boolean way, meaning that the faces can cut away the mesh or bridge mesh parts. */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Push/Pull", DisplayPriority = 1))
	void PushPull() { PostAction(EEditMeshPolygonsToolActions::PushPull); }

	/** Like Extrude, but defaults to moving verts along vertex normals instead of a single direction.*/
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Offset", DisplayPriority = 1))
	void Offset() { PostAction(EEditMeshPolygonsToolActions::Offset); }

	/**
	 * Inset the current set of selected faces. Click in viewport to confirm inset distance.
	 * 
	 * (An Inset operation stitches in a smaller version of selected faces inside the existing ones)
	 */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Inset", DisplayPriority = 2))
	void Inset() { PostAction(EEditMeshPolygonsToolActions::Inset);	}

	/**
	 * Outset the current set of selected faces. Click in viewport to confirm outset distance.
	 * 
	 * (An Outset operation stitches in a larger version of selected faces inside the existing ones)
	 */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Outset", DisplayPriority = 3))
	void Outset() { PostAction(EEditMeshPolygonsToolActions::Outset);	}

	/** Bevel the edge loops around the selected faces, inserting edge-aligned faces that interpolate the normals of the selected faces */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Bevel", DisplayPriority = 4))
	void Bevel() { PostAction(EEditMeshPolygonsToolActions::BevelFaces); }

	/** Merge the current set of selected faces into a single face */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Merge", DisplayPriority = 4))
	void Merge() { PostAction(EEditMeshPolygonsToolActions::Merge);	}

	/** Delete the current set of selected faces */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Delete Faces", DisplayPriority = 4))
	void Delete() { PostAction(EEditMeshPolygonsToolActions::Delete); }

	/** Cut the current set of selected faces. Click twice in viewport to set cut line. */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "CutFaces", DisplayPriority = 5))
	void CutFaces() { PostAction(EEditMeshPolygonsToolActions::CutFaces); }

	/** Recalculate normals for the current set of selected faces */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "RecalcNormals", DisplayPriority = 6))
	void RecalcNormals() { PostAction(EEditMeshPolygonsToolActions::RecalculateNormals); }

	/** Flip normals and face orientation for the current set of selected faces */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Flip", DisplayPriority = 7))
	void Flip() { PostAction(EEditMeshPolygonsToolActions::FlipNormals); }

	/** Retriangulate each of the selected faces */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Retriangulate", DisplayPriority = 9))
	void Retriangulate() { PostAction(EEditMeshPolygonsToolActions::Retriangulate);	}

	/** Split each of the selected faces into a separate polygon for each triangle */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Decompose", DisplayPriority = 10))
	void Decompose() { PostAction(EEditMeshPolygonsToolActions::Decompose);	}

	/** Separate the selected faces at their borders */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Disconnect", DisplayPriority = 11))
	void Disconnect() { PostAction(EEditMeshPolygonsToolActions::Disconnect); }

	/** Duplicate the selected faces at their borders */
	UFUNCTION(CallInEditor, Category = FaceEdits, meta = (DisplayName = "Duplicate", DisplayPriority = 12))
	void Duplicate() { PostAction(EEditMeshPolygonsToolActions::Duplicate); }

	/** Insert a chain of edges across quads (faces with four edges) in the mesh. Due to ambiguity, edges will not be inserted on non-quad faces. */
	UFUNCTION(CallInEditor, Category = ShapeEdits, meta = (DisplayName = "InsertEdgeLoop", DisplayPriority = 13))
	void InsertEdgeLoop() { PostAction(EEditMeshPolygonsToolActions::InsertEdgeLoop); }

	/** Insert a new edge connecting existing edges or vertices on a single face */
	UFUNCTION(CallInEditor, Category = ShapeEdits, meta = (DisplayName = "Insert Edge", DisplayPriority = 14))
	void InsertEdge() { PostAction(EEditMeshPolygonsToolActions::InsertEdge); }

	/** Simplify every polygon group by removing vertices on shared straight edges and retriangulating */
	UFUNCTION(CallInEditor, Category = ShapeEdits, meta = (DisplayName = "SimplifyByGroups", DisplayPriority = 15))
	void SimplifyByGroups() { PostAction(EEditMeshPolygonsToolActions::SimplifyByGroups); }

};



UCLASS(MinimalAPI)
class UEditMeshPolygonsToolActions_Triangles : public UEditMeshPolygonsToolActionPropertySet
{
	GENERATED_BODY()
public:
	/** Extrude the current set of selected faces. Click in viewport to confirm extrude height. */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Extrude", DisplayPriority = 1))
	void Extrude() { PostAction(EEditMeshPolygonsToolActions::Extrude); }

	/** Like Extrude/Offset, but performed in a boolean way, meaning that the faces can cut away the mesh or bridge mesh parts. */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Push/Pull", DisplayPriority = 1))
	void PushPull() { PostAction(EEditMeshPolygonsToolActions::PushPull); }

	/** Like Extrude, but defaults to moving verts along vertex normals instead of a single direction. */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Offset", DisplayPriority = 1))
	void Offset() { PostAction(EEditMeshPolygonsToolActions::Offset); }

	/** Inset the current set of selected faces. Click in viewport to confirm inset distance. */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Inset", DisplayPriority = 2))
	void Inset() { PostAction(EEditMeshPolygonsToolActions::Inset);	}

	/** Outset the current set of selected faces. Click in viewport to confirm outset distance. */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Outset", DisplayPriority = 3))
	void Outset() { PostAction(EEditMeshPolygonsToolActions::Outset);	}

	/** Delete the current set of selected faces */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Delete Faces", DisplayPriority = 4))
	void Delete() { PostAction(EEditMeshPolygonsToolActions::Delete); }

	/** Cut the current set of selected faces. Click twice in viewport to set cut line. */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "CutFaces", DisplayPriority = 5))
	void CutFaces() { PostAction(EEditMeshPolygonsToolActions::CutFaces); }

	/** Recalculate normals for the current set of selected faces */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "RecalcNormals", DisplayPriority = 6))
	void RecalcNormals() { PostAction(EEditMeshPolygonsToolActions::RecalculateNormals); }

	/** Flip normals and face orientation for the current set of selected faces */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Flip", DisplayPriority = 7))
	void Flip() { PostAction(EEditMeshPolygonsToolActions::FlipNormals); }

	/** Separate the selected faces at their borders */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Disconnect", DisplayPriority = 11))
	void Disconnect() { PostAction(EEditMeshPolygonsToolActions::Disconnect); }

	/** Duplicate the selected faces */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Duplicate", DisplayPriority = 12))
	void Duplicate() { PostAction(EEditMeshPolygonsToolActions::Duplicate); }

	/** Insert a new vertex at the center of each selected face */
	UFUNCTION(CallInEditor, Category = TriangleEdits, meta = (DisplayName = "Poke", DisplayPriority = 13))
	void Poke() { PostAction(EEditMeshPolygonsToolActions::PokeSingleFace); }
};





UCLASS(MinimalAPI)
class UEditMeshPolygonsToolUVActions : public UEditMeshPolygonsToolActionPropertySet
{
	GENERATED_BODY()

public:

	/** Assign planar-projection UVs to mesh */
	UFUNCTION(CallInEditor, Category = UVs, meta = (DisplayName = "PlanarProjection", DisplayPriority = 11))
	void PlanarProjection()
	{
		PostAction(EEditMeshPolygonsToolActions::PlanarProjectionUV);
	}
};





UCLASS(MinimalAPI)
class UEditMeshPolygonsToolEdgeActions : public UEditMeshPolygonsToolActionPropertySet
{
	GENERATED_BODY()
public:
	/** Merge selected boundary edges, centering the result */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Weld", DisplayPriority = 0))
	void WeldCentered() { PostAction(EEditMeshPolygonsToolActions::WeldEdgesCentered); }

	/** Merge selected boundary edges, moving the first edge to the second */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Weld To", DisplayPriority = 1))
	void Weld() { PostAction(EEditMeshPolygonsToolActions::WeldEdges); }

	/** Make each selected polygroup edge follow a straight path between its endpoints */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Straighten", DisplayPriority = 2))
	void Straighten() { PostAction(EEditMeshPolygonsToolActions::StraightenEdge); }

	/** Fill the adjacent hole for any selected boundary edges */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Fill Hole", DisplayPriority = 3))
	void FillHole()	{ PostAction(EEditMeshPolygonsToolActions::FillHole); }

	/** Bevel the selected edges, replacing them with angled faces */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Bevel", DisplayPriority = 4))
	void Bevel() { PostAction(EEditMeshPolygonsToolActions::BevelEdges); }
	
	/** Create a new face that connects the selected edges */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Bridge", DisplayPriority = 5))
	void Bridge() { PostAction(EEditMeshPolygonsToolActions::BridgeEdges); }

	/** Duplicate and move boundary edge vertices outwards and connect them to the original boundary to create new faces. */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayPriority = 6))
	void Extrude() { PostAction(EEditMeshPolygonsToolActions::ExtrudeEdges); }

	/** Simplify the underlying triangulation along the selected edges, when doing so won't change the shape or UVs, or make low-quality triangles */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayPriority = 7))
	void Simplify() { PostAction(EEditMeshPolygonsToolActions::SimplifyAlongEdges); }
	
	/** Delete selected edge, implicitly merging any connected faces */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Delete Edges", DisplayPriority = 8))
	void DeleteEdge() { PostAction(EEditMeshPolygonsToolActions::Delete); }

	/** Collapse the selected edges, deleting the attached triangles and merging their vertices into one */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Collapse", DisplayPriority = 5))
	void Collapse() { PostAction(EEditMeshPolygonsToolActions::CollapseEdge); }
};


UCLASS(MinimalAPI)
class UEditMeshPolygonsToolEdgeActions_Triangles : public UEditMeshPolygonsToolActionPropertySet
{
	GENERATED_BODY()
public:
	/** Merge selected boundary edges, centering the result */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Weld", DisplayPriority = 0))
	void WeldCentered() { PostAction(EEditMeshPolygonsToolActions::WeldEdgesCentered); }

	/** Merge selected boundary edges, moving the first edge to the second */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Weld To", DisplayPriority = 1))
	void Weld() { PostAction(EEditMeshPolygonsToolActions::WeldEdges); }

	/** Fill the adjacent hole for any selected boundary edges */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Fill Hole", DisplayPriority = 2))
	void FillHole() { PostAction(EEditMeshPolygonsToolActions::FillHole); }

	/** Create a new face that connects the selected edges */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Bridge", DisplayPriority = 3))
	void Bridge() { PostAction(EEditMeshPolygonsToolActions::BridgeEdges); }
	
	/** Duplicate and move boundary vertices outwards and connect them to the original boundary to create new faces. */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Extrude", DisplayPriority = 4))
	void Extrude() { PostAction(EEditMeshPolygonsToolActions::ExtrudeEdges); }

	/** Collapse the selected edges, deleting the attached triangles and merging its two vertices into one */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Collapse", DisplayPriority = 5))
	void Collapse() { PostAction(EEditMeshPolygonsToolActions::CollapseEdge); }

	/** Flip the selected (non-border, non-seam) edges, replacing them with new edges in the crossing direction */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Flip", DisplayPriority = 6))
	void Flip() { PostAction(EEditMeshPolygonsToolActions::FlipSingleEdge); }

	/** Split the selected edges, inserting a new vertex at each edge midpoint */
	UFUNCTION(CallInEditor, Category = EdgeEdits, meta = (DisplayName = "Split", DisplayPriority = 7))
	void Split() { PostAction(EEditMeshPolygonsToolActions::SplitSingleEdge); }
};



/**
 *
 */
UCLASS(MinimalAPI)
class UEditMeshPolygonsTool : public USingleTargetWithSelectionTool,
	public IToolActivityHost, 
	public IMeshVertexCommandChangeTarget,
	public IInteractiveToolNestedAcceptCancelAPI
{
	GENERATED_BODY()
	using FFrame3d = UE::Geometry::FFrame3d;

public:
	UE_API UEditMeshPolygonsTool();

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;
	UE_API void EnableTriangleMode();

	// used by undo/redo
	UE_API void RebuildTopologyWithGivenExtraCorners(const TSet<int32>& Vids);

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual FBox GetWorldSpaceFocusBox() override;
	UE_API virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) override;

	// IToolActivityHost
	UE_API virtual void NotifyActivitySelfEnded(UInteractiveToolActivity* Activity) override;

	// IMeshVertexCommandChangeTarget
	UE_API virtual void ApplyChange(const FMeshVertexChange* Change, bool bRevert) override;

	// IInteractiveToolNestedAcceptCancelAPI
	virtual bool SupportsNestedCancelCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedCancel() override;
	UE_API virtual bool ExecuteNestedCancelCommand() override;
	virtual bool SupportsNestedAcceptCommand() override { return true; }
	UE_API virtual bool CanCurrentlyNestedAccept() override;
	UE_API virtual bool ExecuteNestedAcceptCommand() override;

public:

	UE_API virtual void RequestAction(EEditMeshPolygonsToolActions ActionType);

	UE_API virtual void RequestSingleShotAction(EEditMeshPolygonsToolActions ActionType);

	UE_API void SetActionButtonsVisibility(bool bVisible);

protected:
	// If bTriangleMode = true, then we use a per-triangle FTriangleGroupTopology instead of polygroup topology.
	// This allows low-level mesh editing with mainly the same code, at a significant cost in overhead.
	// This is a fundamental mode switch, must be set before ::Setup() is called!
	bool bTriangleMode;		

	// TODO: This is a hack to allow us to disallow any actions inside the tool after Setup() is called. We
	// use it if the user tries to run the tool on a mesh that has too many edges for us to render, to avoid
	// hanging the editor.
	bool bToolDisabled = false;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditCommonProperties> CommonProps = nullptr;

	UPROPERTY()
	TObjectPtr<UEditMeshPolygonsToolActions> EditActions = nullptr;
	UPROPERTY()
	TObjectPtr<UEditMeshPolygonsToolActions_Triangles> EditActions_Triangles = nullptr;

	UPROPERTY()
	TObjectPtr<UEditMeshPolygonsToolEdgeActions> EditEdgeActions = nullptr;
	UPROPERTY()
	TObjectPtr<UEditMeshPolygonsToolEdgeActions_Triangles> EditEdgeActions_Triangles = nullptr;

	UPROPERTY()
	TObjectPtr<UEditMeshPolygonsToolUVActions> EditUVActions = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditTopologyProperties> TopologyProperties = nullptr;

	/**
	 * Activity objects that handle multi-interaction operations
	 */
	UPROPERTY()
	TObjectPtr<UPolyEditExtrudeActivity> ExtrudeActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditInsetOutsetActivity> InsetOutsetActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditCutFacesActivity> CutFacesActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditPlanarProjectionUVActivity> PlanarProjectionUVActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditInsertEdgeActivity> InsertEdgeActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditInsertEdgeLoopActivity> InsertEdgeLoopActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditBevelEdgeActivity> BevelEdgeActivity = nullptr;
	UPROPERTY()
	TObjectPtr<UPolyEditExtrudeEdgeActivity> ExtrudeEdgeActivity = nullptr;

	TMap<UInteractiveToolActivity*, FText> ActivityLabels;
	TMap<UInteractiveToolActivity*, FName> ActivityIconNames;

	/**
	 * Points to one of the activities when it is active
	 */
	TObjectPtr<UInteractiveToolActivity> CurrentActivity = nullptr;

	TSharedPtr<UE::Geometry::FDynamicMesh3> CurrentMesh;
	TSharedPtr<UE::Geometry::FGroupTopology> Topology;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3> MeshSpatial;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	FText DefaultMessage;

	UE_API void ResetUserMessage();

	bool bSelectionStateDirty = false;
	UE_API void OnSelectionModifiedEvent();

	UE_API void OnBeginGizmoTransform(UTransformProxy* Proxy);
	UE_API void OnEndGizmoTransform(UTransformProxy* Proxy);
	UE_API void OnGizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void UpdateGizmoFrame(const FFrame3d* UseFrame = nullptr);
	FFrame3d LastGeometryFrame;
	FFrame3d LastTransformerFrame;
	FFrame3d LockedTransfomerFrame;
	bool bInGizmoDrag = false;

	UE::Geometry::FTransformSRT3d BakedTransform; // We bake the scale part of the Target -> World transform
	UE::Geometry::FTransformSRT3d WorldTransform; // Transform from Baked to World

	FFrame3d InitialGizmoFrame;
	FVector3d InitialGizmoScale;
	bool bGizmoUpdatePending = false;
	FFrame3d LastUpdateGizmoFrame;
	FVector3d LastUpdateGizmoScale;
	bool bLastUpdateUsedWorldFrame = false;
	UE_API void ComputeUpdate_Gizmo();

	UE_API UE::Geometry::FDynamicMeshAABBTree3& GetSpatial();
	bool bSpatialDirty;

	// UV Scale factor to apply to texturing on any new geometry (e.g. new faces added by extrude)
	float UVScaleFactor = 1.0f;

	EEditMeshPolygonsToolActions PendingAction = EEditMeshPolygonsToolActions::NoAction;
	bool bTerminateOnPendingActionComplete = false;

	int32 ActivityTimestamp = 1;

	UE_API void StartActivity(TObjectPtr<UInteractiveToolActivity> Activity);
	UE_API void EndCurrentActivity(EToolShutdownType ShutdownType);
	UE_API void SetActionButtonPanelsVisible(bool bVisible);

	// Emit an undoable change to CurrentMesh and update related structures (preview, spatial, etc)
	UE_API void EmitCurrentMeshChangeAndUpdate(const FText& TransactionLabel,
		TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChangeIn,
		const UE::Geometry::FGroupTopologySelection& OutputSelection);

	// Emit an undoable start of an activity
	UE_API void EmitActivityStart(const FText& TransactionLabel);

	UE_API void UpdateGizmoVisibility();

	UE_API void ApplyDelete();
	
	UE_API void ApplyMerge();
	UE_API void ApplyDeleteFaces();
	UE_API void ApplyRecalcNormals();
	UE_API void ApplyFlipNormals();
	UE_API void ApplyRetriangulate();
	UE_API void ApplyDecompose();
	UE_API void ApplyDisconnect();
	UE_API void ApplyDuplicate();
	UE_API void ApplyPokeSingleFace();

	UE_API void ApplyCollapseEdge();
	UE_API void ApplyWeldEdges();
	UE_API void ApplyStraightenEdges();
	UE_API void ApplyDeleteEdges();
	UE_API void ApplyFillHole();
	UE_API void ApplyBridgeEdges();
	UE_API void ApplySimplifyAlongEdges();

	UE_API void ApplyFlipSingleEdge();
	UE_DEPRECATED(5.5, "Use ApplyCollapseEdge instead.")
	UE_API void ApplyCollapseSingleEdge();
	UE_API void ApplySplitSingleEdge();

	UE_API void SimplifyByGroups();

	UE_API void ApplyRegenerateExtraCorners();
	double ExtraCornerDotProductThreshold = -1;

	FFrame3d ActiveSelectionFrameLocal;
	FFrame3d ActiveSelectionFrameWorld;
	TArray<int32> ActiveTriangleSelection;
	UE::Geometry::FAxisAlignedBox3d ActiveSelectionBounds;

	struct FSelectedEdge
	{
		int32 EdgeTopoID;
		TArray<int32> EdgeIDs;
	};
	TArray<FSelectedEdge> ActiveEdgeSelection;

	enum class EPreviewMaterialType
	{
		SourceMaterials, PreviewMaterial, UVMaterial
	};
	UE_API void UpdateEditPreviewMaterials(EPreviewMaterialType MaterialType);
	EPreviewMaterialType CurrentPreviewMaterial;


	//
	// data for current drag
	//
	UE::Geometry::FGroupTopologyDeformer LinearDeformer;
	UE_API void UpdateDeformerFromSelection(const UE::Geometry::FGroupTopologySelection& Selection);

	FMeshVertexChangeBuilder* ActiveVertexChange;
	UE_API void UpdateDeformerChangeFromROI(bool bFinal);
	UE_API void BeginDeformerChange();
	UE_API void EndDeformerChange();

	UE_API bool BeginMeshFaceEditChange();

	UE_API bool BeginMeshEdgeEditChange();
	UE_API bool BeginMeshBoundaryEdgeEditChange(bool bOnlySimple);
	UE_API bool BeginMeshEdgeEditChange(TFunctionRef<bool(int32)> GroupEdgeIDFilterFunc);

	UE_API void UpdateFromCurrentMesh(bool bRebuildTopology);
	int32 ModifiedTopologyCounter = 0;
	bool bWasTopologyEdited = false;

	friend class FEditMeshPolygonsToolMeshChange;
	friend class FPolyEditActivityStartChange;

	// custom setup support
	friend class UEditMeshPolygonsSelectionModeToolBuilder;
	friend class UEditMeshPolygonsActionModeToolBuilder;
	TUniqueFunction<void(UEditMeshPolygonsTool*)> PostSetupFunction;
	UE_API void SetToSelectionModeInterface();

private:
	UE_API void ApplyWeldEdges(double InterpolationT);
	UE_API void ApplyWeldVertices(double InterpolationT);
	UE_API void CollapseGroupEdges(TSet<int32>& GroupEdgesToCollapse, 
		UE::Geometry::FDynamicMeshChangeTracker& ChangeTracker);
};


/**
 * Wraps a FDynamicMeshChange so that it can be expired and so that other data
 * structures in the tool can be updated. On apply/revert, the topology is rebuilt
 * using stored extra corner vids.
 */
class FEditMeshPolygonsToolMeshChange : public FToolCommandChange
{
public:
	FEditMeshPolygonsToolMeshChange(TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChangeIn)
		: MeshChange(MoveTemp(MeshChangeIn))
	{};

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;

	TSet<int32> ExtraCornerVidsBefore;
	TSet<int32> ExtraCornerVidsAfter;
protected:
	TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChange;
};



/**
 * FPolyEditActivityStartChange is used to cancel out of an active action on Undo. 
 * No action is taken on Redo, ie we do not re-start the Tool on Redo.
 */
class FPolyEditActivityStartChange : public FToolCommandChange
{
public:
	FPolyEditActivityStartChange(int32 ActivityTimestampIn)
	{
		ActivityTimestamp = ActivityTimestampIn;
	}
	virtual void Apply(UObject* Object) override {}
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual bool HasExpired(UObject* Object) const override;
	UE_API virtual FString ToString() const override;

protected:
	bool bHaveDoneUndo = false;
	int32 ActivityTimestamp = 0;
};

#undef UE_API
