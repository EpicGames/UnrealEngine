// Copyright Epic Games, Inc. All Rights Reserved.

// HoleFillTool: Fill one or more boundary loops on a selected mesh. Several hole-filling methods are available.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "SingleSelectionTool.h"
#include "CleaningOps/HoleFillOp.h"
#include "BaseTools/MeshSurfacePointTool.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "MeshBoundaryLoops.h"
#include "HoleFillTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UBoundarySelectionMechanic;
class UDynamicMeshReplacementChangeTarget;
class UMeshOpPreviewWithBackgroundCompute;
class UDynamicMeshComponent;
struct FDynamicMeshOpResult;
class UHoleFillTool;

/*
 * Tool builder
 */

UCLASS(MinimalAPI)
class UHoleFillToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};

/*
 * Properties. This class reflects the parameters in FSmoothFillOptions, but is decorated to allow use in the UI system.
 */

UCLASS(MinimalAPI)
class USmoothHoleFillProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** Allow smoothing and remeshing of triangles outside of the fill region */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions)
	bool bConstrainToHoleInterior;

	/** Number of vertex rings outside of the fill region to allow remeshing */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions, 
		meta = (UIMin = "0", ClampMin = "0", EditCondition = "!bConstrainToHoleInterior", Delta = 1, LinearDeltaSensitivity = 50))
	int RemeshingExteriorRegionWidth;

	/** Number of vertex rings outside of the fill region to perform smoothing */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions, meta = (UIMin = "0", ClampMin = "0", Delta = 1, LinearDeltaSensitivity = 50))
	int SmoothingExteriorRegionWidth;

	/** Number of vertex rings away from the fill region boundary to constrain smoothing */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions, meta = (UIMin = "0", ClampMin = "0", Delta = 1, LinearDeltaSensitivity = 50))
	int SmoothingInteriorRegionWidth;

	/** Desired Smoothness. This is not a linear quantity, but larger numbers produce smoother results */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "100.0"))
	float InteriorSmoothness;

	/** Relative triangle density of fill region */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions, meta = (UIMin = "0.001", UIMax = "10.0", ClampMin = "0.001", ClampMax = "10.0"))
	double FillDensityScalar = 1.0;

	/** 
	 * Whether to project to the original mesh during post-smooth remeshing. This can be expensive on large meshes with 
	 * many holes. 
	 */
	UPROPERTY(EditAnywhere, Category = SmoothHoleFillOptions)
	bool bProjectDuringRemesh = false;


	// Set default property values
	USmoothHoleFillProperties()
	{
		// Create a default FSmoothFillOptions and populate this class with its defaults.
		UE::Geometry::FSmoothFillOptions DefaultOptionsObject;
		bConstrainToHoleInterior = DefaultOptionsObject.bConstrainToHoleInterior;
		RemeshingExteriorRegionWidth = DefaultOptionsObject.RemeshingExteriorRegionWidth;
		SmoothingExteriorRegionWidth = DefaultOptionsObject.SmoothingExteriorRegionWidth;
		SmoothingInteriorRegionWidth = DefaultOptionsObject.SmoothingInteriorRegionWidth;
		FillDensityScalar = DefaultOptionsObject.FillDensityScalar;
		InteriorSmoothness = DefaultOptionsObject.InteriorSmoothness;
		bProjectDuringRemesh = DefaultOptionsObject.bProjectDuringRemesh;
	}

	UE::Geometry::FSmoothFillOptions ToSmoothFillOptions() const
	{
		UE::Geometry::FSmoothFillOptions Options;
		Options.bConstrainToHoleInterior = bConstrainToHoleInterior;
		Options.RemeshingExteriorRegionWidth = RemeshingExteriorRegionWidth;
		Options.SmoothingExteriorRegionWidth = SmoothingExteriorRegionWidth;
		Options.SmoothingInteriorRegionWidth = SmoothingInteriorRegionWidth;
		Options.InteriorSmoothness = InteriorSmoothness;
		Options.FillDensityScalar = FillDensityScalar;
		Options.bProjectDuringRemesh = bProjectDuringRemesh;
		return Options;
	}
};

UCLASS(MinimalAPI)
class UHoleFillToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	EHoleFillOpFillType FillType = EHoleFillOpFillType::Minimal;

	/** Clean up triangles that have no neighbors */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bRemoveIsolatedTriangles = false;

	/** Identify and quickly fill single-triangle holes */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bQuickFillSmallHoles = false;

};


UENUM()
enum class EHoleFillToolActions
{
	NoAction,
	SelectAll,
	ClearSelection
};

UCLASS(MinimalAPI)
class UHoleFillToolActions : public UInteractiveToolPropertySet
{
GENERATED_BODY()

	TWeakObjectPtr<UHoleFillTool> ParentTool;

public:

	void Initialize(UHoleFillTool* ParentToolIn)
	{
		ParentTool = ParentToolIn;
	}

	UE_API void PostAction(EHoleFillToolActions Action);

	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayName = "Select All", DisplayPriority = 1))
		void SelectAll()
	{
		PostAction(EHoleFillToolActions::SelectAll);
	}

	UFUNCTION(CallInEditor, Category = SelectionEdits, meta = (DisplayName = "Clear", DisplayPriority = 1))
		void Clear()
	{
		PostAction(EHoleFillToolActions::ClearSelection);
	}
};


UCLASS(MinimalAPI)
class UHoleFillStatisticsProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(VisibleAnywhere, Category = HoleFillStatistics, meta = (NoResetToDefault))
	FString InitialHoles;

	UPROPERTY(VisibleAnywhere, Category = HoleFillStatistics, meta = (NoResetToDefault))
	FString SelectedHoles;

	UPROPERTY(VisibleAnywhere, Category = HoleFillStatistics, meta = (NoResetToDefault))
	FString SuccessfulFills;

	UPROPERTY(VisibleAnywhere, Category = HoleFillStatistics, meta = (NoResetToDefault))
	FString FailedFills;

	UPROPERTY(VisibleAnywhere, Category = HoleFillStatistics, meta = (NoResetToDefault))
	FString RemainingHoles;

	UE_API void Initialize(const UHoleFillTool& HoleFillTool);

	UE_API void Update(const UHoleFillTool& HoleFillTool, const UE::Geometry::FHoleFillOp& HoleFillOp);
};


/*
 * Operator factory
 */

UCLASS(MinimalAPI)
class UHoleFillOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:

	UE_API TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UHoleFillTool> FillTool;
};

/*
 * Tool
 * Inherit from IClickBehaviorTarget so we can click on boundary loops.
 */

UCLASS(MinimalAPI)
class UHoleFillTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	// UMeshSurfacePointTool
	UE_API void Setup() override;
	UE_API void OnTick(float DeltaTime) override;
	UE_API void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	bool HasCancel() const override { return true; }
	bool HasAccept() const override { return true; }
	UE_API bool CanAccept() const override;
	UE_API void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API void OnSelectionModified();

	UE_API virtual void RequestAction(EHoleFillToolActions Action);

protected:

	friend UHoleFillOperatorFactory;
	friend UHoleFillToolBuilder;
	friend UHoleFillStatisticsProperties;

	UPROPERTY()
	TObjectPtr<USmoothHoleFillProperties> SmoothHoleFillProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UHoleFillToolProperties> Properties = nullptr;

	UPROPERTY()
	TObjectPtr<UHoleFillToolActions> Actions = nullptr;

	UPROPERTY()
	TObjectPtr<UHoleFillStatisticsProperties> Statistics = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPROPERTY()
	TObjectPtr<UBoundarySelectionMechanic> SelectionMechanic = nullptr;

	// Input mesh. Ownership shared with Op.
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	// UV Scale factor is cached based on the bounding box of the mesh before any fills are performed
	float MeshUVScaleFactor = 0.0f;

	// Used for hit querying
	UE::Geometry::FDynamicMeshAABBTree3 MeshSpatial;

	TSet<int32> NewTriangleIDs;

	// Create the Preview object
	UE_API void SetupPreview();

	// Invalidate background compute result (some input changed)
	UE_API void InvalidatePreviewResult();

	bool bHavePendingAction = false;
	EHoleFillToolActions PendingAction;
	UE_API virtual void ApplyAction(EHoleFillToolActions ActionType);
	UE_API void SelectAll();
	UE_API void ClearSelection();

	TUniquePtr<UE::Geometry::FMeshBoundaryLoops> BoundaryLoops;

	struct FSelectedBoundaryLoop
	{
		int32 EdgeTopoID;
		TArray<int32> EdgeIDs;
	};
	TArray<FSelectedBoundaryLoop> ActiveBoundaryLoopSelection;
	UE_API void UpdateActiveBoundaryLoopSelection();

	// Just call the SelectionMechanism's Render function
	UE_API void Render(IToolsContextRenderAPI* RenderAPI) override;

	// Populate an array of Edge Loops to be processed by an FHoleFillOp. Returns the edge loops currently selected
	// by this tool.
	UE_API void GetLoopsToFill(TArray<UE::Geometry::FEdgeLoop>& OutLoops) const;

};

#undef UE_API
