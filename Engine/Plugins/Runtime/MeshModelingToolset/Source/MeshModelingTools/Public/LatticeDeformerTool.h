// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelingOperators.h" //IDynamicMeshOperatorFactory
#include "InteractiveTool.h" //UInteractiveToolPropertySet
#include "InteractiveToolBuilder.h" //UInteractiveToolBuilder
#include "InteractiveToolChange.h" //FToolCommandChange
#include "BaseTools/MultiTargetWithSelectionTool.h" // UMultiTargetWithSelectionTool
#include "ModelingToolExternalMeshUpdateAPI.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "Operations/FFDLattice.h"
#include "Solvers/ConstrainedMeshDeformer.h"
#include "LatticeManager.h"

#include "LatticeDeformerTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

namespace UE::Geometry
{
	struct FDynamicSubmesh3;
}

class ULatticeControlPointsMechanic;
class UMeshOpPreviewWithBackgroundCompute;
class UMeshSculptLayerProperties;
class ILatticeStateStorage;

UCLASS(MinimalAPI)
class ULatticeDeformerToolBuilder : public UMultiTargetWithSelectionToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	virtual bool RequiresInputSelection() const override { return false; }

private:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


UENUM()
enum class ELatticeDeformerToolAction  : uint8
{
	NoAction,
	Constrain,
	ClearConstraints
};

UCLASS(MinimalAPI)
class ULatticeDeformerToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<ULatticeDeformerTool> ParentTool;
	void Initialize(ULatticeDeformerTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(ELatticeDeformerToolAction Action);

	/** Number of lattice vertices along the X axis */
	UPROPERTY(EditAnywhere, Category = Resolution, meta = (UIMin = "2", ClampMin = "2", UIMax = "25", ClampMax = "40", EditCondition = "bCanChangeResolution", HideEditConditionToggle))
	int XAxisResolution = 5;

	/** Number of lattice vertices along the Y axis */
	UPROPERTY(EditAnywhere, Category = Resolution, meta = (UIMin = "2", ClampMin = "2", UIMax = "25", ClampMax = "40", EditCondition = "bCanChangeResolution", HideEditConditionToggle))
	int YAxisResolution = 5;

	/** Number of lattice vertices along the Z axis */
	UPROPERTY(EditAnywhere, Category = Resolution, meta = (UIMin = "2", ClampMin = "2", UIMax = "25", ClampMax = "40", EditCondition = "bCanChangeResolution", HideEditConditionToggle))
	int ZAxisResolution = 5;

	/** Relative distance the lattice extends from the mesh */
	UPROPERTY(EditAnywhere, Category = Resolution, meta = (UIMin = "0.01", ClampMin = "0.01", UIMax = "2", ClampMax = "5", EditCondition = "bCanChangeResolution", HideEditConditionToggle))
	float Padding = 0.01f;

	/** Whether to use linear or cubic interpolation to get new mesh vertex positions from the lattice */
	UPROPERTY(EditAnywhere, Category = Interpolation )
	ELatticeInterpolationType InterpolationType = ELatticeInterpolationType::Linear;

	/** Whether to use approximate new vertex normals using the deformer */
	UPROPERTY(EditAnywhere, Category = Interpolation)
	bool bDeformNormals = false;

	// Not user visible - used to disallow changing the lattice resolution after deformation
	UPROPERTY(meta = (TransientToolProperty))
	bool bCanChangeResolution = true;

	/** Whether the gizmo's axes remain aligned with world axes or rotate as the gizmo is transformed */
	UPROPERTY(EditAnywhere, Category = Gizmo)
	EToolContextCoordinateSystem GizmoCoordinateSystem = EToolContextCoordinateSystem::Local;

	/** If Set Pivot Mode is active, the gizmo can be repositioned without moving the selected lattice points */
	UPROPERTY(EditAnywhere, Category = Gizmo)
	bool bSetPivotMode = false;

	/** Whether to use soft deformation of the lattice */
	UPROPERTY(EditAnywhere, Category = Deformation)
	bool bSoftDeformation = false;

	/** Constrain selected lattice points */
	UFUNCTION(CallInEditor, Category = Deformation, meta = (DisplayName = "Constrain"))
	void Constrain() 
	{
		PostAction(ELatticeDeformerToolAction::Constrain);
	}

	/** Clear all constrained lattice points */
	UFUNCTION(CallInEditor, Category = Deformation, meta = (DisplayName = "Clear Constraints"))
	void ClearConstraints()
	{
		PostAction(ELatticeDeformerToolAction::ClearConstraints);
	}

};


UCLASS(MinimalAPI)
class ULatticeDeformerOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<ULatticeDeformerTool> LatticeDeformerTool;
};


/** Deform a mesh using a regular hexahedral lattice */
UCLASS(MinimalAPI)
class ULatticeDeformerTool : public UMultiTargetWithSelectionTool, public IInteractiveToolManageGeometrySelectionAPI, public IModelingToolExternalDynamicMeshUpdateAPI
{
	GENERATED_BODY()

public:

	virtual ~ULatticeDeformerTool() = default;

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API UE::Geometry::FVector3i GetLatticeResolution() const;
	UE_API void SetLatticeStorage(const TScriptInterface<ILatticeStateStorage>& InLatticeStorage);

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	// Input mesh
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	TSharedPtr<UE::Geometry::FDynamicSubmesh3, ESPMode::ThreadSafe> Submesh = nullptr;

	FTransform3d WorldTransform;

	TSharedPtr<UE::Geometry::FFFDLattice, ESPMode::ThreadSafe> Lattice;

	UPROPERTY()
	TObjectPtr<ULatticeControlPointsMechanic> ControlPointsMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<ULatticeDeformerToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshSculptLayerProperties> SculptLayerProperties;

	UPROPERTY()
	TScriptInterface<ILatticeStateStorage> LatticeStorage;

	UPROPERTY()
	bool bLatticeDeformed = false;

	bool bShouldRebuild = false;

	bool bHasSelection = false;

	// IInteractiveToolExternalDynamicMeshUpdateAPI methods
	UE_API virtual bool AllowToolMeshUpdates() const override;
	UE_API virtual void UpdateToolMeshes(TFunctionRef<TUniquePtr<FMeshRegionChangeBase>(UE::Geometry::FDynamicMesh3&, int32 MeshIdx)> UpdateMesh) override;
	UE_API virtual void ProcessToolMeshes(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&, int32 MeshIdx)> ProcessMesh) const override;
	UE_API virtual int32 NumToolMeshes() const override;
	
	// Create and store an FFFDLattice. Pass out the lattice's positions and edges.
	UE_API void InitializeLattice(TArray<FVector3d>& OutLatticePoints, TArray<UE::Geometry::FVector2i>& OutLatticeEdges);

	UE_API void StartPreview();

	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> DeformationSolver;
	TPimplPtr<UE::Geometry::FDynamicGraph3d> LatticeGraph;

	TMap<int32, FVector3d> ConstrainedLatticePoints;
	UE_API void ConstrainSelectedPoints();
	UE_API void ClearConstrainedPoints();
	UE_API void UpdateMechanicColorOverrides();
	UE_API void ResetConstrainedPoints();

	UE_API void RebuildDeformer();
	UE_API void SoftDeformLattice();

	int32 CurrentChangeStamp = 0;

	ELatticeDeformerToolAction PendingAction = ELatticeDeformerToolAction::NoAction;
	UE_API void RequestAction(ELatticeDeformerToolAction Action);
	UE_API void ApplyAction(ELatticeDeformerToolAction Action);

	friend class ULatticeDeformerOperatorFactory;
	friend class ULatticeDeformerToolProperties;
	friend class FLatticeDeformerToolConstrainedPointsChange;
};


// Set of constrained points change
class FLatticeDeformerToolConstrainedPointsChange : public FToolCommandChange
{
public:

	FLatticeDeformerToolConstrainedPointsChange(const TMap<int, FVector3d>& PrevConstrainedLatticePointsIn,
												const TMap<int, FVector3d>& NewConstrainedLatticePointsIn,
												int32 ChangeStampIn) :
		PrevConstrainedLatticePoints(PrevConstrainedLatticePointsIn),
		NewConstrainedLatticePoints(NewConstrainedLatticePointsIn),
		ChangeStamp(ChangeStampIn)
	{}

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;

	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<ULatticeDeformerTool>(Object)->CurrentChangeStamp != ChangeStamp;
	}

	UE_API virtual FString ToString() const override;

protected:

	TMap<int, FVector3d> PrevConstrainedLatticePoints;
	TMap<int, FVector3d> NewConstrainedLatticePoints;
	int32 ChangeStamp;
};

#undef UE_API
