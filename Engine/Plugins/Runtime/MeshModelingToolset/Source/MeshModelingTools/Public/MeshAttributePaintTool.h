// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Changes/ValueWatcher.h"
#include "Changes/IndexedAttributeChange.h"
#include "DynamicMesh/DynamicVerticesOctree3.h"
#include "MeshDescription.h"
#include "Selections/GeometrySelection.h"

#include "MeshAttributePaintTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

struct FMeshDescription;
class UMeshAttributePaintTool;

/**
 * Maps float values to linear color ramp.
 */
class FFloatAttributeColorMapper
{
public:
	virtual ~FFloatAttributeColorMapper() = default;

	FLinearColor LowColor = FLinearColor(0.9f, 0.9f, 0.9f, 1.0f);
	FLinearColor HighColor = FLinearColor(1.0f, 0.0f, 0.0f, 1.0f);

	virtual FLinearColor ToColor(float Value)
	{
		float t = FMath::Clamp(Value, 0.0f, 1.0f);
		return FLinearColor(
			FMathf::Lerp(LowColor.R, HighColor.R, t),
			FMathf::Lerp(LowColor.G, HighColor.G, t),
			FMathf::Lerp(LowColor.B, HighColor.B, t),
			1.0f);
	}

	template<typename VectorType>
	VectorType ToColor(float Value)
	{
		FLinearColor Color = ToColor(Value);
		return VectorType(Color.R, Color.G, Color.B);
	}
};


/**
 * Abstract interface to a single-channel indexed floating-point attribute
 */
class IMeshVertexAttributeAdapter
{
public:
	virtual ~IMeshVertexAttributeAdapter() = default;

	virtual int ElementNum() const = 0;
	virtual float GetValue(int32 Index) const = 0;
	virtual void SetValue(int32 Index, float Value) = 0;
	virtual UE::Geometry::FInterval1f GetValueRange() = 0;
};



/**
 * Abstract interface to a set of single-channel indexed floating-point attributes
 */
class IMeshVertexAttributeSource
{
public:
	virtual ~IMeshVertexAttributeSource() = default;

	virtual TArray<FName> GetAttributeList() = 0;
	virtual TUniquePtr<IMeshVertexAttributeAdapter> GetAttribute(FName AttributeName) = 0;
	/** @return number of indices in each attribute */
	virtual int32 GetAttributeElementNum() = 0;
};






/**
 * Tool Builder for Attribute Paint Tool
 * Note: a new experimental version of this tool is UMeshAttributePaintToolV2
 */
UCLASS(MinimalAPI)
class UMeshAttributePaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	/** Optional color map customization */
	TUniqueFunction<TUniquePtr<FFloatAttributeColorMapper>()> ColorMapFactory;

	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(UMeshSurfacePointTool* NewTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



UENUM()
enum class EMeshAttributePaintMaterialMode
{
	Shaded,
	ColorOnly,
	ExistingMaterial
};


UENUM()
enum class EBrushActionMode
{
	// Clicking adds to the weight, Ctrl subtracts, Shift smooths
	Paint,
	// Clicking sets the weight for the entire mesh component. Ctrl unsets the weight for the entire component, Shift is unused.
	FloodFill,
	// Clicking subtracts from the weight, Ctrl adds, Shift smooths
	Erase,
	// Clicking smooths values, Ctrl and Shift are unused
	Smooth
};


/**
 * Selected-Attribute settings Attribute Paint Tool
 */
UCLASS(MinimalAPI)
class UMeshAttributePaintBrushOperationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** 
	 * If the tool was started with a mesh element selection, this setting hides everything
	 *  except that selection, to make painting it easier. Requires that a mesh element
	 *  selection exist on tool start.
	 */
	UPROPERTY(EditAnywhere, Category = Attribute, meta = (EditCondition = "bToolHasSelection", HideEditConditionToggle))
	bool bIsolateGeometrySelection = false;
	
	//~ For the tool to set, to enable/disable bIsolateGeometrySelection
	UPROPERTY()
	bool bToolHasSelection = false;

	/** The mode of action for the brush. */
	UPROPERTY(EditAnywhere, Category = Attribute, meta=(DisplayName="Action"))
	EBrushActionMode BrushAction = EBrushActionMode::Paint;

	/** The target value that the brush will accumulate towards. */
	UPROPERTY(EditAnywhere, Category = Attribute, meta=(UIMin=0.0, UIMax=1.0, DisplayName="Value", ModelingQuickEdit, ModelingQuickSettings=150))
	float BrushValue = 1.0f;
};


UCLASS(MinimalAPI)
class UMeshAttributePaintToolVisualizationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = Rendering, meta=(DisplayName="Preview", ModelingQuickSettings=1000))
	EMeshAttributePaintMaterialMode MaterialMode = EMeshAttributePaintMaterialMode::Shaded;

	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditCondition ="MaterialMode==EMeshAttributePaintMaterialMode::Shaded"))
	bool bFlatShading = true;
};


UCLASS(MinimalAPI)
class UMeshAttributePaintToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = "Attribute", meta = (DisplayName = "Selected Attribute", GetOptions = GetAttributeNames))
	FString Attribute;

	UFUNCTION()
	const TArray<FString>& GetAttributeNames() { return Attributes; };

	TArray<FString> Attributes;

public:
	/**
	* Initialize the internal array of attribute names
	* @param bInitialize if set, selected Attribute will be reset to the first attribute or empty if there are none.
	*/
	UE_API void Initialize(const TArray<FName>& AttributeNames, bool bInitialize = false);

	/**
	 * Verify that the attribute selection is valid
	 * @param bUpdateIfInvalid if selection is not valid, use attribute at index 0 or empty if there are no attributes
	 * @return true if selection is in the Attributes array
	 */
	UE_API bool ValidateSelectedAttribute(bool bUpdateIfInvalid);

	/**
	 * @return selected attribute index, or -1 if invalid selection
	 */
	UE_API int32 GetSelectedAttributeIndex();
};





UENUM()
enum class EMeshAttributePaintToolActions
{
	NoAction
};



UCLASS(MinimalAPI)
class UMeshAttributePaintEditActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UMeshAttributePaintTool> ParentTool;

	void Initialize(UMeshAttributePaintTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EMeshAttributePaintToolActions Action);
};



/**
 * FCommandChange for color map changes
 */
class FMeshAttributePaintChange : public TCustomIndexedValuesChange<float, int32>
{
public:
	virtual FString ToString() const override
	{
		return FString(TEXT("Paint Attribute"));
	}
};



/**
 * UMeshAttributePaintTool paints single-channel float attributes on a MeshDescription or DynamicMesh
 * Note: a new experimental version of this tool is UMeshAttributePaintToolV2
 */
UCLASS(MinimalAPI)
class UMeshAttributePaintTool : public UDynamicMeshBrushTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void SetWorld(UWorld* World);
	UE_API void SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn);

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	// UBaseBrushTool overrides
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	
	// UMeshSurfacePointTool
	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;

	UE_API virtual void RequestAction(EMeshAttributePaintToolActions ActionType);

	UE_API virtual void SetColorMap(TUniquePtr<FFloatAttributeColorMapper> ColorMap);

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

	UE::Geometry::TDynamicVerticesOctree3<FDynamicMesh3>& GetVerticesOctree() { return VerticesOctree; }

protected:
	UE_API virtual void ApplyStamp(const FBrushStampData& Stamp);


	struct FStampActionData
	{
		TArray<int32> ROIVertices;
		TArray<float> ROIBefore;
		TArray<float> ROIAfter;
	};


	UE_API virtual void ApplyStamp_Paint(const FBrushStampData& Stamp, FStampActionData& ActionData);
	UE_API virtual void ApplyStamp_FloodFill(const FBrushStampData& Stamp, FStampActionData& ActionData);

	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	
	// Should be overriden by subclasses so that settings are not shared across
	// this tool and subclasses.
	UE_API virtual FString GetPropertyCacheIdentifier() const;
	

protected:
	UPROPERTY()
	TObjectPtr<UMeshAttributePaintBrushOperationProperties> BrushActionProps;

	UPROPERTY()
	TObjectPtr<UMeshAttributePaintToolProperties> AttribProps;

	UPROPERTY()
	TObjectPtr<UMeshAttributePaintToolVisualizationProperties> ViewProperties;

	TValueWatcher<int32> SelectedAttributeWatcher;

	//UPROPERTY()
	//UMeshAttributePaintEditActions* AttributeEditActions;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> ActiveOverrideMaterial;

protected:
	UWorld* TargetWorld;

	bool bInRemoveStroke = false;
	bool bInSmoothStroke = false;
	FBrushStampData StartStamp;
	FBrushStampData LastStamp;
	bool bStampPending;

	// Note: This is used if the target supports mesh description read/write; otherwise EditedDynamicMesh is used
	TUniquePtr<FMeshDescription> EditedMesh;
	// This is used if the target doesn't support mesh description read/write; otherwise EditedMesh is used
	TUniquePtr<FDynamicMesh3> EditedDynamicMesh;
	// Can be set by subclasses to determine which to use when both mesh description and dynamic mesh are options.
	bool bPreferMeshDescription = true;

	UE_API double CalculateBrushFalloff(double Distance);
	UE::Geometry::TDynamicVerticesOctree3<FDynamicMesh3> VerticesOctree;
	TArray<int> PreviewBrushROI;
	UE_API void CalculateVertexROI(const FBrushStampData& Stamp, TArray<int>& VertexROI);

	TUniquePtr<FFloatAttributeColorMapper> ColorMapper;
	TUniquePtr<IMeshVertexAttributeSource> AttributeSource;

	struct FAttributeData
	{
		FName Name;
		TUniquePtr<IMeshVertexAttributeAdapter> Attribute;
		TArray<float> CurrentValues;
		TArray<float> InitialValues;
	};
	TArray<FAttributeData> Attributes;
	int32 AttributeBufferCount;
	int32 CurrentAttributeIndex;
	UE::Geometry::FInterval1f CurrentValueRange;

	// actions

	bool bHavePendingAction = false;
	EMeshAttributePaintToolActions PendingAction;
	UE_API virtual void ApplyAction(EMeshAttributePaintToolActions ActionType);

	bool bVisibleAttributeValid = false;
	int32 PendingNewSelectedIndex = -1;
	
	UE_API virtual void InitializeAttributes();
	UE_API void StoreCurrentAttribute();
	UE_API void UpdateVisibleAttribute();
	UE_API void UpdateSelectedAttribute(int32 NewSelectedIndex);

	TUniquePtr<TIndexedValuesChangeBuilder<float, FMeshAttributePaintChange>> ActiveChangeBuilder;
	UE_API void BeginChange();
	UE_API TUniquePtr<FMeshAttributePaintChange> EndChange();
	UE_API virtual void ExternalUpdateValues(int32 AttribIndex, const TArray<int32>& VertexIndices, const TArray<float>& NewValues);

private:
	struct FBrushActionData
	{
		float BrushValue = 1.0f;
		bool bSmooth = false;
		bool bErase = false;
	};
	
	TOptional<UE::Geometry::FGeometrySelection> GeometrySelection;
	TSet<int32> SelectionTids;
	TSet<int32> SelectionVids;
	bool IsFilteringTrianglesBySelection() const;
	bool IsFilteringTrianglesByFrontFacing() const;
	
	void ApplyStamp_Paint(const FBrushStampData& Stamp, FStampActionData& ActionData, const FBrushActionData& BrushData);

	void UpdateMaterialMode(EMeshAttributePaintMaterialMode MaterialMode);
	void UpdateFlatShadingSetting(bool bNewValue);

	bool IsTriangleBackFacing(int32 TriangleID, const FDynamicMesh3* QueryMesh) const;
};







#undef UE_API
