// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Mechanics/PolyLassoMarqueeMechanic.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Sculpting/MeshBrushOpBase.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "TransformTypes.h"
#include "Changes/MeshPolygroupChange.h"
#include "Polygroups/PolygroupSet.h"
#include "FaceGroupUtil.h"

#include "MeshGroupPaintTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class FDynamicMeshTriangleLabelEditBuilder;
class UMeshElementsVisualizer;
class UGroupEraseBrushOpProps;
class UGroupPaintBrushOpProps;
namespace UE::Geometry
{
	class FTriangleLabelAdapter;
}

DECLARE_STATS_GROUP(TEXT("GroupPaintTool"), STATGROUP_GroupPaintTool, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_UpdateROI"), GroupPaintTool_UpdateROI, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_ApplyStamp"), GroupPaintToolApplyStamp, STATGROUP_GroupPaintTool );
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick"), GroupPaintToolTick, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick_ApplyStampBlock"), GroupPaintTool_Tick_ApplyStampBlock, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick_ApplyStamp_Remove"), GroupPaintTool_Tick_ApplyStamp_Remove, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick_ApplyStamp_Insert"), GroupPaintTool_Tick_ApplyStamp_Insert, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick_NormalsBlock"), GroupPaintTool_Tick_NormalsBlock, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick_UpdateMeshBlock"), GroupPaintTool_Tick_UpdateMeshBlock, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Tick_UpdateTargetBlock"), GroupPaintTool_Tick_UpdateTargetBlock, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Normals_Collect"), GroupPaintTool_Normals_Collect, STATGROUP_GroupPaintTool);
DECLARE_CYCLE_STAT(TEXT("GroupPaintTool_Normals_Compute"), GroupPaintTool_Normals_Compute, STATGROUP_GroupPaintTool);





/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UMeshGroupPaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};




/** Mesh Sculpting Brush Types */
UENUM()
enum class EMeshGroupPaintInteractionType : uint8
{
	Brush,
	Fill,
	GroupFill,
	PolyLasso,

	LastValue UMETA(Hidden)
};







/** Mesh Sculpting Brush Types */
UENUM()
enum class EMeshGroupPaintBrushType : uint8
{
	/** Paint active group */
	Paint UMETA(DisplayName = "Paint"),

	/** Erase active group */
	Erase UMETA(DisplayName = "Erase"),

	LastValue UMETA(Hidden)
};


/** Mesh Sculpting Brush Area Types */
UENUM()
enum class EMeshGroupPaintBrushAreaType : uint8
{
	Connected,
	Volumetric
};

/** Mesh Sculpting Brush Types */
UENUM()
enum class EMeshGroupPaintVisibilityType : uint8
{
	None,
	FrontFacing,
	Unoccluded
};




UCLASS(MinimalAPI)
class UGroupPaintBrushFilterProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Primary Brush Mode */
	//UPROPERTY(EditAnywhere, Category = Brush2, meta = (DisplayName = "Brush Type"))
	UPROPERTY()
	EMeshGroupPaintBrushType PrimaryBrushType = EMeshGroupPaintBrushType::Paint;

	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Action"))
	EMeshGroupPaintInteractionType SubToolType = EMeshGroupPaintInteractionType::Brush;

	/** Relative size of brush */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Brush Size", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0", 
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType != EMeshGroupPaintInteractionType::PolyLasso"))
	float BrushSize = 0.25f;

	/** When Volumetric, all faces inside the brush sphere are selected, otherwise only connected faces are selected */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Brush Area Mode",
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType != EMeshGroupPaintInteractionType::PolyLasso"))
	EMeshGroupPaintBrushAreaType BrushAreaMode = EMeshGroupPaintBrushAreaType::Connected;

	/** Allow the Brush to hit the back-side of the mesh */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (DisplayName = "Hit Back Faces",
		HideEditConditionToggle, EditConditionHides, EditCondition = "SubToolType != EMeshGroupPaintInteractionType::PolyLasso"))
	bool bHitBackFaces = true;

	/** The group index that will be assigned to triangles */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (UIMin = 0, ClampMin = 0, Delta = 1, LinearDeltaSensitivity = 50,
		HideEditConditionToggle, EditConditionHides, EditCondition = "bPaintingLabels == false"))
	int32 SetGroup = 1;
	
	/** The group label that will be assigned to triangles */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = bUseLabel))
	FName SetLabel = NAME_None;

	/** If true, only triangles with no group assigned will be painted */
	UPROPERTY(EditAnywhere, Category = ActionType)
	bool bOnlySetUngrouped = false;

	/** Group index to set as Erased value */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (UIMin = 0, ClampMin = 0, Delta = 1, LinearDeltaSensitivity = 50,
		HideEditConditionToggle, EditConditionHides, EditCondition = "bPaintingLabels == false"))
	int32 EraseGroup = 0;
	
	/** Group label to set as Erased value */
	UPROPERTY(EditAnywhere, Category = ActionType, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = bUseLabel))
	FName EraseLabel = NAME_None;

	/** When enabled, only the current group configured in the Paint brush is erased */
	UPROPERTY(EditAnywhere, Category = ActionType)
	bool bOnlyEraseCurrent = false;

	/** The Region affected by the current operation will be bounded by edge angles larger than this threshold */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (UIMin = "0.0", UIMax = "180.0", EditCondition = "SubToolType != EMeshGroupPaintInteractionType::PolyLasso && BrushAreaMode == EMeshGroupPaintBrushAreaType::Connected"))
	float AngleThreshold = 180.0f;

	/** The Region affected by the current operation will be bounded by UV borders/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (EditCondition = "SubToolType != EMeshGroupPaintInteractionType::PolyLasso && BrushAreaMode == EMeshGroupPaintBrushAreaType::Connected"))
	bool bUVSeams = false;

	/** The Region affected by the current operation will be bounded by Hard Normal edges/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (EditCondition = "SubToolType != EMeshGroupPaintInteractionType::PolyLasso && BrushAreaMode == EMeshGroupPaintBrushAreaType::Connected"))
	bool bNormalSeams = false;

	/** Control which triangles can be affected by the current operation based on visibility. Applied after all other filters. */
	UPROPERTY(EditAnywhere, Category = Filters)
	EMeshGroupPaintVisibilityType VisibilityFilter = EMeshGroupPaintVisibilityType::None;


	/** Number of vertices in a triangle the Lasso must hit to be counted as "inside" */
	UPROPERTY(EditAnywhere, Category = Filters, AdvancedDisplay, meta = (UIMin = 1, UIMax = 3, EditCondition = "SubToolType == EMeshGroupPaintInteractionType::PolyLasso"))
	int MinTriVertCount = 1;


	/** Display the Group ID of the last triangle under the cursor */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowHitGroup = false;

	/** Display the Group ID for all visible groups in the mesh */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowAllGroups = false;
	
	/** whether the user is painting labels */
	UPROPERTY()
	bool bPaintingLabels = false;
};





UENUM()
enum class EMeshGroupPaintToolActions
{
	NoAction,

	ClearFrozen,
	FreezeCurrent,
	FreezeOthers,

	GrowCurrent,
	ShrinkCurrent,
	ClearCurrent,
	FloodFillCurrent,
	ClearAll
};


UCLASS(MinimalAPI)
class UMeshGroupPaintToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UMeshGroupPaintTool> ParentTool;

	void Initialize(UMeshGroupPaintTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EMeshGroupPaintToolActions Action);
};



UCLASS(MinimalAPI)
class UMeshGroupPaintToolFreezeActions : public UMeshGroupPaintToolActionPropertySet
{
	GENERATED_BODY()

public:
	UFUNCTION(CallInEditor, Category = Freezing, meta = (DisplayPriority = 1))
	void UnfreezeAll()
	{
		PostAction(EMeshGroupPaintToolActions::ClearFrozen);
	}

	UFUNCTION(CallInEditor, Category = Freezing, meta = (DisplayPriority = 2))
	void FreezeCurrent()
	{
		PostAction(EMeshGroupPaintToolActions::FreezeCurrent);
	}

	UFUNCTION(CallInEditor, Category = Freezing, meta = (DisplayPriority = 3))
	void FreezeOthers()
	{
		PostAction(EMeshGroupPaintToolActions::FreezeOthers);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 10))
	void ClearAll()
	{
		PostAction(EMeshGroupPaintToolActions::ClearAll);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 11))
	void ClearCurrent()
	{
		PostAction(EMeshGroupPaintToolActions::ClearCurrent);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 12))
	void FloodFillCurrent()
	{
		PostAction(EMeshGroupPaintToolActions::FloodFillCurrent);
	}


	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 20))
	void GrowCurrent()
	{
		PostAction(EMeshGroupPaintToolActions::GrowCurrent);
	}

	UFUNCTION(CallInEditor, Category = Operations, meta = (DisplayPriority = 21))
	void ShrinkCurrent()
	{
		PostAction(EMeshGroupPaintToolActions::ShrinkCurrent);
	}


};




/**
 * Mesh Element Paint Tool Class
 */
UCLASS(MinimalAPI)
class UMeshGroupPaintTool : public UMeshSculptToolBase
{
	GENERATED_BODY()

public:
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	UE_API bool IsInBrushSubMode() const;

	UE_API virtual void CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology) override;


public:

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties;

	/** Filters on paint brush */
	UPROPERTY()
	TObjectPtr<UGroupPaintBrushFilterProperties> FilterProperties;


private:
	// This will be of type UGroupPaintBrushOpProps, we keep a ref so we can change active group ID on pick
	UPROPERTY()
	TObjectPtr<UGroupPaintBrushOpProps> PaintBrushOpProperties;

	UPROPERTY()
	TObjectPtr<UGroupEraseBrushOpProps> EraseBrushOpProperties;

public:
	UE_API void AllocateNewGroupAndSetAsCurrentAction();
	UE_API void GrowCurrentGroupAction();
	UE_API void ShrinkCurrentGroupAction();
	UE_API void ClearCurrentGroupAction();
	UE_API void FloodFillCurrentGroupAction();
	UE_API void ClearAllGroupsAction();

	UE_API void SetTrianglesToGroupID(const TSet<int32>& Triangles, int32 ToGroupID, bool bIsErase);

	UE_API bool HaveVisibilityFilter() const;
	UE_API void ApplyVisibilityFilter(const TArray<int32>& Triangles, TArray<int32>& VisibleTriangles);
	UE_API void ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer);

	// we override these so we can update the separate BrushSize property added for this tool
	UE_API virtual void IncreaseBrushRadiusAction();
	UE_API virtual void DecreaseBrushRadiusAction();
	UE_API virtual void IncreaseBrushRadiusSmallStepAction();
	UE_API virtual void DecreaseBrushRadiusSmallStepAction();

protected:
	// UMeshSculptToolBase API
	virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() { return DynamicMeshComponent; }
	virtual FDynamicMesh3* GetBaseMesh() { check(false); return nullptr; }
	virtual const FDynamicMesh3* GetBaseMesh() const { check(false); return nullptr; }

	UE_API virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const override;

	UE_API virtual void OnBeginStroke(const FRay& WorldRay) override;
	UE_API virtual void OnEndStroke() override;
	UE_API virtual void OnCancelStroke() override;

	virtual bool SharesBrushPropertiesChanges() const override { return false; }

	UE_API virtual TUniquePtr<FMeshSculptBrushOp>& GetActiveBrushOp();
	// end UMeshSculptToolBase API



	//
	// Action support
	//

public:
	UE_API virtual void RequestAction(EMeshGroupPaintToolActions ActionType);

	UPROPERTY()
	TObjectPtr<UMeshGroupPaintToolFreezeActions> FreezeActions;

protected:
	bool bHavePendingAction = false;
	EMeshGroupPaintToolActions PendingAction;
	UE_API virtual void ApplyAction(EMeshGroupPaintToolActions ActionType);



	//
	// Marquee Support
	//
public:
	UPROPERTY()
	TObjectPtr<UPolyLassoMarqueeMechanic> PolyLassoMechanic;

protected:
	UE_API void OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled);


	//
	// Internals
	//

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged();
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	int32 GetGroup(const int32 TriangleID) const;
	void SetGroup(const int32 TriangleID, const int32 GroupID, FDynamicMesh3& Mesh) const;
	
	int32 GetGroupToSet() const;
	int32 GetGroupToErase() const;
	
	TUniquePtr<UE::Geometry::FPolygroupSet> ActiveGroupSet;
	TSharedPtr<UE::Geometry::FTriangleLabelAdapter> LabelAdapter;
	
	bool IsPaintingLabel() const
	{
		return LabelAdapter.IsValid();
	}
	
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer();

	UE_API void UpdateSubToolType(EMeshGroupPaintInteractionType NewType);

	UE_API void UpdateBrushType(EMeshGroupPaintBrushType BrushType);

	TSet<int32> AccumulatedTriangleROI;
	bool bUndoUpdatePending = false;
	TArray<int> NormalsBuffer;
	UE_API void WaitForPendingUndoRedoUpdate();

	TArray<int> TempROIBuffer;
	TArray<int> VertexROI;
	TArray<bool> VisibilityFilterBuffer;
	TSet<int> VertexSetBuffer;
	TSet<int> TriangleROI;
	UE_API void UpdateROI(const FSculptBrushStamp& CurrentStamp);

	EMeshGroupPaintBrushType PendingStampType = EMeshGroupPaintBrushType::Paint;

	UE_API bool UpdateStampPosition(const FRay& WorldRay);
	UE_API bool ApplyStamp();

	UE::Geometry::FDynamicMeshOctree3 Octree;

	UE_API bool UpdateBrushPosition(const FRay& WorldRay);

	bool GetInEraseStroke()
	{
		// Re-use the smoothing stroke key (shift) for erase stroke in the group paint tool
		return GetInSmoothingStroke();
	}


	bool bPendingPickGroup = false;
	bool bPendingToggleFreezeGroup = false;


	TArray<int32> ROITriangleBuffer;
	TArray<int32> ROIGroupBuffer;
	UE_API bool SyncMeshWithGroupBuffer(FDynamicMesh3* Mesh);

	TUniquePtr<FDynamicMeshGroupEditBuilder> ActiveGroupEditBuilder;
	TUniquePtr<FDynamicMeshTriangleLabelEditBuilder> ActiveLabelEditBuilder;
	UE_API void BeginChange();
	UE_API void EndChange();
	UE_API void SaveTriangleEdits() const;

	TArray<int32> FrozenGroups;
	UE_API void ToggleFrozenGroup(int32 GroupID);
	UE_API void FreezeOtherGroups(int32 GroupID);
	UE_API void ClearAllFrozenGroups();
	UE_API void EmitFrozenGroupsChange(const TArray<int32>& FromGroups, const TArray<int32>& ToGroups, const FText& ChangeText);

	UE_API FColor GetColorForGroup(int32 GroupID);

	TArray<FVector3d> TriNormals;
	TArray<int32> UVSeamEdges;
	TArray<int32> NormalSeamEdges;
	UE_API void PrecomputeFilterData();


	bool bDrawGroupsDataValid = false;
	UE::Geometry::FGroupVisualizationCache GroupVisualizationCache;

protected:
	virtual bool ShowWorkPlane() const override { return false; }
};



#undef UE_API
