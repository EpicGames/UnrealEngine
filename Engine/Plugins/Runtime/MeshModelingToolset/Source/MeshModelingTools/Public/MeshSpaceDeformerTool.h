// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "BaseTools/MeshSurfacePointTool.h"
#include "ToolDataVisualizer.h"
#include "SpaceDeformerOps/MeshSpaceDeformerOp.h"
#include "SpaceDeformerOps/BendMeshOp.h"
#include "SpaceDeformerOps/TwistMeshOp.h"
#include "SpaceDeformerOps/FlareMeshOp.h"
#include "Components/DynamicMeshComponent.h"
#include "ToolDataVisualizer.h"
#include "BaseGizmos/GizmoInterfaces.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"

#include "MeshSpaceDeformerTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UMeshSpaceDeformerTool;

class UDragAlignmentMechanic;
class UPreviewMesh;
class UCombinedTransformGizmo;
class UTransformProxy;
class UIntervalGizmo;
class UMeshOpPreviewWithBackgroundCompute;
class UGizmoLocalFloatParameterSource;
class UGizmoTransformChangeStateTarget;
class FSelectClickedAction;

/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UMeshSpaceDeformerToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

/** ENonlinearOperation determines which type of nonlinear deformation will be applied*/
UENUM()
enum class  ENonlinearOperationType : int8
{
	/** 
	 * Will bend the mesh in the direction of the gizmo Y axis along the Z axis. A line along the Z
	 * axis from the lower bound to the upper bound would not change length as it bends.
	 */
	Bend,

	/**
	 * Depening on 'Flare Percent", will either flare or squish the mesh along the Gizmo Z axis,
	 * from lower bound to upper bound.
	 */
	Flare		UMETA(DisplayName = "Flare/Squish"),

	/** Twists the mesh along the gizmo Z axis, from lower bound to upper bound. */
	Twist
};

UENUM()
enum class EFlareProfileType : int8
{
	//Displaced by sin(pi x) with x in 0 to 1
	SinMode, 

	//Displaced by sin(pi x)*sin(pi x) with x in 0 to 1. This provides a smooth normal transition.
	SinSquaredMode,
	
	// Displaced by piecewise-linear trianglular mode
	TriangleMode 
};

UCLASS(MinimalAPI)
class UMeshSpaceDeformerToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Operation Type"))
	ENonlinearOperationType SelectedOperationType = ENonlinearOperationType::Bend;

	/** The upper bound to the region of space which the operation will affect. Measured along the gizmo Z axis from the gizmo center. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Upper Bound", UIMin = "0.0", ClampMin = "0.0"))
	float UpperBoundsInterval = 100.0;

	/** The lower bound to the region of space which the operation will affect. Measured along the gizmo Z axis from the gizmo center. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Lower Bound", UIMax = "0", ClampMax = "0"))
	float LowerBoundsInterval = -100.0;

	//~ The two degrees properties (bend vs twist) are separate because they have different default values
	/** 
	 * A line along the Z axis of the gizmo from lower bound to upper bound will be bent into a perfect arc of this
	 * many degrees in the direction of the Y axis without changing length.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "-360", UIMax = "360", ClampMin = "-3600", ClampMax = "3600", 
		EditCondition = "SelectedOperationType == ENonlinearOperationType::Bend", EditConditionHides))
	float BendDegrees = 90;

	/** Degrees of twist from the lower bound to the upper bound along the gizmo Z axis. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "-360", UIMax = "360", ClampMin = "-3600", ClampMax = "3600", 
		EditCondition = "SelectedOperationType == ENonlinearOperationType::Twist", EditConditionHides))
	float TwistDegrees = 180;

	/**
	* Determines the profile used as a displacement
	*/ 
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SelectedOperationType == ENonlinearOperationType::Flare", EditConditionHides))
	EFlareProfileType FlareProfileType = EFlareProfileType::SinMode;

	/**
	 * Determines how much to flare perpendicular to the Z axis. When set to 100%, points are moved double the distance
	 * away from the gizmo Z axis at the most extreme flare point. 0% does not flare at all, whereas -100% pinches all
	 * the way to the gizmo Z axis at the most extreme flare point.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "-100", UIMax = "200", ClampMin = "-1000", ClampMax = "2000", 
		EditCondition = "SelectedOperationType == ENonlinearOperationType::Flare", EditConditionHides))
	float FlarePercentY = 100;

	/**
	 * If true, flaring is applied along the gizmo X and Y axis the same amount.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SelectedOperationType == ENonlinearOperationType::Flare", EditConditionHides))
	bool bLockXAndYFlaring = true;

	/**
	 * Determines how much to flare perpendicular to the Z axis in the X direction if the flaring is not locked
	 * to be the same in the X and Y directions.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "-100", UIMax = "200", ClampMin = "-1000", ClampMax = "2000",
	EditCondition = "SelectedOperationType == ENonlinearOperationType::Flare && !bLockXAndYFlaring", EditConditionHides))
	float FlarePercentX = 100;

	/**
	 * If true, the "bottom" of the mesh relative to the gizmo Z axis will stay in place while the rest bends or twists. If false, the bend
	 * or twist will happen around the gizmo location.
	 */
	UPROPERTY(EditAnywhere, Category = Options, meta = (
		EditCondition = "SelectedOperationType == ENonlinearOperationType::Bend || SelectedOperationType == ENonlinearOperationType::Twist", EditConditionHides))
	bool bLockBottom = false;

	UPROPERTY(EditAnywhere, Category = Options)
	bool bShowOriginalMesh = true;

	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SelectedOperationType == ENonlinearOperationType::Bend", EditConditionHides))
	bool bDrawVisualization = true;

	/** When true, Ctrl+click not only moves the gizmo to the clicked location, but also aligns the Z axis with the normal at that point. */
	UPROPERTY(EditAnywhere, Category = Gizmo)
	bool bAlignToNormalOnCtrlClick = false;
};

UENUM()
enum class EMeshSpaceDeformerToolAction
{
	NoAction,

	ShiftToCenter
};

UCLASS(MinimalAPI)
class UMeshSpaceDeformerToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UMeshSpaceDeformerTool> ParentTool;

	void Initialize(UMeshSpaceDeformerTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EMeshSpaceDeformerToolAction Action);

	/** Move the gizmo to the center of the object without changing the orientation. */
	UFUNCTION(CallInEditor, Category = Gizmo)
	void ShiftToCenter() { PostAction(EMeshSpaceDeformerToolAction::ShiftToCenter); }
};


UCLASS(MinimalAPI)
class USpaceDeformerOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UMeshSpaceDeformerTool> SpaceDeformerTool;  // back pointer
};


/**
 * Applies non-linear deformations to a mesh 
 */
UCLASS(MinimalAPI)
class UMeshSpaceDeformerTool : public USingleSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()
public:
	UE_API UMeshSpaceDeformerTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API void RequestAction(EMeshSpaceDeformerToolAction ActionType);

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// sync the parameters owned by the MeshSpaceDeformerOp 
	UE_API void UpdateOpParameters(UE::Geometry::FMeshSpaceDeformerOp& MeshSpaceDeformerOp) const;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<UMeshSpaceDeformerToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UMeshSpaceDeformerToolActionPropertySet> ToolActions;

	UPROPERTY()
	TObjectPtr<UGizmoTransformChangeStateTarget> StateTarget = nullptr;
	
	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic;

protected:

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;   

protected:	

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalDynamicMesh;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> OriginalMeshPreview;

	UPROPERTY()
	TObjectPtr<UIntervalGizmo> IntervalGizmo;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy;

	/** Interval Parameter sources that reflect UI settings. */

	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> UpIntervalSource;

	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> DownIntervalSource;

	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> ForwardIntervalSource;

	// Button click support
	EMeshSpaceDeformerToolAction PendingAction;
	FVector3d MeshCenter;

	UE::Geometry::FFrame3d GizmoFrame;

	// The length of the third interval gizmo (which sets the intensity of the deformation)
	// when the modifier is set to some reasonable maximum.
	double ModifierGizmoLength;

	TPimplPtr<FSelectClickedAction> SetPointInWorldConnector;

	TArray<FVector3d> VisualizationPoints;
	FToolDataVisualizer VisualizationRenderer;

	UE_API void TransformProxyChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void SetGizmoFrameFromWorldPos(const FVector& Position, const FVector& Normal = FVector(), bool bAlignNormal = false);
	
	UE_API double GetModifierGizmoValue() const;
	UE_API void ApplyModifierGizmoValue(double Value);

	// Apply clicked action
	UE_API void ApplyAction(EMeshSpaceDeformerToolAction Action);

	UE_API void UpdatePreview();

	friend USpaceDeformerOperatorFactory;
};

#undef UE_API
