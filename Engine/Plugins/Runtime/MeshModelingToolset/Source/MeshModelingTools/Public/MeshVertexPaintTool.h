// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "Components/DynamicMeshComponent.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "PropertySets/ColorChannelFilterPropertyType.h"
#include "Mechanics/PolyLassoMarqueeMechanic.h"
#include "Selections/GeometrySelection.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "Sculpting/MeshSculptToolBase.h"
#include "Sculpting/MeshBrushOpBase.h"

// TODO: Refactor UMeshSymmetryProperties out of MeshVertexSculptTool
#include "MeshVertexSculptTool.h" // For UMeshSymmetryProperties

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "TransformTypes.h"
#include "Changes/IndexedAttributeChange.h"
#include "Polygroups/PolygroupSet.h"
#include "FaceGroupUtil.h"

#include "MeshVertexPaintTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UMeshElementsVisualizer;
class UVertexColorPaintBrushOpProps;
class UVertexColorSmoothBrushOpProps;
struct FMeshDescription;


/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UMeshVertexPaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual void InitializeNewTool(UMeshSurfacePointTool* NewTool, const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};




/** Mesh Vertex Paint Primary Interactions */
UENUM()
enum class EMeshVertexPaintInteractionType : uint8
{
	/** Paint Vertices of hit triangles with a smooth falloff */
	Brush UMETA(DisplayName = "Paint Vertices"),
	/** Fill any painted triangles, by setting all 3 vertices to the same color */
	TriFill UMETA(DisplayName = "Paint Triangles"),
	/** Fill any triangles connected to the brushed triangles */
	Fill UMETA(DisplayName = "Flood Fill Connected"),
	/** Fill any polygroups connected to the brushed triangles */
	GroupFill UMETA(DisplayName = "Flood Fill Groups"),
	/** Paint any triangles inside polygonal or freehand Lassos drawn in the viewport */
	PolyLasso,

	LastValue UMETA(Hidden)
};


UENUM()
enum class EMeshVertexPaintColorChannel : uint8
{
	Red = 0,
	Green = 1,
	Blue = 2,
	Alpha = 3
};


// currently in-sync with EVertexColorPaintBrushOpBlendMode
UENUM()
enum class EMeshVertexPaintColorBlendMode : uint8
{
	/** Interpolate between Paint color and existing Color */
	Lerp = 0,
	/** Alpha-Blend the Paint accumulated during each stroke with the existing Colors */
	Mix = 1,
	/** Multiply the Paint color with the existing Color */
	Multiply = 2
};




/** Mesh Vertex Painting Brush Types */
UENUM()
enum class EMeshVertexPaintBrushType : uint8
{
	/** Paint the Primary Color */
	Paint UMETA(DisplayName = "Paint"),

	/** Paint the Erase/Secondary Color */
	Erase UMETA(DisplayName = "Erase"),

	/** Average any seam colors at a vertex */
	Soften UMETA(DisplayName = "Soften"),

	/** Smooth the colors */
	Smooth UMETA(DisplayName = "Smooth"),

	LastValue UMETA(Hidden)
};

/** Secondary/Erase Vertex Color Painting Types */
UENUM()
enum class EMeshVertexPaintSecondaryActionType : uint8
{
	/** Paint the Erase/Secondary Color */
	Erase, 
	/** Blend any split color values at painted vertices  */
	Soften,
	/** Blend vertex colors with nearby vertex colors (ie blur) */
	Smooth
};



/** Brush Area Types */
UENUM()
enum class EMeshVertexPaintBrushAreaType : uint8
{
	/** Brush affects any triangles inside a sphere around the cursor */
	Connected,
	/** Brush affects any triangles geometrically connected to the triangle under the cursor */
	Volumetric
};

/** Visibility Types */
UENUM()
enum class EMeshVertexPaintVisibilityType : uint8
{
	None,
	/** Only paint vertices that are front-facing relative to the current camera direction */
	FrontFacing,
	/** Only paint triangles that are visible. Only considers active mesh, visibility test is based on triangle centers */
	Unoccluded
};

/** Visualization Materials */
UENUM()
enum class EMeshVertexPaintMaterialMode : uint8
{
	/** Display Vertex Colors using a Lit flat-shaded material */
	LitVertexColor,
	/** Display Vertex Colors using an Unlit smooth-shaded material */
	UnlitVertexColor,
	/** Display Materials assigned to target Mesh */
	OriginalMaterial
};



UCLASS(MinimalAPI)
class UVertexPaintBasicProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Primary Brush Mode */
	//UPROPERTY(EditAnywhere, Category = Brush2, meta = (DisplayName = "Brush Type"))
	UPROPERTY()
	EMeshVertexPaintBrushType PrimaryBrushType = EMeshVertexPaintBrushType::Paint;

	/** Painting Operation to apply when left-clicking and dragging */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "Action"))
	EMeshVertexPaintInteractionType SubToolType = EMeshVertexPaintInteractionType::Brush;

	// The strength of the brush effect when stamped on the mesh
	UPROPERTY(EditAnywhere, Category = Settings, meta = (UIMin = 0, UIMax = 1))
	double Strength = 1.0;

	/** The Color that will be assigned to painted triangle vertices */
	UPROPERTY(EditAnywhere, Category = Settings, meta = ())
	FLinearColor PaintColor = FLinearColor::Red;

	/** Should pressure sensitivity affect Paint brush strength? */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bIsPaintPressureEnabled = true;

	/** Controls how painted Colors will be combined with the existing Colors */
	UPROPERTY(EditAnywhere, Category = Settings)
	EMeshVertexPaintColorBlendMode BlendMode = EMeshVertexPaintColorBlendMode::Lerp;

	/** The Brush Operation that will be applied when holding the Shift key when in Painting */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "Secondary Brush"))
	EMeshVertexPaintSecondaryActionType SecondaryActionType = EMeshVertexPaintSecondaryActionType::Erase;

	/** Color to set when using Erase brush */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "SecondaryActionType != EMeshVertexPaintSecondaryActionType::Smooth"))
	FLinearColor EraseColor = FLinearColor::White;

	/** Should pressure sensitivity affect Erase brush strength? */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (EditConditionHides, EditCondition = "SecondaryActionType != EMeshVertexPaintSecondaryActionType::Smooth"))
	bool bIsErasePressureEnabled = true;

	/** Strength of Smooth Brush */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (UIMin = 0, UIMax = 1, EditConditionHides, EditCondition = "SecondaryActionType == EMeshVertexPaintSecondaryActionType::Smooth"))
	float SmoothStrength = 0.25;

	/** Controls which Color Channels will be affected by Operations. Only enabled Channels are rendered. */
	UPROPERTY(EditAnywhere, Category = Settings)
	FModelingToolsColorChannelFilter ChannelFilter;

	/** Create Split Colors / Hard Color Edges at the borders of the painted area. Use Soften operations to un-split. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "Hard Edges"))
	bool bHardEdges = false;
};




UCLASS(MinimalAPI)
class UVertexPaintBrushFilterProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Area Mode specifies the shape of the brush and which triangles will be included relative to the cursor */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (DisplayName = "Brush Area Mode",
		EditConditionHides, EditCondition = "CurrentSubToolType != EMeshVertexPaintInteractionType::PolyLasso"))
		EMeshVertexPaintBrushAreaType BrushAreaMode = EMeshVertexPaintBrushAreaType::Connected;

	/** The Region affected by the current operation will be bounded by edge angles larger than this threshold */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (UIMin = "0.0", UIMax = "180.0", EditCondition = "CurrentSubToolType != EMeshVertexPaintInteractionType::PolyLasso && BrushAreaMode == EMeshVertexPaintBrushAreaType::Connected"))
	float AngleThreshold = 180.0f;

	/** The Region affected by the current operation will be bounded by UV borders/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (DisplayName = "UV Seams", EditCondition = "CurrentSubToolType != EMeshVertexPaintInteractionType::PolyLasso && BrushAreaMode == EMeshVertexPaintBrushAreaType::Connected"))
	bool bUVSeams = false;

	/** The Region affected by the current operation will be bounded by Hard Normal edges/seams */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (DisplayName = "Hard Normals", EditCondition = "CurrentSubToolType != EMeshVertexPaintInteractionType::PolyLasso && BrushAreaMode == EMeshVertexPaintBrushAreaType::Connected"))
	bool bNormalSeams = false;

	/** Control which triangles can be affected by the current operation based on visibility. Applied after all other filters. */
	UPROPERTY(EditAnywhere, Category = Filters)
	EMeshVertexPaintVisibilityType VisibilityFilter = EMeshVertexPaintVisibilityType::None;

	/**
	 * If the tool was started with a mesh element selection, this setting hides everything
	 *  except that selection, to make painting it easier. Requires that a mesh element
	 *  selection exist on tool start.
	 */
	UPROPERTY(EditAnywhere, Category = Filters, meta = (EditCondition = "bToolHasSelection", HideEditConditionToggle))
	bool bIsolateGeometrySelection = false;

	//~ For the tool to set, to enable/disable bIsolateGeometrySelection
	UPROPERTY()
	bool bToolHasSelection = false;

	/** Number of vertices in a triangle the Lasso must hit to be counted as "inside" */
	//UPROPERTY(EditAnywhere, Category = Filters, AdvancedDisplay, meta = (UIMin = 1, UIMax = 3, EditCondition = "CurrentSubToolType == EMeshVertexPaintInteractionType::PolyLasso"))
	UPROPERTY()
	int MinTriVertCount = 1;		// todo: convert to cvar

	/** Specify which Materials should be used to render the Mesh */
	UPROPERTY(EditAnywhere, Category = Visualization)
	EMeshVertexPaintMaterialMode MaterialMode = EMeshVertexPaintMaterialMode::UnlitVertexColor;

	/** Display the Color under the cursor */
	UPROPERTY(EditAnywhere, Category = Visualization, meta=(DisplayName="Show Color Under Cursor"))
	bool bShowHitColor = false;


	// values below are for edit conditions and track the current BasicProperties setting

	UPROPERTY(meta=(TransientToolProperty))
	EMeshVertexPaintInteractionType CurrentSubToolType;
};



UENUM()
enum class EMeshVertexPaintToolActions
{
	NoAction,

	/** Fill all Vertex Colors with the current Paint Color */
	PaintAll,
	/** Fill all Vertex Colors with the current Erase Color */
	EraseAll,
	/** Fill all Vertex Colors with Black (0,0,0,1) */
	FillBlack,
	/** Fill all Vertex Colors with White (1,1,1,1) */
	FillWhite,

	ApplyCurrentUtility
};


UCLASS(MinimalAPI)
class UMeshVertexPaintToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UMeshVertexPaintTool> ParentTool;

	void Initialize(UMeshVertexPaintTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EMeshVertexPaintToolActions Action);
};



UCLASS(MinimalAPI)
class UMeshVertexPaintToolQuickActions : public UMeshVertexPaintToolActionPropertySet
{
	GENERATED_BODY()

public:

	/**
	 * Fill all Vertex Colors with the current Paint color. Current Channel Filter still applies.
	 */
	UFUNCTION(CallInEditor, Category = QuickActions, meta = (DisplayPriority = 12))
	void PaintAll()
	{
		PostAction(EMeshVertexPaintToolActions::PaintAll);
	}

	/**
	 * Fill all Vertex Colors with the current Erase color. Current Channel Filter still applies.
	 */
	UFUNCTION(CallInEditor, Category = QuickActions, meta = (DisplayPriority = 13))
	void EraseAll()
	{
		PostAction(EMeshVertexPaintToolActions::EraseAll);
	}

	/**
	 * Fill all Vertex Colors with the Color (0,0,0,1). Current Channel Filter still applies.
	 */
	UFUNCTION(CallInEditor, Category = QuickActions, meta = (DisplayPriority = 14))
	void FillBlack()
	{
		PostAction(EMeshVertexPaintToolActions::FillBlack);
	}

	/**
	 * Fill all Vertex Colors with the Color (1,1,1,1). Current Channel Filter still applies.
	 */
	UFUNCTION(CallInEditor, Category = QuickActions, meta = (DisplayPriority = 14))
	void FillWhite()
	{
		PostAction(EMeshVertexPaintToolActions::FillWhite);
	}
};



UENUM()
enum class EMeshVertexPaintToolUtilityOperations
{
	/**
	 * Average the current color values at each vertex with split colors, so that there are no split vertices or seams in the color values
	 */
	BlendAllSeams,
	/**
	 * Set selected channels to a fixed value
	 */
	FillChannels,
	/**
	 * Invert channel values
	 */
	InvertChannels,
	/**
	 * Copy the color value from a source channel to all the selected target channels
	 */
	CopyChannelToChannel,
	/**
	 * Swap values between two Channels
	 */
	 SwapChannels,
	 /**
	 * Copy values from WeightMap into Vertex Color channels
	 */
	CopyFromWeightMap,
	/**
	 * Copy current values to any other LODs defined on the target
	 */
	CopyToOtherLODs,
	/**
	 * Copy current values to a specific LOD defined on the target
	 */
	CopyToSingleLOD
};




UCLASS(MinimalAPI)
class UMeshVertexPaintToolUtilityActions : public UMeshVertexPaintToolActionPropertySet
{
	GENERATED_BODY()

public:
	/**
	 * Operation to apply to current Vertex Colors
	 */
	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (NoResetToDefault, DisplayPriority = 1))
	EMeshVertexPaintToolUtilityOperations Operation = EMeshVertexPaintToolUtilityOperations::BlendAllSeams;

	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::CopyChannelToChannel || Operation == EMeshVertexPaintToolUtilityOperations::SwapChannels"))
	EMeshVertexPaintColorChannel SourceChannel = EMeshVertexPaintColorChannel::Red;

	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (UIMin = 0, UIMax = 1, EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::FillChannels"))
	float SourceValue = 0.0f;

	/** Target Vertex Weight Map */
	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (TransientToolProperty, NoResetToDefault, GetOptions = GetWeightMapsFunc, EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::CopyFromWeightMap"))
	FName WeightMap;

	// this function is called provide set of available weight maps
	UFUNCTION()
	TArray<FString> GetWeightMapsFunc() { return WeightMapsList; }

	// internal list used to implement above
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> WeightMapsList;

	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::FillChannels || Operation == EMeshVertexPaintToolUtilityOperations::InvertChannels || Operation == EMeshVertexPaintToolUtilityOperations::CopyChannelToChannel || Operation == EMeshVertexPaintToolUtilityOperations::CopyFromWeightMap"))
	FModelingToolsColorChannelFilter TargetChannels;

	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::SwapChannels"))
	EMeshVertexPaintColorChannel TargetChannel = EMeshVertexPaintColorChannel::Green;

	/** Copy colors to HiRes Source Mesh, if it exists */
	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::CopyToOtherLODs"))
	bool bCopyToHiRes = false;

	/** Target LOD to copy Colors to */
	UPROPERTY(EditAnywhere, Category = UtilityOperations, meta = (TransientToolProperty, DisplayName = "Copy To LOD", NoResetToDefault, GetOptions = GetLODNamesFunc, EditConditionHides, EditCondition = "Operation == EMeshVertexPaintToolUtilityOperations::CopyToSingleLOD"))
	FString CopyToLODName;

	UFUNCTION()
	const TArray<FString>& GetLODNamesFunc() const { return LODNamesList; }

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> LODNamesList;


	/**
	 * Apply the Operation currently selected below
	 */
	UFUNCTION(CallInEditor, Category = UtilityOperations, meta = (DisplayPriority = 10))
	void ApplySelectedOperation()
	{
		PostAction(EMeshVertexPaintToolActions::ApplyCurrentUtility);
	}

};



/**
 * FCommandChange for Vertex Color changes
 */
class FMeshVertexColorPaintChange : public TCustomIndexedValuesChange<FVector4f, int32>
{
public:
	virtual FString ToString() const override
	{
		return FString(TEXT("Paint Vertices"));
	}
};




/**
 * Mesh Vertex Color Painting TOol
 */
UCLASS(MinimalAPI)
class UMeshVertexPaintTool : public UMeshSculptToolBase, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API void SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn);

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API bool IsInBrushSubMode() const;

	UE_API virtual void CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

public:

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties;

	UPROPERTY()
	TObjectPtr<UVertexPaintBasicProperties> BasicProperties;
	
	/** Filters on paint brush */
	UPROPERTY()
	TObjectPtr<UVertexPaintBrushFilterProperties> FilterProperties;

	UPROPERTY()
	TObjectPtr<UMeshSymmetryProperties> SymmetryProperties;

private:
	// This will be of type UVertexPaintBrushOpProps, we keep a ref so we can change active color on pick
	UPROPERTY()
	TObjectPtr<UVertexColorPaintBrushOpProps> PaintBrushOpProperties;

	// This will be of type UVertexPaintBrushOpProps, we keep a ref so we can change active color on pick
	UPROPERTY()
	TObjectPtr<UVertexColorPaintBrushOpProps> EraseBrushOpProperties;

public:
	UE_API void FloodFillColorAction(FLinearColor Color);


	
	UE_API void SetTrianglesToVertexColor(const TArray<int32>& Triangles, const FLinearColor& ToColor);
	UE_API void SetTrianglesToVertexColor(const TSet<int32>& Triangles, const FLinearColor& ToColor);

	UE_API bool HaveVisibilityFilter() const;
	UE_API void ApplyVisibilityFilter(const TArray<int32>& Triangles, TArray<int32>& VisibleTriangles);
	UE_API void ApplyVisibilityFilter(TSet<int32>& Triangles, TArray<int32>& ROIBuffer, TArray<int32>& OutputBuffer);

protected:
	// UMeshSculptToolBase API
	virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() { return DynamicMeshComponent; }
	virtual FDynamicMesh3* GetBaseMesh() { check(false); return nullptr; }
	virtual const FDynamicMesh3* GetBaseMesh() const { check(false); return nullptr; }

	UE_API virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const override;

	UE_API virtual void OnBeginStroke(const FRay& WorldRay) override;
	UE_API virtual void OnEndStroke() override;
	UE_API virtual void OnCancelStroke() override;

	UE_API virtual TUniquePtr<FMeshSculptBrushOp>& GetActiveBrushOp();
	// end UMeshSculptToolBase API

	UE_API virtual FString GetPropertyCacheIdentifier() const;

	//
	// Action support
	//

public:
	UE_API virtual void RequestAction(EMeshVertexPaintToolActions ActionType);

	UPROPERTY()
	TObjectPtr<UMeshVertexPaintToolQuickActions> QuickActions;


public:
	UPROPERTY()
	TObjectPtr<UMeshVertexPaintToolUtilityActions> UtilityActions;
	
	UE_API void ApplyCurrentUtilityAction();
	UE_API void BlendAllSeams();
	UE_API void FillChannels();
	UE_API void InvertChannels();
	UE_API void CopyChannelToChannel();
	UE_API void SwapChannels();
	UE_API void CopyFromWeightMap();
	UE_API void CopyToOtherLODs();
	UE_API void CopyToSpecificLOD();

protected:
	bool bHavePendingAction = false;
	EMeshVertexPaintToolActions PendingAction;
	UE_API virtual void ApplyAction(EMeshVertexPaintToolActions ActionType);



	//
	// Marquee Support
	//
public:
	UPROPERTY()
	TObjectPtr<UPolyLassoMarqueeMechanic> PolyLassoMechanic;

protected:
	UE_API void OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled);


	//
	// Internals
	//

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged();
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	EMeshLODIdentifier SourceLOD;
	TArray<EMeshLODIdentifier> AvailableLODs;
	bool bTargetSupportsLODs = false;

	UE::Geometry::FDynamicMeshColorOverlay* ActiveColorOverlay = nullptr;
	UE::Geometry::FDynamicMeshColorOverlay* GetActiveColorOverlay() const { return ActiveColorOverlay; }

	TUniquePtr<UE::Geometry::FPolygroupSet> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer();

	UE_API void UpdateSubToolType(EMeshVertexPaintInteractionType NewType);
	UE_API void UpdateBrushType(EMeshVertexPaintBrushType BrushType);
	UE_API void UpdateSecondaryBrushType(EMeshVertexPaintSecondaryActionType NewType);
	UE_API void UpdateVertexPaintMaterialMode();

	TSet<int32> AccumulatedTriangleROI;
	bool bUndoUpdatePending = false;
	TArray<int> NormalsBuffer;
	UE_API void WaitForPendingUndoRedoUpdate();

	TArray<int> TempROIBuffer;
	TArray<int> VertexROI;
	TArray<bool> VisibilityFilterBuffer;
	TSet<int> TempVertexSet;
	TSet<int> TriangleROI;
	UE_API void UpdateROI(const FSculptBrushStamp& CurrentStamp);

	// in Mix blending mode we need to accumulate each stroke in a fully separate buffer and blend it
	// with the background colors. So we need that buffer and also save initial colors
	TArray<FVector4f> StrokeInitialColorBuffer;
	TArray<FVector4f> StrokeAccumColorBuffer;

	EMeshVertexPaintBrushType PendingStampType = EMeshVertexPaintBrushType::Paint;

	UE_API bool UpdateStampPosition(const FRay& WorldRay);
	UE_API bool ApplyStamp();

	// initial code here was ported from MeshVertexSculptTool, which requires an Octree. However since
	// mesh shape is static, we can actually use an AABBTree, and in one case a required query (nearest-point)
	// is not supported by Octree (currently). So currently using both (gross)
	UE::Geometry::FDynamicMeshOctree3 Octree;
	UE::Geometry::FDynamicMeshAABBTree3 AABBTree;

	UE_API bool UpdateBrushPosition(const FRay& WorldRay);

	bool GetInEraseStroke()
	{
		// Re-use the smoothing stroke key (shift) for erase stroke in the group paint tool
		return GetInSmoothingStroke();
	}


	bool bPendingPickColor = false;
	bool bPendingPickEraseColor = false;


	TArray<int32> ROITriangleBuffer;
	TSet<int32> ROIElementSet;
	TArray<int32> ROIElementBuffer;
	TArray<FVector4f> ROIColorBuffer;
	UE_API void InitializeElementROIFromTriangleROI(const TArray<int32>& TriangleROI, bool bInitializeFlatBuffers);
	UE_API bool SyncMeshWithColorBuffer(FDynamicMesh3* Mesh);

	TUniquePtr<TIndexedValuesChangeBuilder<FVector4f, FMeshVertexColorPaintChange>> ActiveChangeBuilder;
	UE_API void BeginChange();
	UE_API void EndChange();
	UE_API void ExternalUpdateValues(const TArray<int32>& ElementIDs, const TArray<FVector4f>& NewValues);

	UE_API FColor GetColorForGroup(int32 GroupID);
	UE_API void ApplyChannelFilter(const FVector4f& CurColor, FVector4f& NewColor);
	UE_API void OnChannelFilterModified();

	TArray<FVector3d> TriNormals;
	TArray<int32> UVSeamEdges;
	TArray<int32> NormalSeamEdges;
	UE_API void PrecomputeFilterData();


protected:
	virtual bool ShowWorkPlane() const override { return false; }

	// Currently using flow rate as 'brush strength', so disable temporal stamp spacing
	virtual float GetStampTemporalFlowRate() const override { return 1.0f; }

private:
	TOptional<UE::Geometry::FGeometrySelection> GeometrySelection;
	TSet<int32> SelectionTids;
	UE_API bool ShouldFilterTrianglesBySelection() const;

	TPimplPtr<UE::Geometry::FMeshPlanarSymmetry> Symmetry;
	bool bMeshSymmetryIsValid = false;
	UE_API void TryToInitializeSymmetry();

	bool bApplySymmetry = false;
	TArray<int> SymmetricVertexROI;
	TSet<int> SymmetricTriangleROI;
	TArray<int32> SymmetricROITriangleBuffer;
};



#undef UE_API
