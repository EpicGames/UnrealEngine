// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/NoExportTypes.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/DynamicVerticesOctree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "BaseTools/BaseBrushTool.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "ModelingToolExternalMeshUpdateAPI.h"
#include "TransformTypes.h"
#include "Sculpting/MeshSculptToolBase.h"
#include "Sculpting/MeshBrushOpBase.h"
#include "Image/ImageBuilder.h"
#include "Util/UniqueIndexSet.h"
#include "Polygroups/PolygroupSet.h"
#include "Templates/PimplPtr.h"
#include "MeshVertexSculptTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UMaterialInstanceDynamic;
class FMeshVertexChangeBuilder;
class UPreviewMesh;
namespace UE::Geometry { class FDynamicMeshSculptLayers; }
PREDECLARE_GEOMETRY(class FMeshPlanarSymmetry);
class UMeshSculptLayerProperties;


/**
 * Tool Builder
 */
UCLASS(MinimalAPI)
class UMeshVertexSculptToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	int32 DefaultPrimaryBrushID = -1;
protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
private:
	static FToolTargetTypeRequirements VSculptTypeRequirements;
};





/** Mesh Sculpting Brush Types */
UENUM()
enum class EMeshVertexSculptBrushType : uint8
{
	/** Move vertices parallel to the view plane  */
	Move UMETA(DisplayName = "Move"),

	/** Grab Brush, fall-off alters the influence of the grab */
	PullKelvin UMETA(DisplayName = "Kelvin Grab"),

	/** Grab Brush that may generate cusps, fall-off alters the influence of the grab */
	PullSharpKelvin UMETA(DisplayName = "Sharp Kelvin Grab"),

	/** Smooth mesh vertices  */
	Smooth UMETA(DisplayName = "Smooth"),

	/** Smooth mesh vertices but only in direction of normal (Ctrl to invert) */
	SmoothFill UMETA(DisplayName = "SmoothFill"),

	/** Displace vertices along the pre-stroke surface normal (Ctrl to invert) */
	Offset UMETA(DisplayName = "Sculpt (Normal)"),

	/** Displace vertices towards the camera viewpoint (Ctrl to invert) */
	SculptView UMETA(DisplayName = "Sculpt (Viewpoint)"),

	/** Displaces vertices along the pre-stroke surface normal to a maximum height based on the brush size (Ctrl to invert) */
	SculptMax UMETA(DisplayName = "Sculpt Max"),

	/** Displace vertices along their vertex normals */
	Inflate UMETA(DisplayName = "Inflate"),

	/** Displace vertices along their vertex normals, operating on pre-stroke mesh. */
	InflateStroke UMETA(DisplayName = "Inflate (Stroke)"),

	/** Displace vertices along their vertex normals to a maximum distance based on the brush size (Ctrl to invert) */
	InflateMax UMETA(DisplayName = "Inflate Max"),

	/** Scale Brush will inflate or pinch radially from the center of the brush */
	ScaleKelvin UMETA(DisplayName = "Kelvin Scale"),

	/** Move vertices towards the center of the brush (Ctrl to push away)*/
	Pinch UMETA(DisplayName = "Pinch"),

	/** Twist Brush moves vertices in the plane perpendicular to the local mesh normal */
	TwistKelvin UMETA(DisplayName = "Kelvin Twist"),

	/** Move vertices towards the average plane of the brush stamp region */
	Flatten UMETA(DisplayName = "Flatten"),

	/** Move vertices towards a plane defined by the initial brush position  */
	Plane UMETA(DisplayName = "Plane (Normal)"),

	/** Move vertices towards a view-facing plane defined at the initial brush position */
	PlaneViewAligned UMETA(DisplayName = "Plane (Viewpoint)"),

	/** Move vertices towards a fixed plane in world space, positioned with a 3D gizmo */
	FixedPlane UMETA(DisplayName = "FixedPlane"),

	/** Erase sculpt layers */
	EraseSculptLayer UMETA(DisplayName = "Erase Sculpt Layer"),

	LastValue UMETA(Hidden)

};


/** Brush Triangle Filter Type */
UENUM()
enum class EMeshVertexSculptBrushFilterType : uint8
{
	/** Do not filter brush area */
	None = 0,
	/** Only apply brush to triangles in the same connected mesh component/island */
	Component = 1,
	/** Only apply brush to triangles with the same PolyGroup */
	PolyGroup = 2
};



UCLASS(MinimalAPI)
class UVertexBrushSculptProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	
	UE_DEPRECATED(5.6, "Vertex sculpt now uses an integer for a brush ID to allow custom brush registration")
	UPROPERTY()
	EMeshVertexSculptBrushType PrimaryBrushType = EMeshVertexSculptBrushType::Offset;

	/** Primary Brush Mode */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (DisplayName = "Brush"))
	int32 PrimaryBrushID = 0;

	/** Primary Brush Falloff Type, multiplied by Alpha Mask where applicable */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (DisplayName = "Falloff", ModelingQuickSettings = 300))
	EMeshSculptFalloffType PrimaryFalloffType = EMeshSculptFalloffType::Smooth;

	/** Filter applied to Stamp Region Triangles, based on first Stroke Stamp */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (DisplayName = "Region"))
	EMeshVertexSculptBrushFilterType BrushFilter = EMeshVertexSculptBrushFilterType::None;

	/** When Freeze Target is toggled on, the Brush Target Surface will be Frozen in its current state, until toggled off. Brush strokes will be applied relative to the Target Surface, for applicable Brushes */
	UPROPERTY(EditAnywhere, Category = Sculpting, meta = (EditCondition = "bCanFreezeTarget", HideEditConditionToggle))
	bool bFreezeTarget = false;

	/** Whether to prevent sculpt brushes from moving mesh boundary vertices */
	UPROPERTY(EditAnywhere, Category = Sculpting)
	bool bLockBoundaries = false;

	/** Scale the rendered opacity of the on-surface brush stamp indicator */
	UPROPERTY(EditAnywhere, Category = Sculpting, AdvancedDisplay, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float StampOpacity = 1.0f;

	UPROPERTY()
	bool bCanFreezeTarget = false;

	// parent ref required for details customization
	UPROPERTY(meta = (TransientToolProperty))
	TWeakObjectPtr<UMeshVertexSculptTool> Tool;
};



/**
 * Tool Properties for a brush alpha mask
 */
UCLASS(MinimalAPI)
class UVertexBrushAlphaProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Alpha mask applied to brush stamp. Red channel is used. */
	UPROPERTY(EditAnywhere, Category = Alpha, meta = (DisplayName = "Alpha Mask", ModelingQuickSettings = 400))
	TObjectPtr<UTexture2D> Alpha = nullptr;

	/** Alpha is rotated by this angle, inside the brush stamp frame (vertically aligned) */
	UPROPERTY(EditAnywhere, Category = Alpha, meta = (DisplayName = "Angle", UIMin = "-180.0", UIMax = "180.0", ClampMin = "-360.0", ClampMax = "360.0"))
	float RotationAngle = 0.0;

	/** If true, a random angle in +/- RandomRange is added to Rotation angle for each stamp */
	UPROPERTY(EditAnywhere, Category = Alpha, AdvancedDisplay)
	bool bRandomize = false;

	/** Bounds of random generation (positive and negative) for randomized stamps */
	UPROPERTY(EditAnywhere, Category = Alpha, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "180.0"))
	float RandomRange = 180.0;

	// parent ref required for details customization
	UPROPERTY(meta = (TransientToolProperty))
	TWeakObjectPtr<UMeshVertexSculptTool> Tool;
};




UCLASS(MinimalAPI)
class UMeshSymmetryProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Enable/Disable symmetric sculpting. This option will not be available if symmetry cannot be detected, or a non-symmetric edit has been made */
	UPROPERTY(EditAnywhere, Category = Symmetry, meta = (HideEditConditionToggle, EditCondition = bSymmetryCanBeEnabled, ModelingQuickSettings = 500))
	bool bEnableSymmetry = false;

	// this flag is set/updated by the Tool to enable/disable the bEnableSymmetry toggle
	UPROPERTY(meta = (TransientToolProperty))
	bool bSymmetryCanBeEnabled = false;
};







/**
 * Mesh Vertex Sculpt Tool Class
 */
UCLASS(MinimalAPI)
class UMeshVertexSculptTool : public UMeshSculptToolBase, public IInteractiveToolManageGeometrySelectionAPI, public IModelingToolExternalDynamicMeshUpdateAPI
{
	GENERATED_BODY()
public:
	UE_API bool DoesTargetHaveSculptLayers() const;

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

	// IInteractiveToolExternalDynamicMeshUpdateAPI methods
	virtual bool AllowToolMeshUpdates() const override { return !ActiveVertexChange; }
	UE_API virtual void UpdateToolMeshes(TFunctionRef<TUniquePtr<FMeshRegionChangeBase>(FDynamicMesh3&, int32 MeshIdx)> UpdateMesh) override;
	virtual void ProcessToolMeshes(TFunctionRef<void(const FDynamicMesh3&, int32 MeshIdx)> ProcessMesh) const override
	{
		ProcessMesh(*GetSculptMesh(), 0);
	}
	virtual int32 NumToolMeshes() const override
	{
		return 1;
	}

public:

	/** Properties that control sculpting*/
	UPROPERTY()
	TObjectPtr<UVertexBrushSculptProperties> SculptProperties;

	UPROPERTY()
	TObjectPtr<UVertexBrushAlphaProperties> AlphaProperties;

	UPROPERTY()
	TObjectPtr<UTexture2D> BrushAlpha;

	UPROPERTY()
	TObjectPtr<UMeshSymmetryProperties> SymmetryProperties;

private:
	// TODO: These sculpt layer properties should be moved off this tool, to a separate UI
	UPROPERTY()
	TObjectPtr<UMeshSculptLayerProperties> SculptLayerProperties;

	// Call to sync sculpt layer properties to the current mesh state
	void RefreshSculptLayerProperties();

public:
	UE_API virtual void IncreaseBrushSpeedAction() override;
	UE_API virtual void DecreaseBrushSpeedAction() override;

	UE_API virtual void UpdateBrushAlpha(UTexture2D* NewAlpha);

	UE_API virtual void SetActiveBrushType(int32 Identifier);
	UE_API virtual void SetActiveFalloffType(int32 Identifier);
	UE_API virtual void SetRegionFilterType(int32 Identifier);

	// retrieves a set of FBrushTypeInfos representing the brushes currently available to be enabled
	virtual TSet<UMeshSculptToolBase::FBrushTypeInfo> GetAvailableBrushTypes() { return GetRegisteredPrimaryBrushTypes(); };

	UE_API void SetDefaultPrimaryBrushID(int32 InPrimaryBrushID);

	UE_API bool CanUpdateBrushType() const;

	/**
	* OnDetailsPanelRequestRebuild is broadcast by the tool when it detects it needs to have it's details panel rebuilt outside
	* of normal rebuilding triggers, such as changing property set objects. This is useful in rare circumstances, such as when
	* the tool is using detail customizations and tool properties are changed outside of user interactions, such as via tool
	* preset loading. In these cases, the detail customization widgets might not be updated properly without rebuilding the details
	* panel completely.
	*/

	DECLARE_MULTICAST_DELEGATE(OnInteractiveToolDetailsPanelRequestRebuild);
	OnInteractiveToolDetailsPanelRequestRebuild OnDetailsPanelRequestRebuild;

protected:
	// Overriden by subclasses to choose the brushes they use
	UE_API virtual void RegisterBrushes();
	// Should be overriden by subclasses so that settings are not shared across
	//  this tool and subclasses.
	UE_API virtual FString GetPropertyCacheIdentifier() const override;

	// UMeshSculptToolBase API
	UE_API virtual void SaveActiveStrokeModifiers() override;
	UE_API virtual void InitializeIndicator() override;
	UE_API virtual UPreviewMesh* MakeBrushIndicatorMesh(UObject* Parent, UWorld* World) override;

	virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() override { return DynamicMeshComponent; }
	virtual FDynamicMesh3* GetBaseMesh() override{ return &BaseMesh; }
	virtual const FDynamicMesh3* GetBaseMesh() const override{ return &BaseMesh; }

	UE_API virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const override;
	UE_API bool IsHitTriangleBackFacing(int32 TriangleID, const FDynamicMesh3* QueryMesh) const;

	UE_API virtual void UpdateHoverStamp(const FFrame3d& StampFrameWorld) override;

	UE_API virtual void OnBeginStroke(const FRay& WorldRay) override;
	UE_API virtual void OnEndStroke() override;
	UE_API virtual void OnCancelStroke() override;
	// end UMeshSculptToolBase API

protected:

	UPROPERTY()
	TObjectPtr<AInternalToolFrameworkActor> PreviewMeshActor = nullptr;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = nullptr;

	// realtime visualization
	UE_API void OnDynamicMeshComponentChanged(UDynamicMeshComponent* Component, const FMeshRegionChangeBase* Change, bool bRevert);
	FDelegateHandle OnDynamicMeshComponentChangedHandle;

	UE_DEPRECATED(5.6, "Vertex sculpt now uses an integer for a brush ID to allow custom brush registration")
	UE_API void UpdateBrushType(EMeshVertexSculptBrushType BrushType);

	UE_API virtual void UpdateBrushType(int32 BrushID);
	
	/**
	 * Fetch the BrushFrame in World space with considerations for symmetry. When symmetry is enabled,
	 * the frame will always be in the positive side of the symmetry plane to simplify the mirror ROI
	 * computation. This should be used wherever we use the BrushFrame to compute an ROI to properly
	 * support symmetry.
	 * @return The brush frame in world space.
	 */
	FFrame3d GetSymmetricBrushFrameWorld() const;

	/**
	 * Fetch the BrushFrame in Local space with considerations for symmetry. When symmetry is enabled,
	 * the frame will always be in the positive side of the symmetry plane to simplify the mirror ROI
	 * computation. This should be used wherever we use the BrushFrame to compute an ROI to properly
	 * support symmetry.
	 * 
	 * @return The brush frame in local space.
	 */
	FFrame3d GetSymmetricBrushFrameLocal() const;
protected:

	TUniquePtr<UE::Geometry::FPolygroupSet> ActiveGroupSet;
	TArray<int32> TriangleComponentIDs;

	int32 InitialStrokeTriangleID = -1;

	TSet<int32> AccumulatedTriangleROI;
	UE_DEPRECATED(5.8, "bUndoUpdatePending is no longer used")
	bool bUndoUpdatePending = false;
	UE_DEPRECATED(5.8, "UndoNormalsFuture is no longer used")
	TFuture<bool> UndoNormalsFuture;
	TFuture<bool> UndoUpdateOctreeFuture;
	TFuture<bool> UndoUpdateBaseMeshFuture;
	TFuture<void> UndoUpdateFuture;
	TArray<int> NormalsBuffer;

	UE_DEPRECATED(5.7, "Vertex sculpt now uses a const version of WaitForPendingUndoRedo")
	void WaitForPendingUndoRedo() {};
	UE_API void WaitForPendingUndoRedoUpdate() const;

	TArray<uint32> OctreeUpdateTempBuffer;
	TArray<bool> OctreeUpdateTempFlagBuffer;
	TFuture<void> StampUpdateOctreeFuture;
	UE_DEPRECATED(5.7, "Check StampUpdateOctreeFuture directly instead")
	bool bStampUpdatePending = false;

	UE_DEPRECATED(5.7, "Vertex sculpt now uses a const version of WaitForPendingStampUpdate")
	void WaitForPendingStampUpdate() {};
	UE_API void WaitForPendingStampUpdateConst() const;

	UE_DEPRECATED(5.8, "RangeQueryTriBuffer is not used anymore")
	TArray<int> RangeQueryTriBuffer;
	UE::Geometry::FUniqueIndexSet VertexROIBuilder;
	UE::Geometry::FUniqueIndexSet TriangleROIBuilder;
	UE_DEPRECATED(5.8, "TriangleROIInBuf is not used anymore")
	TArray<UE::Geometry::FIndex3i> TriangleROIInBuf;
	TArray<int> VertexROI;
	TArray<int> TriangleROIArray;
	UE_API void UpdateROI(const FFrame3d& LocalFrame);
	UE_DEPRECATED(5.6, "Use the FFrame3d overload instead.")
	UE_API void UpdateROI(const FVector3d& BrushPos);
private:
	UE_API void ComputeVertexAndTriangleROI(const FFrame3d& LocalFrame);
	UE_API void PrepROIVertPositionBuffers();
protected:
	virtual bool RequireConnectivityToHitPointInStamp() const { return false; }

	UE::Geometry::FUniqueIndexSet NormalsROIBuilder;
	TArray<std::atomic<bool>> NormalsFlags;		// set of per-vertex or per-element-id flags that indicate
												// whether normal needs recompute. Fast to do it this way
												// than to use a TSet or UniqueIndexSet...

	bool bTargetDirty;

	UE_DEPRECATED(5.6, "Vertex sculpt now uses an integer for a brush ID to allow custom brush registration")
	EMeshVertexSculptBrushType PendingStampType = EMeshVertexSculptBrushType::Smooth;
private:
	int32 PendingStampBrushID = (int32)EMeshVertexSculptBrushType::Smooth;
protected:

	UE_API bool UpdateStampPosition(const FRay& WorldRay);
	UE_API TFuture<void> ApplyStamp();
	
	FVector3d PreviousRayDirection;
	bool bMouseMoved = false;
	// The stamp used last time bMouseMoved was true
	FSculptBrushStamp LastMovedStamp;

	FRandomStream StampRandomStream;

	// The base mesh is a second copy of our mesh that we can intentionally not update during
	//  some sculpt strokes so that we can base certain hittesting or processing off that mesh
	//  instead of the latest result, e.g. when using brushes that offset some max
	//  amount, so that we only offset relative to the mesh as it was before the start of the
	//  stroke.
	FDynamicMesh3 BaseMesh;
	UE::Geometry::FDynamicMeshOctree3 BaseMeshSpatial;
	TArray<int32> BaseMeshIndexBuffer;
	bool bCachedFreezeTarget = false;
	UE_API void UpdateBaseMesh(const TSet<int32>* TriangleROI = nullptr);
	UE_API bool GetBaseMeshNearest(int32 VertexID, const FVector3d& Position, double SearchRadius, FVector3d& TargetPosOut, FVector3d& TargetNormalOut);
	TFunction<bool(int32, const FVector3d&, double MaxDist, FVector3d&, FVector3d&)> BaseMeshQueryFunc;

	// triangle octree used for raycast queries; for ROI queries we use the VertexOctree (in private scope, below)
	UE::Geometry::FDynamicMeshOctree3 Octree;
	bool bOctreeUpdated = false;
	TSharedPtr<UE::Geometry::FDynamicMeshOctree3::FTreeCutSet> CutTree;
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> OctreeGeometry;

	UE_API bool UpdateBrushPosition(const FRay& WorldRay);

	double SculptMaxFixedHeight = -1.0;

	bool bHaveBrushAlpha = false;
	UE::Geometry::TImageBuilder<FVector4f> BrushAlphaValues;
	UE::Geometry::FImageDimensions BrushAlphaDimensions;
	UE_API double SampleBrushAlpha(const FSculptBrushStamp& Stamp, const FVector3d& Position) const;

	TArray<FVector3d> ROIPositionBuffer;
	TArray<FVector3d> ROIPrevPositionBuffer;

	TPimplPtr<UE::Geometry::FMeshPlanarSymmetry> Symmetry;
	bool bMeshSymmetryIsValid = false;
	UE_API void TryToInitializeSymmetry();
	friend class FVertexSculptNonSymmetricChange;
	UE_API virtual void UndoRedo_RestoreSymmetryPossibleState(bool bSetToValue);

	bool bApplySymmetry = false;
	TArray<int> SymmetricVertexROI;
	TArray<FVector3d> SymmetricROIPositionBuffer;
	TArray<FVector3d> SymmetricROIPrevPositionBuffer;

	FMeshVertexChangeBuilder* ActiveVertexChange = nullptr;
	UE_API void BeginChange();
	UE_API void EndChange();

	int32 DefaultPrimaryBrushID = -1;


protected:
	virtual bool ShowWorkPlane() const override { return GizmoProperties->bShowPlane && SculptProperties->PrimaryBrushID == (int32)EMeshVertexSculptBrushType::FixedPlane; }

private:
	// Assumes that brush is currently aligned to hit normal
	UE_API void RealignBrush(FMeshSculptBrushOp::EStampAlignmentType AlignmentType);

	// internal implementation of OnDynamicMeshComponentChanged, exposing option to update component rendering as well
	void OnDynamicMeshComponentChanged_Internal(UDynamicMeshComponent* Component, const FMeshRegionChangeBase* Change, bool bRevert, bool bUpdateRender = false);

	double InitialBoundsMaxDim = 0;
	// Cached mesh connectivity for faster iteration
	TArray<TArray<int32, TInlineAllocator<8>>> CachedVtxTriConnectivity, CachedVtxVtxConnectivity;
	// Re-used index set for looking up the initial ROI during connected component growth
	UE::Geometry::FUniqueIndexSet VertexROILookup;
	// Octree used for vertex queries
	UE::Geometry::TDynamicVerticesOctree3<FDynamicMesh3> VertexOctree;
};


#undef UE_API
