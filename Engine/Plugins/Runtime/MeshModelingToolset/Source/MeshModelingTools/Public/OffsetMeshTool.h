// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/BaseMeshProcessingTool.h"
#include "PropertySets/WeightMapSetProperties.h"
#include "OffsetMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API


UENUM()
enum class EOffsetMeshToolOffsetType : uint8
{
	/** Iterative Offsetting with N iterations */
	Iterative UMETA(DisplayName = "Iterative"),

	/** Implicit Offsetting, produces smoother output and does a better job at preserving UVs, but can be very slow on large meshes */
	Implicit UMETA(DisplayName = "Implicit")
};



/** Base properties of Offset */
UCLASS(MinimalAPI)
class UOffsetMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Type of Offsetting to apply */
	UPROPERTY(EditAnywhere, Category = Offset)
	EOffsetMeshToolOffsetType OffsetType = EOffsetMeshToolOffsetType::Iterative;

	/** Offset Distance in World Units */
	UPROPERTY(EditAnywhere, Category = Offset, meta = (UIMin = "-100.0", UIMax = "100.0", ClampMin = "-10000.0", ClampMax = "100000.0"))
	float Distance = 1.0f;

	/** If true, create a thickened shell, instead of only moving the input vertices */
	UPROPERTY(EditAnywhere, Category = Offset, meta = (EditCondition = "OffsetType == EOffsetMeshToolOffsetType::Iterative") )
	bool bCreateShell = false;
};



UCLASS(MinimalAPI)
class UOffsetWeightMapSetProperties : public UWeightMapSetProperties
{
	GENERATED_BODY()
public:

	/** Minimum Offset Distance in World Units, for Weight Map values of zero (clamped to Distance) */
	UPROPERTY(EditAnywhere, Category = WeightMap, meta = (UIMin = "-100.0", UIMax = "100.0", ClampMin = "-10000.0", ClampMax = "100000.0", DisplayPriority = 5))
	float MinDistance = 1.0f;
};


/** Properties for Iterative Offsetting */
UCLASS(MinimalAPI)
class UIterativeOffsetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Number of Offsetting iterations */
	UPROPERTY(EditAnywhere, Category = IterativeOffsetOptions, meta = (UIMin = "1", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	int32 Steps = 10;

	/** Control whether the boundary is allowed to move */
	UPROPERTY(EditAnywhere, Category = IterativeOffsetOptions)
	bool bOffsetBoundaries = true;

	/** Amount of smoothing applied per Offset step */
	UPROPERTY(EditAnywhere, Category = IterativeOffsetOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float SmoothingPerStep = 0.0f;

	/** Reproject smooth vertices onto non-smoothed Offset Surface at each step (expensive but better-preserves uniform distance) */
	UPROPERTY(EditAnywhere, Category = IterativeOffsetOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition="SmoothingPerStep > 0"))
	bool bReprojectSmooth = false;
};


/** Properties for Implicit Offsetting */
UCLASS(MinimalAPI)
class UImplicitOffsetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** How tightly we should constrain the constrained implicit offset to the explicit offset */
	UPROPERTY(EditAnywhere, Category = ImplicitOffsettingOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "100.0"))
	float Smoothness = 0.2f;

	/** If this is false, triangles will be reshaped to be more regular, which will distort UVs */
	UPROPERTY(EditAnywhere, Category = ImplicitOffsettingOptions)
	bool bPreserveUVs = true;
};






/**
 * Mesh Offsetting Tool
 */
UCLASS(MinimalAPI)
class UOffsetMeshTool : public UBaseMeshProcessingTool
{
	GENERATED_BODY()

public:
	UE_API UOffsetMeshTool();

	UE_API virtual void InitializeProperties() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	virtual bool RequiresInitialVtxNormals() const { return true; }
	virtual bool RequiresInitialBoundaryLoops() const { return true; }
	UE_API virtual bool HasMeshTopologyChanged() const override;

	UE_API virtual FText GetToolMessageString() const override;
	UE_API virtual FText GetAcceptTransactionName() const override;

protected:
	UPROPERTY()
	TObjectPtr<UOffsetMeshToolProperties> OffsetProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UIterativeOffsetProperties> IterativeProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UImplicitOffsetProperties> ImplicitProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UOffsetWeightMapSetProperties> WeightMapProperties = nullptr;
};




/**
 *
 */
UCLASS(MinimalAPI)
class UOffsetMeshToolBuilder : public UBaseMeshProcessingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

#undef UE_API
