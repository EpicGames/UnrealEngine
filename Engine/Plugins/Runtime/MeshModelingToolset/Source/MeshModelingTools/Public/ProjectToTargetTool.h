// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemeshMeshTool.h"
#include "ToolBuilderUtil.h"
#include "ProjectToTargetTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

/**
 * Determine if/how we can build UProjectToTargetTool. It requires two selected mesh components.
 */
UCLASS(MinimalAPI)
class UProjectToTargetToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


/**
 * Subclass URemeshMeshToolProperties just so we can set default values for some properties. Setting these values in the
 * Setup function of UProjectToTargetTool turns out to be tricky to achieve with the property cache.
 */
UCLASS(MinimalAPI)
class UProjectToTargetToolProperties : public URemeshMeshToolProperties
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = ProjectionSpace)
	bool bWorldSpace = true;

	UPROPERTY(EditAnywhere, Category = Remeshing)
	bool bParallel = true;

	UPROPERTY(EditAnywhere, Category = NormalFlow, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow", UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "100"))
	int FaceProjectionPassesPerRemeshIteration = 1;

	UPROPERTY(EditAnywhere, Category = NormalFlow, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow", UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "10.0"))
	float SurfaceProjectionSpeed = 0.2f;

	UPROPERTY(EditAnywhere, Category = NormalFlow, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow", UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "10.0"))
	float NormalAlignmentSpeed = 0.2f;

	UPROPERTY(EditAnywhere, Category = NormalFlow, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow"))
	bool bSmoothInFillAreas = true;

	UPROPERTY(EditAnywhere, Category = NormalFlow, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow && bSmoothInFillAreas", UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "10.0"))
	float FillAreaDistanceMultiplier = 0.25f;

	UPROPERTY(EditAnywhere, Category = NormalFlow, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow && bSmoothInFillAreas", UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "10.0"))
	float FillAreaSmoothMultiplier = 0.25f;


	UProjectToTargetToolProperties() 
		: URemeshMeshToolProperties()
	{
		bPreserveSharpEdges = false;
		RemeshType = ERemeshType::NormalFlow;
	}
};


/**
 * Project one mesh surface onto another, while undergoing remeshing. Subclass of URemeshMeshTool to avoid duplication.
 */
UCLASS(MinimalAPI)
class UProjectToTargetTool : public URemeshMeshTool
{
	GENERATED_BODY()

public:

	UProjectToTargetTool(const FObjectInitializer& ObjectInitializer) :
		Super(ObjectInitializer.SetDefaultSubobjectClass<UProjectToTargetToolProperties>(TEXT("RemeshProperties")))
	{}

	UE_API virtual void Setup() override;

	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

private:

	TUniquePtr<UE::Geometry::FDynamicMesh3> ProjectionTarget;
	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> ProjectionTargetSpatial;

};

#undef UE_API
