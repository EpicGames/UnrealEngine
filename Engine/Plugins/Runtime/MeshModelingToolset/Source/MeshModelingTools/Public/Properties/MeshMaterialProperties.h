// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"
#include "InteractiveToolBuilder.h"
#include "MeshMaterialProperties.generated.h"

#define UE_API MESHMODELINGTOOLS_API


// Forward declarations
class UMaterialInterface;
class UMaterialInstanceDynamic;


// Standard material property settings for tools that generate new meshes
UCLASS(MinimalAPI)
class UNewMeshMaterialProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UE_API UNewMeshMaterialProperties();

	/** Material for new mesh */
	UPROPERTY(EditAnywhere, NonTransactional, Category = Material)
	TWeakObjectPtr<UMaterialInterface> Material;

	/** Scale factor for generated UVs */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DisplayName = "UV Scale", HideEditConditionToggle, EditConditionHides, EditCondition = "bShowExtendedOptions", Delta = 0.01, LinearDeltaSensitivity = 1))
	float UVScale = 1.0;

	/** If true, UV scale will be relative to world space. This means objects of different sizes created with the same UV scale have the same average texel size. */
	UPROPERTY(EditAnywhere, Category = Material, meta = (DisplayName = "World Space UV Scale", HideEditConditionToggle, EditConditionHides, EditCondition = "bShowExtendedOptions"))
	bool bWorldSpaceUVScale = false;

	/** If true, overlays preview with wireframe */
	UPROPERTY(EditAnywhere, Category = Material, meta = (HideEditConditionToggle, EditConditionHides, EditCondition = "bShowExtendedOptions"))
	bool bShowWireframe = false;

	/** If true, extended options are available */ 
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowExtendedOptions = true;
};



/** Standard material modes for tools that need to set custom materials for visualization */
UENUM()
enum class ESetMeshMaterialMode : uint8
{
	/** Input material */
	Original,

	/** Checkerboard material */
	Checkerboard,

	/** Override material */
	Override
};

// Standard material property settings for tools that visualize materials on existing meshes (e.g. to help show UVs)
UCLASS(MinimalAPI)
class UExistingMeshMaterialProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Material that will be used on the mesh */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial)
	ESetMeshMaterialMode MaterialMode = ESetMeshMaterialMode::Original;

	/** Number of checkerboard tiles within the 0 to 1 range; only available when Checkerboard is selected as material mode */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial,
		meta = (UIMin = "1.0", UIMax = "40.0", ClampMin = "0.01", ClampMax = "1000.0", EditConditionHides, EditCondition = "MaterialMode == ESetMeshMaterialMode::Checkerboard"))
	float CheckerDensity = 20.0f;

	/** Material to use instead of the original material; only available when Override is selected as material mode */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial, meta = (EditConditionHides, EditCondition = "MaterialMode == ESetMeshMaterialMode::Override"))
	TObjectPtr<UMaterialInterface> OverrideMaterial = nullptr;

	/** Which UV channel to use for visualizing the checkerboard material on the mesh; note that this does not affect the preview layout */
	UPROPERTY(EditAnywhere, Category = PreviewMaterial,
		meta = (DisplayName = "Preview UV Channel", GetOptions = GetUVChannelNamesFunc, EditConditionHides, EditCondition =
			"MaterialMode == ESetMeshMaterialMode::Checkerboard", NoResetToDefault))
	FString UVChannel;

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> UVChannelNamesList;

	UFUNCTION()
	UE_API const TArray<FString>& GetUVChannelNamesFunc() const;

	UPROPERTY(meta = (TransientToolProperty))
	TObjectPtr<UMaterialInstanceDynamic> CheckerMaterial = nullptr;

	// Needs custom restore in order to call setup
	UE_API virtual void RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier = TEXT("")) override;

	UE_API void Setup();

	UE_API void UpdateMaterials();
	UE_API UMaterialInterface* GetActiveOverrideMaterial() const;
	
	UE_API void UpdateUVChannels(int32 UVChannelIndex, const TArray<FString>& UVChannelNames, bool bUpdateSelection = true);
};

UENUM()
enum class EMeshEditingMaterialModes
{
	ExistingMaterial,
	Diffuse,
	Grey,
	Soft,
	Transparent,
	TangentNormal,
	VertexColor,
	CustomImage,
	Custom
};


UCLASS(MinimalAPI)
class UMeshEditingViewProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Toggle drawing of wireframe overlay on/off [Alt+W] */
	UPROPERTY(EditAnywhere, Category = Rendering)
	bool bShowWireframe = false;

	/** Set which material to use on object */
	UPROPERTY(EditAnywhere, Category = Rendering)
	EMeshEditingMaterialModes MaterialMode = EMeshEditingMaterialModes::Diffuse;

	/** Toggle flat shading on/off */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, 
		EditCondition = "MaterialMode != EMeshEditingMaterialModes::ExistingMaterial && MaterialMode != EMeshEditingMaterialModes::Transparent && MaterialMode != EMeshEditingMaterialModes::Custom") )
	bool bFlatShading = true;

	/** Main Color of Material */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshEditingMaterialModes::Diffuse"))
	FLinearColor Color = FLinearColor(0.4f, 0.4f, 0.4f);

	/** Image used in Image-Based Material */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshEditingMaterialModes::CustomImage", TransientToolProperty) )
	TObjectPtr<UTexture2D> Image;

	/** Opacity of transparent material */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshEditingMaterialModes::Transparent", ClampMin = "0", ClampMax = "1.0"))
	double Opacity = 0.65;

	//~ Could have used the same property as Color, above, but the user may want different saved values for the two
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshEditingMaterialModes::Transparent", DisplayName = "Color"))
	FLinearColor TransparentMaterialColor = FLinearColor(0.0606f, 0.309f, 0.842f);

	/** Although a two-sided transparent material causes rendering issues with overlapping faces, it is still frequently useful to see the shape when sculpting around other objects. */
	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshEditingMaterialModes::Transparent"))
	bool bTwoSided = true;

	UPROPERTY(EditAnywhere, Category = Rendering, meta = (EditConditionHides, EditCondition = "MaterialMode == EMeshEditingMaterialModes::Custom"))
	TWeakObjectPtr<UMaterialInterface> CustomMaterial;
};

#undef UE_API
