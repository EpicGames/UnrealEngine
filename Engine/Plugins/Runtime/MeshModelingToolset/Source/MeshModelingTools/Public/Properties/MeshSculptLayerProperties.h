// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "GeometryBase.h"

#include "MeshSculptLayerProperties.generated.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMeshSculptLayers);
class IModelingToolExternalDynamicMeshUpdateAPI;

UCLASS(MinimalAPI)
class UMeshSculptLayerProperties : public UObject
{
	GENERATED_BODY()

public:

	/** Set the active mesh layer */
	UPROPERTY(EditAnywhere, Category = MeshLayers, meta = (ClampMin = 0, HideEditConditionToggle, EditCondition = bCanEditLayers, ModelingQuickSettings))
	int32 ActiveLayer = 0;

	/** Set the mesh layer weights */
	UPROPERTY(EditAnywhere, EditFixedSize, NoClear, Category = MeshLayers, meta = (NoResetToDefault, HideEditConditionToggle, EditCondition = bCanEditLayers, ModelingQuickSettings))
	TArray<double> LayerWeights;

	UPROPERTY(meta = (TransientToolProperty))
	bool bCanEditLayers = false;

	/** Add a sculpt layer */
	MESHMODELINGTOOLS_API void AddLayer();

	/** Remove the active sculpt layer */
	MESHMODELINGTOOLS_API bool RemoveLayer();

	/** Remove the sculpt layer at the provided index */
	UFUNCTION(Category = MeshLayers)
	MESHMODELINGTOOLS_API bool RemoveLayerAtIndex(const int32 IndexOfLayerToRemove);

	/** Return the name of the sculpt layer at the provided index */
	UFUNCTION(Category = MeshLayers)
	MESHMODELINGTOOLS_API FName GetLayerName(const int32 InIndex);

	/** Set the name of the layer at the provided index to the provided name */
	UFUNCTION(Category = MeshLayers)
	MESHMODELINGTOOLS_API void SetLayerName(const int32 InIndex, const FName InLayerName);

	/** Move the layer at the first provided index to the location provided by the second index */
	UFUNCTION(Category = MeshLayers)
	MESHMODELINGTOOLS_API void MoveLayer(const int32 InInitIndex, int32 InTargetIndex);

	/** Sets the currently active sculpt layer to be the one at the provided index */
	UFUNCTION(Category = MeshLayers)
	MESHMODELINGTOOLS_API void SetActiveLayer(const int32 InIndex);

	/** Sets the weight of the layer at the provided index to the provided weight */
	UFUNCTION(Category = MeshLayers)
	MESHMODELINGTOOLS_API void SetLayerWeight(const int32 InIndex, double InWeight, const uint32 ChangeType);

	MESHMODELINGTOOLS_API void Init(IModelingToolExternalDynamicMeshUpdateAPI* InTool, int32 InNumLockedBaseLayers);

	/** Refresh properties from current mesh data */
	MESHMODELINGTOOLS_API void RefreshFromMesh();

#if WITH_EDITOR
	MESHMODELINGTOOLS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	int32 GetNumLockedBaseLayers() const { return NumLockedBaseLayers; }


	DECLARE_MULTICAST_DELEGATE(OnMeshLayersPropertiesPanelRequestRebuild);
	OnMeshLayersPropertiesPanelRequestRebuild OnMeshLayersPanelRequestRebuild;

private:

	IModelingToolExternalDynamicMeshUpdateAPI* Tool = nullptr;

	int32 NumLockedBaseLayers = 0;

	// Mesh before layer change has been applied. Used for tracking mesh changes that occur over multiple frames (i.e., from an interactive drag)
	// TODO: Add/update a mesh FChange to track sculpt layer changes, and use that instead of saving an entire mesh here
	TSharedPtr<FDynamicMesh3> InitialMesh;

	// Helper to set sculpt layers from the current LayerWeights property (accounting for the NumLockedBaseLayers)
	void SetLayerWeights(FDynamicMeshSculptLayers* SculptLayers) const;

	// Update the ActiveLayer and LayerWeights settings from the current sculpt layers
	void UpdateSettingsFromMesh(const FDynamicMeshSculptLayers* SculptLayers);

	// Helper to apply edits to the current sculpt layers if possible, with associated book keeping
	// @param EditFn The edit to apply if possible
	// @param bEmitChange Whether to emit a change object along with the edit
	void EditSculptLayers(TFunctionRef<void(UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)> EditFn, bool bEmitChange);

	// Helper to read current sculpt layers if possible
	bool ReadSculptLayers(TFunctionRef<void(const UE::Geometry::FDynamicMesh3& Mesh, const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers)> ReadFn);
};

DECLARE_LOG_CATEGORY_EXTERN(LogSculptLayerProps, Warning, All);