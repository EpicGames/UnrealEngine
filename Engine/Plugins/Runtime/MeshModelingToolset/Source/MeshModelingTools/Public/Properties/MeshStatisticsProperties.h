// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "GeometryBase.h"
#include "MeshStatisticsProperties.generated.h"

#define UE_API MESHMODELINGTOOLS_API


// predeclarations
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);


UCLASS(MinimalAPI)
class UMeshStatisticsProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(VisibleAnywhere, Category = MeshStatistics, meta = (NoResetToDefault))
	FString Mesh;

	UPROPERTY(VisibleAnywhere, Category = MeshStatistics, AdvancedDisplay, meta = (NoResetToDefault))
	FString UV;

	UPROPERTY(VisibleAnywhere, Category = MeshStatistics, AdvancedDisplay, meta = (NoResetToDefault))
	FString Attributes;

	UE_API void Update(const FDynamicMesh3& Mesh);
};

#undef UE_API
