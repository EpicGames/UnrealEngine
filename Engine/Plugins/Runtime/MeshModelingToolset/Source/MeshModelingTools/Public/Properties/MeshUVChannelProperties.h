// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "GeometryBase.h"
#include "MeshUVChannelProperties.generated.h"

#define UE_API MESHMODELINGTOOLS_API

struct FMeshDescription;
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);


UCLASS(MinimalAPI)
class UMeshUVChannelProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Select UV channel in the mesh */
	UPROPERTY(EditAnywhere, Category = "UV Channel", meta = (DisplayName = "UV Channel", GetOptions = GetUVChannelNamesFunc, NoResetToDefault))
	FString UVChannel;

	UFUNCTION()
	UE_API const TArray<FString>& GetUVChannelNamesFunc() const;

	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> UVChannelNamesList;

	UE_API void Initialize(int32 NumUVChannels, bool bInitializeSelection = true);
	UE_API void Initialize(const FMeshDescription* MeshDescription, bool bInitializeSelection = true);
	UE_API void Initialize(const FDynamicMesh3* Mesh, bool bInitializeSelection = true);

	/**
	 * Verify that the UV channel selection is valid
	 * @param bUpdateIfInvalid if selection is not valid, reset UVChannel to UV0 or empty if no UV channels exist
	 * @return true if selection in UVChannel is an entry in UVChannelNamesList.
	 */
	UE_API bool ValidateSelection(bool bUpdateIfInvalid);

	/**
	 * @param bForceToZeroOnFailure if true, then instead of returning -1 we return 0 so calling code can fallback to default UV paths
	 * @return selected UV channel index, or -1 if invalid selection, or 0 if invalid selection and bool bForceToZeroOnFailure = true
	 */
	UE_API int32 GetSelectedChannelIndex(bool bForceToZeroOnFailure = false);
};

#undef UE_API
