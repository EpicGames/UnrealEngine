// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "GeometryBase.h"
#include "MeshWeightChannelProperties.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);


UCLASS(MinimalAPI)
class UMeshWeightChannelDensityProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Whether to use a vertex weight map layer to control relative mesh density */
	UPROPERTY(EditAnywhere, Category = WeightMap)
	bool bUseWeightMap;

	/** Name of the vertex weight map layer that will be used to adjust relative mesh density */
	UPROPERTY(EditAnywhere, Category = WeightMap, meta = (DisplayName = "Attribute", GetOptions = GetAttributeNames, EditCondition = bUseWeightMap))
	FName SelectedAttribute;

	UFUNCTION()
	const TArray<FName>& GetAttributeNames() { return AttributeNames; };

	TArray<FName> AttributeNames;

	/** Amount to adjust density in regions where the vertex weight map is greater than zero. A positive value here results in a denser mesh, a negative value results in a coarser one */
	UPROPERTY(EditAnywhere, Category = WeightMap, meta = (EditCondition = bUseWeightMap, UIMin = "-2.0", UIMax = "2.0"))
	float RelativeDensity = 1.f;

	/**
	 * Set up the list of available attributes
	 */
	UE_API void InitializeAttributes(const UE::Geometry::FDynamicMesh3* const Mesh);

	/**
	 * If the attribute selection is not valid, use the attribute at index 0 or empty if there are no attributes
	 */
	UE_API void ValidateSelectedAttribute();

	/** @return the selected weight attribute on the given dynamic mesh or INDEX_NONE if not found */
	UE_API int32 GetSelectedWeightAttributeIndex(const UE::Geometry::FDynamicMesh3* const Mesh) const;
};


#undef UE_API
