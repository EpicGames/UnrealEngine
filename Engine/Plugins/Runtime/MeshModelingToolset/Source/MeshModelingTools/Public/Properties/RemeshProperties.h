// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveTool.h"
#include "MeshConstraints.h"

#include "RemeshProperties.generated.h"

using UE::Geometry::EEdgeRefineFlags;

/** Mesh Boundary Constraint Types */
UENUM()
enum class EMeshBoundaryConstraint : uint8
{
	Fixed  = (uint8)EEdgeRefineFlags::FullyConstrained  UMETA(DisplayName = "Fixed"),
	Refine = (uint8)EEdgeRefineFlags::SplitsOnly UMETA(DisplayName = "Refine"),
	Free   = (uint8)EEdgeRefineFlags::NoFlip   UMETA(DisplayName = "Free")
};

/** Group Boundary Constraint Types */
UENUM()
enum class EGroupBoundaryConstraint : uint8
{
	Fixed  = (uint8)EEdgeRefineFlags::FullyConstrained  UMETA(DisplayName = "Fixed"),
	Refine = (uint8)EEdgeRefineFlags::SplitsOnly UMETA(DisplayName = "Refine"),
	Free   = (uint8)EEdgeRefineFlags::NoFlip   UMETA(DisplayName = "Free"),
	Ignore = (uint8)EEdgeRefineFlags::NoConstraint UMETA(DisplayName = "Ignore")
};

/** Material Boundary Constraint Types */
UENUM()
enum class EMaterialBoundaryConstraint : uint8
{
	Fixed  = (uint8)EEdgeRefineFlags::FullyConstrained  UMETA(DisplayName = "Fixed"),
	Refine = (uint8)EEdgeRefineFlags::SplitsOnly UMETA(DisplayName = "Refine"),
	Free   = (uint8)EEdgeRefineFlags::NoFlip   UMETA(DisplayName = "Free"),
	Ignore = (uint8)EEdgeRefineFlags::NoConstraint UMETA(DisplayName = "Ignore")
};

UCLASS(MinimalAPI)
class UMeshConstraintProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** If true, sharp edges are preserved  */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Edge Constraints")
	bool bPreserveSharpEdges;

	/** Mesh Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Boundary Constraints", meta = (DisplayName = "Mesh Boundary"))
	EMeshBoundaryConstraint MeshBoundaryConstraint = EMeshBoundaryConstraint::Free;

	/** Group Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Boundary Constraints", meta = (DisplayName = "Group Boundary"))
	EGroupBoundaryConstraint GroupBoundaryConstraint = EGroupBoundaryConstraint::Free;

	/** Material Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Boundary Constraints", meta = (DisplayName = "Material Boundary"))
	EMaterialBoundaryConstraint MaterialBoundaryConstraint = EMaterialBoundaryConstraint::Free;

	/** Prevent normal flips */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Edge Constraints", meta=(EditCondition = "IsPreventNormalFlipsEnabled") )
	bool bPreventNormalFlips = false;

	/** Prevent introduction of tiny triangles or slivers */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Edge Constraints", meta=(EditCondition = "IsPreventTinyTrianglesEnabled") )
	bool bPreventTinyTriangles = false;

protected:

	UFUNCTION()
	virtual bool IsPreventNormalFlipsEnabled() const { return true; }

	UFUNCTION()
	virtual bool IsPreventTinyTrianglesEnabled() const { return true; }
};

UCLASS(MinimalAPI)
class URemeshProperties : public UMeshConstraintProperties
{
	GENERATED_BODY()

public:
	/** Amount of Vertex Smoothing applied within Remeshing */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (DisplayName = "Smoothing Rate", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float SmoothingStrength = 0.1f;

	/** Enable edge flips in Remeshing (tends to lose edges/detail) */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Edge Constraints", meta = (DisplayName = "Allow Flips"))
	bool bFlips = false;

	/** Enable edge splits in Remeshing */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Edge Constraints", meta = (DisplayName = "Allow Splits"))
	bool bSplits = true;

	/** Enable edge collapses in Remeshing */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Edge Constraints", meta = (DisplayName = "Allow Collapses"))
	bool bCollapses = true;

};
