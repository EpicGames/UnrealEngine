// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h" //UInteractiveToolPropertySet
#include "VectorTypes.h"

#include "RevolveProperties.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UNewMeshMaterialProperties;
PREDECLARE_GEOMETRY(class FCurveSweepOp);

UENUM()
enum class ERevolvePropertiesCapFillMode : uint8
{
	/** No caps will be generated. */
	None,
	/** Caps are triangulated by placing a vertex in the center and creating a fan to the boundary. This works well if the path is convex,
	  * but can create invalid geometry if it is concave. */
	CenterFan,
	/** Caps are triangulated to maximize the minimal angle in the triangles using Delaunay triangulation. */
	Delaunay,
	/** Caps are triangulated using a standard ear clipping approach. This could result in some very thin triangles. */
	EarClipping
};


UENUM()
enum class ERevolvePropertiesPolygroupMode : uint8
{
	/** One PolyGroup for the entire shape */
	PerShape,
	/** One PolyGroup for each geometric face */
	PerFace,
	/** One PolyGroup along the path for each revolution step */
	PerRevolveStep,
	/** One PolyGroup along the revolution steps for each path segment */
	PerPathSegment
};

UENUM()
enum class ERevolvePropertiesQuadSplit : uint8
{
	/** Quads will always be split into triangles the same way regardless of quad shape. */
	Uniform,
	
	/** Quads will be split into triangles by connecting the shortest diagonal. */
	Compact
};

/**
 * Common properties for revolving a PolyPath to create a mesh.
 */
UCLASS(MinimalAPI)
class URevolveProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	URevolveProperties()
	{
		// We want the revolution degrees to be clamped to 360 when there is no offset along axis, and extendable
		// beyond 360 when there is one (to make springs, etc). There's not currently a way to have a conditional
		// clamp value, so instead, we swap the variables that we make visible to the user (RevolveDegrees and
		// RevolveDegreesClamped). However, we have to do some work to keep the values consistent so that they
		// do not jump when they are swapped.
		ClampedRevolutionDegreesWatcherIndex = WatchProperty(RevolveDegreesClamped,
			[this](const double& NewClamped) 
			{ 
				RevolveDegrees = NewClamped; 
				SilentUpdateWatcherAtIndex(RevolutionDegreesWatcherIndex); //avoid triggering that watcher
			});

		RevolutionDegreesWatcherIndex = WatchProperty(RevolveDegrees,
			[this](const double& NewNonClamped)
			{
				RevolveDegreesClamped = FMath::Min(RevolveDegrees, 360.0);
				SilentUpdateWatcherAtIndex(ClampedRevolutionDegreesWatcherIndex); //avoid triggering that watcher
			});
	}

	/** Revolution extent in degrees. Clamped to a maximum of 360 if Height Offset Per Degree is set to 0. */
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (DisplayName = "Degrees", UIMin = "0", UIMax = "360", ClampMin = "0", ClampMax = "360", 
		EditCondition = "HeightOffsetPerDegree == 0", EditConditionHides))
	double RevolveDegreesClamped = 360;

	/** Revolution extent in degrees. Clamped to a maximum of 360 if Height Offset Per Degree is set to 0. */
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (DisplayName = "Degrees", UIMin = "0", UIMax = "3600", ClampMin = "0", ClampMax = "360000",
		EditCondition = "HeightOffsetPerDegree != 0", EditConditionHides))
	double RevolveDegrees = 360;

	/** The angle by which to rotate the path around the axis before beginning the revolve. */
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (DisplayName = "Degrees Offset", UIMin = "-360", UIMax = "360", ClampMin = "-36000", ClampMax = "36000"))
	double RevolveDegreesOffset = 0;

	/** Implicitly defines the number of steps in the revolution such that each step moves the revolution no more than the given number of degrees. This is only available if Explicit Steps is disabled. */
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (UIMin = "1", ClampMin = "1", UIMax = "120", ClampMax = "180", EditCondition = "!bExplicitSteps"))
	double StepsMaxDegrees = 15;

	/** If true, the number of steps can be specified explicitly via Steps. If false, the number of steps is adjusted automatically based on Steps Max Degrees. */
	UPROPERTY(EditAnywhere, Category = Revolve)
	bool bExplicitSteps = false;

	/** Number of steps in the revolution. This is only available if Explicit Steps is enabled. */
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (DisplayName = "Steps", UIMin = "1", ClampMin = "1", UIMax = "100", ClampMax = "5000",
		EditCondition = "bExplicitSteps"))
	int NumExplicitSteps = 24;

	/** How far to move each step along the revolution axis per degree. Non-zero values are useful for creating spirals. */
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (DisplayName = "Height Offset", UIMin = "-10", UIMax = "10", ClampMin = "-100000", ClampMax = "100000"))
	double HeightOffsetPerDegree = 0;

	/** By default, revolution is done counterclockwise if looking down the revolution axis. This reverses the revolution direction to clockwise.*/
	UPROPERTY(EditAnywhere, Category = Revolve, meta = (DisplayName = "Reverse"))
	bool bReverseRevolutionDirection = false;

	/** Flips the mesh inside out. */
	UPROPERTY(EditAnywhere, Category = Revolve)
	bool bFlipMesh = false;

	/** If true, normals are not averaged or shared between triangles beyond the Sharp Normals Degree Threshold. */
	UPROPERTY(EditAnywhere, Category = Revolve)
	bool bSharpNormals = false;

	/** The threshold in degrees beyond which normals are not averaged or shared between triangles anymore. This is only available if Sharp Normals is enabled. */
	UPROPERTY(EditAnywhere, Category = Revolve,	meta = (DisplayName = "Sharp Normals Threshold", ClampMin = "0.0", ClampMax = "90.0",
		EditCondition = "bSharpNormals"))
	double SharpNormalsDegreeThreshold = 0.1;

	/** If true, the path is placed at the midpoint of each step instead of at the start and/or end of a step. For example, this is useful for creating square columns. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (DisplayName = "Path at Midpoint"))
	bool bPathAtMidpointOfStep = false;

	/** How PolyGroups are assigned to shape primitives. If caps are generated, they will always be in separate groups. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (DisplayName = "PolyGroup Mode"))
	ERevolvePropertiesPolygroupMode PolygroupMode = ERevolvePropertiesPolygroupMode::PerFace;

	/** How generated quads are split into triangles. */
	UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay)
	ERevolvePropertiesQuadSplit QuadSplitMode = ERevolvePropertiesQuadSplit::Compact;

	/**
	 * For quad split mode Compact, this biases the length comparison to prefer one diagonal over the other.
	 * For example, a value of 0.01 allows a 1% difference in lengths before the triangulation is flipped. This helps symmetric quads to
	 * be triangulated more uniformly.
	 */
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (DisplayName = "Quad Split Tolerance", ClampMin = "0.0",  ClampMax = "2.0",
	// 	EditCondition = "QuadSplitMode == ERevolvePropertiesQuadSplit::Compact"))
	double QuadSplitCompactTolerance = 0.01;

	// /** Determines how end caps are created. This is not relevant if the end caps are not visible or if the path is not closed. */
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (EditCondition = "HeightOffsetPerDegree != 0 || RevolveDegrees != 360"))
	// ERevolvePropertiesCapFillMode CapFillMode = ERevolvePropertiesCapFillMode::Delaunay;

	/** If true, the ends of a fully revolved path are welded together, rather than duplicating vertices at the seam.
	 * This is not relevant if the revolution is not full and/or there is a height offset. */
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (EditCondition = "HeightOffsetPerDegree == 0 && RevolveDegrees == 360"))
	bool bWeldFullRevolution = true;

	/** If true, vertices close to the axis will not be replicated, instead reusing the same vertex for any adjacent triangles. */
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay)
	bool bWeldVertsOnAxis = true;

	/** If welding vertices on the axis, the distance that a vertex can be from the axis and still be welded. */
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay, meta = (ClampMin = "0.0", ClampMax = "20.0", EditCondition = "bWeldVertsOnAxis"))
	double AxisWeldTolerance = 0.1;

	/** If true, UV coordinates will be flipped in the V direction. */
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay)
	bool bFlipVs = false;

	/* If true, UV layout is not affected by segments of the path that do not result in any triangles. For example, when both ends of
	 * a segment are welded due to being on the revolution axis.*/
	// UPROPERTY(EditAnywhere, Category = Revolve, AdvancedDisplay)
	bool bUVsSkipFullyWeldedEdges = true;


	/**
	 * Sets most of the settings for a FCurveSweepOp except for the profile curve itself. Should be called
	 * after setting the profile curve, as the function reverses it if necessary.
	 *
	 * CurveSweepOpOut.ProfileCurve and CurveSweepOpOut.bProfileCurveIsClosed must be initialized in advance.
	 */
	UE_API void ApplyToCurveSweepOp(const UNewMeshMaterialProperties& MaterialProperties,
		const FVector3d& RevolutionAxisOrigin, const FVector3d& RevolutionAxisDirection,
		UE::Geometry::FCurveSweepOp& CurveSweepOpOut) const;

	protected:
		virtual ERevolvePropertiesCapFillMode GetCapFillMode() const
		{
			return ERevolvePropertiesCapFillMode::Delaunay;
		}
	
		int32 ClampedRevolutionDegreesWatcherIndex;
		int32 RevolutionDegreesWatcherIndex;
};

#undef UE_API
