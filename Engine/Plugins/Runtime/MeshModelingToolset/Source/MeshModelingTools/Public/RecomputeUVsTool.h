// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshOpPreviewHelpers.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/MeshUVChannelProperties.h"
#include "Properties/RecomputeUVsProperties.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Polygroups/PolygroupSet.h"
#include "Drawing/UVLayoutPreview.h"

#include "RecomputeUVsTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API


// Forward declarations
class UDynamicMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class RecomputeUVsOp;
class URecomputeUVsOpFactory;


/**
 *
 */
UCLASS(MinimalAPI)
class URecomputeUVsToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * URecomputeUVsTool Recomputes UVs based on existing segmentations of the mesh
 */
UCLASS(MinimalAPI)
class URecomputeUVsTool : public USingleSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:
	UPROPERTY()
	TObjectPtr<UMeshUVChannelProperties> UVChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<URecomputeUVsToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UExistingMeshMaterialProperties> MaterialSettings = nullptr;

	UPROPERTY()
	bool bCreateUVLayoutViewOnSetup = true;

	UPROPERTY()
	TObjectPtr<UUVLayoutPreview> UVLayoutView = nullptr;

	UPROPERTY()
	TObjectPtr<URecomputeUVsOpFactory> RecomputeUVsOpFactory;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;

	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> ActiveGroupSet;
	UE_API void OnSelectedGroupLayerChanged();
	UE_API void UpdateActiveGroupLayer();
	UE_API int32 GetSelectedUVChannel() const;
	UE_API void OnPreviewMeshUpdated();
};

#undef UE_API
