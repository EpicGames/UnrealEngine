// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "CleaningOps/RemeshMeshOp.h"
#include "Properties/RemeshProperties.h"
#include "Properties/MeshWeightChannelProperties.h"
#include "RemeshMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UMeshStatisticsProperties;
class UMeshElementsVisualizer;
PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3);

/**
 *
 */
UCLASS(MinimalAPI)
class URemeshMeshToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	/** 
	 * Return true if we have one object selected. URemeshMeshTool is a UMultiSelectionTool, however we currently 
	 * only ever apply it to a single mesh. (See comment at URemeshMeshTool definition below.)
	 */
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * Standard properties of the Remesh operation
 */
UCLASS(MinimalAPI)
class URemeshMeshToolProperties : public URemeshProperties
{
	GENERATED_BODY()

public:
	UE_API URemeshMeshToolProperties();

	/** Target triangle count */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (ClampMin = 0, EditCondition = "bUseTargetEdgeLength == false"))
	int TargetTriangleCount;

	/** Smoothing type */
	UPROPERTY(EditAnywhere, Category = Remeshing)
	ERemeshSmoothingType SmoothingType;

	/** Discard UVs and existing normals, allowing the remesher to ignore any UV and normal seams. New per-vertex normals are computed. */
	UPROPERTY(EditAnywhere, Category = Remeshing)
	bool bDiscardAttributes;

	/** Display colors corresponding to the mesh's polygon groups */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowGroupColors = false;

	/** Remeshing type */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay)
	ERemeshType RemeshType;

	/** Number of Remeshing passes */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay, meta = (EditCondition = "RemeshType == ERemeshType::FullPass", UIMin = "0", UIMax = "50", ClampMin = "0", ClampMax = "1000"))
	int RemeshIterations;

	/** Maximum number of Remeshing passes, for Remeshers that have convergence criteria */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay, meta = (EditCondition = "RemeshType != ERemeshType::FullPass", UIMin = "0", UIMax = "200", ClampMin = "0", ClampMax = "200"))
	int MaxRemeshIterations;

	/** For NormalFlowRemesher: extra iterations of normal flow with no remeshing */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay, meta = (EditCondition = "RemeshType == ERemeshType::NormalFlow", UIMin = "0", UIMax = "200", ClampMin = "0", ClampMax = "200"))
	int ExtraProjectionIterations;

	/** If true, the target count is ignored and the target edge length is used directly */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay)
	bool bUseTargetEdgeLength;

	/** Remesh to have edges approximately this length. An attempt at a reasonable value is computed automatically for this field based on the selected target mesh. */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay, meta = (NoSpinbox = "true", EditCondition = "bUseTargetEdgeLength == true", NoResetToDefault))
	float TargetEdgeLength;

	/** Enable projection back to input mesh */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay)
	bool bReproject;

	/** Project constrained vertices back to original constraint curves */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay)
	bool bReprojectConstraints;

	/** Angle threshold in degrees for classifying a boundary vertex as a corner. Corners will be fixed if Reproject Constraints is active. */
	UPROPERTY(EditAnywhere, Category = Remeshing, AdvancedDisplay, meta = (EditCondition = "bReprojectConstraints", UIMin = "0", UIMax = "180", ClampMin = "0", ClampMax = "180"))
	float BoundaryCornerAngleThreshold;

};


/**
 * Simple Mesh Remeshing Tool
 *
 * Note this is a subclass of UMultiSelectionTool, however we currently only ever apply it to one mesh at a time. The
 * function URemeshMeshToolBuilder::CanBuildTool will return true only when a single mesh is selected, and the tool will
 * only be applied to the first mesh in the selection list. The reason we inherit from UMultiSelectionTool is so 
 * that subclasses of this class can work with multiple meshes (see, for example, UProjectToTargetTool.)
 */
UCLASS(MinimalAPI)
class URemeshMeshTool : public UMultiSelectionMeshEditingTool, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:

	UE_API URemeshMeshTool(const FObjectInitializer&);

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<URemeshMeshToolProperties> BasicProperties;

	UPROPERTY()
	TObjectPtr<UMeshWeightChannelDensityProperties> WeightMapProperties;

	UPROPERTY()
	TObjectPtr<UMeshStatisticsProperties> MeshStatisticsProperties;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay;

protected:
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial;
	double InitialMeshArea;

	UE_API void UpdateVisualization();
};

#undef UE_API
