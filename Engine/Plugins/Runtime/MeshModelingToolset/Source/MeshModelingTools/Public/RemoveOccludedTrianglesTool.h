// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "MeshOpPreviewHelpers.h"
#include "CleaningOps/RemoveOccludedTrianglesOp.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshAdapter.h"
#include "BaseTools/SingleClickTool.h"
#include "PropertySets/PolygroupLayersProperties.h"

#include "RemoveOccludedTrianglesTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

class URemoveOccludedTrianglesTool;

/**
 *
 */
UCLASS(MinimalAPI)
class URemoveOccludedTrianglesToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};


// these UIMode enums are versions of the enums in Operations/RemoveOccludedTriangles.h, w/ some removed & some renamed to be more user friendly

UENUM()
enum class EOcclusionTriangleSamplingUIMode : uint8
{
	/** Test for occlusion at vertices */
	Vertices,
	/** Test for occlusion at vertices and triangle centroids */
	VerticesAndCentroids
	//~ currently do not expose centroid-only option; it almost always looks bad
};

UENUM()
enum class EOcclusionCalculationUIMode : uint8
{
	//~ GeneralizedWindingNumber maps to using fast winding number approximation
	/** Test for occlusion by a 3D 'Winding Number' test (Note: Allows internal 'air pockets' to be considered 'not occluded') */
	GeneralizedWindingNumber,
	/** Test for occlusion by casting rays against the mesh */
	RaycastOcclusionSamples
};

UENUM()
enum class EOccludedAction : uint8
{
	Remove,
	SetNewGroup
};

/**
 * Standard properties
 */
UCLASS(MinimalAPI)
class URemoveOccludedTrianglesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API URemoveOccludedTrianglesToolProperties();

	/** The method for deciding whether a triangle is occluded */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation)
	EOcclusionCalculationUIMode OcclusionTestMethod = EOcclusionCalculationUIMode::GeneralizedWindingNumber;

	/** Where to sample triangles to test occlusion */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation)
	EOcclusionTriangleSamplingUIMode TriangleSampling = EOcclusionTriangleSamplingUIMode::VerticesAndCentroids;

	/** The winding isovalue for GeneralizedWindingNumber mode */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation, meta = (UIMin = "-1", UIMax = "1", ClampMin = "-2", ClampMax = "2", EditCondition = "OcclusionTestMethod==EOcclusionCalculationUIMode::GeneralizedWindingNumber"))
	double WindingIsoValue = 0.5;

	/** For raycast-based occlusion tests, optionally add random ray direction to increase the accuracy of the visibility sampling */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", EditCondition = "OcclusionTestMethod==EOcclusionCalculationUIMode::RaycastOcclusionSamples"))
	int AddRandomRays = 0;

	/** Optionally add random samples to each triangle (in addition to those from TriangleSampling) to increase the accuracy of the visibility sampling */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	int AddTriangleSamples = 0;

	/** If false, when multiple meshes are selected the meshes can occlude each other.  When true, we process each selected mesh independently and only consider self-occlusions. */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation)
	bool bOnlySelfOcclude = false;

	/** Shrink (erode) the boundary of the set of triangles to remove. */
	UPROPERTY(EditAnywhere, Category = OcclusionCalculation, meta = (UIMin = "0", ClampMin = "0", Delta = 1, LinearDeltaSensitivity = 1))
	int ShrinkRemoval = 0;

	UPROPERTY(EditAnywhere, Category = RemoveIslands, meta = (UIMin = "0", ClampMin = "0", Delta = 0.5, LinearDeltaSensitivity = 1))
	double MinAreaIsland = 0;

	UPROPERTY(EditAnywhere, Category = RemoveIslands, meta = (UIMin = "0", ClampMin = "0", Delta = 1, LinearDeltaSensitivity = 1))
	int MinTriCountIsland = 0;

	/** What action to perform on occluded triangles */
	UPROPERTY(EditAnywhere, Category = Action)
	EOccludedAction Action = EOccludedAction::Remove;
};




/**
 * Advanced properties
 */
UCLASS(MinimalAPI)
class URemoveOccludedTrianglesAdvancedProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API URemoveOccludedTrianglesAdvancedProperties();

	/** Amount to numerically 'nudge' occlusion sample query points away from the surface (to avoid e.g. all occlusion sample rays hitting the source triangle) */
	// probably not actually a good idea to expose this to the user
	//UPROPERTY(EditAnywhere, Category = NormalsTopology, meta = (UIMin = "0.000000001", UIMax = ".0001", ClampMin = "0.0", ClampMax = "0.01"))
	double NormalOffset = FMathd::ZeroTolerance;
};


/**
 * Factory with enough info to spawn the background-thread Operator to do a chunk of work for the tool
 *  stores a pointer to the tool and enough info to know which specific operator it should spawn
 */
UCLASS(MinimalAPI)
class URemoveOccludedTrianglesOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<URemoveOccludedTrianglesTool> Tool;

	int PreviewIdx;
};


/**
 * Simple Mesh Normal Updating Tool
 */
UCLASS(MinimalAPI)
class URemoveOccludedTrianglesTool : public UMultiSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	friend URemoveOccludedTrianglesOperatorFactory;

	UE_API URemoveOccludedTrianglesTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent) override;
#endif

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


protected:
	UPROPERTY()
	TObjectPtr<URemoveOccludedTrianglesToolProperties> BasicProperties;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayersProperties;

	UPROPERTY()
	TObjectPtr<URemoveOccludedTrianglesAdvancedProperties> AdvancedProperties;

	UPROPERTY()
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> Previews;

	// When multiple meshes in the selection correspond to the same asset, only one needs a PreviewWithBackgroundCompute
	//  all others just get a plain PreviewMesh copy that is updated via OnMeshUpdated broadcast from the source Preview
	UPROPERTY()
	TArray<TObjectPtr<UPreviewMesh>> PreviewCopies;


protected:
	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> OriginalDynamicMeshes;

	// AABB trees and winding trees for every mesh target, with repeated instances as pointers to the same data
	TArray<TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe>> OccluderTrees;
	TArray<TSharedPtr<UE::Geometry::TFastWindingTree<FDynamicMesh3>, ESPMode::ThreadSafe>> OccluderWindings;
	TArray<UE::Geometry::FTransformSRT3d> OccluderTransforms;

	TArray<TArray<int32>> PreviewToCopyIdx;
	TArray<int32> PreviewToTargetIdx;
	TArray<int32> TargetToPreviewIdx;

	// Group IDs for occluded triangles, per mesh in the Previews array, if OccludedAction is SetNewGroup
	TArray<int32> OccludedGroupIDs;
	// Selected layer indices for Group IDs, per mesh in the Previews array
	TArray<int32> OccludedGroupLayers;

	FViewCameraState CameraState;

	UE_API void SetupPreviews();
	UE_API void MakePolygroupLayerProperties();

	UE_API void GenerateAsset(const TArray<FDynamicMeshOpResult>& Results);
};

#undef UE_API
