// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MatrixTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Deformers/Kelvinlets.h"
#include "Sculpting/MeshBrushOpBase.h"
#include "Sculpting/MeshSculptToolBase.h"
#include "Async/ParallelFor.h"
#include "KelvinletBrushOp.generated.h"

// import kelvinlet types
using UE::Geometry::FPullKelvinlet;
using UE::Geometry::FLaplacianPullKelvinlet;
using UE::Geometry::FBiLaplacianPullKelvinlet;
using UE::Geometry::FSharpLaplacianPullKelvinlet;
using UE::Geometry::FSharpBiLaplacianPullKelvinlet;
using UE::Geometry::FAffineKelvinlet;
using UE::Geometry::FTwistKelvinlet;
using UE::Geometry::FPinchKelvinlet;
using UE::Geometry::FScaleKelvinlet;
using UE::Geometry::FBlendPullSharpKelvinlet;
using UE::Geometry::FBlendPullKelvinlet;
using UE::Geometry::FLaplacianTwistPullKelvinlet;
using UE::Geometry::FBiLaplacianTwistPullKelvinlet;

UCLASS(MinimalAPI)
class UBaseKelvinletBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()

public:

	/** How much the mesh resists shear */
	UPROPERTY()
	float Stiffness = 1.f;

	/** How compressible the spatial region is: 1 - 2 x Poisson ratio */
	UPROPERTY()
	float Incompressiblity = 1.f;

	/** Integration steps*/
	UPROPERTY()
	int32 BrushSteps = 3;

	virtual float GetStiffness() { return Stiffness; }
	virtual float GetIncompressiblity() { return Incompressiblity; }
	virtual int32 GetNumSteps() { return BrushSteps; }

};





class FBaseKelvinletBrushOp : public FMeshSculptBrushOp
{
public:
	
	void SetBaseProperties(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp)
	{
		// pointer to mesh
		Mesh = SrcMesh;
		UBaseKelvinletBrushOpProps* BaseKelvinletBrushOpProps = GetPropertySetAs<UBaseKelvinletBrushOpProps>();

		float Stiffness = BaseKelvinletBrushOpProps->GetStiffness();
		float Incompressibility = BaseKelvinletBrushOpProps->GetIncompressiblity();
		int NumIntSteps = BaseKelvinletBrushOpProps->GetNumSteps();

		Mu = FMath::Max(Stiffness, 0.f);
		Nu = FMath::Clamp(0.5f * (1.f - 2.f * Incompressibility), 0.f, 0.5f);
		
		Size = FMath::Max(Stamp.Radius, 0.01);

		NumSteps = NumIntSteps;
	}


	template <typename KelvinletType>
	void ApplyKelvinlet(const KelvinletType& Kelvinlet, const FFrame3d& LocalFrame, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) const 
	{

		if (NumSteps == 0)
		{
			DisplaceKelvinlet(Kelvinlet, LocalFrame, Vertices, NewPositionsOut);
		}
		else
		{
			IntegrateKelvinlet(Kelvinlet, LocalFrame, Vertices, NewPositionsOut, IntegrationTime, NumSteps);
		}
	}



	virtual void ApplyStamp(const FDynamicMesh3*SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override {}

protected:

	void ApplyFalloff(const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) const
	{
		int32 NumV = Vertices.Num();
		bool bParallel = true;
		ParallelFor(NumV, [&Stamp, &Vertices, &NewPositionsOut, this](int32 k)
			{
				int32 VertIdx = Vertices[k];
				FVector3d OrigPos = Mesh->GetVertex(VertIdx);

				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);

				double Alpha = FMath::Clamp(Falloff, 0., 1.);

				NewPositionsOut[k] = Alpha * NewPositionsOut[k] + (1. - Alpha) * OrigPos;
			}, bParallel);
	}


	// NB: this just moves the verts, but doesn't update the normal.  The kelvinlets will have to be extended if we want
	// to do the Jacobian Transpose operation on the normals - but for now, we should just rebuild the normals after the brush
	template <typename KelvinletType>
	void DisplaceKelvinlet(const KelvinletType& Kelvinlet, const FFrame3d& LocalFrame, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) const
	{
		int32 NumV = Vertices.Num();
		NewPositionsOut.SetNum(NumV, EAllowShrinking::No);

		const bool bForceSingleThread = false;

		ParallelFor(NumV, [&Kelvinlet, &Vertices, &NewPositionsOut, &LocalFrame, this](int32 k)
			{
				int VertIdx = Vertices[k];
				FVector3d Pos = Mesh->GetVertex(VertIdx);
				Pos = LocalFrame.ToFramePoint(Pos);

				Pos = Kelvinlet.Evaluate(Pos) + Pos;

				// Update the position in the ROI Array
				NewPositionsOut[k] = LocalFrame.FromFramePoint(Pos);
			}
		, bForceSingleThread);

	}

	template <typename KelvinletType>
	void IntegrateKelvinlet(const KelvinletType& Kelvinlet, const FFrame3d& LocalFrame, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut, const double Dt, const int32 Steps) const
	{
		int NumV = Vertices.Num();
		NewPositionsOut.SetNum(NumV, EAllowShrinking::No);

		const bool bForceSingleThread = false;

		ParallelFor(NumV, [&Kelvinlet, &Vertices, &NewPositionsOut, &LocalFrame, Dt, Steps, this](int32 k)
			{
				int VertIdx = Vertices[k];
				FVector3d Pos = LocalFrame.ToFramePoint(Mesh->GetVertex(VertIdx));

				double TimeScale = 1. / (Steps);
				// move with several time steps
				for (int i = 0; i < Steps; ++i)
				{
					// the position after deformation
					Pos = Kelvinlet.IntegrateRK3(Pos, Dt * TimeScale);
				}
				// Update the position in the ROI Array
				NewPositionsOut[k] = LocalFrame.FromFramePoint(Pos);
			}, bForceSingleThread);


	}

protected:


	// physical properties.
	float Mu;
	float Nu;
	// model regularization parameter
	float Size;
	
	// integration parameters
	int32 NumSteps;
	double IntegrationTime = 1.;
	
	const FDynamicMesh3* Mesh;
};





UCLASS(MinimalAPI)
class UScaleKelvinletBrushOpProps : public   UBaseKelvinletBrushOpProps
{
	GENERATED_BODY()

public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = KelvinScaleBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "10.", ClampMin = "0.0", ClampMax = "10.", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = KelvinScaleBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;


	virtual float GetStrength() override { return Strength; }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

class FScaleKelvinletBrushOp : public  FBaseKelvinletBrushOp
{

public:
	
	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		SetBaseProperties(SrcMesh, Stamp);

		float Strength = GetPropertySetAs<UScaleKelvinletBrushOpProps>()->GetStrength();

		float Speed = Strength * 0.25 * FMath::Sqrt(Stamp.Radius) * Stamp.Direction;
		FScaleKelvinlet ScaleKelvinlet(Speed, 0.35 * Size, Mu, Nu);

		ApplyKelvinlet(ScaleKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}

};

UCLASS(MinimalAPI)
class UPullKelvinletBrushOpProps : public UBaseKelvinletBrushOpProps
{
	GENERATED_BODY()
public:
	
	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = KelvinGrabBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.1f;

	// these get routed into the Stamp
	virtual float GetFalloff() override { return Falloff; }
};

class FPullKelvinletBrushOp : public  FBaseKelvinletBrushOp
{

public:

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		SetBaseProperties(SrcMesh, Stamp);

		// compute the displacement vector in the local frame of the stamp
		FVector3d Force = Stamp.LocalFrame.ToFrameVector(Stamp.LocalFrame.Origin - Stamp.PrevLocalFrame.Origin);
		Size *= 0.6f;

		FLaplacianPullKelvinlet LaplacianPullKelvinlet(Force, Size, Mu, Nu);
		FBiLaplacianPullKelvinlet BiLaplacianPullKelvinlet(Force, Size, Mu, Nu);

		const double Alpha = Stamp.Falloff;
		// Lerp between a broad and a narrow kelvinlet based on the fall-off 
		FBlendPullKelvinlet BlendPullKelvinlet(BiLaplacianPullKelvinlet, LaplacianPullKelvinlet, Alpha);
		ApplyKelvinlet(BlendPullKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::ActivePlane;
	}

	virtual bool IgnoreZeroMovements() const override
	{
		return true;
	}
};

UCLASS(MinimalAPI)
class USharpPullKelvinletBrushOpProps : public UBaseKelvinletBrushOpProps
{
	GENERATED_BODY()
public:

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = KelvinSharpGrabBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	// these get routed into the Stamp
	virtual float GetFalloff() override { return Falloff; }
};

class FSharpPullKelvinletBrushOp : public  FBaseKelvinletBrushOp
{

public:

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		SetBaseProperties(SrcMesh, Stamp);

		// compute the displacement vector in the local frame of the stamp
		FVector3d Force = Stamp.LocalFrame.ToFrameVector(Stamp.LocalFrame.Origin - Stamp.PrevLocalFrame.Origin);

		FSharpLaplacianPullKelvinlet SharpLaplacianPullKelvinlet(Force, Size, Mu, Nu);
		FSharpBiLaplacianPullKelvinlet SharpBiLaplacianPullKelvinlet(Force, Size, Mu, Nu);

		const double Alpha = Stamp.Falloff;
		// Lerp between a broad and a narrow Kelvinlet based on the fall-off 
		FBlendPullSharpKelvinlet BlendPullSharpKelvinlet(SharpBiLaplacianPullKelvinlet, SharpLaplacianPullKelvinlet, Alpha);

		ApplyKelvinlet(BlendPullSharpKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::ActivePlane;
	}

	virtual bool IgnoreZeroMovements() const override
	{
		return true;
	}

};

class FLaplacianPullKelvinletBrushOp : public FBaseKelvinletBrushOp
{
public:

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		SetBaseProperties(SrcMesh, Stamp);

		// compute the displacement vector in the local frame of the stamp
		FVector3d Force = Stamp.LocalFrame.ToFrameVector(Stamp.LocalFrame.Origin - Stamp.PrevLocalFrame.Origin);

		FLaplacianPullKelvinlet PullKelvinlet(Force, Size, Mu, Nu);
		ApplyKelvinlet(PullKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::ActivePlane;
	}

	virtual bool IgnoreZeroMovements() const override
	{
		return true;
	}
};

class FBiLaplacianPullKelvinletBrushOp : public FBaseKelvinletBrushOp
{
public:

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		SetBaseProperties(SrcMesh, Stamp);

		// compute the displacement vector in the local frame of the stamp
		FVector3d Force = Stamp.LocalFrame.ToFrameVector(Stamp.LocalFrame.Origin - Stamp.PrevLocalFrame.Origin);

		FBiLaplacianPullKelvinlet PullKelvinlet(Force, Size, Mu, Nu);
		ApplyKelvinlet(PullKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::ActivePlane;
	}

	virtual bool IgnoreZeroMovements() const override
	{
		return true;
	}

};

UCLASS(MinimalAPI)
class UTwistKelvinletBrushOpProps : public  UBaseKelvinletBrushOpProps
{
	GENERATED_BODY()

public:
	/** Twisting strength of the Brush */
	UPROPERTY(EditAnywhere, Category = KelvinTwistBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "10.", ClampMin = "0.0", ClampMax = "10.", ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = KelvinTwistBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;


	virtual float GetStrength() override { return Strength; }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

class FTwistKelvinletBrushOp : public FBaseKelvinletBrushOp
{
public:


	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		SetBaseProperties(SrcMesh, Stamp);

		float Strength = GetPropertySetAs<UTwistKelvinletBrushOpProps>()->GetStrength();

		float Speed = Strength * Stamp.Direction * 10;
		
		// In the local frame, twist axis is in "z" direction 
		FVector3d LocalTwistAxis = FVector3d(0., 0., Speed);

		FTwistKelvinlet TwistKelvinlet(LocalTwistAxis, Size, Mu, Nu);
		ApplyKelvinlet(TwistKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::ActivePlane;
	}
};

class FPinchKelvinletBrushOp : public FBaseKelvinletBrushOp
{
public:
	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		// is this really the vector we want?
		// compute the displacement vector in the local frame of the stamp
		FVector3d Dir = Stamp.LocalFrame.ToFrameVector(Stamp.LocalFrame.Origin - Stamp.PrevLocalFrame.Origin);

		FMatrix3d ForceMatrix = UE::Geometry::CrossProductMatrix(Dir);
		// make symmetric
		ForceMatrix.Row0[1] = -ForceMatrix.Row0[1];
		ForceMatrix.Row0[2] = -ForceMatrix.Row0[2];
		ForceMatrix.Row1[2] = -ForceMatrix.Row1[2];
		FPinchKelvinlet PinchKelvinlet(ForceMatrix, Size, Mu, Nu);
		ApplyKelvinlet(PinchKelvinlet, Stamp.LocalFrame, Vertices, NewPositionsOut);

		ApplyFalloff(Stamp, Vertices, NewPositionsOut);
	}
};


/**--------------------- Glue Layer Currently Used in the DynamicMeshSculptTool:   --------------------------
* 
* We should be able to remove these once the dynamic mesh sculpt tool starts using the new brush infrastructure ( used by the MeshVertexSculptTool)
*/

enum class EKelvinletBrushMode
{
	ScaleKelvinlet,
	PinchKelvinlet,
	TwistKelvinlet,
	PullKelvinlet,
	LaplacianPullKelvinlet,
	BiLaplacianPullKelvinlet,
	BiLaplacianTwistPullKelvinlet,
	LaplacianTwistPullKelvinlet,
	SharpPullKelvinlet
};

class FKelvinletBrushOp
{
public:


	struct FKelvinletBrushOpProperties
	{
		FKelvinletBrushOpProperties(const EKelvinletBrushMode& BrushMode, const UKelvinBrushProperties& Properties, double BrushRadius, double BrushFalloff) 
			: Mode(BrushMode)
			, Direction(1.0, 0.0, 0.0)
		{
			Speed = 0.;
			FallOff = BrushFalloff;
			Mu = FMath::Max(Properties.Stiffness, 0.f);
			Nu = FMath::Clamp(0.5f * (1.f - 2.f * Properties.Incompressiblity), 0.f, 0.5f);
			Size = FMath::Max(BrushRadius * Properties.FallOffDistance, 0.f);
			NumSteps = Properties.BrushSteps;
		}


		EKelvinletBrushMode Mode;
		FVector Direction;

		double Speed;   // Optionally used
		double FallOff; // Optionally used
		double Mu; // Shear Modulus
		double Nu; // Poisson ratio

		// regularization parameter
		double Size;

		int NumSteps;

	};

	FKelvinletBrushOp(const FDynamicMesh3& DynamicMesh) :
		Mesh(&DynamicMesh)
	{}

	void ExtractTransform(const FMatrix& WorldToBrush)
	{
		// Extract the parts of the transform and account for the vector * matrix  format of FMatrix by transposing.

		// Transpose of the 3x3 part
		WorldToBrushMat.Row0[0] = WorldToBrush.M[0][0];
		WorldToBrushMat.Row0[1] = WorldToBrush.M[0][1];
		WorldToBrushMat.Row0[2] = WorldToBrush.M[0][2];

		WorldToBrushMat.Row1[0] = WorldToBrush.M[1][0];
		WorldToBrushMat.Row1[1] = WorldToBrush.M[1][1];
		WorldToBrushMat.Row1[2] = WorldToBrush.M[1][2];

		WorldToBrushMat.Row2[0] = WorldToBrush.M[2][0];
		WorldToBrushMat.Row2[1] = WorldToBrush.M[2][1];
		WorldToBrushMat.Row2[2] = WorldToBrush.M[2][2];

		Translation[0] = WorldToBrush.M[3][0];
		Translation[1] = WorldToBrush.M[3][1];
		Translation[2] = WorldToBrush.M[3][2];

		// The matrix should be unitary (det +/- 1) but want this to work with 
		// more general input if needed so just make sure we can invert the matrix
		check(FMath::Abs(WorldToBrush.Determinant()) > 1.e-4);

		BrushToWorldMat = WorldToBrushMat.Inverse();

	};

	~FKelvinletBrushOp() = default;

public:
	const FDynamicMesh3* Mesh;


	double TimeStep = 1.0;
	double NumSteps = 0.0;

	// To be applied as WorlToBrushMat * v + Trans
	// Note: could use FTransform3d in TransformTypes.h
	FMatrix3d WorldToBrushMat;
	FMatrix3d BrushToWorldMat;
	FVector3d Translation;

public:



	void ApplyBrush(const FKelvinletBrushOpProperties& Properties, const FMatrix& WorldToBrush, const TArray<int>& VertRIO, TArray<FVector3d>& ROIPositionBuffer)
	{

		ExtractTransform(WorldToBrush);

		NumSteps = Properties.NumSteps;

		switch (Properties.Mode)
		{

		case EKelvinletBrushMode::ScaleKelvinlet:
		{
			double Scale = Properties.Direction.X;
			FScaleKelvinlet ScaleKelvinlet(Scale, Properties.Size, Properties.Mu, Properties.Nu);
			ApplyKelvinlet(ScaleKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::PullKelvinlet:
		{
			FVector Dir = Properties.Direction;
			FVector3d Force(Dir.X, Dir.Y, Dir.Z);

			FLaplacianPullKelvinlet LaplacianPullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);
			FBiLaplacianPullKelvinlet BiLaplacianPullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);

			const double Alpha = Properties.FallOff;
			// Lerp between a broad and a narrow kelvinlet based on the fall-off 
			FBlendPullKelvinlet BlendPullKelvinlet(BiLaplacianPullKelvinlet, LaplacianPullKelvinlet, Alpha);
			ApplyKelvinlet(BlendPullKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::SharpPullKelvinlet:
		{
			FVector Dir = Properties.Direction;
			FVector3d Force(Dir.X, Dir.Y, Dir.Z);

			FSharpLaplacianPullKelvinlet SharpLaplacianPullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);
			FSharpBiLaplacianPullKelvinlet SharpBiLaplacianPullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);

			const double Alpha = Properties.FallOff;
			// Lerp between a broad and a narrow Kelvinlet based on the fall-off 
			FBlendPullSharpKelvinlet BlendPullSharpKelvinlet(SharpBiLaplacianPullKelvinlet, SharpLaplacianPullKelvinlet, Alpha);
			ApplyKelvinlet(BlendPullSharpKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::LaplacianPullKelvinlet:
		{
			FVector Dir = Properties.Direction;
			FVector3d Force(Dir.X, Dir.Y, Dir.Z);
			FLaplacianPullKelvinlet PullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);
			ApplyKelvinlet(PullKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::BiLaplacianPullKelvinlet:
		{
			FVector Dir = Properties.Direction;
			FVector3d Force(Dir.X, Dir.Y, Dir.Z);
			FBiLaplacianPullKelvinlet PullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);
			ApplyKelvinlet(PullKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::TwistKelvinlet:
		{
			FVector TwistAxis = Properties.Direction;
			FTwistKelvinlet TwistKelvinlet((FVector3d)TwistAxis, Properties.Size, Properties.Mu, Properties.Nu);
			ApplyKelvinlet(TwistKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::PinchKelvinlet:
		{
			FVector Dir = Properties.Direction;

			FMatrix3d ForceMatrix = UE::Geometry::CrossProductMatrix(FVector3d(Dir.X, Dir.Y, Dir.Z));
			// make symmetric
			ForceMatrix.Row0[1] = -ForceMatrix.Row0[1];
			ForceMatrix.Row0[2] = -ForceMatrix.Row0[2];
			ForceMatrix.Row1[2] = -ForceMatrix.Row1[2];
			FPinchKelvinlet PinchKelvinlet(ForceMatrix, Properties.Size, Properties.Mu, Properties.Nu);
			ApplyKelvinlet(PinchKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;
		}
		case EKelvinletBrushMode::LaplacianTwistPullKelvinlet:
		{
			FVector3d TwistAxis = (FVector3d)Properties.Direction;
			UE::Geometry::Normalize(TwistAxis);
			TwistAxis *= Properties.Speed;
			FTwistKelvinlet TwistKelvinlet(TwistAxis, Properties.Size, Properties.Mu, Properties.Nu);

			FVector Dir = Properties.Direction;
			FVector3d Force(Dir.X, Dir.Y, Dir.Z);
			FLaplacianPullKelvinlet PullKelvinlet(Force, Properties.Size, Properties.Mu, Properties.Nu);

			FLaplacianTwistPullKelvinlet TwistPullKelvinlet(TwistKelvinlet, PullKelvinlet, 0.5);
			ApplyKelvinlet(TwistPullKelvinlet, VertRIO, ROIPositionBuffer, TimeStep, NumSteps);
			break;

		}
		default:
			check(0);
		}
	}


	// NB: this just moves the verts, but doesn't update the normal.  The kelvinlets will have to be extended if we want
	// to do the Jacobian Transpose operation on the normals - but for now, we should just rebuild the normals after the brush
	template <typename KelvinletType>
	void DisplaceKelvinlet(const KelvinletType& Kelvinlet, const TArray<int>& VertexROI, TArray<FVector3d>& ROIPositionBuffer)
	{
		int NumV = VertexROI.Num();
		ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

		const bool bForceSingleThread = false;

		ParallelFor(NumV, [&Kelvinlet, &VertexROI, &ROIPositionBuffer, this](int k)

		{
			int VertIdx = VertexROI[k];
			FVector3d Pos = Mesh->GetVertex(VertIdx);
			Pos = XForm(Pos);

			Pos = Kelvinlet.Evaluate(Pos) + Pos;

			// Update the position in the ROI Array
			ROIPositionBuffer[k] = InvXForm(Pos);
		}
		, bForceSingleThread);

	}

	template <typename KelvinletType>
	void IntegrateKelvinlet(const KelvinletType& Kelvinlet, const TArray<int>& VertexROI, TArray<FVector3d>& ROIPositionBuffer, const double Dt, const int Steps)
	{
		int NumV = VertexROI.Num();
		ROIPositionBuffer.SetNum(NumV, EAllowShrinking::No);

		const bool bForceSingleThread = false;

		ParallelFor(NumV, [&Kelvinlet, &VertexROI, &ROIPositionBuffer, Dt, Steps, this](int k)
		{
			int VertIdx = VertexROI[k];
			FVector3d Pos = XForm(Mesh->GetVertex(VertIdx));

			double TimeScale = 1. / (Steps);
			// move with several time steps
			for (int i = 0; i < Steps; ++i)
			{
				// the position after deformation
				Pos = Kelvinlet.IntegrateRK3(Pos, Dt * TimeScale);
			}
			// Update the position in the ROI Array
			ROIPositionBuffer[k] = InvXForm(Pos);
		}, bForceSingleThread);


	}

	template <typename KelvinletType>
	void ApplyKelvinlet(const KelvinletType& Kelvinlet, const TArray<int>& VertexROI, TArray<FVector3d>& ROIPositionBuffer, const double Dt, const int NumIntegrationSteps)
	{

		if (NumIntegrationSteps == 0)
		{
			DisplaceKelvinlet(Kelvinlet, VertexROI, ROIPositionBuffer);
		}
		else
		{
			IntegrateKelvinlet(Kelvinlet, VertexROI, ROIPositionBuffer, Dt, NumIntegrationSteps);
		}
	}

private:

	// apply the transform.
	FVector3d XForm(const FVector3d& Pos) const
	{
		return WorldToBrushMat * Pos + Translation;
	}

	// apply the inverse transform.
	FVector3d InvXForm(const FVector3d& Pos) const
	{
		return BrushToWorldMat * (Pos - Translation);
	}

};
