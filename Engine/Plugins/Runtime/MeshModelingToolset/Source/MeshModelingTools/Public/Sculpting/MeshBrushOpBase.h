// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "FrameTypes.h"
#include "InteractiveTool.h"
#include "MeshBrushOpBase.generated.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
using UE::Geometry::FFrame3d;
using UE::Geometry::FMatrix3d;

enum class ESculptBrushOpTargetType : uint8
{
	// The "live" mesh that is updated by each stamp
	SculptMesh,
	// The "base" mesh that is typically updated after every stroke
	TargetMesh,
	ActivePlane
};


UENUM()
enum class EPlaneBrushSideMode : uint8
{
	BothSides = 0,
	PushDown = 1,
	PullTowards = 2
};


struct FSculptBrushStamp
{
	FFrame3d WorldFrame;
	FFrame3d LocalFrame;
	double Radius;
	double Falloff;
	double Power;
	double Direction;
	double Depth;
	double DeltaTime;

	FFrame3d PrevWorldFrame;
	FFrame3d PrevLocalFrame;

	FDateTime TimeStamp;

	// only initialized if current op requires it
	FFrame3d RegionPlane;

	// stamp alpha
	TFunction<double(const FSculptBrushStamp& Stamp, const FVector3d& Position)> StampAlphaFunc;
	bool HasAlpha() const { return !!StampAlphaFunc; }

	FSculptBrushStamp()
	{
		TimeStamp = FDateTime::Now();
	}
};


struct FSculptBrushOptions
{
	//bool bPreserveUVFlow = false;

	FFrame3d ConstantReferencePlane;
};


class FMeshSculptFallofFunc
{
public:
	TUniqueFunction<double(const FSculptBrushStamp& StampInfo, const FVector3d& Position)> FalloffFunc;

	inline double Evaluate(const FSculptBrushStamp& StampInfo, const FVector3d& Position) const
	{
		return FalloffFunc(StampInfo, Position);
	}
};



UCLASS(MinimalAPI)
class UMeshSculptBrushOpProps : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	virtual float GetStrength() { return 1.0f; }
	virtual float GetDepth() { return 0.0f; }
	virtual float GetFalloff() { return 0.5f; }

	// support for this is optional, used by UI level to edit brush props via hotkeys/etc
	virtual void SetStrength(float NewStrength) { }
	virtual void SetFalloff(float NewFalloff) { }

	// is Pressure Sensitivity for Brush Strength currently toggled on?
	virtual bool GetStrengthPressureEnabled() { return  SupportsStrengthPressure() && bIsStrengthPressureEnabled; }

	// is Pressure Sensitivity for Brush Strength supported?
	virtual bool SupportsStrengthPressure() { return false; }

	/** Should pressure affect brush strength? */
	UPROPERTY(EditAnywhere, Category = BrushOp)
	bool bIsStrengthPressureEnabled = true;
};

/** Mesh Sculpting Brush Stroke Types */
UENUM()
enum class EMeshSculptStrokeType : uint8
{
	// Brushes stamp at regular distance intervals as the stroke is drawn
	Spacing = 0,
	// Brushes stamp at regular time intervals as the stroke is drawn
	Airbrush = 1,
	// Brushes stamp once per frame as the stroke is drawn.
	Dots = 2,

	LastValue UMETA(Hidden)
};

class FMeshSculptBrushOp
{
public:
	// Determines what region of a mesh a brush wants to affect
	enum class EBrushRegionType
	{
		// Affect a sphere around the current brush location
		LocalSphere,
		// Affect a cylinder centered at current brush location and extending infinitely upwards and downwards
		//  (based on local frame)
		InfiniteCylinder,
		// Affect a cylinder whose axis starts at a reference sphere center, passes through the brush position,
		//  and extends further away from the sphere.
		CylinderOnSphere,
	};

	// Determines how the brush expects its local frame to be aligned as the mouse is moved
	enum class EStampAlignmentType
	{
		// Align brush to the hit normal
		HitNormal,
		// Align brush to have its local plane face the camera
		Camera,
		// Align brush such that its local plane is parallel to a reference plane
		ReferencePlane,
		// Align brush such that its local plane normal points away from a reference sphere center
		ReferenceSphere,
	};

	// Determines what kind of reference plane the brush wants to use (which will be accessed 
	//  via CurrentOptions.ConstantReferencePlane)
	enum class EReferencePlaneType
	{
		// Reference plane is not used
		None,
		// The reference plane is expected to have been caluclated using triangles in the region
		//  of interest affected by the brush at its initial application (at start of stroke)
		InitialROI,
		// Like InitialROI, but the plane normal is aligned to camera direction, so only the
		//  centroid is calculated from the region of interest.
		InitialROI_ViewAligned,
		// The brush expects custom "work" plane (typically set via gizmo)
		WorkPlane,
	};

	// Determines which vertices should be moved by the stroke
	enum class EROIUpdateType
	{
		// ROI is initialized from the first stamp and remains constant
		UseInitial,
		// ROI grows as new stamps overlap with more vertices
		UpdatePerStamp
	};

	// Determines which mesh should be used by ApplyStamp
	enum class EStampInputMeshType
	{
		// The "live" mesh that is updated by each stamp
		SculptMesh,
		// The "base" mesh that is typically updated after every stroke
		BaseMesh
	};

	virtual ~FMeshSculptBrushOp() = default;

	TWeakObjectPtr<UMeshSculptBrushOpProps> PropertySet;

	template<typename PropType> 
	PropType* GetPropertySetAs()
	{
		ensure(PropertySet.IsValid());
		PropType* CastResult = Cast<PropType>(PropertySet.Get());
		ensure(CastResult);
		return CastResult;
	}

	template<typename PropType>
	const PropType* GetPropertySetAs() const
	{
		ensure(PropertySet.IsValid());
		PropType* CastResult = Cast<PropType>(PropertySet.Get());
		ensure(CastResult);
		return CastResult;
	}


	TSharedPtr<FMeshSculptFallofFunc> Falloff;
	FSculptBrushOptions CurrentOptions;

	const FMeshSculptFallofFunc& GetFalloff() const { return *Falloff; }

	virtual void ConfigureOptions(const FSculptBrushOptions& Options)
	{
		CurrentOptions = Options;
	}

	virtual void BeginStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& InitialVertices) {}
	virtual void EndStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& FinalVertices) {}
	virtual void CancelStroke() {}
	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewValuesOut) = 0;



	//
	// overrideable Brush Op configuration things
	//

	virtual ESculptBrushOpTargetType GetBrushTargetType() const
	{
		return ESculptBrushOpTargetType::SculptMesh;
	}

	// Determines what region of a mesh a brush wants to affect
	virtual EBrushRegionType GetBrushRegionType() const
	{ 
		return EBrushRegionType::LocalSphere;
	}

	// Determines how the brush expects its local frame to be aligned as the mouse is moved
	virtual EStampAlignmentType GetStampAlignmentType() const
	{
		if (GetAlignStampToView())
		{
			return EStampAlignmentType::Camera;
		}
		return EStampAlignmentType::HitNormal;
	}

	//~ TODO: Deprecate this in favor of GetStampAlignmentType
	virtual bool GetAlignStampToView() const
	{
		return false;
	}

	virtual bool IgnoreZeroMovements() const
	{
		return false;
	}

	// If this is true and there is no mouse movement, the stamp will be applied with the same
	//  Local/World frames as the previous application. Irrelevant if IgnoreZeroMovements()
	//  returns true, since in that case the stamp requires mouse movement to be applied.
	// Useful for height brushes which change the mesh (and therefore the hit location) but
	//  want to continue being applied in the same vertical region while the mouse is not moved.
	virtual bool UseLastStampFrameOnZeroMovement() const
	{
		return false;
	}

	// Determines what kind of plane (if any) the brush wants to be stored at the start of
	//  the stroke in CurrentOptions.ConstantReferencePlane.
	virtual EReferencePlaneType GetReferencePlaneType() const
	{
		return EReferencePlaneType::None;
	}

	// Whether the brush wants an average plane to be computed at each stamp application out 
	//  of the affected triangles (accessed through Stamp.RegionPlane)
	virtual bool WantsStampRegionPlane() const
	{
		return false;
	}

	virtual bool UsesAlpha() const
	{
		return false;
	}

	UE_DEPRECATED(5.7, "Deprecated in favor of SupportsStrokeType to provide more fine grained description of what strokes brushes support.")
	virtual bool SupportsVariableSpacing() const
	{
		return false;
	}

	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
			return true;
		case EMeshSculptStrokeType::Spacing:
			return false;
		default:
			return false;
		}
	}

	virtual EROIUpdateType GetROIUpdateType() const
	{
		return EROIUpdateType::UpdatePerStamp;
	}

	virtual EStampInputMeshType GetStampInputMeshType() const
	{
		return EStampInputMeshType::SculptMesh;
	}
};




class FMeshSculptBrushOpFactory
{
public:
	virtual ~FMeshSculptBrushOpFactory() = default;
	virtual TUniquePtr<FMeshSculptBrushOp> Build() = 0;
};

template<typename OpType>
class TBasicMeshSculptBrushOpFactory : public FMeshSculptBrushOpFactory
{
public:
	virtual TUniquePtr<FMeshSculptBrushOp> Build() override
	{
		return MakeUnique<OpType>();
	}
};


class FLambdaMeshSculptBrushOpFactory : public FMeshSculptBrushOpFactory
{
public:
	TUniqueFunction<TUniquePtr<FMeshSculptBrushOp>(void)> BuildFunc;

	FLambdaMeshSculptBrushOpFactory()
	{
	}

	FLambdaMeshSculptBrushOpFactory(TUniqueFunction<TUniquePtr<FMeshSculptBrushOp>(void)> BuildFuncIn)
	{
		BuildFunc = MoveTemp(BuildFuncIn);
	}

	virtual TUniquePtr<FMeshSculptBrushOp> Build() override
	{
		return BuildFunc();
	}
};
