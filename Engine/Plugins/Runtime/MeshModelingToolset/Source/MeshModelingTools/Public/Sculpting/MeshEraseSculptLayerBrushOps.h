// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshSculptLayers.h"
#include "Async/ParallelFor.h"

#include "MeshEraseSculptLayerBrushOps.generated.h"

UCLASS(MinimalAPI)
class UEraseSculptLayerBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()

public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = EraseSculptLayerBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "10.", ClampMin = "0.0", ClampMax = "10.", ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = EraseSculptLayerBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;


	virtual float GetStrength() override { return Strength; }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};


class FEraseSculptLayerBrushOp : public FMeshSculptBrushOp
{
public:

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		static const double BrushSpeedTuning = 1.0;
		double UsePower = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;
		double MaxOffset = Stamp.Radius;
		if (!Mesh->HasAttributes())
		{
			return;
		}

		const UE::Geometry::FDynamicMeshSculptLayers* SculptLayers = Mesh->Attributes()->GetSculptLayers();
		if (!SculptLayers)
		{
			return;
		}

		int32 ActiveLayer = SculptLayers->GetActiveLayer();
		TConstArrayView<double> LayerWeights = SculptLayers->GetLayerWeights();
		
		ParallelFor(Vertices.Num(), [this, &Mesh, &Stamp, &Vertices, &NewPositionsOut, UsePower, MaxOffset, SculptLayers, ActiveLayer, LayerWeights](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d BasePos(0);
			for (int32 LayerIdx = 0, NumLayers = SculptLayers->NumLayers(); LayerIdx < NumLayers; ++LayerIdx)
			{
				if (LayerIdx != ActiveLayer)
				{
					FVector3d Offset;
					SculptLayers->GetLayer(LayerIdx)->GetValue(VertIdx, Offset);
					BasePos += Offset * LayerWeights[LayerIdx];
				}
			}
		
			FVector3d MoveVec = (BasePos - OrigPos);
			double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);
			double MoveDist = Falloff * UsePower;
			if (MoveVec.SquaredLength() < MoveDist * MoveDist)
			{
				NewPositionsOut[k] = BasePos;
			}
			else
			{
				UE::Geometry::Normalize(MoveVec);
				NewPositionsOut[k] = OrigPos + MoveDist * MoveVec;
			}
		});
	}


	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::SculptMesh;
	}


	virtual bool IgnoreZeroMovements() const
	{
		return false;
	}
};

