// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshWeights.h"
#include "Async/ParallelFor.h"
#include "MeshGroupPaintBrushOps.generated.h"




class FMeshTriangleGroupEditBrushOp : public FMeshSculptBrushOp
{
public:

	// not supported for this kind of brush op
	virtual void ApplyStamp(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<FVector3d>& NewColorsOut) override
	{
		check(false);
	}


	virtual void ApplyStampByTriangles(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Triangles,
		TArray<int32>& NewGroupsOut) = 0;

};






UCLASS(MinimalAPI)
class UGroupEraseBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Group to set as Erased value */
	UPROPERTY(EditAnywhere, Category = EraseBrush, meta = (DisplayName = "Erase Group", UIMin = 1))
	int32 Group = 0;

	/** When enabled, only the current group configured in the Paint brush is erased */
	UPROPERTY(EditAnywhere, Category = EraseBrush, meta = (DisplayName = "Only Current"))
	bool bOnlyEraseCurrent = false;

	virtual int32 GetGroup() { return Group; }

	// this lambda defines the "current" group for bOnlyEraseCurrent mode
	TUniqueFunction<int32()> GetCurrentGroupLambda = []() { return -1; };
};



class FGroupEraseBrushOp : public FMeshTriangleGroupEditBrushOp
{
public:

	virtual void ApplyStampByTriangles(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Triangles,
		TArray<int32>& NewGroupsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		UGroupEraseBrushOpProps* Props = GetPropertySetAs<UGroupEraseBrushOpProps>();
		int32 EraseGroup = (int32)Props->GetGroup();
		bool bOnlyEraseCurrent = Props->bOnlyEraseCurrent;
		int32 EraseCurrentGroup = Props->GetCurrentGroupLambda();

		int32 NumTriangles = Triangles.Num();
		for (int32 k = 0; k < NumTriangles; ++k)
		{
			int32 TriIdx = Triangles[k];
			int32 CurGroupID = Mesh->GetTriangleGroup(TriIdx);
			if (bOnlyEraseCurrent)
			{
				NewGroupsOut[k] = (CurGroupID == EraseCurrentGroup) ? EraseGroup : CurGroupID;
			}
			else
			{
				NewGroupsOut[k] = EraseGroup;
			}
		}
	}
};




UCLASS(MinimalAPI)
class UGroupPaintBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** The group that will be assigned to triangles within the brush region */
	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Group", UIMin = 0))
	int32 Group = 1;

	/** If true, only triangles with no group assigned will be painted */
	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Only Ungrouped"))
	bool bOnlyPaintUngrouped = false;

	virtual int32 GetGroup() { return Group; }
};



class FGroupPaintBrushOp : public FMeshTriangleGroupEditBrushOp
{
public:

	virtual void ApplyStampByTriangles(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Triangles,
		TArray<int32>& NewGroupsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		UGroupPaintBrushOpProps* Props = GetPropertySetAs<UGroupPaintBrushOpProps>();
		int32 SetToGroup = (int32)Props->GetGroup();
		bool bOnlyUngrouped = Props->bOnlyPaintUngrouped;

		int32 NumTriangles = Triangles.Num();
		for ( int32 k = 0; k < NumTriangles; ++k)
		{
			int32 TriIdx = Triangles[k];
			int32 OrigGroupID = Mesh->GetTriangleGroup(TriIdx);
			if (bOnlyUngrouped == false || OrigGroupID == 0)
			{
				NewGroupsOut[k] = SetToGroup;
			}
			else
			{
				NewGroupsOut[k] = OrigGroupID;
			}
		}
	}
};
