// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshWeights.h"
#include "Async/ParallelFor.h"
#include "MeshInflateBrushOps.generated.h"


UCLASS(MinimalAPI)
class UInflateBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = InflateBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = InflateBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

// Like FSurfaceSculptBrushOp (moves vertices along individual vertex normals), but operates on the
//  sculpt mesh (i.e. the live mesh) instead of the pre-stroke mesh.
class FInflateBrushOp : public FMeshSculptBrushOp
{

public:
	double BrushSpeedTuning = 2.0;

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::SculptMesh;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);
			FVector3d Normal = UE::Geometry::FMeshNormals::ComputeVertexNormal(*Mesh, VertIdx);
			FVector3d MoveVec = UsePower * Normal;

			double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);

			FVector3d NewPos = OrigPos + Falloff * MoveVec;
			NewPositionsOut[k] = NewPos;
		});
	}

};

