// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshWeights.h"
#include "Async/ParallelFor.h"
#include "PropertySets/AxisFilterPropertyType.h"
#include "MeshMoveBrushOps.generated.h"



UCLASS(MinimalAPI)
class UMoveBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = MoveBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 1.0;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = MoveBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }

	/** Axis filters restrict mesh movement to World X/Y/Z axes */
	UPROPERTY(EditAnywhere, Category = MoveBrush)
	FModelingToolsAxisFilter AxisFilters;
};


class FMoveBrushOp : public FMeshSculptBrushOp
{
private:	
	FSculptBrushStamp FirstStamp;
	
public:
	
	virtual void BeginStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& InitialVertices) override
	{
		FirstStamp = Stamp;
	};

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		FVector3d MoveVec = Stamp.LocalFrame.Origin - FirstStamp.LocalFrame.Origin;
		
		UMoveBrushOpProps* Props = GetPropertySetAs<UMoveBrushOpProps>();

		// if we have axis constraints we want to apply them in world space, and then remap
		// that world vector to local space
		if (Props->AxisFilters.AnyAxisFiltered())
		{
			FVector3d WorldMoveVec = Stamp.WorldFrame.Origin - FirstStamp.WorldFrame.Origin;
			WorldMoveVec = MoveVec.Length() * UE::Geometry::Normalized(WorldMoveVec);		// apply local space scaling
			WorldMoveVec.X = (Props->AxisFilters.bAxisX) ? WorldMoveVec.X : 0;
			WorldMoveVec.Y = (Props->AxisFilters.bAxisY) ? WorldMoveVec.Y : 0;
			WorldMoveVec.Z = (Props->AxisFilters.bAxisZ) ? WorldMoveVec.Z : 0;
			MoveVec = FirstStamp.LocalFrame.FromFrameVector(FirstStamp.WorldFrame.ToFrameVector(WorldMoveVec));
		}

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			double Falloff = GetFalloff().Evaluate(FirstStamp, OrigPos);

			FVector3d NewPos = OrigPos + Falloff * MoveVec;
			NewPositionsOut[k] = NewPos;
		});
	}


	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::ActivePlane;
	}

	virtual bool IgnoreZeroMovements() const override
	{
		return true;
	}

	virtual EROIUpdateType GetROIUpdateType() const override
	{
		return EROIUpdateType::UseInitial;
	}

	virtual EStampInputMeshType GetStampInputMeshType() const override
	{
		return EStampInputMeshType::BaseMesh;
	}
};
