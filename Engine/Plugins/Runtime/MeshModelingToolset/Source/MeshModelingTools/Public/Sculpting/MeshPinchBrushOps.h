// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"
#include "LineTypes.h"
#include "MeshPinchBrushOps.generated.h"


UCLASS(MinimalAPI)
class UPinchBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = PinchBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = PinchBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.75;

	/** Depth of Brush into surface along surface normal */
	UPROPERTY(EditAnywhere, Category = PinchBrush, meta = (UIMin = "-0.5", UIMax = "0.5", ClampMin = "-1.0", ClampMax = "1.0"))
	float Depth = 0;

	/** When enabled, brush will damp motion of vertices that would move perpendicular to brush stroke direction */
	UPROPERTY(EditAnywhere, Category = PinchBrush)
	bool bPerpDamping = true;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual float GetDepth() override { return Depth; }
	virtual bool SupportsStrengthPressure() override { return true; }
};


class FPinchBrushOp : public FMeshSculptBrushOp
{

public:
	double BrushSpeedTuning = 3.0;

	FVector3d LastSmoothBrushPosLocal;
	FVector3d LastSmoothBrushNormalLocal;;

	virtual void BeginStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& InitialVertices) 
	{
		LastSmoothBrushPosLocal = Stamp.LocalFrame.Origin;
		LastSmoothBrushNormalLocal = Stamp.LocalFrame.Z();
	}


	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		// hardcoded LazyBrush
		FVector3d NewSmoothBrushPosLocal = UE::Geometry::Lerp(LastSmoothBrushPosLocal, Stamp.LocalFrame.Origin, 0.75);
		FVector3d NewSmoothBrushNormal = UE::Geometry::Lerp(LastSmoothBrushNormalLocal, Stamp.LocalFrame.Z(), 0.75);
		UE::Geometry::Normalize(NewSmoothBrushNormal);

		FVector3d MotionVec = NewSmoothBrushPosLocal - LastSmoothBrushPosLocal;
		bool bHaveMotion = (MotionVec.Length() > FMathd::ZeroTolerance);
		UE::Geometry::Normalize(MotionVec);
		UE::Geometry::FLine3d MoveLine(LastSmoothBrushPosLocal, MotionVec);

		FVector3d DepthPosLocal = NewSmoothBrushPosLocal - (Stamp.Depth * Stamp.Radius * NewSmoothBrushNormal);
		double UseSpeed = Stamp.Direction * Stamp.Radius * Stamp.Power * Stamp.DeltaTime * BrushSpeedTuning;

		LastSmoothBrushPosLocal = NewSmoothBrushPosLocal;

		bool bLimitDrag = GetPropertySetAs<UPinchBrushOpProps>()->bPerpDamping;

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d Delta = DepthPosLocal - OrigPos;
			UE::Geometry::Normalize(Delta);
			FVector3d MoveVec = Delta;

			// pinch uses 1/x falloff
			double Dist = UE::Geometry::Distance(OrigPos, NewSmoothBrushPosLocal);
			double NormalizedDistance = (Dist / Stamp.Radius) + 0.0001;
			double LinearFalloff = FMathd::Clamp(1.0 - NormalizedDistance, 0.0, 1.0);
			double InvFalloff = LinearFalloff * LinearFalloff * LinearFalloff;
			double UseFalloff = FMathd::Lerp(LinearFalloff, InvFalloff, Stamp.Falloff);

			if (bLimitDrag && bHaveMotion && UseFalloff < 0.7f)
			{
				double AnglePower = 1.0 - FMathd::Abs( UE::Geometry::Normalized(MoveVec).Dot(MotionVec));
				UseFalloff *= AnglePower;
			}

			FVector3d NewPos = OrigPos + UseFalloff * UseSpeed * MoveVec;
			NewPositionsOut[k] = NewPos;
		});
	}


	virtual ESculptBrushOpTargetType GetBrushTargetType() const
	{
		return ESculptBrushOpTargetType::TargetMesh;
	}
};


