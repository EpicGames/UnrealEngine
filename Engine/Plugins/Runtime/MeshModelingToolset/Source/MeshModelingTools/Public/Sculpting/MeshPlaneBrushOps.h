// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Async/ParallelFor.h"
#include "MeshPlaneBrushOps.generated.h"





UCLASS(MinimalAPI)
class UBasePlaneBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	virtual EPlaneBrushSideMode GetWhichSide() { return EPlaneBrushSideMode::BothSides; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

UCLASS(MinimalAPI)
class UPlaneBrushOpProps : public UBasePlaneBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = PlaneBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = PlaneBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	/** Depth of Brush into surface along surface normal */
	UPROPERTY(EditAnywhere, Category = PlaneBrush, meta = (UIMin = "-0.5", UIMax = "0.5", ClampMin = "-1.0", ClampMax = "1.0"))
	float Depth = 0;

	/** Control whether effect of brush should be limited to one side of the Plane  */
	UPROPERTY(EditAnywhere, Category = PlaneBrush)
	EPlaneBrushSideMode WhichSide = EPlaneBrushSideMode::BothSides;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual float GetDepth() override { return Depth; }
	virtual EPlaneBrushSideMode GetWhichSide() { return WhichSide; }
};


UCLASS(MinimalAPI)
class UViewAlignedPlaneBrushOpProps : public UBasePlaneBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = ViewPlaneBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = ViewPlaneBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	/** Depth of Brush into surface along view ray */
	UPROPERTY(EditAnywhere, Category = ViewPlaneBrush, meta = (UIMin = "-0.5", UIMax = "0.5", ClampMin = "-1.0", ClampMax = "1.0"))
	float Depth = 0;

	/** Control whether effect of brush should be limited to one side of the Plane  */
	UPROPERTY(EditAnywhere, Category = ViewPlaneBrush)
	EPlaneBrushSideMode WhichSide = EPlaneBrushSideMode::BothSides;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual float GetDepth() override { return Depth; }
	virtual EPlaneBrushSideMode GetWhichSide() { return WhichSide; }
};


UCLASS(MinimalAPI)
class UFixedPlaneBrushOpProps : public UBasePlaneBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = FixedPlaneBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = FixedPlaneBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	/** Control whether effect of brush should be limited to one side of the Plane  */
	UPROPERTY(EditAnywhere, Category = FixedPlaneBrush)
	EPlaneBrushSideMode WhichSide = EPlaneBrushSideMode::BothSides;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual EPlaneBrushSideMode GetWhichSide() { return WhichSide; }
};



class FPlaneBrushOp : public FMeshSculptBrushOp
{

public:
	double BrushSpeedTuning = 6.0;

	FFrame3d StrokePlane;
	
	virtual EReferencePlaneType GetReferencePlaneType() const
	{
		// FPlaneBrushOp doesn't have one type of reference plane it expects. Instead, it can
		//  be used as multiple different brushes depending on what reference plane it gets 
		//  (for instance, a flatten brush, or bring to work plane brush, etc.). 
		// Ideally we would have some enum on the brush instance that will change which 
		//  reference plane type it requests, so that each brush instance can be configured
		//  to act as a particular brush type. What we do instead (for now) is rely on the
		//  fact that we configure each of these different plane brush types with a different
		//  property set, so by looking at our property set, we know what brush we are.

		if (ExactCast<UViewAlignedPlaneBrushOpProps>(PropertySet.Get()))
		{
			return EReferencePlaneType::InitialROI_ViewAligned;
		}
		else if (ExactCast<UFixedPlaneBrushOpProps>(PropertySet.Get()))
		{
			return EReferencePlaneType::WorkPlane;
		}
		else // UPlaneBrushOpProps
		{
			return EReferencePlaneType::InitialROI;
		}
	}

	virtual void BeginStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& InitialVertices) override
	{
		StrokePlane = CurrentOptions.ConstantReferencePlane;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		static const double PlaneSigns[3] = { 0, -1, 1 };
		UBasePlaneBrushOpProps* Props = GetPropertySetAs<UBasePlaneBrushOpProps>();
		double PlaneSign = PlaneSigns[(int32)Props->GetWhichSide()];

		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		double UseSpeed = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);
			FVector3d PlanePos = StrokePlane.ToPlane(OrigPos, 2);
			FVector3d Delta = PlanePos - OrigPos;

			double Dot = Delta.Dot(StrokePlane.Z());
			FVector3d NewPos = OrigPos;
			if (Dot * PlaneSign >= 0)
			{
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);
				FVector3d MoveVec = Falloff * UseSpeed * Delta;
				double MaxDist = UE::Geometry::Normalize(Delta);
				NewPos = (MoveVec.SquaredLength() > MaxDist * MaxDist) ?
					PlanePos : (OrigPos + Falloff * MoveVec);
			}

			NewPositionsOut[k] = NewPos;
		});
	}

};

