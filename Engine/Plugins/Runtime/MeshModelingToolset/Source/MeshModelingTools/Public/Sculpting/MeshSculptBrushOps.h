// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshWeights.h"
#include "Async/ParallelFor.h"
#include "MeshSculptBrushOps.generated.h"


UCLASS(MinimalAPI)
class UStandardSculptBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SculptBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = SculptBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 1.0;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

namespace UE::Geometry
{
// Brush that sculpts all vertices in one direction, set by the normal at the brush position.
//  Operates on pre-stroke mesh.
class FSingleNormalSculptBrushOp : public FMeshSculptBrushOp
{
public:
	double BrushSpeedTuning = 6.0;

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		// Get our stamp information from the pre-stroke mesh
		return ESculptBrushOpTargetType::TargetMesh;
	}
	virtual bool SupportsVariableSpacing() const override
	{
		return true;
	}

	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}

	virtual bool UsesAlpha() const override
	{
		return true;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		bool bHaveAlpha = Stamp.HasAlpha();
		FVector3d OffsetDirection = Stamp.LocalFrame.Z();

		ParallelFor(Vertices.Num(), [this, Mesh, &Stamp, &Vertices, &NewPositionsOut, UsePower, bHaveAlpha, &OffsetDirection](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			double Alpha = (bHaveAlpha) ? Stamp.StampAlphaFunc(Stamp, OrigPos) : 1.0;

			FVector3d MoveVec = UsePower * OffsetDirection;
			double Falloff = GetFalloff().Evaluate(Stamp, OrigPos) * Alpha;
			FVector3d NewPos = OrigPos + Falloff * MoveVec;
			NewPositionsOut[k] = NewPos;
		});
	}
};
}// end namespace UE::Geometry
//~ TODO: The rest should be moved to UE::Geometry namespace as well

// Uses the vertex normal at the "base" mesh (i.e. pre-stroke mesh) vertex to move vertices.
//  Basically just like the inflate brush, but using the pre-stroke mesh.
class FSurfaceSculptBrushOp : public FMeshSculptBrushOp
{
public:
	double BrushSpeedTuning = 6.0;

	typedef TUniqueFunction<bool(int32, const FVector3d&, double, FVector3d&, FVector3d&)> NearestQueryFuncType;

	NearestQueryFuncType BaseMeshNearestQueryFunc;

	FSurfaceSculptBrushOp(NearestQueryFuncType QueryFunc)
	{
		BaseMeshNearestQueryFunc = MoveTemp(QueryFunc);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::TargetMesh;
	}
	virtual bool SupportsVariableSpacing() const override
	{
		return true;
	}

	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}

	virtual bool UsesAlpha() const override
	{
		return true;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		bool bHaveAlpha = Stamp.HasAlpha();

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d BasePos, BaseNormal;
			bool bFoundBasePos = BaseMeshNearestQueryFunc(VertIdx, OrigPos, 4.0 * Stamp.Radius, BasePos, BaseNormal);
			if (bFoundBasePos == false)
			{
				NewPositionsOut[k] = OrigPos;
			}
			else
			{
				double Alpha = (bHaveAlpha) ? Stamp.StampAlphaFunc(Stamp, OrigPos) : 1.0;

				FVector3d MoveVec = UsePower * BaseNormal;
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos) * Alpha;
				FVector3d NewPos = OrigPos + Falloff * MoveVec;
				NewPositionsOut[k] = NewPos;
			}
		});
	}

};





UCLASS(MinimalAPI)
class UViewAlignedSculptBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SculptToViewBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = SculptToViewBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 1.0;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};


// Starts with the base mesh, and moves all vertices towards the camera (by asking the owning tool to align
//  the stamp normal to view, and using that value).
class FViewAlignedSculptBrushOp : public FMeshSculptBrushOp
{
public:
	double BrushSpeedTuning = 6.0;

	typedef TUniqueFunction<bool(int32, const FVector3d&, double, FVector3d&, FVector3d&)> NearestQueryFuncType;

	NearestQueryFuncType BaseMeshNearestQueryFunc;

	FViewAlignedSculptBrushOp(NearestQueryFuncType QueryFunc)
	{
		BaseMeshNearestQueryFunc = MoveTemp(QueryFunc);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::TargetMesh;
	}

	virtual bool GetAlignStampToView() const override
	{
		return true;
	}
	virtual bool SupportsVariableSpacing() const override
	{
		return true;
	}
	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}
	virtual bool UsesAlpha() const override
	{
		return true;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		FVector3d StampNormal = Stamp.LocalFrame.Z();

		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		bool bHaveAlpha = Stamp.HasAlpha();

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d BasePos, BaseNormal;
			bool bFoundBasePos = BaseMeshNearestQueryFunc(VertIdx, OrigPos, 4.0 * Stamp.Radius, BasePos, BaseNormal);
			if (bFoundBasePos == false)
			{
				NewPositionsOut[k] = OrigPos;
			}
			else
			{
				double Alpha = (bHaveAlpha) ? Stamp.StampAlphaFunc(Stamp, OrigPos) : 1.0;

				FVector3d MoveVec = UsePower * StampNormal;
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos) * Alpha;
				FVector3d NewPos = OrigPos + Falloff * MoveVec;
				NewPositionsOut[k] = NewPos;
			}
		});
	}

};





UCLASS(MinimalAPI)
class USculptMaxBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SculptMaxBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = SculptMaxBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	/** Maximum height as fraction of brush size */
	UPROPERTY(EditAnywhere, Category = SculptMaxBrush, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float MaxHeight = 0.5;

	/** If true, maximum height is defined using the FixedHeight constant instead of brush-relative size */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = SculptMaxBrush)
	bool bUseFixedHeight = false;

	/** Maximum height in world-space dimension */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = SculptMaxBrush)
	float FixedHeight = 0.0;


	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};

namespace UE::Geometry
{
// Brush that sculpts all vertices in one direction, set by the normal at the brush position, 
//  and capped by a max value. Operates on pre-stroke mesh.
class FSingleNormalMaxSculptBrushOp : public FSingleNormalSculptBrushOp
{
public:
	typedef TUniqueFunction<bool(int32, const FVector3d&, double, FVector3d&, FVector3d&)> NearestQueryFuncType;

	// To cap the movement distance, we need to know the original position on the pre-sculpt mesh
	NearestQueryFuncType BaseMeshNearestQueryFunc;

	FSingleNormalMaxSculptBrushOp(NearestQueryFuncType QueryFunc)
	{
		BaseMeshNearestQueryFunc = MoveTemp(QueryFunc);
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		USculptMaxBrushOpProps* Props = GetPropertySetAs<USculptMaxBrushOpProps>();
		double MaxOffset = (Props->bUseFixedHeight) ? Props->FixedHeight : (Props->MaxHeight * Stamp.Radius);

		bool bHaveAlpha = Stamp.HasAlpha();
		FVector3d OffsetDirection = Stamp.LocalFrame.Z();

		ParallelFor(Vertices.Num(), [this, Mesh, &Stamp, &Vertices, &NewPositionsOut, UsePower, MaxOffset, bHaveAlpha, &OffsetDirection](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d BasePos, BaseNormal;
			bool bFoundBasePos = BaseMeshNearestQueryFunc(VertIdx, OrigPos, 4.0 * Stamp.Radius, BasePos, BaseNormal);
			if (bFoundBasePos == false)
			{
				NewPositionsOut[k] = OrigPos;
			}
			else
			{
				double Alpha = (bHaveAlpha) ? Stamp.StampAlphaFunc(Stamp, OrigPos) : 1.0;

				FVector3d MoveVec = UsePower * OffsetDirection;
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos) * Alpha;
				FVector3d NewPos = OrigPos + Falloff * MoveVec;

				FVector3d DeltaPos = NewPos - BasePos;
				if (DeltaPos.SquaredLength() > MaxOffset * MaxOffset)
				{
					UE::Geometry::Normalize(DeltaPos);
					NewPos = BasePos + MaxOffset * DeltaPos;
				}

				NewPositionsOut[k] = NewPos;
			}
		});
	}
};
}//end UE::Geometry

// Like FSurfaceSculptBrushOp, but caps the movement to some maximum distance. In other words, it is
//  an inflate (movement along individual vertex normals) based off of base (i.e. pre-stroke) mesh
//  normals, but capped in offset distance from the pre-stroke mesh.
class FSurfaceMaxSculptBrushOp : public FMeshSculptBrushOp
{
public:
	double BrushSpeedTuning = 6.0;

	typedef TUniqueFunction<bool(int32, const FVector3d&, double, FVector3d&, FVector3d&)> NearestQueryFuncType;

	NearestQueryFuncType BaseMeshNearestQueryFunc;

	FSurfaceMaxSculptBrushOp(NearestQueryFuncType QueryFunc)
	{
		BaseMeshNearestQueryFunc = MoveTemp(QueryFunc);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::TargetMesh;
	}
	virtual bool SupportsVariableSpacing() const override
	{
		return true;
	}
	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}
	virtual bool UsesAlpha() const override
	{
		return true;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		USculptMaxBrushOpProps* Props = GetPropertySetAs<USculptMaxBrushOpProps>();
		double MaxOffset = (Props->bUseFixedHeight) ? Props->FixedHeight : (Props->MaxHeight * Stamp.Radius);

		bool bHaveAlpha = Stamp.HasAlpha();

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d BasePos, BaseNormal;
			bool bFoundBasePos = BaseMeshNearestQueryFunc(VertIdx, OrigPos, 4.0 * Stamp.Radius, BasePos, BaseNormal);
			if (bFoundBasePos == false)
			{
				NewPositionsOut[k] = OrigPos;
			}
			else
			{
				double Alpha = (bHaveAlpha) ? Stamp.StampAlphaFunc(Stamp, OrigPos) : 1.0;

				FVector3d MoveVec = UsePower * BaseNormal;
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos) * Alpha;
				FVector3d NewPos = OrigPos + Falloff * MoveVec;

				FVector3d DeltaPos = NewPos - BasePos;
				if (DeltaPos.SquaredLength() > MaxOffset * MaxOffset)
				{
					UE::Geometry::Normalize(DeltaPos);
					NewPos = BasePos + MaxOffset * DeltaPos;
				}

				NewPositionsOut[k] = NewPos;
			}
		});
	}

};
