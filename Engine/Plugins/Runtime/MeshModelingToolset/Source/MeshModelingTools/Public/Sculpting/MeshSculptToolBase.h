// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MeshSurfacePointTool.h"
#include "BaseTools/BaseBrushTool.h"
#include "Sculpting/MeshBrushOpBase.h"
#include "BoxTypes.h"
#include "Properties/MeshMaterialProperties.h"
#include "Changes/ValueWatcher.h"
#include "Components/BaseDynamicMeshComponent.h"
#include "TransactionUtil.h"
#include "MeshSculptToolBase.generated.h"

#define UE_API MESHMODELINGTOOLS_API


class ULocalTwoAxisPropertyEditInputBehavior;
class UMaterialInstanceDynamic;
class UCombinedTransformGizmo;
class UTransformProxy;
class UPreviewMesh;
class UPreviewGeometry;

/**
 * Type of Brush Size currently active in FBrushToolRadius
 */
UENUM()
enum class EBrushToolSizeType : uint8
{
	/** Brush size is a dimensionless scale relative to the target object size */
	Adaptive = 0,
	/** Brush size is defined in world dimensions */
	World = 1
};

/**
 * FBrushToolRadius is used to define the size of 3D "brushes" used in (eg) sculpting tools.
 * The brush size can be defined in various ways.
 */
USTRUCT()
struct FBrushToolRadius
{
	GENERATED_BODY()

	/** Specify the type of brush size currently in use */
	UPROPERTY(EditAnywhere, Category = Brush)
	EBrushToolSizeType SizeType = EBrushToolSizeType::Adaptive;

	/** Adaptive brush size is used to interpolate between an object-specific minimum and maximum brush size */
	UPROPERTY(EditAnywhere, Category = Brush, AdvancedDisplay, meta = (EditCondition = "SizeType == EBrushToolSizeType::Adaptive",
		DisplayName = "Size", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0"))
	float AdaptiveSize = 0.25;

	/** World brush size is a dimension in world coordinates */
	UPROPERTY(EditAnywhere, Category = Brush, AdvancedDisplay, meta = (EditCondition = "SizeType == EBrushToolSizeType::World",
		DisplayName = "World Radius", UIMin = "1.0", UIMax = "1000.0", ClampMin = "0.1", ClampMax = "50000.0"))
	float WorldRadius = 100.0;

	UPROPERTY(EditAnywhere, Category = Brush)
	bool bToolSupportsPressureSensitivity = false;
	
	UPROPERTY(EditAnywhere, Category = Brush, meta = (EditCondition = "bToolSupportsPressureSensitivity"))
	bool bEnablePressureSensitivity = false;

	/**
	 * WorldSizeRange defines the min/max dimensions for Adaptive brush size
	 */
	TInterval<float> WorldSizeRange = TInterval<float>(1.0f, 1000.0f);

	//
	// util functions
	//

	/** Set the WorldSizeRange value and optionally clamp the WorldRadius based on this new range */
	UE_API void InitializeWorldSizeRange(TInterval<float> Range, bool bValidateWorldRadius = true);
	/** Return the set/calculated world-space radius for the current settings */
	UE_API float GetWorldRadius() const;
	/** Increase the current radius dimension by a fixed step (or a smaller fixed step) */
	UE_API void IncreaseRadius(bool bSmallStep);
	/** Decrease the current radius dimension by a fixed step (or a smaller fixed step) */
	UE_API void DecreaseRadius(bool bSmallStep);
};

 
/** Mesh Sculpting Brush Types */
UENUM()
enum class EMeshSculptFalloffType : uint8
{
	Smooth = 0,
	Linear = 1,
	Inverse = 2,
	Round = 3,
	BoxSmooth = 4,
	BoxLinear = 5,
	BoxInverse = 6,
	BoxRound = 7,

	LastValue UMETA(Hidden)
};


UCLASS(MinimalAPI)
class USculptBrushProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = Stroke)
	EMeshSculptStrokeType StrokeType = EMeshSculptStrokeType::Spacing;

	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayPriority = 1, ModelingQuickEdit, ModelingQuickSettings = 100))
	FBrushToolRadius BrushSize;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", HideEditConditionToggle, EditConditionHides, EditCondition="bShowFalloff", DisplayPriority = 3))
	float BrushFalloffAmount;

	/** If false, then BrushFalloffAmount will not be shown in DetailsView panels (otherwise no effect) */
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowFalloff = true;

	/** Depth of Brush into surface along view ray or surface normal, depending on the Active Brush Type */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = "-0.5", UIMax = "0.5", ClampMin = "-1.0", ClampMax = "1.0", DisplayPriority = 5, HideEditConditionToggle, EditConditionHides, EditCondition = "bShowPerBrushProps"))
	float Depth = 0;

	/** Allow the Brush to hit the back-side of the mesh */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayPriority = 6))
	bool bHitBackFaces = true;

	/** Brush stamps are applied at this time interval. 0 for a single stamp, 1 for continuous stamps, 0.5 is a stamp every half-second */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Flow", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", DisplayPriority = 7, HideEditConditionToggle, EditConditionHides, EditCondition = "bShowFlowRate && StrokeType == EMeshSculptStrokeType::Airbrush"))
	float FlowRate = 1.0f;

	/** When moving the mouse quickly in Spacing mode, determines how initial rays are upsampled before being filtered by StampSpacing. */
	UPROPERTY(EditAnywhere, Category = Brush, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "2.0", ClampMin = "0.0", ClampMax = "10.0",
		DisplayPriority = 8, HideEditConditionToggle, EditConditionHides,
		EditCondition = "bShowSpacing && StrokeType == EMeshSculptStrokeType::Spacing"))
	float RayDensity = 1.0f;

	/** Space out stamp centers at distances Spacing*BrushDiameter along the stroke (so Spacing of 1 means that stamps will be adjacent but non-overlapping). Zero spacing means continuous stamps. */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1000.0", 
		DisplayPriority = 8, DisplayName = "Stamp Spacing", HideEditConditionToggle, EditConditionHides, 
		EditCondition = "bShowSpacing && StrokeType == EMeshSculptStrokeType::Spacing"))
	float Spacing = .05f;

	/** Lazy brush smooths out the brush path by averaging the cursor positions */
	UPROPERTY(EditAnywhere, Category = Brush, meta = (DisplayName = "Lazy", UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "1.0", DisplayPriority = 9, HideEditConditionToggle, EditConditionHides, EditCondition = "bShowLazyness"))
	float Lazyness = 0;

	/**  */
	UPROPERTY( meta = (TransientToolProperty))
	bool bShowPerBrushProps = true;

	/**  */
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowLazyness = true;

	/**  */
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowFlowRate = true;

	UPROPERTY(meta = (TransientToolProperty))
	bool bShowSpacing = true;
};




UCLASS(MinimalAPI)
class UKelvinBrushProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Brush Fall off as fraction of brush size*/
	UPROPERTY(EditAnywhere, Category = Kelvin, meta = (UIMin = "0.0", ClampMin = "0.0"))
	float FallOffDistance = 1.f;

	/** How much the mesh resists shear */
	UPROPERTY(EditAnywhere, Category = Kelvin, meta = (UIMin = "0.0", ClampMin = "0.0"))
	float Stiffness = 1.f;

	/** How compressible the spatial region is: 1 - 2 x Poisson ratio */
	UPROPERTY(EditAnywhere, Category = Kelvin, meta = (UIMin = "0.0", UIMax = "0.1", ClampMin = "0.0", ClampMax = "1.0"))
	float Incompressiblity = 1.f;

	/** Integration steps*/
	UPROPERTY(EditAnywhere, Category = Kelvin, meta = (UIMin = "0.0", UIMax = "100", ClampMin = "0.0", ClampMax = "100"))
	int BrushSteps = 3;
};




UCLASS(MinimalAPI)
class UWorkPlaneProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY( meta = (TransientToolProperty) )
	bool bPropertySetEnabled = true;

	/** Toggle whether Work Plane Positioning Gizmo is visible */
	UPROPERTY(EditAnywhere, Category = TargetPlane, meta = (HideEditConditionToggle, EditCondition = "bPropertySetEnabled == true", ModelingQuickSettings))
	bool bShowGizmo = true;

	/** Toggle whether Work Plane is visible */
	UPROPERTY(EditAnywhere, Category = TargetPlane)
	bool bShowPlane = true;

	UPROPERTY(EditAnywhere, Category = TargetPlane, AdvancedDisplay, meta = (HideEditConditionToggle, EditCondition = "bPropertySetEnabled == true"))
	FVector Position = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = TargetPlane, AdvancedDisplay, meta = (HideEditConditionToggle, EditCondition = "bPropertySetEnabled == true"))
	FQuat Rotation = FQuat::Identity;

	// Recenter the gizmo around the target position (without changing work plane), if it is "too far" (> 10 meters + max bounds dim) from that position currently
	void RecenterGizmoIfFar(FVector CenterPosition, double BoundsMaxDim, double TooFarDistance = 1000)
	{
		double DistanceTolSq = (BoundsMaxDim + TooFarDistance) * (BoundsMaxDim + TooFarDistance);
		if (FVector::DistSquared(CenterPosition, Position) > DistanceTolSq)
		{
			FVector Normal = Rotation.GetAxisZ();
			Position = CenterPosition - (CenterPosition - Position).ProjectOnToNormal(Normal);
		}
	}
};




UCLASS(MinimalAPI)
class USculptMaxBrushProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Specify maximum displacement height (relative to brush size) */
	UPROPERTY(EditAnywhere, Category = SculptMaxBrush, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float MaxHeight = 0.5;

	/** Use maximum height from last brush stroke, regardless of brush size. Note that spatial brush falloff still applies.  */
	UPROPERTY(EditAnywhere, Category = SculptMaxBrush)
	bool bFreezeCurrentHeight = false;
};




/**
 * Base Tool for mesh sculpting tools, provides some shared functionality
 */
UCLASS(MinimalAPI)
class UMeshSculptToolBase : public UMeshSurfacePointTool
{
	GENERATED_BODY()
protected:
	using FFrame3d = UE::Geometry::FFrame3d;
public:

	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// UMeshSurfacePointTool API
	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual void OnCancelDrag() override;
	// end UMeshSurfacePointTool API

protected:
	UE_API virtual void OnTick(float DeltaTime) override;

	
	UE_API virtual void OnCompleteSetup();
	virtual void OnBeginStroke(const FRay& WorldRay) { check(false); }
	virtual void OnEndStroke() { check(false); }
	virtual void OnCancelStroke() { check(false); }

public:
	/** Properties that control brush size/etc */
	UPROPERTY()
	TObjectPtr<USculptBrushProperties> BrushProperties;

	/** Properties for 3D workplane / gizmo */
	UPROPERTY()
	TObjectPtr<UWorkPlaneProperties> GizmoProperties;


protected:
	UWorld* TargetWorld;		// required to spawn UPreviewMesh/etc
	FViewCameraState CameraState;

	/** Initial transformation on target mesh */
	UE::Geometry::FTransformSRT3d InitialTargetTransform;
	/** Active transformation on target mesh, includes baked scale */
	UE::Geometry::FTransformSRT3d CurTargetTransform;

	UE_API FRay3d GetLocalRay(const FRay& WorldRay) const;

	/** Returns true if BrushSize Pressure Sensitivity is both supported by the tool and currently enabled. */
	UE_API bool GetBrushSizePressureSensitivityEnabled() const;

public:
	/** Returns true if Pressure Sensitivity for Brush Strength is both supported by the tool and currently enabled. */
	UE_API bool GetBrushStrengthPressureEnabled();
	/** Sets the Pressure Sensitivity for Brush Strength to be enabled/disabled */
	UE_API void SetBrushStrengthPressureEnabled(const bool bEnabled);

protected:
	/**
	 * Subclass must implement this and return relevant rendering component
	 */
	virtual UBaseDynamicMeshComponent* GetSculptMeshComponent() { check(false); return nullptr; }

	virtual FDynamicMesh3* GetSculptMesh() { return GetSculptMeshComponent()->GetMesh(); }
	virtual const FDynamicMesh3* GetSculptMesh() const { return const_cast<UMeshSculptToolBase*>(this)->GetSculptMeshComponent()->GetMesh(); }

	virtual FDynamicMesh3* GetBaseMesh() { check(false); return nullptr; }
	virtual const FDynamicMesh3* GetBaseMesh() const { check(false); return nullptr; }

	// For any subclass where this returns false, BrushProperties will not be automatically saved/restored, so the class won't use BrushProperties changes made in other tools.
	virtual bool SharesBrushPropertiesChanges() const { return true; }
	
	// Should be overriden by subclasses so that settings are not shared across this tool and subclasses.
	UE_API virtual FString GetPropertyCacheIdentifier() const;


	/**
	 * Subclass calls this to set up editing component
	 */
	UE_API virtual void InitializeSculptMeshComponent(UBaseDynamicMeshComponent* Component, AActor* Actor);


	/**
	 * Subclass can override this to change what results are written.
	 * Default is to apply a default vertex positions update to the target object.
	 */
	UE_API virtual void CommitResult(UBaseDynamicMeshComponent* Component, bool bModifiedTopology);


	//
	// Brush Types
	//
public:
	struct FBrushTypeInfo
	{
		FText Name;
		int32 Identifier;

		bool operator==( const FBrushTypeInfo& Other ) const
		{
			return Other.Identifier == Identifier &&
					Other.Name.EqualTo(Name);
		}

		friend uint32 GetTypeHash(const FBrushTypeInfo& Other)
		{
			return HashCombine(GetTypeHash(Other.Identifier), GetTypeHash(Other.Name.ToString()));
		}
	};
	const TSet<FBrushTypeInfo>& GetRegisteredPrimaryBrushTypes() const { return RegisteredPrimaryBrushTypes; }
	const TSet<FBrushTypeInfo>& GetRegisteredSecondaryBrushTypes() const { return RegisteredSecondaryBrushTypes; }

protected:
	TSet<FBrushTypeInfo> RegisteredPrimaryBrushTypes;
	TSet<FBrushTypeInfo> RegisteredSecondaryBrushTypes;

	UPROPERTY()
	TMap<int32, TObjectPtr<UMeshSculptBrushOpProps>> BrushOpPropSets;

	TMap<int32, TUniquePtr<FMeshSculptBrushOpFactory>> BrushOpFactories;

	UPROPERTY()
	TMap<int32, TObjectPtr<UMeshSculptBrushOpProps>> SecondaryBrushOpPropSets;

	TMap<int32, TUniquePtr<FMeshSculptBrushOpFactory>> SecondaryBrushOpFactories;

	UE_API void RegisterBrushType(int32 Identifier, FText Name, TUniquePtr<FMeshSculptBrushOpFactory> Factory, UMeshSculptBrushOpProps* PropSet);
	UE_API void RegisterSecondaryBrushType(int32 Identifier, FText Name, TUniquePtr<FMeshSculptBrushOpFactory> Factory, UMeshSculptBrushOpProps* PropSet);

	UE_API virtual void SaveAllBrushTypeProperties(UInteractiveTool* SaveFromTool);
	UE_API virtual void RestoreAllBrushTypeProperties(UInteractiveTool* RestoreToTool);

protected:
	TUniquePtr<FMeshSculptBrushOp> PrimaryBrushOp;
	UMeshSculptBrushOpProps* PrimaryVisiblePropSet = nullptr;		// BrushOpPropSets prevents GC of this

	TUniquePtr<FMeshSculptBrushOp> SecondaryBrushOp;
	UMeshSculptBrushOpProps* SecondaryVisiblePropSet = nullptr;

	bool bBrushOpPropsVisible = true;

	UE_API void SetActivePrimaryBrushType(int32 Identifier);
	UE_API void SetActiveSecondaryBrushType(int32 Identifier);
	UE_API virtual TUniquePtr<FMeshSculptBrushOp>& GetActiveBrushOp();
	UE_API void SetBrushOpPropsVisibility(bool bVisible);

	//
	// Falloff types
	//
public:
	struct FFalloffTypeInfo
	{
		FText Name;
		FString StringIdentifier;
		int32 Identifier;
	};
	const TArray<FFalloffTypeInfo>& GetRegisteredPrimaryFalloffTypes() const { return RegisteredPrimaryFalloffTypes; }

	/** Set the active falloff type for the primary brush */
	UE_API virtual void SetPrimaryFalloffType(EMeshSculptFalloffType Falloff);

protected:
	TSharedPtr<FMeshSculptFallofFunc> PrimaryFalloff;

	TArray<FFalloffTypeInfo> RegisteredPrimaryFalloffTypes;
	UE_API void RegisterStandardFalloffTypes();


	//
	// Brush Size
	//
protected:
	UE::Geometry::FInterval1d BrushRelativeSizeRange;
	double CurrentBrushRadius = 1.0;
	UE_API virtual void InitializeBrushSizeRange(const UE::Geometry::FAxisAlignedBox3d& TargetBounds);
	UE_API virtual void CalculateBrushRadius();
	UE_API virtual double GetActiveBrushRadius();
	virtual double GetCurrentBrushRadius() const { return CurrentBrushRadius; }

	double CurrentBrushFalloff = 0.5;
	virtual double GetCurrentBrushFalloff() const { return CurrentBrushFalloff; }

	double ActivePressure = 1.0;
	virtual double GetActivePressure() const { return ActivePressure; }

	// returns brush strength, factoring in pressure sensitivity if applicable
	UE_API virtual double GetActiveBrushStrength();
	UE_API virtual double GetCurrentBrushStrength();
	UE_API virtual double GetCurrentBrushDepth();

public:
	UE_API virtual void IncreaseBrushRadiusAction();
	UE_API virtual void DecreaseBrushRadiusAction();
	UE_API virtual void IncreaseBrushRadiusSmallStepAction();
	UE_API virtual void DecreaseBrushRadiusSmallStepAction();


	// client currently needs to implement these...
	virtual void IncreaseBrushSpeedAction() {}
	virtual void DecreaseBrushSpeedAction() {}
	virtual void NextBrushModeAction() {}
	virtual void PreviousBrushModeAction() {}


public:
	// IInteractiveToolCameraFocusAPI override to focus on brush w/ 'F'
	UE_API virtual FBox GetWorldSpaceFocusBox() override;



	//
	// Brush/Stroke stuff
	//
protected:
	FFrame3d LastBrushFrameWorld;
	FFrame3d LastBrushFrameLocal;
	int32 LastBrushTriangleID = IndexConstants::InvalidID;
	double MinimumBrushAdaptiveSizeRatio = 0.01;

	const FFrame3d& GetBrushFrameWorld() const { return LastBrushFrameWorld; }
	const FFrame3d& GetBrushFrameLocal() const { return LastBrushFrameLocal; }
	int32 GetBrushTriangleID() const { return LastBrushTriangleID; }
	UE_API void UpdateBrushFrameWorld(const FVector3d& NewPosition, const FVector3d& NewNormal);
	UE_API void AlignBrushToView();

	bool GetBrushCanHitBackFaces() const { return BrushProperties->bHitBackFaces; }

	// These routines handle the actual ray intersection queries againt the sculpt and target meshes. There 
	// are two versions for each - a const version and a non-const version. The versions have different names
	// to avoid running afoul of deprecation rules. Subclasses may leave the non-const versions unimplemented,
	// as the default implementation simply calls the const version. Subclasses should implement the const versions,
	// to provide correct ray intersection logic for thier specific sculpt/target situation.

	/** @return hit triangle at ray position - subclass *may* implement this */
	virtual int32 FindHitSculptMeshTriangle(const FRay3d& LocalRay) { return FindHitSculptMeshTriangleConst(LocalRay); }
	/** @return hit triangle at ray position - subclass *must* implement this */
	virtual int32 FindHitSculptMeshTriangleConst(const FRay3d& LocalRay) const { check(false); return IndexConstants::InvalidID; }
	/** @return hit triangle at ray position - subclass *may* implement this */
	virtual int32 FindHitTargetMeshTriangle(const FRay3d& LocalRay) { return FindHitTargetMeshTriangleConst(LocalRay); }
	/** @return hit triangle at ray position - subclass *should* implement this for most brushes  */
	virtual int32 FindHitTargetMeshTriangleConst(const FRay3d& LocalRay) const { check(false); return IndexConstants::InvalidID;	}

	bool ProjectWorldRayOnActivePlane(const FRay& WorldRay, FVector3d& ProjectedPosition, FVector3d& ProjectedNormal) const;
	bool ProjectWorldRayOnTargetMesh(const FRay& WorldRay, bool bFallbackToViewPlane, FVector3d& ProjectedPosition, FVector3d& ProjectedNormal, int32& HitTriangle) const;
	bool ProjectWorldRayOnSculptMesh(const FRay& WorldRay, bool bFallbackToViewPlane, FVector3d& ProjectedPosition, FVector3d& ProjectedNormal, int32& HitTriangle) const;

	UE_API virtual bool UpdateBrushPositionOnActivePlane(const FRay& WorldRay);
	UE_API virtual bool UpdateBrushPositionOnTargetMesh(const FRay& WorldRay, bool bFallbackToViewPlane);
	UE_API virtual bool UpdateBrushPositionOnSculptMesh(const FRay& WorldRay, bool bFallbackToViewPlane);

	/**
	* Provides a hook for informing subclasses which triangle was last hit during a brush update.
	* Subclass should implement this if it needs to know which triangle was hit
	* @param TriangleID the triangle last hit during brush updates on the Sculpt Mesh
	* @param LocalRay the ray in local space which caused the triangle to be hit
	*/
	virtual void UpdateHitSculptMeshTriangle(int32 TriangleID, const FRay3d& LocalRay) {};

	/**
	* Provides a hook for informing subclasses which triangle was last hit during a brush update.
	* Subclass should implement this if it needs to know which triangle was hit
	* @param TriangleID the triangle last hit during brush updates on the Target Mesh
	* @param LocalRay the ray in local space which caused the triangle to be hit
	*/
	virtual void UpdateHitTargetMeshTriangle(int32 TriangleID, const FRay3d& LocalRay) {};

	//
	// Brush Target Plane is plane that some brushes move on
	//
protected:
	FFrame3d ActiveBrushTargetPlaneWorld;
	UE_API virtual void UpdateBrushTargetPlaneFromHit(const FRay& WorldRay, const FHitResult& Hit);


	//
	// Stroke Modifiers
	//
protected:
	bool bInStroke = false;
	bool bSmoothing = false;
	bool bInvert = false;
	UE_API virtual void SaveActiveStrokeModifiers();
	virtual bool InStroke() const { return bInStroke; }
	virtual bool GetInSmoothingStroke() const { return bSmoothing; }
	virtual bool GetInInvertStroke() const { return bInvert; }

private:
	struct FInputRayData
	{
		FRay Ray;
		double Timestamp;
		double Pressure;
	};
	TArray<FInputRayData> PendingInputRays;
	// If this is not the first time we're processing PendingInputRays during a stroke, the previous ray
	TOptional<FInputRayData> PreviousInputRay;
	TOptional<FVector3d> PreviousUnfilteredHitPosition;
	double PreviousUnfilteredHitRadius = 0;
	
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> StrokeGeometry;
protected:

	// when in a stroke, this function determines when a new stamp should be emitted, based on spacing and flow rate settings
	UE_API virtual void UpdateStampPendingState();

	// for tracking stroke time and length, to apply spacing and flow rate settings
	double ActiveStrokeTime = 0.0;
	double ActiveStrokePathArcLen = 0.0;
	int LastFlowTimeStamp = 0;
	int LastSpacingTimestamp = 0;
	UE_API virtual void ResetStrokeTime();
	UE_API virtual void AccumulateStrokeTime(float DeltaTime);

	//
	// Stamps
	//
protected:
	bool bIsStampPending = false;
	FRay PendingStampRay;

private:
	struct FStampRayData
	{
		FRay StampRay;
		double Pressure;
	};
	// Sequence of stamps to apply. Initialized by UpdateStampPendingState and immediately
	//  applied, though potentially with filtering
	TArray<FStampRayData> PendingStampRays;
protected:

	FSculptBrushStamp HoverStamp;
	FSculptBrushStamp CurrentStamp;
	FSculptBrushStamp LastStamp;
	UE_API virtual void UpdateHoverStamp(const FFrame3d& StampFrameWorld);
	virtual bool IsStampPending() const { return bIsStampPending; }
	virtual const FRay& GetPendingStampRayWorld() const { return PendingStampRay;  }
	// Temporal Flow Rate defines the frequency of stamp placement. 1 is max rate, 0 is no stamps. Defaults to BrushProperties->FlowRate, but subclasses can re-use that setting for other things
	UE_API virtual float GetStampTemporalFlowRate() const;

	UE_API void ProcessPerTickStamps(TFunction<bool(const FRay& StampRay)> UpdateStampPosition, TFunction<void(int StampIndex, const FRay& StampRay)> ExecuteStampOperation);
	UE_API void ProcessPerTickStamps(TFunction<bool(const FRay& StampRay)> UpdateStampPosition, TFunction<void(int StampCount )> PreExecuteStampsOperation, TFunction<void(int StampIndex, const FRay& StampRay)> ExecuteStampOperation, TFunction<void()> PostExecuteStampsOperation );

	//
	// Stamp ROI Plane is a plane used by some brush ops
	//
protected:
	FFrame3d StampRegionPlane;
	UE_API virtual FFrame3d ComputeStampRegionPlane(const FFrame3d& StampFrame, const TArray<int32>& StampTriangles, bool bIgnoreDepth, bool bViewAligned, bool bInvDistFalloff = true);
	UE_API virtual FFrame3d ComputeStampRegionPlane(const FFrame3d& StampFrame, const TSet<int32>& StampTriangles, bool bIgnoreDepth, bool bViewAligned, bool bInvDistFalloff = true);


	// Stroke plane is a plane used by some brush ops
	//
protected:
	FFrame3d StrokePlane;
	virtual const FFrame3d& GetCurrentStrokeReferencePlane() const { return StrokePlane; }

	UE_API virtual void UpdateStrokeReferencePlaneForROI(const FFrame3d& StampFrame, const TArray<int32>& TriangleROI, bool bViewAligned);
	UE_API virtual void UpdateStrokeReferencePlaneFromWorkPlane();


	//
	// Display / Material
	//
public:
	UPROPERTY()
	TObjectPtr<UMeshEditingViewProperties> ViewProperties;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> ActiveOverrideMaterial;

protected:
	UE_API virtual void SetViewPropertiesEnabled(bool bNewValue);
	UE_API virtual void UpdateWireframeVisibility(bool bNewValue);
	UE_API virtual void UpdateMaterialMode(EMeshEditingMaterialModes NewMode);
	UE_API virtual void UpdateFlatShadingSetting(bool bNewValue);
	UE_API virtual void UpdateColorSetting(FLinearColor NewColor);
	UE_API virtual void UpdateTransparentColorSetting(FLinearColor NewColor);
	UE_API virtual void UpdateOpacitySetting(double Opacity);
	UE_API virtual void UpdateTwoSidedSetting(bool bOn);
	UE_API virtual void UpdateCustomMaterial(TWeakObjectPtr<UMaterialInterface> NewMaterial);
	UE_API virtual void UpdateImageSetting(UTexture2D* NewImage);



	//
	// brush indicator
	//
protected:
	// subclasses should call this to create indicator in their ::Setup()
	UE_API virtual void InitializeIndicator();

	// Called by InitializeIndicator to create a mesh for the brush ROI indicator. Default is sphere.
	UE_API virtual UPreviewMesh* MakeBrushIndicatorMesh(UObject* Parent, UWorld* World);

	UE_API virtual void ConfigureIndicator(bool bVolumetric);
	UE_API virtual bool GetIsVolumetricIndicator();

	UE_API virtual void SetIndicatorVisibility(bool bVisible);
	UE_API virtual bool GetIndicatorVisibility() const;

	UE_API virtual void SetupBrushEditBehaviorSetup(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior);

	UE_API void MapHorizontalBrushEditBehaviorToBrushSize(ULocalTwoAxisPropertyEditInputBehavior& OutBehavior);

protected:
	UPROPERTY()
	TObjectPtr<UBrushStampIndicator> BrushIndicator;

	UPROPERTY()
	bool bIsVolumetricIndicator;


	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> BrushIndicatorMaterial;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> BrushIndicatorMesh;

	
	TWeakObjectPtr<ULocalTwoAxisPropertyEditInputBehavior> BrushEditBehavior;


	//
	// Work Plane
	//
public:
	// plane gizmo
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> PlaneTransformGizmo;

	UPROPERTY()
	TObjectPtr<UTransformProxy> PlaneTransformProxy;

protected:
	UE_API virtual void UpdateWorkPlane();
	virtual bool ShowWorkPlane() const { return false; };
	// Only relevant when ShowWorkPlane() returns true. Determines whether the plane
	//  grid lines are drawn.
	bool bDrawWorkPlaneGridLines = true;

protected:
	TValueWatcher<FVector> GizmoPositionWatcher;
	TValueWatcher<FQuat> GizmoRotationWatcher;

	UE_API virtual void UpdateGizmoFromProperties();
	UE_API virtual void PlaneTransformChanged(UTransformProxy* Proxy, FTransform Transform);

	enum class EPendingWorkPlaneUpdate
	{
		NoUpdatePending,
		MoveToHitPositionNormal,
		MoveToHitPosition,
		MoveToHitPositionViewAligned
	};
	EPendingWorkPlaneUpdate PendingWorkPlaneUpdate;
	UE_API virtual void SetFixedSculptPlaneFromWorldPos(const FVector& Position, const FVector& Normal, EPendingWorkPlaneUpdate UpdateType);
	UE_API virtual void UpdateFixedSculptPlanePosition(const FVector& Position);
	UE_API virtual void UpdateFixedSculptPlaneRotation(const FQuat& Rotation);
	UE_API virtual void UpdateFixedPlaneGizmoVisibility(bool bVisible);

protected:
	UE::TransactionUtil::FLongTransactionTracker LongTransactions;

};

#undef UE_API
