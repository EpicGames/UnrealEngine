// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Util/UniqueIndexSet.h"


namespace UE 
{ 
namespace SculptUtil 
{
	using namespace UE::Geometry;

	/** 
	 * Recompute overlay normals for the overlay elements belonging to all the ModifiedTris. 
	 * ElementSetBuffer and NormalsBuffer will be populated with all the normal overlay element IDs (you provide to allow for re-use of allocated memory)
	 */
	void RecalculateNormals_Overlay(FDynamicMesh3* Mesh, const TSet<int32>& ModifiedTris, TSet<int32>& ElementSetBuffer, TArray<int32>& NormalsBuffer);

	/**
	 * Recompute mesh vertex normals for the vertices belonging to all the ModifiedTris.
	 * VertexSetBuffer and NormalsBuffer will be populated with all the vertex IDs (you provide to allow for re-use of allocated memory)
	 */
	void RecalculateNormals_PerVertex(FDynamicMesh3* Mesh, const TSet<int32>& ModifiedTris, TSet<int32>& VertexSetBuffer, TArray<int32>& NormalsBuffer);

	/**
	 * Recompute normals for either the normals overlay or vertex normals of the mesh that belong to all the ModifiedTris
	 * TempSetBuffer and NormalsBuffer will be populated with either the recomputed overlay element IDs or vertex IDs
	 */
	void RecalculateROINormals(FDynamicMesh3* Mesh, const TSet<int32>& TriangleROI, TSet<int32>& TempSetBuffer, TArray<int32>& NormalsBuffer, bool bForceVertex = false);




	/**
	 * Recompute overlay normals for the overlay elements belonging to all the ModifiedTris.
	 * ElementSetTemp will be populated with all the normal overlay element IDs (you provide to allow for re-use of allocated memory)
	 */
	void RecalculateNormals_Overlay(FDynamicMesh3* Mesh, const TSet<int32>& ModifiedTris, FUniqueIndexSet& ElementSetTemp);

	/**
	 * Recompute mesh vertex normals for the vertices belonging to all the ModifiedTris.
	 * VertexSetTemp will be populated with all the vertex IDs (you provide to allow for re-use of allocated memory)
	 */
	void RecalculateNormals_PerVertex(FDynamicMesh3* Mesh, const TSet<int32>& ModifiedTris, FUniqueIndexSet& VertexSetTemp);

	/**
	 * Recompute normals for either the normals overlay or vertex normals of the mesh that belong to all the ModifiedTris
	 * IndexSetTemp will be populated with either the recomputed overlay element IDs or vertex IDs
	 */
	void RecalculateROINormals(FDynamicMesh3* Mesh, const TSet<int32>& TriangleROI, FUniqueIndexSet& IndexSetTemp, bool bForceVertex = false);



	/**
	 * Precalculate IndexSetTemp for the modified TriangleROI, the indices will either be overlay normal element IDs, or vertex IDs
	 * @param bIsOverlayElementsOut will be returned as true for overlay normals, false for vertex normals
	 */
	void PrecalculateNormalsROI(const FDynamicMesh3* Mesh, const TArray<int32>& TriangleROI, FUniqueIndexSet& IndexSetTemp, bool& bIsOverlayElementsOut, bool bForceVertex = false);

	/**
	 * Recalculate either overlay normals or vertex normals for the given Indices
	 * @param bIsOverlayElements if true, Indices is interpreted as overlay element IDs, otherwise as vertex IDs
	 */
	void RecalculateROINormals(FDynamicMesh3* Mesh, const TArray<int32>& Indices, bool bIsOverlayElements);



	/**
	 * Precalculate ROIFlags for the modified TriangleROI, the indices will either be overlay normal element IDs, or vertex IDs.
	 * ROIFlags will be resized to either the max overlay element ID, or the max vertex ID, and indices in the ROI will be set to true
	 * @param bIsOverlayElementsOut will be returned as true for overlay normals, false for vertex normals
	 */
	void PrecalculateNormalsROI(const FDynamicMesh3* Mesh, const TArray<int32>& TriangleROI, TArray<std::atomic<bool>>& ROIFlags, bool& bIsOverlayElementsOut, bool bForceVertex = false);


	/**
	 * Recalculate either overlay normals or vertex normals for the given indices that are true in ROIFlags
	 * @param bIsOverlayElements if true, ROIFlags is interpreted as overlay element ID flags, otherwise as vertex IDs
	 */
	void RecalculateROINormals(FDynamicMesh3* Mesh, TArray<std::atomic<bool>>& ROIFlags, bool bIsOverlayElements);

	/**
	 * Recalculate either overlay normals or vertex normals for the given indices defined by a set of triangles
	 * @param bIsOverlayElements if true, normals are computed on the overlay. Otherwise, they are computed as vertex normals
	 */
	void RecalculateROINormalForTriangles(FDynamicMesh3* Mesh, TArray<int> Triangles, bool bIsOverlayElements);

/* end namespace UE::SculptUtil */  } }