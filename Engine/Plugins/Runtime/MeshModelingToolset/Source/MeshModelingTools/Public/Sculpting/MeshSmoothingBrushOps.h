// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshWeights.h"
#include "DynamicMesh/MeshNormals.h"
#include "Async/ParallelFor.h"
#include "MeshSmoothingBrushOps.generated.h"


UCLASS(MinimalAPI)
class UBaseSmoothBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	virtual bool GetPreserveUVFlow() { return false; }
	// @return a float value in the [0,1] range controlling the desired relative weight of PolyGroup edges in smoothing (1 = maximally favor PolyGroup edges, 0 = ignore PolyGroup edges)
	virtual float GetPolyGroupEdgesWeight() { return 0.f; }
	virtual bool SupportsStrengthPressure() override { return true; }
	virtual int32 GetIterations() { return 1; }
};


UCLASS(MinimalAPI)
class USmoothBrushOpProps : public UBaseSmoothBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (ClampMin = 1, UIMax = 50))
	int32 Iterations = 1;

	/** If true, try to preserve the shape of the UV/3D mapping. This will limit Smoothing and Remeshing in some cases. */
	UPROPERTY(EditAnywhere, Category = SmoothBrush)
	bool bPreserveUVFlow = true;

	/** If true, favor smoothing along PolyGroup edges where they are present. Can help to preserve quads. No effect where PolyGroups are not defined. */
	UPROPERTY(EditAnywhere, Category = SmoothBrush)
	bool bFavorPolyGroupEdges = false;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool GetPreserveUVFlow() override { return bPreserveUVFlow; }
	virtual float GetPolyGroupEdgesWeight() override { return (float)bFavorPolyGroupEdges; }
	virtual int32 GetIterations() { return Iterations; }
};



UCLASS(MinimalAPI)
class USecondarySmoothBrushOpProps : public UBaseSmoothBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush, meta = (ClampMin = 1, UIMax = 50))
	int32 Iterations = 1;

	/** If true, try to preserve the shape of the UV/3D mapping. This will limit Smoothing and Remeshing in some cases. */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush)
	bool bPreserveUVFlow = true;

	/** If true, favor smoothing along PolyGroup edges where they are present. Can help to preserve quads. No effect where PolyGroups are not defined. */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush)
	bool bFavorPolyGroupEdges = false;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool GetPreserveUVFlow() override { return bPreserveUVFlow; }
	virtual float GetPolyGroupEdgesWeight() override { return (float)bFavorPolyGroupEdges; }
	virtual int32 GetIterations() { return Iterations; }
};



class FSmoothBrushOp : public FMeshSculptBrushOp
{

public:

	static void ApplyIterativeStamp(const FDynamicMesh3* Mesh, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut, int32 NumIters,
		TFunctionRef<FVector3d(int32 Iter, FVector3d OrigPos, int32 VID, TFunctionRef<FVector3d(int32)> GetVertexFn)> GetStampVertPos)
	{
		TMap<int32, int32> VIDToIdx;
		if (NumIters > 0)
		{
			// We need this map for iterations after the first, to go from neighbor vertex ID -> index in NewPositionsOut
			// (otherwise we'd use stale positions for the neighbor vertices)
			VIDToIdx.Reserve(Vertices.Num());
			for (int32 Idx = 0; Idx < Vertices.Num(); ++Idx)
			{
				VIDToIdx.Add(Vertices[Idx], Idx);
			}
		}

		auto MeshVertPos = [&Mesh](int32 VID) { return Mesh->GetVertex(VID); };
		auto BufferVertPos = [&Mesh, &VIDToIdx, &NewPositionsOut](int32 VID)
		{
			if (int32* Idx = VIDToIdx.Find(VID))
			{
				return NewPositionsOut[*Idx];
			}
			else
			{
				return Mesh->GetVertex(VID);
			}
		};
		for (int32 Iter = 0; Iter < NumIters; ++Iter)
		{
			ParallelFor(Vertices.Num(), [&](int32 k)
			{
				int32 VertIdx = Vertices[k];

				FVector3d OrigPos = Iter > 0 ? NewPositionsOut[k] : Mesh->GetVertex(VertIdx);
				TFunctionRef<FVector3d(int32)> GetVertexFn = Iter == 0 ?
					TFunctionRef<FVector3d(int32)>(MeshVertPos) :
					TFunctionRef<FVector3d(int32)>(BufferVertPos);
				NewPositionsOut[k] = GetStampVertPos(Iter, OrigPos, VertIdx, GetVertexFn);
			});
		}
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;
		bool bPreserveUVFlow = GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetPreserveUVFlow();
		int32 NumIters = GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetIterations();
		ApplyIterativeStamp(Mesh, Vertices, NewPositionsOut, NumIters,
		[&](int32 Iter, FVector3d OrigPos, int32 VID, TFunctionRef<FVector3d(int32)> GetVertexFn)->FVector3d
		{
			double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);
			FVector3d SmoothedPos = OrigPos;
			if (bPreserveUVFlow)
			{
				SmoothedPos = UE::Geometry::FMeshWeights::CotanCentroidSafe(*Mesh, VID, GetVertexFn, 10.0);
			}
			else
			{
				SmoothedPos = UE::Geometry::FMeshWeights::UniformCentroid(*Mesh, VID, GetVertexFn);
			}
			double PolyGroupEdgesWeight = (double)GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetPolyGroupEdgesWeight();
			if (PolyGroupEdgesWeight > 0 && Mesh->HasTriangleGroups())
			{
				FVector3d PolyGroupNbrSum = FVector3d::Zero();
				int32 FoundGroupEdges = 0;
				Mesh->EnumerateVertexEdges(VID, [&PolyGroupNbrSum, &FoundGroupEdges, &Mesh, &GetVertexFn, VID](int32 EID)
				{
					const FDynamicMesh3::FEdge& Edge = Mesh->GetEdgeRef(EID);
					if (Mesh->GetTriangleGroup(Edge.Tri.A) != Mesh->GetTriangleGroup(Edge.Tri.B))
					{
						FoundGroupEdges++;
						PolyGroupNbrSum += GetVertexFn(Edge.Vert.OtherElement(VID));
					}
				});
				if (FoundGroupEdges > 0)
				{
					double UseGroupEdgeWeight = FMath::Min(1., PolyGroupEdgesWeight);
					SmoothedPos = UE::Geometry::Lerp(SmoothedPos, PolyGroupNbrSum / (double)FoundGroupEdges, UseGroupEdgeWeight);
				}
			}
			
			return UE::Geometry::Lerp(OrigPos, SmoothedPos, Falloff * Stamp.Power);
		});
	}
};





UCLASS(MinimalAPI)
class USmoothFillBrushOpProps : public UBaseSmoothBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SmoothFillBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = SmoothFillBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	/** If true, try to preserve the shape of the UV/3D mapping. This will limit Smoothing and Remeshing in some cases. */
	UPROPERTY(EditAnywhere, Category = SmoothFillBrush)
	bool bPreserveUVFlow = false;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool GetPreserveUVFlow() override { return bPreserveUVFlow; }
};



class FSmoothFillBrushOp : public FMeshSculptBrushOp
{

public:

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;
		double Direction = Stamp.Direction;
		bool bPreserveUVFlow = GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetPreserveUVFlow();

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];

			FVector3d OrigPos = Mesh->GetVertex(VertIdx);
			// TODO: If adding Iterations option to this brush, need to also make a ComputeVertexNormals that takes a TFunctionRef for getting vertex positions
			FVector3d Normal = UE::Geometry::FMeshNormals::ComputeVertexNormal(*Mesh, VertIdx);

			double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);

			FVector3d SmoothedPos = OrigPos;
			if (bPreserveUVFlow)
			{
				SmoothedPos = UE::Geometry::FMeshWeights::CotanCentroidSafe(*Mesh, VertIdx, 10.0);
			}
			else
			{
				SmoothedPos = UE::Geometry::FMeshWeights::UniformCentroid(*Mesh, VertIdx);
			}

			FVector3d NewPos = UE::Geometry::Lerp(OrigPos, SmoothedPos, Falloff * Stamp.Power);

			if ((NewPos - OrigPos).Dot(Normal) * Direction < 0)
			{
				NewPositionsOut[k] = OrigPos;
			}
			else
			{
				NewPositionsOut[k] = NewPos;
			}
		});
	}
};





UCLASS(MinimalAPI)
class UFlattenBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = FlattenBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = FlattenBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	/** Control whether effect of brush should be limited to one side of the Plane  */
	UPROPERTY(EditAnywhere, Category = FlattenBrush)
	EPlaneBrushSideMode WhichSide = EPlaneBrushSideMode::BothSides;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};


class FFlattenBrushOp : public FMeshSculptBrushOp
{

public:
	double BrushSpeedTuning = 5.0;

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		static const double PlaneSigns[3] = { 0, -1, 1 };
		double PlaneSign = PlaneSigns[(int32)GetPropertySetAs<UFlattenBrushOpProps>()->WhichSide];

		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		// Note that (unlike w/ other brushes) we don't multiply by a Stamp.Radius factor here, 
		// because we make speed proportional to distance from target plane, which already gives an implicit ~Stamp.Radius factor
		// (as distance from plane tends to increase proportionally to distance from brush center, for a surface that isn't already plane-aligned)
		double UseSpeed = Stamp.Power * Stamp.DeltaTime * BrushSpeedTuning;
		const FFrame3d& FlattenPlane = Stamp.RegionPlane;
		FVector3d PlaneZ = FlattenPlane.Z();

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);
			FVector3d Normal = UE::Geometry::FMeshNormals::ComputeVertexNormal(*Mesh, VertIdx);
			FVector3d PlanePos = FlattenPlane.ToPlane(OrigPos, 2);
			FVector3d MoveDelta = PlanePos - OrigPos;

			double PlaneDot = MoveDelta.Dot(FlattenPlane.Z());
			FVector3d NewPos = OrigPos;
			if (PlaneDot * PlaneSign >= 0)
			{
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);
				FVector3d MoveVec = Falloff * UseSpeed * MoveDelta;
				double MaxDist = UE::Geometry::Normalize(MoveDelta);
				NewPos = (MoveVec.SquaredLength() > MaxDist * MaxDist) ?
					PlanePos : (OrigPos + Falloff * MoveVec);
			}

			NewPositionsOut[k] = NewPos;
		});
	}


	virtual bool WantsStampRegionPlane() const
	{
		return true;
	}
};






UCLASS(MinimalAPI)
class UEraseBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = EraseBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = EraseBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }
};


class FEraseToBaseMeshBrushOp : public FMeshSculptBrushOp
{
public:
	double BrushSpeedTuning = 3.0;

	typedef TUniqueFunction<bool(int32, const FVector3d&, double, FVector3d&, FVector3d&)> NearestQueryFuncType;

	NearestQueryFuncType BaseMeshNearestQueryFunc;

	FEraseToBaseMeshBrushOp(NearestQueryFuncType QueryFunc)
	{
		BaseMeshNearestQueryFunc = MoveTemp(QueryFunc);
	}

	virtual ESculptBrushOpTargetType GetBrushTargetType() const override
	{
		return ESculptBrushOpTargetType::TargetMesh;
	}

	virtual void ApplyStamp(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override
	{
		double UsePower = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;
		double MaxOffset = Stamp.Radius;

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VertIdx = Vertices[k];
			FVector3d OrigPos = Mesh->GetVertex(VertIdx);

			FVector3d BasePos, BaseNormal;
			bool bFoundBasePos = BaseMeshNearestQueryFunc(VertIdx, OrigPos, 4.0 * Stamp.Radius, BasePos, BaseNormal);
			if (bFoundBasePos == false)
			{
				NewPositionsOut[k] = OrigPos;
			}
			else
			{
				FVector3d MoveVec = (BasePos - OrigPos); 
				double Falloff = GetFalloff().Evaluate(Stamp, OrigPos);
				double MoveDist = Falloff * UsePower;
				if (MoveVec.SquaredLength() < MoveDist * MoveDist)
				{
					NewPositionsOut[k] = BasePos;
				}
				else
				{
					UE::Geometry::Normalize(MoveVec);
					NewPositionsOut[k] = OrigPos + MoveDist * MoveVec;
				}
			}
		});
	}

};
