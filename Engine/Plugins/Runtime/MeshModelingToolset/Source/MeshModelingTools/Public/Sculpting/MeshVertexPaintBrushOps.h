// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshWeights.h"
#include "MathUtil.h"
#include "VectorTypes.h"
#include "Async/ParallelFor.h"
#include "MeshVertexPaintBrushOps.generated.h"


UENUM()
enum class EVertexColorPaintBrushOpBlendMode
{
	Lerp = 0,
	Mix = 1,
	Multiply = 2
};



namespace UELocal
{
	template <typename RealType>
	inline UE::Math::TVector4<RealType> BlendColors_Mix(
		const UE::Math::TVector4<RealType>& BackColor,
		const UE::Math::TVector4<RealType>& ForeColor,
		RealType Alpha)
	{
		UE::Math::TVector4<RealType> Result;
		RealType OneMinusAlpha = (RealType)1 - Alpha;

		//for (int32 k = 0; k < 3; ++k)
		//{
		//	Result[k] = TMathUtil<RealType>::Sqrt( OneMinusAlpha*(A[k]*A[k])  + Alpha*(B[k]*B[k]) );
		//	Result[k] = TMathUtil<RealType>::Clamp(Result[k], (RealType)0, (RealType)1);
		//}
		//Result.W = TMathUtil<RealType>::Clamp(OneMinusAlpha*A.W + Alpha*B.W, (RealType)0, (RealType)1);

		// pretty sure this is assuming linear color space...

		RealType ForeAlpha = ForeColor.W * Alpha;
		Result.W = ForeAlpha + BackColor.W * (1 - ForeAlpha);
		for (int32 k = 0; k < 3; ++k)
		{
			Result[k] = ForeColor[k]*ForeAlpha + BackColor[k]*BackColor.W*(1-ForeAlpha);
			if (Result.W > TMathUtil<RealType>::ZeroTolerance)
			{
				Result[k] /= Result.W;
			}
		}

		Result.X = FMathf::Clamp(Result.X, (RealType)0, (RealType)1);
		Result.Y = FMathf::Clamp(Result.Y, (RealType)0, (RealType)1);
		Result.Z = FMathf::Clamp(Result.Z, (RealType)0, (RealType)1);
		Result.W = FMathf::Clamp(Result.W, (RealType)0, (RealType)1);

		return Result;
	}


	template <typename RealType>
	inline UE::Math::TVector4<RealType> BlendColors_Lerp(
		const UE::Math::TVector4<RealType>& A,
		const UE::Math::TVector4<RealType>& B,
		RealType Alpha)
	{
		UE::Math::TVector4<RealType> Result;
		RealType OneMinusAlpha = (RealType)1 - Alpha;
		for (int32 k = 0; k < 4; ++k)
		{
			Result[k] = TMathUtil<RealType>::Clamp(OneMinusAlpha*A[k] + Alpha*B[k], (RealType)0, (RealType)1);
		}
		return Result;
	}

	template <typename RealType>
	inline UE::Math::TVector4<RealType> BlendColors_Multiply(
		const UE::Math::TVector4<RealType>& A,
		const UE::Math::TVector4<RealType>& B,
		RealType Alpha)
	{
		UE::Math::TVector4<RealType> Result;
		Result.X = A.X * B.X;
		Result.Y = A.Y * B.Y;
		Result.Z = A.Z * B.Z;
		Result.W = A.W * B.W;
		return BlendColors_Lerp(A, Result, Alpha);
	}


	template <typename RealType>
	inline UE::Math::TVector4<RealType> BlendColors(
		const UE::Math::TVector4<RealType>& Background,
		const UE::Math::TVector4<RealType>& Foreground,
		RealType Alpha,
		EVertexColorPaintBrushOpBlendMode BlendMode)
	{
		switch (BlendMode)
		{
		case EVertexColorPaintBrushOpBlendMode::Multiply:
			return BlendColors_Multiply(Background, Foreground, Alpha);
			break;
		case EVertexColorPaintBrushOpBlendMode::Mix:
			return BlendColors_Mix(Background, Foreground, Alpha);
			break;
		case EVertexColorPaintBrushOpBlendMode::Lerp:
		default:
			return BlendColors_Lerp(Background, Foreground, Alpha);
		}
	}
}



UCLASS(MinimalAPI)
class UVertexColorBaseBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()
public:

	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = PaintBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 1.0;

	UPROPERTY(EditAnywhere, Category = PaintBrush)
	EVertexColorPaintBrushOpBlendMode BlendMode = EVertexColorPaintBrushOpBlendMode::Mix;

	/** If bApplyFalloff is disabled, 1.0 is used as "falloff" for all vertices */
	UPROPERTY(EditAnywhere, Category = PaintBrush)
	bool bApplyFalloff = true;

	virtual bool SupportsStrengthPressure() override { return true; };
	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual void SetFalloff(float NewFalloff) override { Falloff = FMathf::Clamp(NewFalloff, 0.0f, 1.0f); }
};





UCLASS(MinimalAPI)
class UVertexColorPaintBrushOpProps : public UVertexColorBaseBrushOpProps
{
	GENERATED_BODY()
public:
	/** The paint color */
	UPROPERTY(EditAnywhere, Category = Paint, meta = (DisplayName = "Color"))
	FLinearColor Color;

	virtual FLinearColor GetColor() const { return Color; }
};


UCLASS(MinimalAPI)
class UVertexColorSoftenBrushOpProps : public UVertexColorBaseBrushOpProps
{
	GENERATED_BODY()
public:

	virtual float GetFalloff() override { return 1.0f; }
};

UCLASS(MinimalAPI)
class UVertexColorSmoothBrushOpProps : public UVertexColorBaseBrushOpProps
{
	GENERATED_BODY()
public:
};



namespace UE::Geometry
{


class FMeshVertexColorBrushOp : public FMeshSculptBrushOp
{
public:

	// not supported for this kind of brush op
	virtual void ApplyStamp(
		const FDynamicMesh3* Mesh,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices,
		TArray<FVector3d>& NewColorsOut) override
	{
		check(false);
	}


	virtual void ApplyStampToVertexColors(
		const FDynamicMesh3* Mesh,
		const FDynamicMeshColorOverlay* VertexColors,
		TArray<FVector4f>& InitialVertexColorBuffer,
		TArray<FVector4f>& StrokeVertexColorBuffer,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& ElementIDs,
		TArray<FVector4f>& NewColorsOut) = 0;

};




class FVertexPaintBrushOp : public FMeshVertexColorBrushOp
{
public:
	double BrushSpeedTuning = 6.0;

	virtual void ApplyStampToVertexColors(
		const FDynamicMesh3* Mesh,
		const FDynamicMeshColorOverlay* VertexColors,
		TArray<FVector4f>& InitialVertexColorBuffer,
		TArray<FVector4f>& StrokeVertexColorBuffer,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& ElementIDs,
		TArray<FVector4f>& NewColorsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		//double UsePower = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;
		double UsePower = Stamp.Power;

		UVertexColorPaintBrushOpProps* Props = GetPropertySetAs<UVertexColorPaintBrushOpProps>();
		FVector4f SetColor = (FVector4f)Props->GetColor();
		float BlendStrokeAlpha = SetColor.W; 
		if (Props->BlendMode == EVertexColorPaintBrushOpBlendMode::Mix)
		{
			SetColor.W = 1.0f;
		}
		bool bApplyFalloff = Props->bApplyFalloff;

		int32 NumElements = ElementIDs.Num();
		ensure(NewColorsOut.Num() == NumElements);
		ParallelFor(NumElements, [&](int32 k)
		{
			int32 ElementID = ElementIDs[k];
			int32 VertexID = VertexColors->GetParentVertex(ElementID);
			FVector3d VertexPos = Mesh->GetVertex(VertexID);
			double PositionFalloff = (bApplyFalloff) ? GetFalloff().Evaluate(Stamp, VertexPos) : 1.0;
			double T = PositionFalloff * UsePower;

			if (Props->BlendMode == EVertexColorPaintBrushOpBlendMode::Mix)
			{
				FVector4f CurStrokeColor = StrokeVertexColorBuffer[ElementID];
				StrokeVertexColorBuffer[ElementID] = UELocal::BlendColors_Mix(CurStrokeColor, SetColor, (float)T);
				FVector4f OverColor = StrokeVertexColorBuffer[ElementID];

				FVector4f BaseColor = InitialVertexColorBuffer[ElementID];
				NewColorsOut[k] = UELocal::BlendColors(BaseColor, OverColor, BlendStrokeAlpha, Props->BlendMode);
			}
			else
			{
				FVector4f CurColor = VertexColors->GetElement(ElementIDs[k]);
				NewColorsOut[k] = UELocal::BlendColors(CurColor, SetColor, (float)T, Props->BlendMode);
			}
		});

	}

	virtual bool SupportsVariableSpacing() const override
	{
		return true;
	}

	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}
};



/**
 * Soften brush averages colors at any vertices that have split elements, effectively un-splitting them
 * Currently this averaging is done in a single step, ie falloff/etc is ignored, but it could be done more gradually...
 */
class FVertexColorSoftenBrushOp : public FMeshVertexColorBrushOp
{
public:
	TArray<int32> TempElementIDs;

	virtual void ApplyStampToVertexColors(
		const FDynamicMesh3* Mesh,
		const FDynamicMeshColorOverlay* VertexColors,
		TArray<FVector4f>& InitialVertexColorBuffer,
		TArray<FVector4f>& StrokeVertexColorBuffer,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& ElementIDs,
		TArray<FVector4f>& NewColorsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		int32 NumElements = ElementIDs.Num();
		ensure(NewColorsOut.Num() == NumElements);
		for (int32 k = 0; k < NumElements; ++k)
		{
			int32 VertexID = VertexColors->GetParentVertex(ElementIDs[k]);
			TempElementIDs.Reset();
			VertexColors->GetVertexElements(VertexID, TempElementIDs);
			if (TempElementIDs.Num() > 1)
			{
				FVector4f SumColor = FVector4f::Zero();
				for (int32 ElementID : TempElementIDs)
				{
					SumColor += VertexColors->GetElement(ElementID);
				}
				SumColor /= (double)TempElementIDs.Num();
				NewColorsOut[k] = SumColor;
			}
			else
			{
				NewColorsOut[k] = VertexColors->GetElement(ElementIDs[k]);
			}
		}

	}
};




class FVertexColorSmoothBrushOp : public FMeshVertexColorBrushOp
{
public:
	double BrushSpeedTuning = 6.0;
	TArray<int32> TempElementIDs;
	TArray<int32> ElementIDVertices;
	TArray<int32> ElementIDVertexIndices;
	TArray<int32> UniqueVertices;
	TArray<FVector4f> SoftenedVertexColors;
	TArray<FVector4f> SmoothedVertexColors;


	virtual void ApplyStampToVertexColors(
		const FDynamicMesh3* Mesh,
		const FDynamicMeshColorOverlay* VertexColors,
		TArray<FVector4f>& InitialVertexColorBuffer,
		TArray<FVector4f>& StrokeVertexColorBuffer,
		const FSculptBrushStamp& Stamp,
		const TArray<int32>& ElementIDs,
		TArray<FVector4f>& NewColorsOut) override
	{
		const FVector3d& StampPos = Stamp.LocalFrame.Origin;

		//double UsePower = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;
		double UsePower = Stamp.Power;

		UVertexColorSmoothBrushOpProps* Props = GetPropertySetAs<UVertexColorSmoothBrushOpProps>();

		int32 NumElements = ElementIDs.Num();
		ElementIDVertices.SetNum(NumElements, EAllowShrinking::No);
		ElementIDVertexIndices.SetNum(NumElements, EAllowShrinking::No);
		ensure(NewColorsOut.Num() == NumElements);
		UniqueVertices.Reset();
		for (int32 k = 0; k < NumElements; ++k)
		{
			int32 VertexID = VertexColors->GetParentVertex(ElementIDs[k]);
			ElementIDVertices[k] = VertexID;
			int32 VertexIndex = (int32)UniqueVertices.AddUnique(VertexID);
			ElementIDVertexIndices[k] = VertexIndex;
		}

		// compute color at each vertex that blends any split elements
		int32 NumVertices = UniqueVertices.Num();
		SoftenedVertexColors.SetNum(NumVertices, EAllowShrinking::No);
		ParallelFor(NumVertices, [&](int32 Index)
		{
			TArray<int32> Temp;
			Temp.Reserve(16);
			VertexColors->GetVertexElements(UniqueVertices[Index], Temp);
			SoftenedVertexColors[Index] = FVector4f::Zero();
			int32 Count = 0;
			for (int32 ElementID : Temp)
			{
				if (ElementIDs.Contains(ElementID))
				{
					SoftenedVertexColors[Index] += VertexColors->GetElement(ElementID);
					Count++;
				}
			}
			if (Count > 1)
			{
				SoftenedVertexColors[Index] /= (double)Count;
			}
		});
		

		// now compute one-ring-smoothed vertex colors
		SmoothedVertexColors.SetNum(NumVertices, EAllowShrinking::No);
		ParallelFor(NumVertices, [&](int32 Index)
		{
			FVector4f SmoothedColor = FVector4f::Zero();
			int32 NbrCount = 0;
			int32 VertexID = UniqueVertices[Index];
			for (int32 NbrVertexID : Mesh->VtxVerticesItr(VertexID))
			{
				int32 NbrIndex = UniqueVertices.IndexOfByKey(NbrVertexID);
				if (NbrIndex != INDEX_NONE)
				{
					const float GeometryWeight = 1.0f;
					SmoothedColor += GeometryWeight * SoftenedVertexColors[NbrIndex];
					NbrCount++;
				}
			}
			if (NbrCount > 1)
			{
				SmoothedColor /= (double)NbrCount;
			}
			SmoothedVertexColors[Index] = SmoothedColor;
		} );

		// now blend each element color w/ it's existing smoothed color
		ParallelFor(NumElements, [&](int32 k)
		{
			int32 VertexID = ElementIDVertices[k];

			FVector3d VertexPos = Mesh->GetVertex(VertexID);
			double PositionFalloff = GetFalloff().Evaluate(Stamp, VertexPos);
			double T = PositionFalloff * UsePower;

			FVector4f CurColor = VertexColors->GetElement(ElementIDs[k]);
			int32 VertIndex = ElementIDVertexIndices[k];
			FVector4f SetColor = SmoothedVertexColors[VertIndex];

			//NewColorsOut[k] = UELocal::BlendColors_Mix(CurColor, SetColor, (float)T);
			NewColorsOut[k] = UELocal::BlendColors_Lerp(CurColor, SetColor, (float)T);
		});

	}
};


}
