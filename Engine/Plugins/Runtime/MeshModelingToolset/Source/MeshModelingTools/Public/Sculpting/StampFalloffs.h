// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Sculpting/MeshBrushOpBase.h"


namespace UE
{
	namespace SculptFalloffs
	{
		using namespace UE::Geometry;

		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeStandardSmoothFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double Dist = Distance(Position, Stamp.LocalFrame.Origin);
				double FalloffT = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double UnitDistance = Dist / Stamp.Radius;
				if (UnitDistance > FalloffT)
				{
					UnitDistance = FMathd::Clamp((UnitDistance - FalloffT) / (1.0 - FalloffT), 0.0, 1.0);
					double Weight = (1.0 - UnitDistance * UnitDistance);
					return Weight * Weight * Weight;
				}
				return 1.0;
			};
		}



		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeLinearFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double Dist = Distance(Position, Stamp.LocalFrame.Origin);
				double FalloffT = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double UnitDistance = Dist / Stamp.Radius;
				if (UnitDistance > FalloffT)
				{
					UnitDistance = FMathd::Clamp((UnitDistance - FalloffT) / (1.0 - FalloffT), 0.0, 1.0);
					double Weight = FMathd::Clamp(1.0 - UnitDistance, 0.0, 1.0);
					return Weight;
				}
				return 1.0;
			};
		}




		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeInverseFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double Dist = Distance(Position, Stamp.LocalFrame.Origin);
				double FalloffT = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double UnitDistance = Dist / Stamp.Radius;
				if (UnitDistance > FalloffT)
				{
					UnitDistance = FMathd::Clamp((UnitDistance - FalloffT) / (1.0 - FalloffT), 0.0, 1.0);
					double Weight = FMathd::Clamp(1.0 - UnitDistance, 0.0, 1.0);
					return Weight * Weight * Weight;
				}
				return 1.0;
			};
		}



		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeRoundFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double Dist = Distance(Position, Stamp.LocalFrame.Origin);
				double FalloffT = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double UnitDistance = Dist / Stamp.Radius;
				if (UnitDistance > FalloffT)
				{
					UnitDistance = FMathd::Clamp((UnitDistance - FalloffT) / (1.0 - FalloffT), 0.0, 1.0);
					return FMathd::Clamp(1.0 - UnitDistance * UnitDistance, 0.0, 1.0);
				}
				return 1.0;
			};
		}



		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeSmoothBoxFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double f = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double InnerRadius = f * Stamp.Radius;
				double FalloffWidth = Stamp.Radius - InnerRadius;
				double BoxHalfWidth = FMathd::InvSqrt2 * InnerRadius;

				FVector3d LocalPos = Stamp.LocalFrame.ToFramePoint(Position);
				FAxisAlignedBox3d Box(FVector3d::Zero(), BoxHalfWidth);
				if (Box.Contains(LocalPos))
				{
					return 1.0;
				}
				else
				{
					double DistSqr = Box.DistanceSquared(LocalPos);
					if (DistSqr < FalloffWidth * FalloffWidth)
					{
						double t = FMathd::Clamp(FMathd::Sqrt(DistSqr) / FalloffWidth, 0.0, 1.0);
						double w = (1.0 - t * t);
						return w * w * w;
					}
					return 0.0;
				}
			};
		}



		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeLinearBoxFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double f = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double InnerRadius = f * Stamp.Radius;
				double FalloffWidth = Stamp.Radius - InnerRadius;
				double BoxHalfWidth = FMathd::InvSqrt2 * InnerRadius;

				FVector3d LocalPos = Stamp.LocalFrame.ToFramePoint(Position);
				FAxisAlignedBox3d Box(FVector3d::Zero(), BoxHalfWidth);
				if (Box.Contains(LocalPos))
				{
					return 1.0;
				}
				else
				{
					double DistSqr = Box.DistanceSquared(LocalPos);
					if (DistSqr < FalloffWidth * FalloffWidth)
					{
						double t = FMathd::Clamp(FMathd::Sqrt(DistSqr) / FalloffWidth, 0.0, 1.0);
						return (1.0 - t);
					}
					return 0.0;
				}
			};
		}



		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeRoundBoxFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double f = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double InnerRadius = f * Stamp.Radius;
				double FalloffWidth = Stamp.Radius - InnerRadius;
				double BoxHalfWidth = FMathd::InvSqrt2 * InnerRadius;

				FVector3d LocalPos = Stamp.LocalFrame.ToFramePoint(Position);
				FAxisAlignedBox3d Box(FVector3d::Zero(), BoxHalfWidth);
				if (Box.Contains(LocalPos))
				{
					return 1.0;
				}
				else
				{
					double DistSqr = Box.DistanceSquared(LocalPos);
					if (DistSqr < FalloffWidth * FalloffWidth)
					{
						double t = FMathd::Clamp(FMathd::Sqrt(DistSqr) / FalloffWidth, 0.0, 1.0);
						return (1.0 - t*t);
					}
					return 0.0;
				}
			};
		}



		static TUniqueFunction<double(const FSculptBrushStamp&, const FVector3d&)> MakeInverseBoxFalloff()
		{
			return [](const FSculptBrushStamp& Stamp, const FVector3d& Position)
			{
				double f = FMathd::Clamp(1.0 - Stamp.Falloff, 0.0, 1.0);
				double InnerRadius = f * Stamp.Radius;
				double FalloffWidth = Stamp.Radius - InnerRadius;
				double BoxHalfWidth = FMathd::InvSqrt2 * InnerRadius;

				FVector3d LocalPos = Stamp.LocalFrame.ToFramePoint(Position);
				FAxisAlignedBox3d Box(FVector3d::Zero(), BoxHalfWidth);
				if (Box.Contains(LocalPos))
				{
					return 1.0;
				}
				else
				{
					double DistSqr = Box.DistanceSquared(LocalPos);
					if (DistSqr < FalloffWidth * FalloffWidth)
					{
						double t = FMathd::Clamp(FMathd::Sqrt(DistSqr) / FalloffWidth, 0.0, 1.0);
						double w = (1.0 - t);
						return w * w * w;
					}
					return 0.0;
				}
			};
		}




	}
}


