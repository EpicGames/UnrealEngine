// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/BaseMeshProcessingTool.h"
#include "SmoothMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API


UENUM()
enum class ESmoothMeshToolSmoothType : uint8
{
	/** Iterative smoothing with N iterations */
	Iterative UMETA(DisplayName = "Fast Iterative"),

	/** Implicit smoothing, produces smoother output and does a better job at preserving UVs, but can be very slow on large meshes */
	Implicit UMETA(DisplayName = "Fast Implicit"),

	/** Iterative implicit-diffusion smoothing with N iterations */
	Diffusion UMETA(DisplayName = "Iterative Diffusion")
};



/** PropertySet for properties affecting the Smoother. */
UCLASS(MinimalAPI)
class USmoothMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Type of smoothing to apply */
	UPROPERTY(EditAnywhere, Category = SmoothingType)
	ESmoothMeshToolSmoothType SmoothingType = ESmoothMeshToolSmoothType::Iterative;
};



/** Properties for Iterative Smoothing */
UCLASS(MinimalAPI)
class UIterativeSmoothProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Amount of smoothing allowed per step. Smaller steps will avoid things like collapse of small/thin features. */
	UPROPERTY(EditAnywhere, Category = IterativeSmoothingOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float SmoothingPerStep = 0.8f;

	/** Number of Smoothing iterations */
	UPROPERTY(EditAnywhere, Category = IterativeSmoothingOptions, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	int32 Steps = 10;

	/** If this is false, the smoother will try to reshape the triangles to be more regular, which will distort UVs */
	UPROPERTY(EditAnywhere, Category = IterativeSmoothingOptions)
	bool bSmoothBoundary = true;
};



/** Properties for Diffusion Smoothing */
UCLASS(MinimalAPI)
class UDiffusionSmoothProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Amount of smoothing allowed per step. Smaller steps will avoid things like collapse of small/thin features. */
	UPROPERTY(EditAnywhere, Category = DiffusionSmoothingOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float SmoothingPerStep = 0.8f;

	/** Number of Smoothing iterations */
	UPROPERTY(EditAnywhere, Category = DiffusionSmoothingOptions, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000"))
	int32 Steps = 1;

	/** If this is false, the smoother will try to reshape the triangles to be more regular, which will distort UVs */
	UPROPERTY(EditAnywhere, Category = DiffusionSmoothingOptions)
	bool bPreserveUVs = true;
};




/** Properties for Implicit smoothing */
UCLASS(MinimalAPI)
class UImplicitSmoothProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Smoothing speed */
	//UPROPERTY(EditAnywhere, Category = ImplicitSmoothing, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	UPROPERTY()
	float SmoothSpeed = 0.1f;

	/** Desired Smoothness. This is not a linear quantity, but larger numbers produce smoother results */
	UPROPERTY(EditAnywhere, Category = ImplicitSmoothingOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "100.0"))
	float Smoothness = 0.2f;

	/** If this is false, the smoother will try to reshape the triangles to be more regular, which will distort UVs */
	UPROPERTY(EditAnywhere, Category = ImplicitSmoothingOptions)
	bool bPreserveUVs = true;

	/** Magic number that allows you to try to correct for shrinking caused by smoothing */
	UPROPERTY(EditAnywhere, Category = ImplicitSmoothingOptions, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "10.0"))
	float VolumeCorrection = 0.0f;
};



UCLASS(MinimalAPI)
class USmoothWeightMapSetProperties : public UWeightMapSetProperties
{
	GENERATED_BODY()
public:

	/** Fractional Minimum Smoothing Parameter in World Units, for Weight Map values of zero */
	UPROPERTY(EditAnywhere, Category = WeightMap, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "1.0", DisplayPriority = 5))
	float MinSmoothMultiplier = 0.0f;
};




/**
 * Mesh Smoothing Tool
 */
UCLASS(MinimalAPI)
class USmoothMeshTool : public UBaseMeshProcessingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API USmoothMeshTool();

	UE_API virtual void InitializeProperties() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	virtual bool RequiresInitialVtxNormals() const { return true; }
	UE_API virtual bool HasMeshTopologyChanged() const override;

	UE_API virtual FText GetToolMessageString() const override;
	UE_API virtual FText GetAcceptTransactionName() const override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:
	UPROPERTY()
	TObjectPtr<USmoothMeshToolProperties> SmoothProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UIterativeSmoothProperties> IterativeProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UDiffusionSmoothProperties> DiffusionProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UImplicitSmoothProperties> ImplicitProperties = nullptr;

	UPROPERTY()
	TObjectPtr<USmoothWeightMapSetProperties> WeightMapProperties = nullptr;
};




/**
 *
 */
UCLASS(MinimalAPI)
class USmoothMeshToolBuilder : public UBaseMeshProcessingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

#undef UE_API
