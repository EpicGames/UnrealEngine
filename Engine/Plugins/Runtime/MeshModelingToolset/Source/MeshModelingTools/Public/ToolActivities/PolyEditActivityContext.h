// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "GeometryBase.h"

#include "PolyEditActivityContext.generated.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(class FDynamicMeshChange);
PREDECLARE_GEOMETRY(struct FGroupTopologySelection);
PREDECLARE_GEOMETRY(class FGroupTopology);
class FToolCommandChange;
class UPolyEditCommonProperties;
class UPolygonSelectionMechanic;
class UMeshOpPreviewWithBackgroundCompute;

UCLASS(MinimalAPI)
class UPolyEditActivityContext : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	TObjectPtr<UPolyEditCommonProperties> CommonProperties;

	/** 
	 * The CurrentMesh is the authoritative current version of the mesh, which would be baked back
	 * to the asset on accept.
	 */
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> CurrentMesh;

	/**
	 * The mesh stored in the preview is not authoritative. It may be altered in various ways as the 
	 * user previews potential changes, and may be reset back to CurrentMesh if an activity
	 * is cancelled.
	 */
	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	/** 
	 * The activity should update CurrentTopology and MeshSpatial if it alters CurrentMesh. 
	 */
	TSharedPtr<UE::Geometry::FGroupTopology, ESPMode::ThreadSafe> CurrentTopology;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> MeshSpatial;

	bool bTriangleMode = false;

	/** 
	 * The activity may use the selection mechanic to get (or alter) the current selection, though
	 * if selection is just being changed at the end of the activity, it should probably be done
	 * through EmitCurrentMeshChangeAndUpdate so that it is lumped with the same undo transaction.
	 */
	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic;

	/**
	 * Given a change to CurrentMesh (already applied and stored in MeshChangeIn), use this
	 * function to properly emit it from the host. Be aware that the host will update any 
	 * related data structures at this time (spatial, preview, etc) so that state is consistent
	 * with undo/redo.
	 */
	TUniqueFunction<void (const FText& TransactionLabel,
			TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChangeIn,
			const UE::Geometry::FGroupTopologySelection& OutputSelection)> 
		EmitCurrentMeshChangeAndUpdate;

	/**
	 * If an activity starts running, it should use this function to have the host emit an
	 * appropriate undoable transaction with the given transaction label.
	 */
	TUniqueFunction<void(const FText& TransactionLabel)> EmitActivityStart;

	/**
	 * Gets broadcast when the CurrentMesh is modified by an undo/redo transaction emitted via
	 * EmitCurrentMeshChangeAndUpdate. Only activities that issue multiple transactions via 
	 * EmitCurrentMeshChangeAndUpdate during the same invocation need to use this, since a 
	 * transaction that ends immediately after the call will not have to deal with the undo.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnUndoRedo, bool bGroupTopologyChanged);
	FOnUndoRedo OnUndoRedo;
};
