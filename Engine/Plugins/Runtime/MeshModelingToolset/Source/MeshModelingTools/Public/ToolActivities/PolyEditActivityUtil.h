// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UInteractiveTool;
class UPolyEditActivityContext;
class UPolyEditPreviewMesh;

namespace UE {
namespace Geometry {
namespace PolyEditActivityUtil {

	enum class EPreviewMaterialType
	{
		SourceMaterials, PreviewMaterial, UVMaterial
	};

	UPolyEditPreviewMesh* CreatePolyEditPreviewMesh(UInteractiveTool& Tool, const UPolyEditActivityContext& ActivityContext);

	void UpdatePolyEditPreviewMaterials(UInteractiveTool& Tool, const UPolyEditActivityContext& ActivityContext,
		UPolyEditPreviewMesh& EditPreviewMesh, EPreviewMaterialType MaterialType);

}}}