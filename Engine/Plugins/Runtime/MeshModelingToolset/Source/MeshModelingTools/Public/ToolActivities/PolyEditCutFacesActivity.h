// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InteractiveToolActivity.h"
#include "InteractiveToolChange.h" //FToolCommandChange
#include "ToolContextInterfaces.h" // FViewCameraState

#include "PolyEditCutFacesActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UPolyEditActivityContext;
class UPolyEditPreviewMesh;
class UCollectSurfacePathMechanic;

UENUM()
enum class EPolyEditCutPlaneOrientation
{
	FaceNormals,
	ViewDirection
};

UCLASS(MinimalAPI)
class UPolyEditCutProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Cut)
	EPolyEditCutPlaneOrientation Orientation = EPolyEditCutPlaneOrientation::FaceNormals;

	UPROPERTY(EditAnywhere, Category = Cut)
	bool bSnapToVertices = true;
};


/**
 *
 */
UCLASS(MinimalAPI)
class UPolyEditCutFacesActivity : public UInteractiveToolActivity,
	public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}

protected:
	UE_API void Clear();
	UE_API void BeginCutFaces();
	UE_API void ApplyCutFaces();

	UPROPERTY()
	TObjectPtr<UPolyEditCutProperties> CutProperties;

	UPROPERTY()
	TObjectPtr<UPolyEditPreviewMesh> EditPreview;

	UPROPERTY()
	TObjectPtr<UCollectSurfacePathMechanic> SurfacePathMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	bool bIsRunning = false;
	int32 ActivityStamp = 1;

	FViewCameraState CameraState;

	friend class FPolyEditCutFacesActivityFirstPointChange;
};

/**
 * This should get emitted when setting the first point so that we can undo it.
 */
class FPolyEditCutFacesActivityFirstPointChange : public FToolCommandChange
{
public:
	FPolyEditCutFacesActivityFirstPointChange(int32 CurrentActivityStamp)
		: ActivityStamp(CurrentActivityStamp)
	{};

	virtual void Apply(UObject* Object) override {};
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return bHaveDoneUndo || Cast<UPolyEditCutFacesActivity>(Object)->ActivityStamp != ActivityStamp;
	}
	virtual FString ToString() const override
	{
		return TEXT("FPolyEditCutFacesActivityFirstPointChange");
	}

protected:
	int32 ActivityStamp;
	bool bHaveDoneUndo = false;
};

#undef UE_API
