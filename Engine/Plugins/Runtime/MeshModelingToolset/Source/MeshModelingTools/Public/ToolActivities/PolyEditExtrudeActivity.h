// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "DeformationOps/ExtrudeOp.h" // EPolyEditExtrudeMode
#include "GeometryBase.h"
#include "GroupTopology.h" // FGroupTopologySelection
#include "InteractiveToolActivity.h"
#include "FrameTypes.h"

#include "PolyEditExtrudeActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UPolyEditActivityContext;
class UPolyEditPreviewMesh;
class UPlaneDistanceFromHitMechanic;
PREDECLARE_GEOMETRY(class FDynamicMesh3);

UENUM()
enum class EPolyEditExtrudeDirection
{
	SelectionNormal,
	WorldX,
	WorldY,
	WorldZ,
	LocalX,
	LocalY,
	LocalZ
};

UENUM()
enum class EPolyEditExtrudeDistanceMode
{
	/** Set distance by clicking in the viewport. */
	ClickInViewport,

	/** Set distance with an explicit numerical value, then explictly accept. */
	Fixed,

	//~ TODO: Add someday
	// Gizmo,
};


// There is a lot of overlap in the options for Extrude, Offset, and Push/Pull, and they map to
// the same op behind the scenes. However, we want to keep them as separate buttons to keep some
// amount of shallowness in the UI, to make it more likely that new users will find the setting
// they are looking for.
// A couple of settings are entirely replicated: namely, doing an offset or "extrude" with SelectedTriangleNormals
// or SelectedTriangleNormalsEven as the movement direction is actually equivalent. Properly speaking, these
// two should only be options under Offset, not Extrude, but we keep them as (non-default) options 
// in both because an "extrude along local normals" is a common operation that some users are likely
// to look for under extrude, regardless of it not lining up with the physical meaning of extrusion.

// We use different property set objects so that we can customize category names, etc, as well as
// have different defaults and saved settings.

UENUM()
enum class EPolyEditExtrudeModeOptions
{
	// Extrude all triangles in the same direction regardless of their facing.
	SingleDirection = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SingleDirection),

	// Take the angle-weighed average of the selected triangles around each
	// extruded vertex to determine vertex movement direction.
	SelectedTriangleNormals = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SelectedTriangleNormals),

	// Like Selected Triangle Normals, but also adjusts the distances moved in
	// an attempt to keep triangles parallel to their original facing.
	SelectedTriangleNormalsEven = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SelectedTriangleNormalsEven),
};

UENUM()
enum class EPolyEditOffsetModeOptions
{
	// Vertex normals, regardless of selection.
	VertexNormals = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::VertexNormals),

	// Take the angle-weighed average of the selected triangles around
	// offset vertex to determine vertex movement direction.
	SelectedTriangleNormals = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SelectedTriangleNormals),

	// Like Selected Triangle Normals, but also adjusts the distances moved in
	// an attempt to keep triangles parallel to their original facing.
	SelectedTriangleNormalsEven = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SelectedTriangleNormalsEven),
};

UENUM()
enum class EPolyEditPushPullModeOptions
{
	// Take the angle-weighed average of the selected triangles around
	// offset vertex to determine vertex movement direction.
	SelectedTriangleNormals = static_cast<int>(EPolyEditOffsetModeOptions::SelectedTriangleNormals),

	// Like Selected Triangle Normals, but also adjusts the distances moved in
	// an attempt to keep triangles parallel to their original facing.
	SelectedTriangleNormalsEven = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SelectedTriangleNormalsEven),

	// Move all triangles in the same direction regardless of their facing.
	SingleDirection = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::SingleDirection),

	// Vertex normals, regardless of selection.
	VertexNormals = static_cast<int>(UE::Geometry::FExtrudeOp::EDirectionMode::VertexNormals),
};

UCLASS(MinimalAPI)
class UPolyEditExtrudeProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** How the extrude distance is set. */
	UPROPERTY(EditAnywhere, Category = Extrude)
	EPolyEditExtrudeDistanceMode DistanceMode = EPolyEditExtrudeDistanceMode::ClickInViewport;

	/** Distance to extrude. */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (UIMin = "-1000", UIMax = "1000", ClampMin = "-10000", ClampMax = "10000",
		EditConditionHides, EditCondition = "DistanceMode == EPolyEditExtrudeDistanceMode::Fixed"))
	double Distance = 100;

	/** How to move the vertices during the extrude */
	UPROPERTY(EditAnywhere, Category = Extrude)
	EPolyEditExtrudeModeOptions DirectionMode = EPolyEditExtrudeModeOptions::SingleDirection;

	/** Direction in which to extrude. */
	UPROPERTY(EditAnywhere, Category = Extrude,
		meta = (EditConditionHides, EditCondition = "DirectionMode == EPolyEditExtrudeModeOptions::SingleDirection"))
	EPolyEditExtrudeDirection Direction = EPolyEditExtrudeDirection::SelectionNormal;

	/** Controls the maximum distance vertices can move from the target distance in order to stay parallel with their source triangles. */
	UPROPERTY(EditAnywhere, Category = Extrude,
		meta = (ClampMin = "1", EditConditionHides, EditCondition = "DirectionMode == EPolyEditExtrudeModeOptions::SelectedTriangleNormalsEven"))
	double MaxDistanceScaleFactor = 4.0;

	/** Controls whether extruding an entire open-border patch should create a solid or an open shell */
	UPROPERTY(EditAnywhere, Category = Extrude)
	bool bShellsToSolids = true;
	
	/** What axis to measure the extrusion distance along. */
	UPROPERTY(EditAnywhere, Category = Extrude, AdvancedDisplay, meta = (EditConditionHides, 
		EditCondition = "DirectionMode != EPolyEditExtrudeModeOptions::SingleDirection && DistanceMode == EPolyEditExtrudeDistanceMode::ClickInViewport"))
	EPolyEditExtrudeDirection MeasureDirection = EPolyEditExtrudeDirection::SelectionNormal;

	/** 
	 * When extruding regions that touch the mesh border, assign the side groups (groups on the 
	 * stitched side of the extrude) in a way that considers edge colinearity. For instance, when
	 * true, extruding a flat rectangle will give four different groups on its sides rather than
	 * one connected group.
	 */
	UPROPERTY(EditAnywhere, Category = Extrude, AdvancedDisplay)
	bool bUseColinearityForSettingBorderGroups = true;
};

UCLASS(MinimalAPI)
class UPolyEditOffsetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** How the offset distance is set. */
	UPROPERTY(EditAnywhere, Category = Offset)
	EPolyEditExtrudeDistanceMode DistanceMode = EPolyEditExtrudeDistanceMode::ClickInViewport;

	/** Offset distance. */
	UPROPERTY(EditAnywhere, Category = Offset,
		meta = (EditConditionHides, EditCondition = "DistanceMode == EPolyEditExtrudeDistanceMode::Fixed"))
	double Distance = 100;

	/** Which way to move vertices during the offset */
	UPROPERTY(EditAnywhere, Category = Offset)
	EPolyEditOffsetModeOptions DirectionMode = EPolyEditOffsetModeOptions::VertexNormals;

	/** Controls the maximum distance vertices can move from the target distance in order to stay parallel with their source triangles. */
	UPROPERTY(EditAnywhere, Category = Offset,
		meta = (ClampMin = "1", EditConditionHides, EditCondition = "DirectionMode == EPolyEditOffsetModeOptions::SelectedTriangleNormalsEven"))
	double MaxDistanceScaleFactor = 4.0;

	/** Controls whether offsetting an entire open-border patch should create a solid or an open shell */
	UPROPERTY(EditAnywhere, Category = Offset)
	bool bShellsToSolids = true;

	/** What axis to measure the offset distance along. */
	UPROPERTY(EditAnywhere, Category = Offset, AdvancedDisplay, meta = (EditConditionHides,
		EditCondition = "DistanceMode == EPolyEditExtrudeDistanceMode::ClickInViewport"))
	EPolyEditExtrudeDirection MeasureDirection = EPolyEditExtrudeDirection::SelectionNormal;

	/**
	 * When offsetting regions that touch the mesh border, assign the side groups (groups on the
	 * stitched side of the offset) in a way that considers edge colinearity. For instance, when
	 * true, extruding a flat rectangle will give four different groups on its sides rather than
	 * one connected group.
	 */
	UPROPERTY(EditAnywhere, Category = Offset, AdvancedDisplay)
	bool bUseColinearityForSettingBorderGroups = true;
};

UCLASS(MinimalAPI)
class UPolyEditPushPullProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** How the extrusion distance is set. */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions)
	EPolyEditExtrudeDistanceMode DistanceMode = EPolyEditExtrudeDistanceMode::ClickInViewport;

	/** Extrusion distance. */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions,
		meta = (EditConditionHides, EditCondition = "DistanceMode == EPolyEditExtrudeDistanceMode::Fixed"))
	double Distance = 100;

	/** Which way to move vertices while extruding. */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions)
	EPolyEditPushPullModeOptions DirectionMode = EPolyEditPushPullModeOptions::SelectedTriangleNormals;

	/** Direction in which to extrude. */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions,
		meta = (DisplayName="Direction", EditConditionHides, EditCondition = "DirectionMode == EPolyEditPushPullModeOptions::SingleDirection"))
	EPolyEditExtrudeDirection SingleDirection = EPolyEditExtrudeDirection::SelectionNormal;

	/** Controls the maximum distance vertices can move from the target distance in order to stay parallel with their source triangles. */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions,
		meta = (ClampMin = "1", EditConditionHides, EditCondition = "DirectionMode == EPolyEditPushPullModeOptions::SelectedTriangleNormalsEven"))
	double MaxDistanceScaleFactor = 4.0;

	/** Controls whether offsetting an entire open-border patch should create a solid or an open shell */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions)
	bool bShellsToSolids = true;

	/** What axis to measure the extrusion distance along. */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions, AdvancedDisplay, meta = (EditConditionHides, 
		EditCondition = "DirectionMode != EPolyEditPushPullModeOptions::SingleDirection && DistanceMode == EPolyEditExtrudeDistanceMode::ClickInViewport"))
	EPolyEditExtrudeDirection MeasureDirection = EPolyEditExtrudeDirection::SelectionNormal;

	/**
	 * When offsetting regions that touch the mesh border, assign the side groups (groups on the
	 * stitched side of the extrude) in a way that considers edge colinearity. For instance, when
	 * true, extruding a flat rectangle will give four different groups on its sides rather than
	 * one connected group.
	 */
	UPROPERTY(EditAnywhere, Category = ExtrusionOptions, AdvancedDisplay)
	bool bUseColinearityForSettingBorderGroups = true;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UPolyEditExtrudeActivity : public UInteractiveToolActivity,
	public UE::Geometry::IDynamicMeshOperatorFactory,
	public IClickBehaviorTarget, public IHoverBehaviorTarget

{
	GENERATED_BODY()

public:
	using FExtrudeOp = UE::Geometry::FExtrudeOp;

	enum class EPropertySetToUse
	{
		Extrude,
		Offset,
		PushPull
	};

	// Set to different values depending on whether we're using this activity on behalf of
	// extrude, offset, or push/pull
	FExtrudeOp::EExtrudeMode ExtrudeMode = FExtrudeOp::EExtrudeMode::MoveAndStitch;
	EPropertySetToUse PropertySetToUse = EPropertySetToUse::Extrude;

	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	virtual bool HasAccept() const { return true; };
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}

	UPROPERTY()
	TObjectPtr<UPolyEditExtrudeProperties> ExtrudeProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditOffsetProperties> OffsetProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditPushPullProperties> PushPullProperties = nullptr;

protected:

	UE_API FVector3d GetExtrudeDirection() const;
	UE_API virtual void BeginExtrude();
	UE_API virtual void ApplyExtrude();
	UE_API virtual void ReinitializeExtrudeHeightMechanic();
	UE_API virtual void EndInternal();

	UPROPERTY()
	TObjectPtr<UPlaneDistanceFromHitMechanic> ExtrudeHeightMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	TSharedPtr<UE::Geometry::FDynamicMesh3> PatchMesh;
	TArray<int32> NewSelectionGids;

	bool bIsRunning = false;

	UE::Geometry::FGroupTopologySelection ActiveSelection;
	UE::Geometry::FFrame3d ActiveSelectionFrameWorld;
	float UVScaleFactor = 1.0f;

	bool bRequestedApply = false;
};

#undef UE_API
