// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ModelingOperators.h" //IDynamicMeshOperatorFactory
#include "InteractiveTool.h" //UInteractiveToolPropertySet
#include "InteractiveToolActivity.h"
#include "InteractiveToolBuilder.h" //UInteractiveToolBuilder
#include "InteractiveToolChange.h" //FToolCommandChange
#include "MeshOpPreviewHelpers.h" //FDynamicMeshOpResult
#include "Operations/GroupEdgeInserter.h"
#include "Selection/MeshTopologySelector.h"
#include "SingleSelectionTool.h"
#include "ToolDataVisualizer.h"
#include "GroupTopology.h"
#include "PolyEditInsertEdgeActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMeshChange);
class UPolyEditActivityContext;

UENUM()
enum class EGroupEdgeInsertionMode
{
	/** Existing groups will be deleted and new triangles will be created for the new groups.
	 Keeps topology simple but breaks non-planar groups. */
	Retriangulate,

	/** Keeps existing triangles and cuts them to create a new path. May result in fragmented triangles over time.*/
	PlaneCut
};

UCLASS(MinimalAPI)
class UGroupEdgeInsertionProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Determines how group edges are added to the geometry */
	UPROPERTY(EditAnywhere, Category = InsertEdge)
	EGroupEdgeInsertionMode InsertionMode = EGroupEdgeInsertionMode::PlaneCut;

	/** If true, edge insertions are chained together so that each end point becomes the new start point */
	UPROPERTY(EditAnywhere, Category = InsertEdge)
	bool bContinuousInsertion = true;
	
	/** How close a new loop edge needs to pass next to an existing vertex to use that vertex rather than creating a new one (used for plane cut). */
	UPROPERTY(EditAnywhere, Category = InsertEdge, AdvancedDisplay, meta = (UIMin = "0", UIMax = "0.01", ClampMin = "0", ClampMax = "10"))
	double VertexTolerance = 0.001;
};

/** Interactive activity for inserting a group edge into a mesh. */
UCLASS(MinimalAPI)
class UPolyEditInsertEdgeActivity : public UInteractiveToolActivity, 
	public UE::Geometry::IDynamicMeshOperatorFactory, 
	public IHoverBehaviorTarget, public IClickBehaviorTarget
{
	GENERATED_BODY()

	using FGroupEdgeInserter = UE::Geometry::FGroupEdgeInserter;
	using FDynamicMesh3 = UE::Geometry::FDynamicMesh3;

	enum class EState
	{
		GettingStart,
		GettingEnd,
		WaitingForInsertComplete
	};

public:

	friend class FGroupEdgeInsertionFirstPointChange;

	UPolyEditInsertEdgeActivity() {};

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property);

	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	// IClickBehaviorTarget
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

protected:
	UPROPERTY()
	TObjectPtr<UGroupEdgeInsertionProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	bool bIsRunning = false;

	FTransform TargetTransform;
	TSharedPtr<FMeshTopologySelector, ESPMode::ThreadSafe> TopologySelector;

	TArray<TPair<FVector3d, FVector3d>> PreviewEdges;
	TArray<FVector3d> PreviewPoints;

	FViewCameraState CameraState;

	FToolDataVisualizer ExistingEdgesRenderer;
	FToolDataVisualizer PreviewEdgeRenderer;
	FMeshTopologySelector::FSelectionSettings TopologySelectorSettings;


	// Inputs from user interaction:
	FGroupEdgeInserter::FGroupEdgeSplitPoint StartPoint;
	int32 StartTopologyID = FDynamicMesh3::InvalidID;
	bool bStartIsCorner = false;

	FGroupEdgeInserter::FGroupEdgeSplitPoint EndPoint;
	int32 EndTopologyID = FDynamicMesh3::InvalidID;
	bool bEndIsCorner = false;

	int32 CommonGroupID = FDynamicMesh3::InvalidID;
	int32 CommonBoundaryIndex = FDynamicMesh3::InvalidID;

	FRay LastEndPointWorldRay;

	// State control:
	EState ToolState = EState::GettingStart;

	bool bShowingBaseMesh = false;
	bool bLastComputeSucceeded = false;
	TSharedPtr<UE::Geometry::FGroupTopology, ESPMode::ThreadSafe> LatestOpTopologyResult;
	TSharedPtr<TSet<int32>, ESPMode::ThreadSafe> LatestOpChangedTids;

	int32 CurrentChangeStamp = 0;

	// Safe inputs for the background compute to use, untouched by undo/redo/other CurrentMesh updates.
	TSharedPtr<const UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> ComputeStartMesh;
	TSharedPtr<const UE::Geometry::FGroupTopology, ESPMode::ThreadSafe> ComputeStartTopology;
	UE_API void UpdateComputeInputs();

	UE_API void SetupPreview();

	UE_API bool TopologyHitTest(const FRay& WorldRay, FVector3d& RayPositionOut, FRay3d* LocalRayOut = nullptr);
	UE_API bool GetHoveredItem(const FRay& WorldRay, FGroupEdgeInserter::FGroupEdgeSplitPoint& PointOut,
		int32& TopologyElementIDOut, bool& bIsCornerOut, FVector3d& PositionOut, FRay3d* LocalRayOut = nullptr);

	UE_API void ConditionallyUpdatePreview(const FGroupEdgeInserter::FGroupEdgeSplitPoint& NewEndPoint, 
		int32 NewEndTopologyID, bool bNewEndIsCorner, int32 NewCommonGroupID, int32 NewBoundaryIndex);

	UE_API void ClearPreview(bool bClearDrawnElements);

	UE_API void GetCornerTangent(int32 CornerID, int32 GroupID, int32 BoundaryIndex, FVector3d& TangentOut);

	/**
	 * Expires the tool-associated changes in the undo/redo stack. The ComponentTarget
	 * changes will stay (we want this).
	 */
	inline void ExpireChanges()
	{
		++CurrentChangeStamp;
	}
};

/**
 * This should get emitted when selecting the first point in an edge insertion so that we can undo it.
 */
class FGroupEdgeInsertionFirstPointChange : public FToolCommandChange
{
public:
	FGroupEdgeInsertionFirstPointChange(int32 CurrentChangeStamp)
		: ChangeStamp(CurrentChangeStamp)
	{};

	virtual void Apply(UObject* Object) override {};
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		UPolyEditInsertEdgeActivity* Activity = Cast<UPolyEditInsertEdgeActivity>(Object);
		return bHaveDoneUndo || Activity->CurrentChangeStamp != ChangeStamp
			// We only allow undo if we're looking for the next point or waiting for completion, i.e. when
			// we have a start point to undo.
			|| Activity->ToolState == UPolyEditInsertEdgeActivity::EState::GettingStart;
	}
	virtual FString ToString() const override
	{
		return TEXT("FGroupEdgeInsertionFirstPointChange");
	}

protected:
	int32 ChangeStamp;
	bool bHaveDoneUndo = false;
};

#undef UE_API
