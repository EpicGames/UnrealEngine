// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ModelingOperators.h" //IDynamicMeshOperatorFactory
#include "InteractiveTool.h" //UInteractiveToolPropertySet
#include "InteractiveToolActivity.h"
#include "InteractiveToolBuilder.h" //UInteractiveToolBuilder
#include "InteractiveToolChange.h" //FToolCommandChange
#include "MeshOpPreviewHelpers.h" //FDynamicMeshOpResult
#include "Selection/MeshTopologySelector.h"
#include "SingleSelectionTool.h"
#include "ToolDataVisualizer.h"

#include "PolyEditInsertEdgeLoopActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMeshChange);
class UPolyEditActivityContext;

UENUM()
enum class EEdgeLoopPositioningMode
{
	/** Edge loops will be evenly centered within a group. Allows for multiple insertions at a time. */
	Even,

	/** Edge loops will fall at the same length proportion at each edge they intersect (e.g., a quarter way down). */
	ProportionOffset,

	/** Edge loops will fall a constant distance away from the start of each edge they intersect
	 (e.g., 20 units down). Clamps to end if edge is too short. */
	 DistanceOffset
};

UENUM()
enum class EEdgeLoopInsertionMode
{
	/** Existing groups will be deleted and new triangles will be created for the new groups.
	 Keeps topology simple but breaks non-planar groups. */
	Retriangulate,

	/** Keeps existing triangles and cuts them to create a new path. May result in fragmented triangles over time.*/
	PlaneCut
};

UCLASS(MinimalAPI)
class UEdgeLoopInsertionProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Determines how edge loops position themselves vertically relative to loop direction. */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop)
	EEdgeLoopPositioningMode PositionMode = EEdgeLoopPositioningMode::ProportionOffset;

	/** Determines how edge loops are added to the geometry */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop)
	EEdgeLoopInsertionMode InsertionMode = EEdgeLoopInsertionMode::PlaneCut;

	/** How many loops to insert at a time. Only used with "even" positioning mode. */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop, meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "500",
		EditCondition = "PositionMode == EEdgeLoopPositioningMode::Even", EditConditionHides))
	int32 NumLoops = 1;

	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "1.0",
		EditCondition = "PositionMode == EEdgeLoopPositioningMode::ProportionOffset && !bInteractive", EditConditionHides))
	double ProportionOffset = 0.5;

	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop, AdvancedDisplay, meta = (UIMin = "0", ClampMin = "0",
		EditCondition = "PositionMode == EEdgeLoopPositioningMode::DistanceOffset && !bInteractive", EditConditionHides))
	double DistanceOffset = 10.0;

	/** When false, the distance/proportion offset is numerically specified, and mouse clicks just choose the edge. */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop, AdvancedDisplay, meta = (
		EditCondition = "PositionMode != EEdgeLoopPositioningMode::Even", EditConditionHides))
	bool bInteractive = true;

	/** Measure the distance offset from the opposite side of the edges. */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop, meta = (
		EditCondition = "PositionMode == EEdgeLoopPositioningMode::DistanceOffset", EditConditionHides))
	bool bFlipOffsetDirection = false;

	/** When true, non-quad-like groups that stop the loop will be highlighted, with X's marking the corners. */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop)
	bool bHighlightProblemGroups = true;

	/** How close a new loop edge needs to pass next to an existing vertex to use that vertex rather than creating a new one. */
	UPROPERTY(EditAnywhere, Category = InsertEdgeLoop, AdvancedDisplay, meta = (UIMin = "0", UIMax = "0.01", ClampMin = "0", ClampMax = "10"))
	double VertexTolerance = 0.001;
};

/** Interactive activity for inserting (group) edge loops into a mesh. */
UCLASS(MinimalAPI)
class UPolyEditInsertEdgeLoopActivity : public UInteractiveToolActivity, 
	public UE::Geometry::IDynamicMeshOperatorFactory,
	public IHoverBehaviorTarget, public IClickBehaviorTarget
{
	GENERATED_BODY()
public:

	friend class FEdgeLoopInsertionChange;

	UPolyEditInsertEdgeLoopActivity() {};

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property);

	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IDynamicMeshOperatorFactory
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	// IClickBehaviorTarget
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

protected:
	UPROPERTY()
	TObjectPtr<UEdgeLoopInsertionProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	bool bIsRunning = false;

	FTransform TargetTransform;
	TSharedPtr<FMeshTopologySelector, ESPMode::ThreadSafe> TopologySelector;

	TArray<TPair<FVector3d, FVector3d>> PreviewEdges;

	// Used to highlight problematic topology (non-quad groups) when it stops a loop.
	TArray<TPair<FVector3d, FVector3d>> ProblemTopologyEdges;
	TArray<FVector3d> ProblemTopologyVerts;

	FViewCameraState CameraState;

	FToolDataVisualizer PreviewEdgeRenderer;
	FToolDataVisualizer ProblemTopologyRenderer;
	FMeshTopologySelector::FSelectionSettings TopologySelectorSettings;
	float ProblemVertTickWidth = 8;

	UE_API void SetupPreview();

	UE_API FInputRayHit HitTest(const FRay& WorldRay);
	UE_API bool UpdateHoveredItem(const FRay& WorldRay);

	// Safe inputs for the background compute to use, untouched by undo/redo/other CurrentMesh updates.
	TSharedPtr<const UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> ComputeStartMesh;
	TSharedPtr<const UE::Geometry::FGroupTopology, ESPMode::ThreadSafe> ComputeStartTopology;
	UE_API void UpdateComputeInputs();

	UE_API void ConditionallyUpdatePreview(int32 NewGroupID, double* NewInputLength = nullptr);
	UE_API void ClearPreview();

	// Taken from user interaction, read as inputs by the op factory
	int32 InputGroupEdgeID = FDynamicMesh3::InvalidID;
	double InteractiveInputLength = 0;

	// On valid clicks, we wait to finish the background op and apply it before taking more input.
	// Gets reset OnTick when the result is ready.
	bool bWaitingForInsertionCompletion = false;

	// Copied over on op completion
	bool bLastComputeSucceeded = false;
	TSharedPtr<UE::Geometry::FGroupTopology, ESPMode::ThreadSafe> LatestOpTopologyResult;
	TSharedPtr<TSet<int32>, ESPMode::ThreadSafe> LatestOpChangedTids;
};

#undef UE_API
