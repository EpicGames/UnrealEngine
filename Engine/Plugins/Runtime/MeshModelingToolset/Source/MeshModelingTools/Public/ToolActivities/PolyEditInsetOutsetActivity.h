// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "InteractiveToolActivity.h"

#include "PolyEditInsetOutsetActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UPolyEditActivityContext;
class UPolyEditPreviewMesh;
class USpatialCurveDistanceMechanic;

UCLASS(MinimalAPI)
class UPolyEditInsetOutsetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Amount of smoothing applied to the boundary */
	UPROPERTY(EditAnywhere, Category = "Region", 
		meta = (UIMin = "0.0", UIMax = "1.0", EditCondition = "bBoundaryOnly == false"))
	float Softness = 0.5;

	/** Controls whether operation will move interior vertices as well as border vertices */
	UPROPERTY(EditAnywhere, Category = "Region", AdvancedDisplay)
	bool bBoundaryOnly = false;

	/** Tweak area scaling when solving for interior vertices */
	UPROPERTY(EditAnywhere, Category = "Region", AdvancedDisplay, 
		meta = (UIMin = "0.0", UIMax = "1.0", EditCondition = "bBoundaryOnly == false"))
	float AreaScale = true;

	/** When insetting, determines whether vertices in inset region should be projected back onto input surface */
	UPROPERTY(EditAnywhere, Category = "Region", Meta = (EditCondition = "!bOutset", HideEditConditionToggle, EditConditionHides))
	bool bReproject = true;

	//~ This is not user editable- it gets set by PolyEdit depending on whether the user clicks
	//~ inset or outset. Currently, both operations share the same code, and one may argue that
	//~ we should just determine which to do based on where the user clicks. However, our long
	//~ term plan is that they will be more differentiated in operation, to the point that we
	//~ may split them into separate activities.
	UPROPERTY()
	bool bOutset = false;
};


/**
 *
 */
UCLASS(MinimalAPI)
class UPolyEditInsetOutsetActivity : public UInteractiveToolActivity,
	public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}

	UPROPERTY()
	TObjectPtr<UPolyEditInsetOutsetProperties> Settings;

protected:
	UE_API void Clear();
	UE_API bool BeginInset();
	UE_API void ApplyInset();

	bool bIsRunning = false;
	bool bPreviewUpdatePending = false;

	UPROPERTY()
	TObjectPtr<UPolyEditPreviewMesh> EditPreview;

	UPROPERTY()
	TObjectPtr<USpatialCurveDistanceMechanic> CurveDistMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	float UVScaleFactor = 1.0f;
};

#undef UE_API
