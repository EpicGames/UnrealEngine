// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "BoxTypes.h"
#include "InteractiveToolActivity.h"
#include "ToolActivities/PolyEditActivityUtil.h"
#include "ToolContextInterfaces.h" // FViewCameraState

#include "PolyEditPlanarProjectionUVActivity.generated.h"

#define UE_API MESHMODELINGTOOLS_API

class UPolyEditActivityContext;
class UPolyEditPreviewMesh;
class UCollectSurfacePathMechanic;

UCLASS(MinimalAPI)
class UPolyEditSetUVProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = PlanarProjectUV)
	bool bShowMaterial = false;
};


/**
 *
 */
UCLASS(MinimalAPI)
class UPolyEditPlanarProjectionUVActivity : public UInteractiveToolActivity,
	public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:
	// IInteractiveToolActivity
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual bool CanStart() const override;
	UE_API virtual EToolActivityStartResult Start() override;
	virtual bool IsRunning() const override { return bIsRunning; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual EToolActivityEndResult End(EToolShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}

protected:
	UE_API void Clear();
	UE_API void BeginSetUVs();
	UE_API void UpdateSetUVS();
	UE_API void ApplySetUVs();

	UPROPERTY()
	TObjectPtr<UPolyEditSetUVProperties> SetUVProperties;

	UPROPERTY()
	TObjectPtr<UPolyEditPreviewMesh> EditPreview;

	UPROPERTY()
	TObjectPtr<UCollectSurfacePathMechanic> SurfacePathMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UPolyEditActivityContext> ActivityContext;

	bool bIsRunning = false;

	bool bPreviewUpdatePending = false;
	UE::Geometry::PolyEditActivityUtil::EPreviewMaterialType CurrentPreviewMaterial;
	UE::Geometry::FAxisAlignedBox3d ActiveSelectionBounds;
	FViewCameraState CameraState;
};

#undef UE_API
