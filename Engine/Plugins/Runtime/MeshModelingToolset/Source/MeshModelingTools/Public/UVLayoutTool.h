// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "GeometryBase.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "ToolDataVisualizer.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/MeshUVChannelProperties.h"
#include "Drawing/UVLayoutPreview.h"

#include "UVLayoutTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API


// Forward declarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UUVLayoutProperties;
class UUVLayoutOperatorFactory;
PREDECLARE_GEOMETRY(class FDynamicMesh3);

/**
 *
 */
UCLASS(MinimalAPI)
class UUVLayoutToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


/**
 * The level editor version of the UV layout tool.
 */
UCLASS(MinimalAPI)
class UUVLayoutTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:

	UE_API UUVLayoutTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	UE_API int32 GetSelectedUVChannel() const;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<UMeshUVChannelProperties> UVChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVLayoutProperties> BasicProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UExistingMeshMaterialProperties> MaterialSettings = nullptr;

	UPROPERTY()
	TArray<TObjectPtr<UMeshOpPreviewWithBackgroundCompute>> Previews;

	UPROPERTY()
	TArray<TObjectPtr<UUVLayoutOperatorFactory>> Factories;

	TArray<TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe>> OriginalDynamicMeshes;

	FViewCameraState CameraState;

	UE_API void UpdateNumPreviews();

	UE_API void UpdateVisualization();
	
	UE_API void UpdatePreviewMaterial();

	UE_API void OnPreviewMeshUpdated(UMeshOpPreviewWithBackgroundCompute* Compute);

	UE_API void GenerateAsset(const TArray<FDynamicMeshOpResult>& Results);

	UPROPERTY()
	TObjectPtr<UUVLayoutPreview> UVLayoutView = nullptr;
};

#undef UE_API
