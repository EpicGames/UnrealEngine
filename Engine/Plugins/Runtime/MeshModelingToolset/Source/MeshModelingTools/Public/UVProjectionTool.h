// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "MeshOpPreviewHelpers.h"
#include "ToolDataVisualizer.h"
#include "ParameterizationOps/UVProjectionOp.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/MeshUVChannelProperties.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Selection/SelectClickedAction.h"
#include "OrientedBoxTypes.h"

#include "UVProjectionTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API


// Forward declarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UCombinedTransformGizmo;
class UTransformProxy;
class USingleClickInputBehavior;
class UUVProjectionTool;

/**
 *
 */
UCLASS(MinimalAPI)
class UUVProjectionToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	virtual bool RequiresInputSelection() const override { return false; }
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};


UENUM()
enum class EUVProjectionToolActions
{
	NoAction,
	AutoFit,
	AutoFitAlign,
	Reset
};


UCLASS(MinimalAPI)
class UUVProjectionToolEditActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UUVProjectionTool> ParentTool;

	void Initialize(UUVProjectionTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EUVProjectionToolActions Action);

	/** Automatically fit the UV Projection Dimensions based on the current projection orientation */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "AutoFit", DisplayPriority = 1))
	void AutoFit()
	{
		PostAction(EUVProjectionToolActions::AutoFit);
	}

	/** Automatically orient the projection and then automatically fit the UV Projection Dimensions */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "AutoFitAlign", DisplayPriority = 2))
	void AutoFitAlign()
	{
		PostAction(EUVProjectionToolActions::AutoFitAlign);
	}

	/** Re-initialize the projection based on the UV Projection Initialization property */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Reset", DisplayPriority = 3))
	void Reset()
	{
		PostAction(EUVProjectionToolActions::Reset);
	}
};


UENUM()
enum class EUVProjectionToolInitializationMode
{
	/** Initialize projection to bounding box center */
	Default,
	/** Initialize projection based on previous usage of the Project tool */
	UsePrevious,
	/** Initialize projection using Auto Fitting for the initial projection type */
	AutoFit,
	/** Initialize projection using Auto Fitting with Alignment for the initial projection type */
	AutoFitAlign
};


/**
 * Standard properties
 */
UCLASS(MinimalAPI)
class UUVProjectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Shape and/or algorithm to use for UV projection */
	UPROPERTY(EditAnywhere, Category = "UV Projection")
	EUVProjectionMethod ProjectionType = EUVProjectionMethod::Plane;

	/** Width, length, and height of the projection shape before rotation */
	UPROPERTY(EditAnywhere, Category = "UV Projection", meta = (Delta = 0.5, LinearDeltaSensitivity = 1))
	FVector Dimensions = FVector(100.0f, 100.0f, 100.0f);

	/** If true, changes to Dimensions result in all components be changed proportionally */
	UPROPERTY(EditAnywhere, Category = "UV Projection")
	bool bProportionalDimensions = false;

	/** Determines how projection settings will be initialized; this only takes effect if the projection shape dimensions or position are unchanged */
	UPROPERTY(EditAnywhere, Category = "UV Projection")
	EUVProjectionToolInitializationMode Initialization = EUVProjectionToolInitializationMode::Default;

	//
	// Cylinder projection options
	//

	/** Angle in degrees to determine whether faces should be assigned to the cylinder or the flat end caps */
	UPROPERTY(EditAnywhere, Category = CylinderProjection, meta = (DisplayName = "Split Angle", UIMin = "0", UIMax = "90",
		EditCondition = "ProjectionType == EUVProjectionMethod::Cylinder", EditConditionHides))
	float CylinderSplitAngle = 45.0f;

	//
	// ExpMap projection options
	//

	/** Blend between surface normals and projection normal; ExpMap projection becomes Plane projection when this value is 1 */
	UPROPERTY(EditAnywhere, Category = "ExpMap Projection", meta = (DisplayName = "Normal Blending", UIMin = "0", UIMax = "1",
		EditCondition = "ProjectionType == EUVProjectionMethod::ExpMap", EditConditionHides))
	float ExpMapNormalBlending = 0.0f;

	/** Number of smoothing steps to apply; this slightly increases distortion but produces more stable results. */
	UPROPERTY(EditAnywhere, Category = "ExpMap Projection", meta = (DisplayName = "Smoothing Steps", UIMin = "0", UIMax = "100",
		EditCondition = "ProjectionType == EUVProjectionMethod::ExpMap", EditConditionHides))
	int ExpMapSmoothingSteps = 0;

	/** Smoothing parameter; larger values result in faster smoothing in each step. */
	UPROPERTY(EditAnywhere, Category = "ExpMap Projection", meta = (DisplayName = "Smoothing Alpha", UIMin = "0", UIMax = "1",
		EditCondition = "ProjectionType == EUVProjectionMethod::ExpMap", EditConditionHides))
	float ExpMapSmoothingAlpha = 0.25f;

	//
	// UV-space transform options
	//

	/** Rotation in degrees applied after computing projection */
	UPROPERTY(EditAnywhere, Category = "UV Transform", meta = (ClampMin = -360, ClampMax = 360))
	float Rotation = 0.0;

	/** Scaling applied after computing projection */
	UPROPERTY(EditAnywhere, Category = "UV Transform", meta = (Delta = 0.01, LinearDeltaSensitivity = 1))
	FVector2D Scale = FVector2D::UnitVector;

	/** Translation applied after computing projection */
	UPROPERTY(EditAnywhere, Category = "UV Transform")
	FVector2D Translation = FVector2D::ZeroVector;

	//
	// Saved State. These are used internally to support UsePrevious initialization mode
	//

	UPROPERTY()
	FVector SavedDimensions = FVector::ZeroVector;

	UPROPERTY()
	bool bSavedProportionalDimensions = false;

	UPROPERTY()
	FTransform SavedTransform;
};


/**
 * Factory with enough info to spawn the background-thread Operator to do a chunk of work for the tool
 *  stores a pointer to the tool and enough info to know which specific operator it should spawn
 */
UCLASS(MinimalAPI)
class UUVProjectionOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()
public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UUVProjectionTool> Tool;
};


/**
 * UV projection tool
 */
UCLASS(MinimalAPI)
class UUVProjectionTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()

public:

	friend UUVProjectionOperatorFactory;

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;
	
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void RequestAction(EUVProjectionToolActions ActionType);

protected:

	UPROPERTY()
	TObjectPtr<UMeshUVChannelProperties> UVChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVProjectionToolProperties> BasicProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UUVProjectionToolEditActions> EditActions = nullptr;

	UPROPERTY()
	TObjectPtr<UExistingMeshMaterialProperties> MaterialSettings = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> CheckerMaterial = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo = nullptr;
	
	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UUVProjectionOperatorFactory> OperatorFactory = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> EdgeRenderer = nullptr;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> TriangleROI;
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> VertexROI;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> InputMeshROISpatial;
	TSet<int32> TriangleROISet;

	FTransform3d WorldTransform;
	UE::Geometry::FAxisAlignedBox3d WorldBounds;

	FVector CachedDimensions;
	FVector InitialDimensions;
	bool bInitialProportionalDimensions;
	FTransform InitialTransform;
	int32 DimensionsWatcher = -1;
	int32 DimensionsModeWatcher = -1;
	bool bTransformModified = false;
	UE_API void OnInitializationModeChanged();
	UE_API void ApplyInitializationMode();

	FViewCameraState CameraState;

	FToolDataVisualizer ProjectionShapeVisualizer;

	UE_API void InitializeMesh();
	UE_API void UpdateNumPreviews();

	UE_API void TransformChanged(UTransformProxy* Proxy, FTransform Transform);

	UE_API void OnMaterialSettingsChanged();
	UE_API void OnMeshUpdated(UMeshOpPreviewWithBackgroundCompute* PreviewCompute);

	UE_API UE::Geometry::FOrientedBox3d GetProjectionBox() const;


	//
	// Support for ctrl+click to set plane from hit point
	//

	TUniquePtr<FSelectClickedAction> SetPlaneCtrlClickBehaviorTarget;

	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickToSetPlaneBehavior;

	UE_API void UpdatePlaneFromClick(const FVector3d& Position, const FVector3d& Normal, bool bTransitionOnly);

	//
	// Support for Action Buttons
	//

	bool bHavePendingAction = false;
	EUVProjectionToolActions PendingAction;
	UE_API virtual void ApplyAction(EUVProjectionToolActions ActionType);
	UE_API void ApplyAction_AutoFit(bool bAlign);
	UE_API void ApplyAction_Reset();

};

#undef UE_API
