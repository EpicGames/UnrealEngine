// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "PreviewMesh.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "WeldMeshEdgesTool.generated.h"

#define UE_API MESHMODELINGTOOLS_API

// predeclarations
struct FMeshDescription;
class UMeshElementsVisualizer;
class UWeldMeshEdgesTool;
class FWeldMeshEdgesOp;

/**
 *
 */
UCLASS(MinimalAPI)
class UWeldMeshEdgesToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

	virtual bool RequiresInputSelection() const override { return false; };
};

UENUM()
enum class EWeldMeshEdgesAttributeUIMode : uint8
{
	/** Do not weld split-attributes*/
	None,
	/** Apply attribute welding only along the current mesh welds */
	OnWeldedMeshEdgesOnly,
	/** Apply attribute welding to all split-attributes */
	OnFullMesh
};


UCLASS(MinimalAPI)
class UWeldMeshEdgesToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Edges are considered matching if both pairs of endpoint vertices are closer than this distance */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.000001", UIMax = "0.01", ClampMin = "0.00000001", ClampMax = "1000.0"))
	float Tolerance = FMathf::ZeroTolerance;

	/** Only merge unambiguous pairs that have unique duplicate-edge matches */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bOnlyUnique = false;

	/** If enabled, after an initial attempt at Welding, attempt to resolve remaining open edges in T-junction configurations via edge splits, and then retry Weld */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bResolveTJunctions = false;

	/** If enabled, will split bowtie vertices before welding. This can in some cases enable more edges to be successfully welded */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bSplitBowties = true;

	/** Initial number of open boundary edges */
	UPROPERTY(VisibleAnywhere, Category = Statistics)
	int32 InitialEdges;

	/** Number of remaining open boundary edges */
	UPROPERTY(VisibleAnywhere, Category = Statistics)
	int32 RemainingEdges;

	/** Controls split-attribute welding performed after the Mesh weld.  Applies to normals, tangents, UVs and colors. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (DisplayName = "Split-Attribute Welding"))
	EWeldMeshEdgesAttributeUIMode AttrWeldingMode = EWeldMeshEdgesAttributeUIMode::OnWeldedMeshEdgesOnly;


	/** Threshold on the angle between normals used to determine if split normals should be merged*/
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "AttrWeldingMode != EWeldMeshEdgesAttributeUIMode::None", UIMin = "0.0", UIMax = "180.0", ClampMin = "0.0", ClampMax = "180.0", DisplayName="Normal Angle Threshold"))
	float SplitNormalThreshold = 0.1f;

	/** Threshold on the angle between tangent used to determine if split tangents should be merged*/
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "AttrWeldingMode != EWeldMeshEdgesAttributeUIMode::None", UIMin = "0.0", UIMax = "180.0", ClampMin = "0.0", ClampMax = "180.0", DisplayName="Tangent Angle Threshold"))
	float SplitTangentsThreshold = 0.1f;

	/** Threshold uv-distance used to determine if split UVs should be merged */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "AttrWeldingMode != EWeldMeshEdgesAttributeUIMode::None",  UIMin = "0" , DisplayName = "UV Distance Threshold"))
	float SplitUVThreshold = 0.01f;

	/** Threshold color-distance used to determine if split colors should be merged */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay, meta = (EditCondition = "AttrWeldingMode != EWeldMeshEdgesAttributeUIMode::None",  UIMin = "0", DisplayName = "Color Distance Threshold"))
	float SplitColorThreshold = 0.01f;

};




UCLASS(MinimalAPI)
class UWeldMeshEdgesOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UWeldMeshEdgesTool> WeldMeshEdgesTool;
};


/**
 * Mesh Weld Edges Tool
 */
UCLASS(MinimalAPI)
class UWeldMeshEdgesTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()
public:
	UE_API UWeldMeshEdgesTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	// update parameters in FWeldMeshEdgesOp based on current Settings
	UE_API void UpdateOpParameters(FWeldMeshEdgesOp& Op) const;

protected:
	UPROPERTY()
	TObjectPtr<UWeldMeshEdgesToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> PreviewCompute = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay = nullptr;

	UPROPERTY()
	TObjectPtr<UWeldMeshEdgesOperatorFactory> OperatorFactory;

protected:

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> SourceMesh;
	
	// If there is an active selection, SelectedEdges will be initialized
	TSet<int32> SelectedEdges;



};

#undef UE_API
