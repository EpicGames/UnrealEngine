// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshModelingToolsEditorOnly : ModuleRules
{
	public MeshModelingToolsEditorOnly(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InteractiveToolsFramework",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore",
				"GeometryFramework",
				"DynamicMesh",
				"MeshConversion",
				"MeshModelingTools",
				"ModelingComponents",
				"ModelingComponentsEditorOnly",
				"ModelingOperators",
				"ModelingOperatorsEditorOnly",
				"PropertyEditor",
				"SkeletalMeshUtilitiesCommon",
				"SkeletalMeshModifiers"
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"RenderCore",
				"InputCore",
				"UnrealEd",
				"Persona",
				"AdvancedPreviewScene",
				"SceneOutliner",
				"Slate",
				"Json",
				"ApplicationCore",
				"SlateCore",
				"ToolWidgets"
				// ... add private dependencies that you statically link with here ...	
			}
		);
	}
}