// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttributeEditorTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "ToolSetupUtil.h"
#include "MathUtil.h"
#include "MeshDescriptionToDynamicMesh.h"   // for FMeshDescriptionToDynamicMesh::IsReservedAttributeName
#include "AssetUtils/MeshDescriptionUtil.h"

#include "MeshDescription.h"
#include "SkeletalMeshAttributes.h"
#include "StaticMeshOperations.h"

// for lightmap access
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"

#include "Components/PrimitiveComponent.h"

#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeEditorTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UAttributeEditorTool"

/*
 * ToolBuilder
 */

UMultiSelectionMeshEditingTool* UAttributeEditorToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UAttributeEditorTool>(SceneState.ToolManager);
}

bool UAttributeEditorToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return UMultiSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

const FToolTargetTypeRequirements& UAttributeEditorToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}


void UAttributeEditorActionPropertySet::PostAction(EAttributeEditorToolActions Action)
{
	if (ParentTool.IsValid() && Cast<UAttributeEditorTool>(ParentTool))
	{
		Cast<UAttributeEditorTool>(ParentTool)->RequestAction(Action);
	}
}




/*
 * Tool
 */



TArray<FString> UAttributeEditorUVActions::GetUVLayerNamesFunc()
{
	return UVLayerNamesList;
}


TArray<FString> UAttributeEditorModifyAttributeActions::GetAttributeNamesFunc()
{
	return AttributeNamesList;
}




UAttributeEditorTool::UAttributeEditorTool()
{
}


void UAttributeEditorTool::Setup()
{
	UInteractiveTool::Setup();

	NormalsActions = NewObject<UAttributeEditorNormalsActions>(this);
	NormalsActions->Initialize(this);
	AddToolPropertySource(NormalsActions);

	if (Targets.Num() == 1)
	{
		UPrimitiveComponent* TargetComponent = UE::ToolTarget::GetTargetComponent(Targets[0]);
		bTargetIsStaticMesh = Cast<UStaticMeshComponent>(TargetComponent) != nullptr;

		UVActions = NewObject<UAttributeEditorUVActions>(this);
		UVActions->Initialize(this);
		AddToolPropertySource(UVActions);

		if (bTargetIsStaticMesh)
		{
			LightmapUVActions = NewObject<UAttributeEditorLightmapUVActions>(this);
			LightmapUVActions->Initialize(this);
			AddToolPropertySource(LightmapUVActions);
		}

		NewAttributeProps = NewObject<UAttributeEditorNewAttributeActions>(this);
		NewAttributeProps->Initialize(this);
		AddToolPropertySource(NewAttributeProps);

		ModifyAttributeProps = NewObject<UAttributeEditorModifyAttributeActions>(this);
		ModifyAttributeProps->Initialize(this);
		AddToolPropertySource(ModifyAttributeProps);
		//SetToolPropertySourceEnabled(ModifyAttributeProps, false);

		CopyAttributeProps = NewObject<UAttributeEditorCopyAttributeActions>(this);
		CopyAttributeProps->Initialize(this);
		AddToolPropertySource(CopyAttributeProps);
		SetToolPropertySourceEnabled(CopyAttributeProps, false);

		AttributeProps = NewObject<UAttributeEditorAttribProperties>(this);
		AddToolPropertySource(AttributeProps);

		InitializeAttributeLists();
	}

	SetToolDisplayName(LOCTEXT("ToolName", "Edit Attributes"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartAttribEditor", "Inspect and Modify Attributes of a StaticMesh Asset"),
		EToolMessageLevel::UserNotification);
}


struct FAttributeEditorAttribInfo
{
	FName Name;
	EAttributeEditorElementType ElementType;
	EAttributeEditorAttribType DataType;
	bool bIsAutoGenerated;
};

template<typename AttribSetType>
void ExtractAttribList(const FMeshDescription* Mesh, AttribSetType& AttribSet, EAttributeEditorElementType ElemType, TArray<FAttributeEditorAttribInfo>& AttribList, TArray<FString>& StringList)
{
	AttribList.Reset();
	StringList.Reset();

	static FString EnumStrings[] = {
		"Int32", "Boolean", "Float", "Vector2", "Vector3", "Vector4", "String", "Unknown"
	};

	AttribSet.ForEach([&](const FName AttributeName, auto AttributesRef)
	{
		FAttributeEditorAttribInfo AttribInfo;
		AttribInfo.Name = AttributeName;
		AttribInfo.ElementType = ElemType;
		AttribInfo.DataType = EAttributeEditorAttribType::Unknown;
		AttribInfo.bIsAutoGenerated = (AttributesRef.GetFlags() & EMeshAttributeFlags::AutoGenerated) != EMeshAttributeFlags::None;
		if (AttribSet.template HasAttributeOfType<int32>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::Int32;
		}
		else if (AttribSet.template HasAttributeOfType<float>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::Float;
		}
		else if (AttribSet.template HasAttributeOfType<bool>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::Boolean;
		}
		else if (AttribSet.template HasAttributeOfType<FVector2f>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::Vector2;
		}
		else if (AttribSet.template HasAttributeOfType<FVector3f>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::Vector3;
		}
		else if (AttribSet.template HasAttributeOfType<FVector4f>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::Vector4;
		}
		else if (AttribSet.template HasAttributeOfType<FName>(AttributeName))
		{
			AttribInfo.DataType = EAttributeEditorAttribType::String;
		}
		AttribList.Add(AttribInfo);
	});
	AttribList.Sort([](const FAttributeEditorAttribInfo& A, const FAttributeEditorAttribInfo& B)
	{
		return A.Name.LexicalLess(B.Name);
	});

	for (const FAttributeEditorAttribInfo& AttribInfo : AttribList)
	{	
		FString UIString = (AttribInfo.bIsAutoGenerated) ?
			FString::Printf(TEXT("%s - %s (autogen)"), *(AttribInfo.Name.ToString()), *EnumStrings[(int32)AttribInfo.DataType])
			: FString::Printf(TEXT("%s - %s"), *(AttribInfo.Name.ToString()), *EnumStrings[(int32)AttribInfo.DataType]);
		StringList.Add(UIString);
	}
}



static const FAttributesSetBase* GetAttributeSetByType(const FMeshDescription* Mesh, EAttributeEditorElementType ElemType)
{
	switch (ElemType)
	{
	case EAttributeEditorElementType::Vertex:
		return &Mesh->VertexAttributes();
	case EAttributeEditorElementType::VertexInstance:
		return &Mesh->VertexInstanceAttributes();
	case EAttributeEditorElementType::Triangle:
		return &Mesh->TriangleAttributes();
	case EAttributeEditorElementType::Polygon:
		return &Mesh->PolygonAttributes();
	case EAttributeEditorElementType::Edge:
		return &Mesh->EdgeAttributes();
	case EAttributeEditorElementType::PolygonGroup:
		return &Mesh->PolygonGroupAttributes();
	}
	check(false);
	return nullptr;
}
static FAttributesSetBase* GetAttributeSetByType(FMeshDescription* Mesh, EAttributeEditorElementType ElemType)
{
	switch (ElemType)
	{
	case EAttributeEditorElementType::Vertex:
		return &Mesh->VertexAttributes();
	case EAttributeEditorElementType::VertexInstance:
		return &Mesh->VertexInstanceAttributes();
	case EAttributeEditorElementType::Triangle:
		return &Mesh->TriangleAttributes();
	case EAttributeEditorElementType::Polygon:
		return &Mesh->PolygonAttributes();
	case EAttributeEditorElementType::Edge:
		return &Mesh->EdgeAttributes();
	case EAttributeEditorElementType::PolygonGroup:
		return &Mesh->PolygonGroupAttributes();
	}
	check(false);
	return nullptr;
}


static bool HasAttribute(const FMeshDescription* Mesh, EAttributeEditorElementType ElemType, FName AttributeName)
{
	const FAttributesSetBase* AttribSetBase = GetAttributeSetByType(Mesh, ElemType);
	return (AttribSetBase) ? AttribSetBase->HasAttribute(AttributeName) : false;
}

static bool AddAttribute(FMeshDescription* Mesh, EAttributeEditorElementType ElemType, EAttributeEditorAttribType AttribType, FName AttributeName)
{
	FAttributesSetBase* AttribSetBase = GetAttributeSetByType(Mesh, ElemType);
	if (AttribSetBase != nullptr)
	{
		switch (AttribType)
		{
		case EAttributeEditorAttribType::Int32:
			AttribSetBase->RegisterAttribute<int32>(AttributeName, 1, 0, EMeshAttributeFlags::None);
			return true;
		case EAttributeEditorAttribType::Boolean:
			AttribSetBase->RegisterAttribute<bool>(AttributeName, 1, false, EMeshAttributeFlags::None);
			return true;
		case EAttributeEditorAttribType::Float:
			AttribSetBase->RegisterAttribute<float>(AttributeName, 1, 0.0f, EMeshAttributeFlags::Lerpable);
			return true;
		case EAttributeEditorAttribType::Vector2:
			AttribSetBase->RegisterAttribute<FVector2f>(AttributeName, 1, FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable);
			return true;
		case EAttributeEditorAttribType::Vector3:
			AttribSetBase->RegisterAttribute<FVector3f>(AttributeName, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Lerpable);
			return true;
		case EAttributeEditorAttribType::Vector4:
			AttribSetBase->RegisterAttribute<FVector4f>(AttributeName, 1, FVector4f(0,0,0,1), EMeshAttributeFlags::Lerpable);
			return true;
		}
	}
	return false;
}



static bool RemoveAttribute(FMeshDescription* Mesh, EAttributeEditorElementType ElemType, FName AttributeName)
{
	FAttributesSetBase* AttribSetBase = GetAttributeSetByType(Mesh, ElemType);
	if (AttribSetBase != nullptr)
	{
		AttribSetBase->UnregisterAttribute(AttributeName);
		return true;
	}
	return false;
}



void UAttributeEditorTool::InitializeAttributeLists()
{
	const FMeshDescription* Mesh = UE::ToolTarget::GetMeshDescription(Targets[0]);

	TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs =
		Mesh->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);

	UVActions->UVLayerNamesList.Reset();
	for ( int32 k = 0; k < InstanceUVs.GetNumChannels(); ++k )
	{
		UVActions->UVLayerNamesList.Add(FString::Printf(TEXT("UV%d"), k));
	}
	if (!UVActions->UVLayerNamesList.IsEmpty())
	{
		UVActions->UVLayer = UVActions->UVLayerNamesList[0];
	}

	if (bTargetIsStaticMesh && LightmapUVActions != nullptr)
	{
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[0]));
		if (StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			const UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			const FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
			LightmapUVActions->bGenerateLightmapUVs = BuildSettings.bGenerateLightmapUVs;
			LightmapUVActions->SourceUVIndex = BuildSettings.SrcLightmapIndex;
			LightmapUVActions->DestinationUVIndex = BuildSettings.DstLightmapIndex;

			bHaveAutoGeneratedLightmapUVSet = (LightmapUVActions->DestinationUVIndex >= InstanceUVs.GetNumChannels());
		}
	}

	TArray<FAttributeEditorAttribInfo> VertexAttributes, InstanceAttributes, TriangleAttributes, PolygonAttributes, EdgeAttributes, GroupAttributes;
	ExtractAttribList(Mesh, Mesh->VertexAttributes(), EAttributeEditorElementType::Vertex, VertexAttributes, AttributeProps->VertexAttributes);
	ExtractAttribList(Mesh, Mesh->VertexInstanceAttributes(), EAttributeEditorElementType::VertexInstance, InstanceAttributes, AttributeProps->InstanceAttributes);
	ExtractAttribList(Mesh, Mesh->TriangleAttributes(), EAttributeEditorElementType::Triangle, TriangleAttributes, AttributeProps->TriangleAttributes);
	ExtractAttribList(Mesh, Mesh->PolygonAttributes(), EAttributeEditorElementType::Polygon, PolygonAttributes, AttributeProps->PolygonAttributes);
	ExtractAttribList(Mesh, Mesh->EdgeAttributes(), EAttributeEditorElementType::Edge, EdgeAttributes, AttributeProps->EdgeAttributes);
	ExtractAttribList(Mesh, Mesh->PolygonGroupAttributes(), EAttributeEditorElementType::PolygonGroup, GroupAttributes, AttributeProps->GroupAttributes);

	TArray<FString> OldAttributeNames = ModifyAttributeProps->AttributeNamesList;

	ModifyAttributeProps->AttributeNamesList.Reset();
	CopyAttributeProps->FromAttribute.Reset();
	CopyAttributeProps->ToAttribute.Reset();

	//TArray<TArray<FAttributeEditorAttribInfo>*> AttribInfos = {
	//	&this->VertexAttributes, &this->InstanceAttributes,
	//	&this->TriangleAttributes, & this->PolygonAttributes,
	//	& this->EdgeAttributes, & this->GroupAttributes };
	TArray<TArray<FAttributeEditorAttribInfo>*> AttribInfos = {
		&VertexAttributes, &PolygonAttributes, &TriangleAttributes };

	for (TArray<FAttributeEditorAttribInfo>* AttribInfoList : AttribInfos)
	{
		for (FAttributeEditorAttribInfo& AttribInfo : *AttribInfoList)
		{
			if (FSkeletalMeshAttributes::IsReservedAttributeName(AttribInfo.Name) == false)
			{
				ModifyAttributeProps->AttributeNamesList.Add(AttribInfo.Name.ToString());
			}
			//CopyAttributeProps->FromAttribute.Add(AttribInfo.Name);
			//CopyAttributeProps->ToAttribute.Add(AttribInfo.Name);
		}
	}

	if (!ModifyAttributeProps->AttributeNamesList.Contains(ModifyAttributeProps->Attribute))
	{
		ModifyAttributeProps->Attribute.Reset();
	}

	// If we've added a new attribute to the list, set it as the selected attribute (useful when undoing an attribute deletion)
	for (const FString& AttributeName : ModifyAttributeProps->AttributeNamesList)
	{
		if (!OldAttributeNames.Contains(AttributeName))
		{
			ModifyAttributeProps->Attribute = AttributeName;
			break;
		}
	}

	bAttributeListsValid = true;
}



void UAttributeEditorTool::OnShutdown(EToolShutdownType ShutdownType)
{
}


void UAttributeEditorTool::RequestAction(EAttributeEditorToolActions ActionType)
{
	if (PendingAction == EAttributeEditorToolActions::NoAction)
	{
		PendingAction = ActionType;
	}
}


void UAttributeEditorTool::OnTick(float DeltaTime)
{
	switch (PendingAction)
	{
	case EAttributeEditorToolActions::ClearNormals:
		ClearNormals();
		break;
	case EAttributeEditorToolActions::DiscardTangents:
		DiscardTangents();
		break;
	case EAttributeEditorToolActions::ClearAllUVs:
		ClearUVs();
		break;
	case EAttributeEditorToolActions::AddUVSet:
		AddUVSet();
		break;
	case EAttributeEditorToolActions::DeleteSelectedUVSet:
		DeleteSelectedUVSet();
		break;
	case EAttributeEditorToolActions::DuplicateSelectedUVSet:
		DuplicateSelectedUVSet();
		break;
	case EAttributeEditorToolActions::AddAttribute:
		AddNewAttribute();
		break;
	case EAttributeEditorToolActions::AddWeightMapLayer:
		AddNewWeightMap();
		break;
	case EAttributeEditorToolActions::AddPolyGroupLayer:
		AddNewGroupsLayer();
		break;
	case EAttributeEditorToolActions::DeleteAttribute:
		DeleteAttribute();
		break;

	case EAttributeEditorToolActions::EnableLightmapUVs:
		SetLightmapUVsEnabled(true);
		break;
	case EAttributeEditorToolActions::DisableLightmapUVs:
		SetLightmapUVsEnabled(false);
		break;
	case EAttributeEditorToolActions::ResetLightmapUVChannels:
		ResetLightmapUVsChannels();
		break;
	}
	PendingAction = EAttributeEditorToolActions::NoAction;

	if (bAttributeListsValid == false && Targets.Num() == 1)
	{
		InitializeAttributeLists();
	}
}






void UAttributeEditorTool::ClearNormals()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("ClearNormalsTransactionMessage", "Reset Normals"));

	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FMeshDescription EditedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx]);

		TEdgeAttributesRef<bool> EdgeHardnesses = EditedMesh.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);
		if (EdgeHardnesses.IsValid())
		{
			for (FEdgeID ElID : EditedMesh.Edges().GetElementIDs())
			{
				EdgeHardnesses[ElID] = false;
			}
		}
		FStaticMeshOperations::ComputeTriangleTangentsAndNormals(EditedMesh, FMathf::Epsilon);
		FStaticMeshOperations::RecomputeNormalsAndTangentsIfNeeded(EditedMesh, EComputeNTBsFlags::WeightedNTBs | EComputeNTBsFlags::Normals);

		UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[ComponentIdx], MoveTemp(EditedMesh));
	}
	GetToolManager()->EndUndoTransaction();
}



void UAttributeEditorTool::DiscardTangents()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("DiscardTangentsTransactionMessage", "Discard Tangents"));

	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FMeshDescription EditedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx]);

		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = EditedMesh.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = EditedMesh.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);
		const FVertexInstanceArray& Instances = EditedMesh.VertexInstances();
		if (VertexInstanceTangents.IsValid())
		{
			for (const FVertexInstanceID InstanceID : Instances.GetElementIDs())
			{
				VertexInstanceTangents.Set(InstanceID, FVector3f::ZeroVector);
			}
		}
		if (VertexInstanceBinormalSigns.IsValid())
		{
			for (const FVertexInstanceID InstanceID : Instances.GetElementIDs())
			{
				VertexInstanceBinormalSigns.Set(InstanceID, 1.0f);
			}
		}

		UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[ComponentIdx], MoveTemp(EditedMesh));
	}
	GetToolManager()->EndUndoTransaction();
}



void UAttributeEditorTool::ClearUVs()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("ClearUVsTransactionMessage", "Clear Selected UVs"));
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FMeshDescription EditedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx]);

		TVertexInstanceAttributesRef<FVector2f> InstanceUVs =
			EditedMesh.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		int32 NumChannels = InstanceUVs.GetNumChannels();
		const FVertexInstanceArray& Instances = EditedMesh.VertexInstances();
		for (int LayerIndex = NumChannels-1; LayerIndex >= 0; LayerIndex--)
		{
			if (!FStaticMeshOperations::RemoveUVChannel(EditedMesh, LayerIndex))
			{
				for (const FVertexInstanceID ElID : Instances.GetElementIDs())
				{
					InstanceUVs.Set(ElID, LayerIndex, FVector2f::ZeroVector);
				}
			}
		}

		UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[ComponentIdx], MoveTemp(EditedMesh));

		if (bHaveAutoGeneratedLightmapUVSet)
		{
			UpdateAutoGeneratedLightmapUVChannel(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]), InstanceUVs.GetNumChannels());
		}
	}

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	// update attrib lists
	bAttributeListsValid = false;
}




void UAttributeEditorTool::DeleteSelectedUVSet()
{
	int32 DeleteIndex = UVActions->UVLayerNamesList.IndexOfByKey(UVActions->UVLayer);
	if (DeleteIndex == INDEX_NONE)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("CannotFindUVSet", "Selected UV Set Not Found"), EToolMessageLevel::UserWarning);
		return;
	}
	if (DeleteIndex == 0 && UVActions->UVLayerNamesList.Num() == 1)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("CannotDeleteLastUVSet", "Cannot Delete Last UV Set. UVs will be cleared to Zero."), EToolMessageLevel::UserWarning);
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("ClearUVsTransactionMessage", "Clear Selected UVs"));
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FMeshDescription EditedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx]);

		TVertexInstanceAttributesRef<FVector2f> InstanceUVs =
			EditedMesh.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		if (!FStaticMeshOperations::RemoveUVChannel(EditedMesh, DeleteIndex))
		{
			const FVertexInstanceArray& Instances = EditedMesh.VertexInstances();
			for (const FVertexInstanceID InstanceID : Instances.GetElementIDs())
			{
				InstanceUVs.Set(InstanceID, DeleteIndex, FVector2f::ZeroVector);
			}
		}

		UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[ComponentIdx], MoveTemp(EditedMesh));

		if (bHaveAutoGeneratedLightmapUVSet)
		{
			UpdateAutoGeneratedLightmapUVChannel(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]), InstanceUVs.GetNumChannels());
		}
	}

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	bAttributeListsValid = false;
}


void UAttributeEditorTool::AddUVSet()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("AddUVSetMessage", "Add UV Set"));
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FMeshDescription EditedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx]);

		TVertexInstanceAttributesRef<FVector2f> InstanceUVs =
			EditedMesh.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		int32 NewChannelIndex = InstanceUVs.GetNumChannels();
		if (FStaticMeshOperations::AddUVChannel(EditedMesh))
		{
			GetToolManager()->DisplayMessage(FText::Format(LOCTEXT("AddedNewUVSet", "Added UV{0}"), FText::FromString(FString::FromInt(NewChannelIndex))), EToolMessageLevel::UserWarning);
			UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[ComponentIdx], MoveTemp(EditedMesh));

			if (bHaveAutoGeneratedLightmapUVSet)
			{
				UpdateAutoGeneratedLightmapUVChannel(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]), InstanceUVs.GetNumChannels());
			}
		}
		else
		{
			GetToolManager()->DisplayMessage(LOCTEXT("FailedToAddUVSet", "Adding UV Set Failed"), EToolMessageLevel::UserWarning);
		}
	}

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	bAttributeListsValid = false;
}



void UAttributeEditorTool::DuplicateSelectedUVSet()
{
	int32 SourceIndex = UVActions->UVLayerNamesList.IndexOfByKey(UVActions->UVLayer);
	if (SourceIndex == INDEX_NONE)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("CannotFindUVSet", "Selected UV Set Not Found"), EToolMessageLevel::UserWarning);
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("DuplicateUVSetMessage", "Duplicate UV Set"));
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		FMeshDescription EditedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Targets[ComponentIdx]);

		TVertexInstanceAttributesRef<FVector2f> InstanceUVs =
			EditedMesh.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		int32 NewChannelIndex = InstanceUVs.GetNumChannels();
		if (FStaticMeshOperations::AddUVChannel(EditedMesh))
		{
			const FVertexInstanceArray& Instances = EditedMesh.VertexInstances();
			for (const FVertexInstanceID InstanceID : Instances.GetElementIDs())
			{
				FVector2f SourceUV = InstanceUVs.Get(InstanceID, SourceIndex);
				InstanceUVs.Set(InstanceID, NewChannelIndex, SourceUV);
			}

			UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[ComponentIdx], MoveTemp(EditedMesh));

			if (bHaveAutoGeneratedLightmapUVSet)
			{
				UpdateAutoGeneratedLightmapUVChannel(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]), InstanceUVs.GetNumChannels());
			}

			GetToolManager()->DisplayMessage(FText::Format(LOCTEXT("Copied UV Set", "Copied UV{0} to UV{1}"), 
				FText::FromString(FString::FromInt(SourceIndex)), FText::FromString(FString::FromInt(NewChannelIndex))), EToolMessageLevel::UserWarning);
		}
		else
		{
			GetToolManager()->DisplayMessage(LOCTEXT("FailedToAddUVSet", "Adding UV Set Failed"), EToolMessageLevel::UserWarning);
		}
	}

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	bAttributeListsValid = false;
}





void UAttributeEditorTool::AddNewAttribute(EAttributeEditorElementType ElemType, EAttributeEditorAttribType DataType, FName AttributeName)
{
	if (AttributeName.IsNone())
	{
		GetToolManager()->DisplayMessage(LOCTEXT("InvalidAttributeName", "Invalid attribute name"), EToolMessageLevel::UserWarning);
		return;
	}

	const FMeshDescription* CurMesh = UE::ToolTarget::GetMeshDescription(Targets[0]);
	if (HasAttribute(CurMesh, ElemType, AttributeName))
	{
		GetToolManager()->DisplayMessage(LOCTEXT("ErrorAddingDuplicateNameMessage", "Attribute with this name already exists"), EToolMessageLevel::UserWarning);
		return;
	}

	FMeshDescription NewMesh = *CurMesh;
	if (AddAttribute(&NewMesh, ElemType, DataType, AttributeName) == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("FailedAddingNewMessage", "Unknown error adding new Attribute"), EToolMessageLevel::UserWarning);
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("NewAttributeTransactionMessage", "Add Attribute"));
	UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[0], &NewMesh);

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	bAttributeListsValid = false;
}


void UAttributeEditorTool::AddNewAttribute()
{
	if (NewAttributeProps->DataType == EAttributeEditorAttribType::Unknown)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("ErrorAddingTypeMessage", "Currently cannot add this attribute type"), EToolMessageLevel::UserWarning);
		return;
	}

	AddNewAttribute(NewAttributeProps->ElementType, NewAttributeProps->DataType, FName(NewAttributeProps->NewName));
}

void UAttributeEditorTool::AddNewWeightMap()
{
	AddNewAttribute(EAttributeEditorElementType::Vertex, EAttributeEditorAttribType::Float, FName(NewAttributeProps->NewName));
}

void UAttributeEditorTool::AddNewGroupsLayer()
{
	AddNewAttribute(EAttributeEditorElementType::Triangle, EAttributeEditorAttribType::Int32, FName(NewAttributeProps->NewName));
}


void UAttributeEditorTool::ClearAttribute()
{
}

void UAttributeEditorTool::DeleteAttribute()
{
	const FMeshDescription* CurMesh = UE::ToolTarget::GetMeshDescription(Targets[0]);
	FName SelectedName(ModifyAttributeProps->Attribute);

	// We check on the skeletal mesh attributes because it is a superset of the static mesh
	// attributes.
	if (FSkeletalMeshAttributes::IsReservedAttributeName(SelectedName))
	{
		GetToolManager()->DisplayMessage(LOCTEXT("CannotDeleteReservedNameError", "Cannot delete reserved mesh Attributes"), EToolMessageLevel::UserWarning);
		return;
	}

	EAttributeEditorElementType ElemType = EAttributeEditorElementType::Vertex;
	bool bIsDeletableAttribute = false;
	if (HasAttribute(CurMesh, EAttributeEditorElementType::Vertex, SelectedName))
	{
		bIsDeletableAttribute = true;
		ElemType = EAttributeEditorElementType::Vertex;
	}
	else if (HasAttribute(CurMesh, EAttributeEditorElementType::PolygonGroup, SelectedName))
	{
		bIsDeletableAttribute = true;
		ElemType = EAttributeEditorElementType::Polygon;
	}
	else if (HasAttribute(CurMesh, EAttributeEditorElementType::Triangle, SelectedName))
	{
		bIsDeletableAttribute = true;
		ElemType = EAttributeEditorElementType::Triangle;
	}

	if (!bIsDeletableAttribute)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("CannotDeleteAttribError", "Cannot delete the selected attribute"), EToolMessageLevel::UserWarning);
		return;
	}

	FMeshDescription NewMesh = *CurMesh;
	if (RemoveAttribute(&NewMesh, ElemType, SelectedName) == false)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("FailedRemovingNewMessage", "Unknown error removing Attribute"), EToolMessageLevel::UserWarning);
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("RemoveAttributeTransactionMessage", "Remove Attribute"));
	UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[0], &NewMesh);

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	bAttributeListsValid = false;
}



void UAttributeEditorTool::SetLightmapUVsEnabled(bool bEnabled)
{
	if (bEnabled)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("EnableLightmapVUs", "Enable Lightmap UVs"));
	}
	else
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("DisableLightmapUVs", "Disable Lightmap UVs"));
	}
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]));
		if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			StaticMesh->Modify();
			FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
			BuildSettings.bGenerateLightmapUVs = bEnabled;

			StaticMesh->PostEditChange();
		}
	}

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	// update attrib lists
	bAttributeListsValid = false;
}



void UAttributeEditorTool::ResetLightmapUVsChannels()
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("ResetLightmapUVs", "Reset Lightmap UVs"));
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		const FMeshDescription* SourceMesh = UE::ToolTarget::GetMeshDescription(Targets[ComponentIdx]);
		TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs = 
			SourceMesh->VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
		int32 SetChannel = FMath::Max(InstanceUVs.GetNumChannels(), 1);

		UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(UE::ToolTarget::GetTargetComponent(Targets[ComponentIdx]));
		if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh() != nullptr)
		{
			UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
			StaticMesh->Modify();
			FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
			BuildSettings.SrcLightmapIndex = 0;
			BuildSettings.DstLightmapIndex = SetChannel;
			StaticMesh->PostEditChange();
		}
	}

	EmitAttributesChange();

	GetToolManager()->EndUndoTransaction();

	// update attrib lists
	bAttributeListsValid = false;
}



void UAttributeEditorTool::UpdateAutoGeneratedLightmapUVChannel(UPrimitiveComponent* TargetComponent, int32 NewMaxUVChannels)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(TargetComponent);
	if (StaticMeshComponent && StaticMeshComponent->GetStaticMesh() != nullptr)
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		StaticMesh->Modify();

		FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
		BuildSettings.DstLightmapIndex = NewMaxUVChannels;
	}
}

void UAttributeEditorTool::EmitAttributesChange()
{
	TUniquePtr<FAttributeEditor_AttributeListsChange> AttributesChange = MakeUnique<FAttributeEditor_AttributeListsChange>();
	GetToolManager()->EmitObjectChange(this, MoveTemp(AttributesChange), LOCTEXT("AttributesChange", "Attributes Change"));
}

void FAttributeEditor_AttributeListsChange::Apply(UObject* Object)
{
	// We just need the properties to update from the newly-changed static mesh component
	UAttributeEditorTool* Tool = CastChecked<UAttributeEditorTool>(Object);
	Tool->bAttributeListsValid = false;
}

void FAttributeEditor_AttributeListsChange::Revert(UObject* Object)
{
	UAttributeEditorTool* Tool = CastChecked<UAttributeEditorTool>(Object);
	Tool->bAttributeListsValid = false;
}

FString FAttributeEditor_AttributeListsChange::ToString() const
{
	return FString(TEXT("AttributeLists Change"));
}


#undef LOCTEXT_NAMESPACE

