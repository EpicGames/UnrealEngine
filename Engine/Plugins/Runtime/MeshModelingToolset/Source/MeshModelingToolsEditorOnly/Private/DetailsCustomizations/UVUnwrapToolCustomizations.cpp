// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/UVUnwrapToolCustomizations.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SComboButton.h"
#include "IDetailChildrenBuilder.h"
#include "Internationalization/BreakIterator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "SAssetView.h"

#include "PropertyRestriction.h"

#include "Properties/RecomputeUVsProperties.h"


#define LOCTEXT_NAMESPACE "UVUnwrapDetailsCustomization"

namespace UVUnwrapDetailsCustomizationLocal
{

	template<class UENUM_TYPE>
	FPropertyAccess::Result GetPropertyValueAsEnum(const TSharedPtr<IPropertyHandle> Property, UENUM_TYPE& Value)
	{
		if(Property.IsValid())
		{
			uint8 ValueAsByte;
			FPropertyAccess::Result Result = Property->GetValue(/*out*/ ValueAsByte);

			if (Result == FPropertyAccess::Success)
			{
				Value = (UENUM_TYPE)ValueAsByte;
				return FPropertyAccess::Success;
			}
			else
			{
				return FPropertyAccess::Fail;
			}
		}
		return FPropertyAccess::Fail;
	}

}


//
// UVEditorRecomputeUVsTool
//


TSharedRef<IDetailCustomization> FRecomputeUVsToolDetails::MakeInstance()
{
	return MakeShareable(new FRecomputeUVsToolDetails);
}


void FRecomputeUVsToolDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> IslandGenerationModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(URecomputeUVsToolProperties, IslandGeneration), URecomputeUVsToolProperties::StaticClass());
	TSharedRef<IPropertyHandle> UnwrapTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(URecomputeUVsToolProperties, UnwrapType), URecomputeUVsToolProperties::StaticClass());
	TSharedRef<IPropertyHandle> LayoutTypeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(URecomputeUVsToolProperties, LayoutType), URecomputeUVsToolProperties::StaticClass());

	IslandGenerationModeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FRecomputeUVsToolDetails::EvaluateLayoutTypeRestrictions, IslandGenerationModeHandle, UnwrapTypeHandle, LayoutTypeHandle));
	UnwrapTypeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateRaw(this, &FRecomputeUVsToolDetails::EvaluateLayoutTypeRestrictions, IslandGenerationModeHandle, UnwrapTypeHandle, LayoutTypeHandle));

	static FText RestrictReason = LOCTEXT("LayoutTypeRestrictionText", "Requires existing UVs be used for IslandMode and Island Merging not selected as the unwrap strategy.");
	LayoutTypeEnumRestriction = MakeShareable(new FPropertyRestriction(RestrictReason));
	LayoutTypeHandle->AddRestriction(LayoutTypeEnumRestriction.ToSharedRef());

	EvaluateLayoutTypeRestrictions(IslandGenerationModeHandle, UnwrapTypeHandle, LayoutTypeHandle);	
}

void FRecomputeUVsToolDetails::EvaluateLayoutTypeRestrictions(TSharedRef<IPropertyHandle> IslandGenerationModeHandle, TSharedRef<IPropertyHandle> UnwrapTypeHandle, TSharedRef<IPropertyHandle> LayoutTypeHandle)
{
	ERecomputeUVsPropertiesIslandMode IslandMode = ERecomputeUVsPropertiesIslandMode::PolyGroups;
	ERecomputeUVsPropertiesUnwrapType UnwrapType = ERecomputeUVsPropertiesUnwrapType::Conformal;
	FPropertyAccess::Result IslandModeResult = UVUnwrapDetailsCustomizationLocal::GetPropertyValueAsEnum(IslandGenerationModeHandle, IslandMode);
	FPropertyAccess::Result UnwrapTypeResult = UVUnwrapDetailsCustomizationLocal::GetPropertyValueAsEnum(UnwrapTypeHandle, UnwrapType);

	UEnum* ImportTypeEnum = StaticEnum<ERecomputeUVsPropertiesLayoutType>();

	LayoutTypeEnumRestriction->RemoveAll();
	if (IslandModeResult != FPropertyAccess::Fail &&
		UnwrapTypeResult != FPropertyAccess::Fail &&
		(IslandMode == ERecomputeUVsPropertiesIslandMode::PolyGroups ||
		 UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging))
	{
		uint8 EnumValue = static_cast<uint8>(ImportTypeEnum->GetValueByName("EUVEditorRecomputeUVsPropertiesLayoutType::NormalizeToExistingBounds"));
		LayoutTypeEnumRestriction->AddDisabledValue("NormalizeToExistingBounds");

		uint8 Value = 0;
		LayoutTypeHandle->GetValue(Value);
		// The goal here is to shift our selected layout type away from the offending option, either up or down one.
		// This code assumes though that there are options other than our offending enum value to move off to. If it's the only one,
		// this code will end up doing the wrong thing.
		if (Value == EnumValue)
		{
			if (Value > 0)
			{
				Value = EnumValue - 1;
			}
			else
			{
				Value = EnumValue + 1;
			}
		}
		LayoutTypeHandle->SetValue(Value, EPropertyValueSetFlags::NotTransactable | EPropertyValueSetFlags::InteractiveChange);
	}


}

#undef LOCTEXT_NAMESPACE