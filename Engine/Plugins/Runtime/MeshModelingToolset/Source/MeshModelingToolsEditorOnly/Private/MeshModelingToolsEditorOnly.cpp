// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshModelingToolsEditorOnly.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "DetailsCustomizations/UVUnwrapToolCustomizations.h"
#include "Properties/RecomputeUVsProperties.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "SkeletalMesh/SkinWeightsPaintTool.h"

#define LOCTEXT_NAMESPACE "FMeshModelingToolsEditorOnlyModule"

DEFINE_LOG_CATEGORY(LogMeshModelingToolsEditor);

void FMeshModelingToolsEditorOnlyModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMeshModelingToolsEditorOnlyModule::OnPostEngineInit);
	
	FSkeletonEditingToolCommands::Register();
	FSkinWeightsPaintToolCommands::Register();
}

void FMeshModelingToolsEditorOnlyModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (const FName& ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
		for (const FName& PropertyName : PropertiesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}
}

void FMeshModelingToolsEditorOnlyModule::OnPostEngineInit()
{

	// Register details view customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	/// Unwrap
	PropertyModule.RegisterCustomClassLayout("RecomputeUVsToolProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FRecomputeUVsToolDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(URecomputeUVsToolProperties::StaticClass()->GetFName());


}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FMeshModelingToolsEditorOnlyModule, MeshModelingToolsEditorOnly)