// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizeMeshTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ParameterizationOps/ParameterizeMeshOp.h"
#include "Properties/ParameterizeMeshProperties.h"
#include "Polygroups/PolygroupUtil.h"
#include "Polygroups/PolygroupSet.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParameterizeMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UParameterizeMeshTool"

/*
 * ToolBuilder
 */

USingleSelectionMeshEditingTool* UParameterizeMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	UParameterizeMeshTool* NewTool = NewObject<UParameterizeMeshTool>(SceneState.ToolManager);
	return NewTool;
}

bool UParameterizeMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return USingleSelectionMeshEditingToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return ToolBuilderUtil::ComponentTypeCouldHaveUVs(Component); }) == 1;
}

/*
 * Tool
 */


void UParameterizeMeshTool::Setup()
{
	UInteractiveTool::Setup();

	InputMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*InputMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
	FTransform InputTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);

	UE::ToolTarget::HideSourceObject(Target);

	// initialize our properties

	UVChannelProperties = NewObject<UMeshUVChannelProperties>(this);
	UVChannelProperties->RestoreProperties(this);
	UVChannelProperties->Initialize(InputMesh.Get(), false);
	UVChannelProperties->ValidateSelection(true);
	UVChannelProperties->WatchProperty(UVChannelProperties->UVChannel, [this](const FString& NewValue) 
	{
		MaterialSettings->UpdateUVChannels(UVChannelProperties->UVChannelNamesList.IndexOfByKey(UVChannelProperties->UVChannel),
		                                   UVChannelProperties->UVChannelNamesList);
	});
	AddToolPropertySource(UVChannelProperties);

	Settings = NewObject<UParameterizeMeshToolProperties>(this);
	Settings->RestoreProperties(this);
	AddToolPropertySource(Settings);
	Settings->WatchProperty(Settings->Method, [&](EParameterizeMeshUVMethod) { OnMethodTypeChanged(); });


	UVAtlasProperties = NewObject<UParameterizeMeshToolUVAtlasProperties>(this);
	UVAtlasProperties->RestoreProperties(this);
	AddToolPropertySource(UVAtlasProperties);
	SetToolPropertySourceEnabled(UVAtlasProperties, false);

	XAtlasProperties = NewObject<UParameterizeMeshToolXAtlasProperties>(this);
	XAtlasProperties->RestoreProperties(this);
	AddToolPropertySource(XAtlasProperties);
	SetToolPropertySourceEnabled(XAtlasProperties, false);

	PatchBuilderProperties = NewObject<UParameterizeMeshToolPatchBuilderProperties>(this);
	PatchBuilderProperties->RestoreProperties(this);
	AddToolPropertySource(PatchBuilderProperties);
	SetToolPropertySourceEnabled(PatchBuilderProperties, false);

	PolygroupLayerProperties = NewObject<UPolygroupLayersProperties>(this);
	PolygroupLayerProperties->RestoreProperties(this, TEXT("ModelingUVTools"));
	PolygroupLayerProperties->InitializeGroupLayers(InputMesh.Get());
	AddToolPropertySource(PolygroupLayerProperties);
	PatchBuilderProperties->bPolygroupsEnabled = true;
	UVAtlasProperties->bPolygroupsEnabled = true;
	SetToolPropertySourceEnabled(PolygroupLayerProperties, (Settings->Method == EParameterizeMeshUVMethod::PatchBuilder && 
		                                                    PatchBuilderProperties->bUsePolygroups)
														   || (Settings->Method == EParameterizeMeshUVMethod::UVAtlas &&
															   UVAtlasProperties->bUsePolygroups));

	MaterialSettings = NewObject<UExistingMeshMaterialProperties>(this);
	MaterialSettings->MaterialMode = ESetMeshMaterialMode::Checkerboard;
	MaterialSettings->RestoreProperties(this, TEXT("ModelingUVTools"));
	AddToolPropertySource(MaterialSettings);

	if (bCreateUVLayoutViewOnSetup)
	{
		UVLayoutView = NewObject<UUVLayoutPreview>(this);
		UVLayoutView->CreateInWorld(GetTargetWorld());
		UVLayoutView->SetSourceMaterials(MaterialSet);
		FBox Bounds = UE::ToolTarget::GetTargetActor(Target)->GetComponentsBoundingBox(true /*bNonColliding*/);
		if (!Bounds.IsValid) // If component did not have valid bounds ...
		{
			// Try getting bounds from mesh
			Bounds = (FBox)InputMesh->GetBounds();
			Bounds = Bounds.TransformBy(InputTransform);
			// If mesh is also empty, just create some small valid Bounds
			if (!Bounds.IsValid)
			{
				UE_LOGF(LogGeometry, Warning, "Auto UV Tool started on mesh with empty bounding box");
				constexpr double SmallSize = FMathd::ZeroTolerance;
				Bounds = FBox(FVector(-1, -1, -1) * SmallSize, FVector(1, 1, 1) * SmallSize);
			}
		}
		UVLayoutView->SetSourceWorldPosition(InputTransform, Bounds);
		UVLayoutView->Settings->bEnabled = false;
		UVLayoutView->Settings->bShowWireframe = false;
		UVLayoutView->Settings->RestoreProperties(this, TEXT("ParameterizeMeshTool"));
		AddToolPropertySource(UVLayoutView->Settings);
	}

	Factory = NewObject<UParameterizeMeshOperatorFactory>();
	Factory->TargetTransform = InputTransform;
	Factory->Settings = Settings;
	Factory->GetPolygroups = [this]() {
		if (PolygroupLayerProperties->HasSelectedPolygroup() == false)
		{
			return MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(InputMesh.Get());
		}
		else
		{
			FName SelectedName = PolygroupLayerProperties->ActiveGroupLayer;
			FDynamicMeshPolygroupAttribute* FoundAttrib = UE::Geometry::FindPolygroupLayerByName(*InputMesh, SelectedName);
			ensureMsgf(FoundAttrib, TEXT("Selected attribute not found! Falling back to Default group layer."));
			return MakeShared<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>(InputMesh.Get(), FoundAttrib);
		}
	};
	Factory->UVAtlasProperties = UVAtlasProperties;
	Factory->XAtlasProperties = XAtlasProperties;
	Factory->PatchBuilderProperties = PatchBuilderProperties;
	Factory->OriginalMesh = InputMesh;
	Factory->GetSelectedUVChannel = [this]() { return UVChannelProperties->UVChannelNamesList.IndexOfByKey(UVChannelProperties->UVChannel); };

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(GetTargetWorld(), Factory);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->PreviewMesh->ReplaceMesh(*InputMesh);
	Preview->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));
	Preview->PreviewMesh->SetTransform(InputTransform);

	Preview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Op)
		{
			OnPreviewMeshUpdated();
		});
	// force update
	MaterialSettings->UpdateMaterials();
	Preview->OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
	Preview->InvalidateResult();    // start compute

	SetToolDisplayName(LOCTEXT("ToolNameGlobal", "AutoUV"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool_Global", "Automatically partition the selected Mesh into UV islands, flatten, and pack into a single UV chart"),
		EToolMessageLevel::UserNotification);
}

void UParameterizeMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	SetToolPropertySourceEnabled(PolygroupLayerProperties, (Settings->Method == EParameterizeMeshUVMethod::PatchBuilder && 
		                                                    PatchBuilderProperties->bUsePolygroups)
														   || (Settings->Method == EParameterizeMeshUVMethod::UVAtlas &&
															   UVAtlasProperties->bUsePolygroups));

	if (PropertySet == UVChannelProperties 
		|| PropertySet == Settings 
		|| PropertySet == UVAtlasProperties  
		|| PropertySet == XAtlasProperties  
		|| PropertySet == PatchBuilderProperties
		|| PropertySet == PolygroupLayerProperties)
	{
		Preview->InvalidateResult();
	}

	MaterialSettings->UpdateMaterials();
	Preview->OverrideMaterial = MaterialSettings->GetActiveOverrideMaterial();
}


void UParameterizeMeshTool::OnMethodTypeChanged()
{
	SetToolPropertySourceEnabled(UVAtlasProperties, Settings->Method == EParameterizeMeshUVMethod::UVAtlas);
	SetToolPropertySourceEnabled(XAtlasProperties, Settings->Method == EParameterizeMeshUVMethod::XAtlas);
	SetToolPropertySourceEnabled(PatchBuilderProperties, Settings->Method == EParameterizeMeshUVMethod::PatchBuilder);

	Preview->InvalidateResult();
}


void UParameterizeMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (UVLayoutView)
	{
		UVLayoutView->Settings->SaveProperties(this, TEXT("ParameterizeMeshTool"));
		UVLayoutView->Disconnect();
	}

	UVChannelProperties->SaveProperties(this);
	Settings->SaveProperties(this);
	MaterialSettings->SaveProperties(this, TEXT("ModelingUVTools"));
	UVAtlasProperties->SaveProperties(this);
	XAtlasProperties->SaveProperties(this);
	PatchBuilderProperties->SaveProperties(this);
	PolygroupLayerProperties->SaveProperties(this, TEXT("ModelingUVTools"));

	FDynamicMeshOpResult Result = Preview->Shutdown();
	
	// Restore (unhide) the source meshes
	UE::ToolTarget::ShowSourceObject(Target);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("ParameterizeMesh", "Auto UVs"));
		FDynamicMesh3* NewDynamicMesh = Result.Mesh.Get();
		if (ensure(NewDynamicMesh))
		{
			UE::ToolTarget::CommitDynamicMeshUVUpdate(Target, NewDynamicMesh);
		}
		GetToolManager()->EndUndoTransaction();
	}

	Factory = nullptr;
}

void UParameterizeMeshTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (UVLayoutView)
	{
		UVLayoutView->Render(RenderAPI);
	}
}

void UParameterizeMeshTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);

	if (UVLayoutView)
	{
		UVLayoutView->OnTick(DeltaTime);
	}
}

bool UParameterizeMeshTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidResult();
}

void UParameterizeMeshTool::OnPreviewMeshUpdated()
{
	if (UVLayoutView)
	{
		int32 UVChannel = UVChannelProperties ? UVChannelProperties->GetSelectedChannelIndex(true) : 0;
		Preview->PreviewMesh->ProcessMesh([&](const FDynamicMesh3& NewMesh)
		{
			UVLayoutView->UpdateUVMesh(&NewMesh, UVChannel);
		});
	}

	if (MaterialSettings)
	{
		MaterialSettings->UpdateMaterials();
	}
}


#undef LOCTEXT_NAMESPACE

