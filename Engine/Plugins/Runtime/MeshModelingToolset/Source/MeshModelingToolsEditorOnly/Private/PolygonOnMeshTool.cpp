// Copyright Epic Games, Inc. All Rights Reserved.

#include "PolygonOnMeshTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ToolSceneQueriesUtil.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "ToolDataVisualizer.h"
#include "Util/ColorConstants.h"
#include "Drawing/LineSetComponent.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolygonOnMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPolygonOnMeshTool"

/*
 * ToolBuilder
 */


USingleSelectionMeshEditingTool* UPolygonOnMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UPolygonOnMeshTool>(SceneState.ToolManager);
}




/*
 * Tool
 */

void UPolygonOnMeshToolActionPropertySet::PostAction(EPolygonOnMeshToolActions Action)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(Action);
	}
}




UPolygonOnMeshTool::UPolygonOnMeshTool()
{
	SetToolDisplayName(LOCTEXT("PolygonOnMeshToolName", "PolyCut"));
}

void UPolygonOnMeshTool::SetWorld(UWorld* World)
{
	this->TargetWorld = World;
}

void UPolygonOnMeshTool::Setup()
{
	UInteractiveTool::Setup();


	// register click and hover behaviors
	USingleClickInputBehavior* ClickBehavior = NewObject<USingleClickInputBehavior>(this);
	ClickBehavior->Initialize(this);
	AddInputBehavior(ClickBehavior);
	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);

	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	WorldTransform = (FTransform3d)TargetComponent->GetWorldTransform();

	// hide input StaticMeshComponent
	TargetComponent->SetOwnerVisibility(false);

	BasicProperties = NewObject<UPolygonOnMeshToolProperties>(this);
	BasicProperties->RestoreProperties(this);
	AddToolPropertySource(BasicProperties);

	ActionProperties = NewObject<UPolygonOnMeshToolActionPropertySet>(this);
	ActionProperties->Initialize(this);
	AddToolPropertySource(ActionProperties);

	// initialize the PreviewMesh+BackgroundCompute object
	SetupPreview();

	DrawnLineSet = NewObject<ULineSetComponent>(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetupAttachment(Preview->PreviewMesh->GetRootComponent());
	DrawnLineSet->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetToolManager()));
	DrawnLineSet->RegisterComponent();

	Preview->OnOpCompleted.AddLambda(
		[this](const FDynamicMeshOperator* Op)
		{
			const FEmbedPolygonsOp* PolygonsOp = (const FEmbedPolygonsOp*)(Op);
			EdgesOnFailure = PolygonsOp->EdgesOnFailure;
			EmbeddedEdges = PolygonsOp->EmbeddedEdges;
			bOperationSucceeded = PolygonsOp->bOperationSucceeded;
		}
	);
	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute* UpdatedPreview)
		{
			GetToolManager()->PostInvalidation();
			UpdateVisualization();
			if (!bOperationSucceeded)
			{
				GetToolManager()->DisplayMessage(LOCTEXT("FailNotification", "Unable to complete cut."),
					EToolMessageLevel::UserWarning);
			}
			else if (EdgesOnFailure.Num() > 0)
			{
				GetToolManager()->DisplayMessage(LOCTEXT("BoundaryEdgesNotification", "Cut resulted in one or more boundary edges."),
					EToolMessageLevel::UserWarning);
			}
			else if (UpdatedPreview->HaveEmptyResult())
			{
				UpdateAcceptWarnings(EAcceptWarning::EmptyForbidden);
			}
			else
			{
				GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
			}
		}
	);

	DrawPlaneWorld = FFrame3d(WorldTransform.GetTranslation());

	PlaneMechanic = NewObject<UConstructionPlaneMechanic>(this);
	PlaneMechanic->Setup(this);
	PlaneMechanic->Initialize(TargetWorld, DrawPlaneWorld);
	//PlaneMechanic->UpdateClickPriority(ClickBehavior->GetPriority().MakeHigher());
	PlaneMechanic->OnPlaneChanged.AddLambda([this]() {
		DrawPlaneWorld = PlaneMechanic->Plane;
		UpdateDrawPlane();
	});
	PlaneMechanic->SetPlaneCtrlClickBehaviorTarget->InvisibleComponentsToHitTest.Add(TargetComponent->GetOwnerComponent());

	// Convert input mesh description to dynamic mesh
	OriginalDynamicMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*OriginalDynamicMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
	// TODO: consider adding an AABB tree construction here?  tradeoff vs doing a raycast against full every time a param change happens ...

	LastDrawnPolygon = FPolygon2d();
	UpdatePolygonType();
	UpdateDrawPlane();

	GetToolManager()->DisplayMessage(
		LOCTEXT("PolygonOnMeshToolDescription", "Cut the Mesh with a swept Polygon, creating a Hole or new Polygroup. Use the Draw Polygon button to draw a custom polygon on the work plane. Ctrl-click to reposition the work plane."),
		EToolMessageLevel::UserNotification);
}


void UPolygonOnMeshTool::UpdateVisualization()
{
	FColor PartialPathEdgeColor(240, 15, 15);
	float PartialPathEdgeThickness = 2.0f;
	float PartialPathEdgeDepthBias = 3.0f;

	FColor EmbedEdgeColor(100, 240, 100);
	float EmbedEdgeThickness = 1.0f;
	float EmbedEdgeDepthBias = 1.0f;

	const FDynamicMesh3* TargetMesh = Preview->PreviewMesh->GetPreviewDynamicMesh();
	FVector3d A, B;

	DrawnLineSet->Clear();
	for (int EID : EmbeddedEdges)
	{
		TargetMesh->GetEdgeV(EID, A, B);
		DrawnLineSet->AddLine((FVector)A, (FVector)B, EmbedEdgeColor, EmbedEdgeThickness, EmbedEdgeDepthBias);
	}
	// We show problematic edges whether or not we consider the operation to be successful because in the case of
	// booleans, we consider boundary edge results to be important to show but not important enough to be considered
	// a failure (and in fact are not a failure if you're cutting an open mesh, like a simple rectangle)
	for (int EID : EdgesOnFailure)
	{
		TargetMesh->GetEdgeV(EID, A, B);
		DrawnLineSet->AddLine((FVector)A, (FVector)B, PartialPathEdgeColor, PartialPathEdgeThickness, PartialPathEdgeDepthBias);
	}

	// In the case where we don't allow failed results, it can be disorienting to show the broken mesh (especially since sometimes
	// most of it may be cut away). But user can change this behavior.
	if (!bOperationSucceeded && !BasicProperties->bCanAcceptFailedResult && !BasicProperties->bShowIntermediateResultOnFailure)
	{
		// Reset the preview.
		Preview->PreviewMesh->UpdatePreview(OriginalDynamicMesh.Get());
	}
}


void UPolygonOnMeshTool::UpdatePolygonType()
{
	if (BasicProperties->Shape == EPolygonType::Circle)
	{
		ActivePolygon = FPolygon2d::MakeCircle(BasicProperties->Width*0.5, BasicProperties->Subdivisions);
	}
	else if (BasicProperties->Shape == EPolygonType::Square)
	{
		ActivePolygon = FPolygon2d::MakeRectangle(FVector2d::Zero(), BasicProperties->Width, BasicProperties->Width);
	}
	else if (BasicProperties->Shape == EPolygonType::Rectangle)
	{
		ActivePolygon = FPolygon2d::MakeRectangle(FVector2d::Zero(), BasicProperties->Width, BasicProperties->Height);
	}
	else if (BasicProperties->Shape == EPolygonType::RoundRect)
	{
		double Corner = BasicProperties->CornerRatio * FMath::Min(BasicProperties->Width, BasicProperties->Height) * 0.49;
		ActivePolygon = FPolygon2d::MakeRoundedRectangle(FVector2d::Zero(), BasicProperties->Width, BasicProperties->Height, Corner, BasicProperties->Subdivisions);
	}
	else if (BasicProperties->Shape == EPolygonType::Custom)
	{
		if (LastDrawnPolygon.VertexCount() == 0)
		{
			GetToolManager()->DisplayMessage(LOCTEXT("PolygonOnMeshDrawMessage", "Click the Draw Polygon button to draw a custom polygon"), EToolMessageLevel::UserWarning);
			ActivePolygon = FPolygon2d::MakeCircle(BasicProperties->Width*0.5, BasicProperties->Subdivisions);
		}
		else
		{
			ActivePolygon = LastDrawnPolygon;
		}
	}
}

void UPolygonOnMeshTool::UpdateDrawPlane()
{
	Preview->InvalidateResult();
}


void UPolygonOnMeshTool::SetupPreview()
{
	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(this->TargetWorld, this);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);

	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);
	Preview->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	Preview->SetVisibility(true);
}


void UPolygonOnMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	PlaneMechanic->Shutdown();
	if (DrawPolygonMechanic != nullptr)
	{
		DrawPolygonMechanic->Shutdown();
	}
	BasicProperties->SaveProperties(this);

	// Restore (unhide) the source meshes
	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);

	TArray<FDynamicMeshOpResult> Results;
	Results.Emplace(Preview->Shutdown());
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Results);
	}
}


TUniquePtr<FDynamicMeshOperator> UPolygonOnMeshTool::MakeNewOperator()
{
	TUniquePtr<FEmbedPolygonsOp> EmbedOp = MakeUnique<FEmbedPolygonsOp>();
	EmbedOp->bDiscardAttributes = false;
	EmbedOp->Operation = BasicProperties->Operation;
	EmbedOp->bCutWithBoolean = BasicProperties->bCutWithBoolean;
	bool bOpLeavesOpenBoundaries =
		BasicProperties->Operation == EEmbeddedPolygonOpMethod::TrimInside ||
		BasicProperties->Operation == EEmbeddedPolygonOpMethod::TrimOutside;
	EmbedOp->bAttemptFixHolesOnBoolean = !bOpLeavesOpenBoundaries && BasicProperties->bTryToFixHoles;

	// Match the world plane in the local space
	FVector3d LocalOrigin = WorldTransform.InverseTransformPosition(DrawPlaneWorld.Origin);
	FVector3d TempLocalX = WorldTransform.InverseTransformVector(DrawPlaneWorld.GetAxis(0));
	FVector3d TempLocalY = WorldTransform.InverseTransformVector(DrawPlaneWorld.GetAxis(1));
	FVector3d LocalZ = TempLocalX.Cross(TempLocalY);
	FFrame3d LocalFrame(LocalOrigin, LocalZ);
	EmbedOp->PolygonFrame = LocalFrame;
	
	// Transform the active polygon by the polygon scale put it on the local space's frame
	EmbedOp->EmbedPolygon = ActivePolygon;
	const TArray<FVector2d>& Vertices = EmbedOp->EmbedPolygon.GetVertices();
	for (int32 Idx = 0; Idx < EmbedOp->EmbedPolygon.VertexCount(); ++Idx)
	{
		FVector2d World2d = Vertices[Idx] * BasicProperties->PolygonScale;
		FVector2d LocalPos = LocalFrame.ToPlaneUV(WorldTransform.InverseTransformPosition(DrawPlaneWorld.FromPlaneUV(World2d)));
		EmbedOp->EmbedPolygon.Set(Idx, LocalPos);
	}

	// TODO: scale any extrude by ToLocal.TransformVector(LocalFrame.Z()).Length() ??
	// EmbedOp->ExtrudeDistance = Tool->BasicProperties->ExtrudeDistance;

	EmbedOp->OriginalMesh = OriginalDynamicMesh;
	EmbedOp->SetResultTransform(WorldTransform);

	return EmbedOp;
}




void UPolygonOnMeshTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	PlaneMechanic->Render(RenderAPI);
	
	if (DrawPolygonMechanic != nullptr)
	{
		DrawPolygonMechanic->Render(RenderAPI);
	}
	else
	{
		FToolDataVisualizer Visualizer;
		Visualizer.BeginFrame(RenderAPI);
		double Scale = BasicProperties->PolygonScale;
		const TArray<FVector2d>& Vertices = ActivePolygon.GetVertices();
		int32 NumVertices = Vertices.Num();
		FVector3d PrevPosition = DrawPlaneWorld.FromPlaneUV(Scale * Vertices[0]);
		for (int32 k = 1; k <= NumVertices; ++k)
		{
			FVector3d NextPosition = DrawPlaneWorld.FromPlaneUV(Scale * Vertices[k%NumVertices]);
			Visualizer.DrawLine(PrevPosition, NextPosition, LinearColors::VideoRed3f(), 3.0f, false);
			PrevPosition = NextPosition;
		}
	}
}

void UPolygonOnMeshTool::OnTick(float DeltaTime)
{
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(this->CameraState);

	PlaneMechanic->Tick(DeltaTime);
	Preview->Tick(DeltaTime);

	if (PendingAction != EPolygonOnMeshToolActions::NoAction)
	{
		if (PendingAction == EPolygonOnMeshToolActions::DrawPolygon)
		{
			BeginDrawPolygon();
		}
		PendingAction = EPolygonOnMeshToolActions::NoAction;
	}
}



void UPolygonOnMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	UpdatePolygonType();
	Preview->InvalidateResult();
}

void UPolygonOnMeshTool::RequestAction(EPolygonOnMeshToolActions ActionType)
{
	if (PendingAction != EPolygonOnMeshToolActions::NoAction || DrawPolygonMechanic != nullptr)
	{
		return;
	}
	PendingAction = ActionType;
}



void UPolygonOnMeshTool::BeginDrawPolygon()
{
	check(DrawPolygonMechanic == nullptr);

	GetToolManager()->DisplayMessage(LOCTEXT("PolygonOnMeshBeginDrawMessage", "Click repeatedly on the plane to draw a polygon, and on start point to finish."), EToolMessageLevel::UserWarning);

	DrawPolygonMechanic = NewObject<UCollectSurfacePathMechanic>(this);
	DrawPolygonMechanic->Setup(this);
	double SnapTol = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	DrawPolygonMechanic->SpatialSnapPointsFunc = [this, SnapTol](FVector3d Position1, FVector3d Position2)
	{
		return true && ToolSceneQueriesUtil::PointSnapQuery(this->CameraState, Position1, Position2, SnapTol);
	};
	DrawPolygonMechanic->SetDrawClosedLoopMode();

	DrawPolygonMechanic->InitializePlaneSurface(DrawPlaneWorld);
}


void UPolygonOnMeshTool::CompleteDrawPolygon()
{
	check(DrawPolygonMechanic != nullptr);

	GetToolManager()->DisplayMessage(FText::GetEmpty(), EToolMessageLevel::UserWarning);

	FFrame3d DrawFrame = DrawPlaneWorld;
	FPolygon2d TmpPolygon;
	for (const FFrame3d& Point : DrawPolygonMechanic->HitPath)
	{
		TmpPolygon.AppendVertex(DrawFrame.ToPlaneUV(Point.Origin));
	}
	if (TmpPolygon.IsClockwise() == true)
	{
		TmpPolygon.Reverse();
	}

	// check for self-intersections and other invalids

	LastDrawnPolygon = TmpPolygon;
	BasicProperties->Shape = EPolygonType::Custom;
	BasicProperties->PolygonScale = 1.0;
	UpdatePolygonType();
	Preview->InvalidateResult();

	DrawPolygonMechanic->Shutdown();
	DrawPolygonMechanic = nullptr;
}



bool UPolygonOnMeshTool::CanAccept() const
{
	return Super::CanAccept() && Preview != nullptr && Preview->HaveValidNonEmptyResult() 
		&& (bOperationSucceeded || BasicProperties->bCanAcceptFailedResult);
}


void UPolygonOnMeshTool::GenerateAsset(const TArray<FDynamicMeshOpResult>& Results)
{
	GetToolManager()->BeginUndoTransaction(LOCTEXT("PolygonOnMeshToolTransactionName", "Cut Hole"));
	
	check(Results.Num() > 0);
	check(Results[0].Mesh.Get() != nullptr);
	UE::ToolTarget::CommitMeshDescriptionUpdateViaDynamicMesh(Target, *Results[0].Mesh.Get(), true);

	GetToolManager()->EndUndoTransaction();
}





bool UPolygonOnMeshTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	if (DrawPolygonMechanic != nullptr)
	{
		FFrame3d HitPoint;
		if (DrawPolygonMechanic->IsHitByRay(FRay3d(Ray), HitPoint))
		{
			OutHit.Distance = FRay3d(Ray).GetParameter(HitPoint.Origin);
			OutHit.ImpactPoint = (FVector)HitPoint.Origin;
			OutHit.ImpactNormal = (FVector)HitPoint.Z();
			return true;
		}
		return false;
	}

	return false;
}


FInputRayHit UPolygonOnMeshTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FHitResult OutHit;
	if (HitTest(ClickPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}
	return (DrawPolygonMechanic != nullptr) ? FInputRayHit(TNumericLimits<float>::Max()) : FInputRayHit();
}

void UPolygonOnMeshTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (DrawPolygonMechanic != nullptr)
	{
		if (DrawPolygonMechanic->TryAddPointFromRay((FRay3d)ClickPos.WorldRay))
		{
			if (DrawPolygonMechanic->IsDone())
			{
				CompleteDrawPolygon();
				//GetToolManager()->EmitObjectChange(this, MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp), LOCTEXT("DrawPolyPathBeginOffset", "Set Offset"));
				//OnCompleteSurfacePath();
			}
			else
			{
				//GetToolManager()->EmitObjectChange(this, MakeUnique<FDrawPolyPathStateChange>(CurrentCurveTimestamp), LOCTEXT("DrawPolyPathBeginPath", "Begin Path"));
			}
		}
	}

}



FInputRayHit UPolygonOnMeshTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FHitResult OutHit;
	if (HitTest(PressPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}
	return (DrawPolygonMechanic != nullptr) ? FInputRayHit(TNumericLimits<float>::Max()) : FInputRayHit();
}


bool UPolygonOnMeshTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (DrawPolygonMechanic != nullptr)
	{
		DrawPolygonMechanic->UpdatePreviewPoint((FRay3d)DevicePos.WorldRay);
	}
	return true;
}





#undef LOCTEXT_NAMESPACE

