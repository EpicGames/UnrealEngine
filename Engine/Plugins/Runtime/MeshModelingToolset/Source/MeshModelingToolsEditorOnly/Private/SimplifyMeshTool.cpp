// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimplifyMeshTool.h"
#include "InteractiveToolManager.h"
#include "Properties/RemeshProperties.h"
#include "Properties/MeshStatisticsProperties.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "ToolBuilderUtil.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "StaticMeshAttributes.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "ModelingToolTargetUtil.h"
#include "ToolSetupUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Util/ColorConstants.h"


//#include "ProfilingDebugging/ScopedTimers.h" // enable this to use the timer.
#include "Modules/ModuleManager.h"
#include "IMeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimplifyMeshTool)


#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#endif

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USimplifyMeshTool"

DEFINE_LOG_CATEGORY_STATIC(LogMeshSimplification, Log, All);

/*
 * ToolBuilder
 */
USingleSelectionMeshEditingTool* USimplifyMeshToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<USimplifyMeshTool>(SceneState.ToolManager);
}

/*
 * Tool
 */
USimplifyMeshToolProperties::USimplifyMeshToolProperties()
{
	SimplifierType = ESimplifyType::QEM;
	TargetMode = ESimplifyTargetType::Percentage;
	TargetPercentage = 50;
	TargetTriangleCount = 1000;
	TargetVertexCount = 1000;
	MinimalAngleThreshold = 0.01f;
	TargetEdgeLength = 5.0;
	bReproject = false;
	bPreventNormalFlips = true;
	bDiscardAttributes = false;
	bGeometricConstraint = false;
	bShowGroupColors = false;
	GroupBoundaryConstraint = EGroupBoundaryConstraint::Ignore;
	MaterialBoundaryConstraint = EMaterialBoundaryConstraint::Ignore;
}

void USimplifyMeshTool::Setup()
{
	UInteractiveTool::Setup();

	// hide component and create + show preview
	UE::ToolTarget::HideSourceObject(Target);

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(GetTargetWorld(), this);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	Preview->ConfigureMaterials( MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	// some of this could be done async...
	{
		// if in editor, create progress indicator dialog because building mesh copies can be slow (for very large meshes)
#if WITH_EDITOR
		static const FText SlowTaskText = LOCTEXT("SimplifyMeshInit", "Building mesh simplification data...");

		FScopedSlowTask SlowTask(3.0f, SlowTaskText);
		SlowTask.MakeDialog();

		// Declare progress shortcut lambdas
		auto EnterProgressFrame = [&SlowTask](int Progress)
		{
			SlowTask.EnterProgressFrame((float)Progress);
		};
#else
		auto EnterProgressFrame = [](int Progress) {};
#endif
		EnterProgressFrame(2);

		// Note that we only copy over a mesh description if we use the UEStandard path (conditionally done in MakeNewOperator)
		// to avoid doing conversions back and forth for non mesh-description-backed targets (where the conversions can 
		// occasionally do odd things to the attributes), and to avoid the slow copy unless the user needs it.
		static FGetMeshParameters GetMeshParams;
		GetMeshParams.bWantMeshTangents = true;
		OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(Target, GetMeshParams));

		EnterProgressFrame(1);
		OriginalMeshSpatial = MakeShared<FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(OriginalMesh.Get(), true);
	}

	Preview->PreviewMesh->SetTransform((FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target));
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->PreviewMesh->UpdatePreview(OriginalMesh.Get());

	// initialize our properties
	SimplifyProperties = NewObject<USimplifyMeshToolProperties>(this);
	SimplifyProperties->RestoreProperties(this);
	AddToolPropertySource(SimplifyProperties);

	SimplifyProperties->WatchProperty(SimplifyProperties->bShowGroupColors,
									  [this](bool bNewValue) { UpdateVisualization(); });

	WeightMapProperties = NewObject<UMeshWeightChannelDensityProperties>(this);
	WeightMapProperties->RestoreProperties(this);
	AddToolPropertySource(WeightMapProperties);

	WeightMapProperties->InitializeAttributes(OriginalMesh.Get());
	WeightMapProperties->ValidateSelectedAttribute();

	// Attribute properties
	WeightMapProperties->WatchProperty(WeightMapProperties->SelectedAttribute,
		[this](const FName& NewName)
		{
			WeightMapProperties->ValidateSelectedAttribute();
		});
	

	MeshStatisticsProperties = NewObject<UMeshStatisticsProperties>(this);
	AddToolPropertySource(MeshStatisticsProperties);

	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(Preview->PreviewMesh->GetWorld(), Preview->PreviewMesh->GetTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->bShowWireframe = true;
		MeshElementsDisplay->Settings->RestoreProperties(this, TEXT("Simplify"));
		AddToolPropertySource(MeshElementsDisplay->Settings);
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		Preview->ProcessCurrentMesh(ProcessFunc);
	});


	Preview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute)
	{
		Compute->ProcessCurrentMesh([&](const FDynamicMesh3& ReadMesh)
		{
			MeshStatisticsProperties->Update(ReadMesh);
			MeshElementsDisplay->NotifyMeshChanged();
		});
	});

	UpdateVisualization();
	Preview->InvalidateResult();

	SetToolDisplayName(LOCTEXT("ToolName", "Simplify"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Reduce the number of triangles in the selected Mesh using various strategies."),
		EToolMessageLevel::UserNotification);
}


bool USimplifyMeshTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidResult();
}


void USimplifyMeshTool::OnShutdown(EToolShutdownType ShutdownType)
{
	SimplifyProperties->SaveProperties(this);
	WeightMapProperties->SaveProperties(this);
	
	if (ensure(MeshElementsDisplay->Settings))
	{
		MeshElementsDisplay->Settings->SaveProperties(this, TEXT("Simplify"));
	}
	MeshElementsDisplay->Disconnect();

	UE::ToolTarget::ShowSourceObject(Target);
	FDynamicMeshOpResult Result = Preview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("SimplifyMeshToolTransactionName", "Simplify Mesh"));
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, *Result.Mesh, true);
		GetToolManager()->EndUndoTransaction();
	}
}


void USimplifyMeshTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);
	MeshElementsDisplay->OnTick(DeltaTime);
}

TUniquePtr<FDynamicMeshOperator> USimplifyMeshTool::MakeNewOperator()
{
	TUniquePtr<FSimplifyMeshOp> Op = MakeUnique<FSimplifyMeshOp>();

	if (SimplifyProperties->SimplifierType == ESimplifyType::UEStandard && OriginalMeshDescription == nullptr)
	{
		// if in editor, create progress indicator dialog because building mesh copies can be slow (for very large meshes)
		// This is especially needed here because for Reasons, copying meshdescription is pretty slow
#if WITH_EDITOR
		static const FText SlowTaskText = LOCTEXT("SimplifyMeshInit", "Building mesh simplification data...");

		FScopedSlowTask SlowTask(1.0f, SlowTaskText);
		SlowTask.MakeDialog();

#endif

		FGetMeshParameters GetMeshParams;
		GetMeshParams.bWantMeshTangents = true;

		IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target);
		if (MeshDescriptionProvider)
		{
			OriginalMeshDescription = MakeShared<FMeshDescription, ESPMode::ThreadSafe>(
				MeshDescriptionProvider->GetMeshDescriptionCopy(GetMeshParams));
		}
		else
		{
			// Target doesn't have a mesh description, need to make one from the dynamic mesh
			OriginalMeshDescription = MakeShared<FMeshDescription, ESPMode::ThreadSafe>();
			FStaticMeshAttributes Attributes(*OriginalMeshDescription);
			Attributes.Register();
			FDynamicMeshToMeshDescription Converter;
			Converter.Convert(OriginalMesh.Get(), *OriginalMeshDescription, GetMeshParams.bWantMeshTangents);
		}
	}

	Op->bDiscardAttributes = SimplifyProperties->bDiscardAttributes;
	// We always want attributes enabled on result even if we discard them initially
	Op->bResultMustHaveAttributesEnabled = true;
	Op->bPreventNormalFlips = SimplifyProperties->bPreventNormalFlips;
	Op->bPreserveSharpEdges = SimplifyProperties->bPreserveSharpEdges;
	Op->bAllowSeamCollapse = !SimplifyProperties->bPreserveSharpEdges;
	Op->bPreventTinyTriangles = SimplifyProperties->bPreventTinyTriangles;
	Op->bReproject = SimplifyProperties->bReproject;
	Op->SimplifierType = SimplifyProperties->SimplifierType;
	Op->TargetCount = ( SimplifyProperties->TargetMode == ESimplifyTargetType::VertexCount) ?  SimplifyProperties->TargetVertexCount : SimplifyProperties->TargetTriangleCount;
	Op->MinimalPlanarAngleThresh = SimplifyProperties->MinimalAngleThreshold;
	Op->TargetEdgeLength = SimplifyProperties->TargetEdgeLength;
	Op->TargetMode = SimplifyProperties->TargetMode;
	Op->TargetPercentage = SimplifyProperties->TargetPercentage;
	Op->TargetQuadricError = SimplifyProperties->TargetQuadricErrorRoot * SimplifyProperties->TargetQuadricErrorRoot;
	Op->MeshBoundaryConstraint = (EEdgeRefineFlags)SimplifyProperties->MeshBoundaryConstraint;
	Op->GroupBoundaryConstraint = (EEdgeRefineFlags)SimplifyProperties->GroupBoundaryConstraint;
	Op->MaterialBoundaryConstraint = (EEdgeRefineFlags)SimplifyProperties->MaterialBoundaryConstraint;
	Op->bGeometricDeviationConstraint = SimplifyProperties->bGeometricConstraint;
	Op->GeometricTolerance = SimplifyProperties->GeometricTolerance;
	Op->RegularizeWeight = SimplifyProperties->RegularizeWeight;
	Op->PolyEdgeAngleTolerance = SimplifyProperties->PolyEdgeAngleTolerance;
	Op->BoundaryEdgeAngleTolerance = SimplifyProperties->BoundaryEdgeAngleTolerance;
	FTransform LocalToWorld = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	Op->SetTransform(LocalToWorld);

	Op->OriginalMeshDescription = OriginalMeshDescription;
	Op->OriginalMesh = OriginalMesh;
	Op->OriginalMeshSpatial = OriginalMeshSpatial;

	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	Op->MeshReduction = MeshReductionModule.GetStaticMeshReductionInterface();

	if (WeightMapProperties->bUseWeightMap)
	{
		int32 WeightMapAttributeLayerIndex = WeightMapProperties->GetSelectedWeightAttributeIndex(OriginalMesh.Get());

		if (WeightMapAttributeLayerIndex >= 0)
		{
			TFunction<double(const FDynamicMesh3& Mesh, int VertexA, int VertexB)> EdgeMetricScale =
			[WeightMapAttributeLayerIndex, RelativeDensity = WeightMapProperties->RelativeDensity](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
			{
				if (Mesh.HasAttributes() && Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex))
				{
					const FDynamicMeshWeightAttribute* const AttributeLayer = Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex);

					float WeightValueA;
					AttributeLayer->GetValue(VertexA, &WeightValueA);
					WeightValueA = FMath::Clamp(WeightValueA, 0.0f, 1.0f);

					float WeightValueB;
					AttributeLayer->GetValue(VertexB, &WeightValueB);
					WeightValueB = FMath::Clamp(WeightValueB, 0.0f, 1.0f);

					// choose the weight that will lead to a larger measure, so we choose the more conservative of the two weights
					const float UseWeight = RelativeDensity < 0 ? FMath::Max(WeightValueA, WeightValueB) : FMath::Min(WeightValueA, WeightValueB);

					return FMath::Pow(0.5, -RelativeDensity * (double)UseWeight);
				}

				return 1.0;
			};

			TFunction<double(const FDynamicMesh3& Mesh, int Vertex)> SquaredVertexMetricScale =
			[WeightMapAttributeLayerIndex, RelativeDensity = WeightMapProperties->RelativeDensity](const FDynamicMesh3& Mesh, int Vertex) -> double
			{
				if (Mesh.HasAttributes() && Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex))
				{
					const FDynamicMeshWeightAttribute* const AttributeLayer = Mesh.Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex);

					float WeightValue;
					AttributeLayer->GetValue(Vertex, &WeightValue);
					WeightValue = FMath::Clamp(WeightValue, 0.0f, 1.0f);

					// We use a squared error metric for the quadric error scale to get a similar scale in effect as the tolerance/edge-len scales, since it is a squared error term
					return FMath::Pow(0.5, -2. * RelativeDensity * (double)WeightValue);
				}

				return 1.0;
			};

			if (bool(SimplifyProperties->ApplyWeightMapTo & (uint32)EApplyWeightMapToSimplifyMeasure::GeometricTolerance))
			{
				Op->CustomGeometricErrorScaleF = EdgeMetricScale;
			}
			if (bool(SimplifyProperties->ApplyWeightMapTo & (uint32)EApplyWeightMapToSimplifyMeasure::QuadricError))
			{
				Op->CustomQuadricErrorScaleF = SquaredVertexMetricScale;
			}
			// edge scale is only useful to the op if the simplify target is edge length
			if (SimplifyProperties->TargetMode == ESimplifyTargetType::EdgeLength && bool(SimplifyProperties->ApplyWeightMapTo & (uint32)EApplyWeightMapToSimplifyMeasure::EdgeLengths))
			{
				Op->CustomEdgeLengthScaleF = EdgeMetricScale;
			}

			TFunction<double(int Vertex)> ClusterEdgeTargetLengthScale =
			[AttributeLayer = OriginalMesh->Attributes()->GetWeightLayer(WeightMapAttributeLayerIndex),
				RelativeDensity = WeightMapProperties->RelativeDensity](int Vertex) -> double
			{
				float WeightValue;
				AttributeLayer->GetValue(Vertex, &WeightValue);
				WeightValue = FMath::Clamp(WeightValue, 0.0f, 1.0f);

				// Note: We use positive RelativeDensity here instead of negative, b/c cluster simplifier scales target length rather measured length
				return FMath::Pow(0.5, RelativeDensity * WeightValue);
			};
			Op->ClusterTargetEdgeLengthScaleF = ClusterEdgeTargetLengthScale;
		}
	}

	return Op;
}


void USimplifyMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if ( Property )
	{
		if ( Property->GetFName() == GET_MEMBER_NAME_CHECKED(USimplifyMeshToolProperties, bShowGroupColors) )
		{
			UpdateVisualization();
		}
		else
		{
			Preview->InvalidateResult();
		}
	}
}

void USimplifyMeshTool::UpdateVisualization()
{
	if (SimplifyProperties->bShowGroupColors)
	{
		Preview->OverrideMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager());
		Preview->PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
		{
			return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
		},
		UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	else
	{
		Preview->OverrideMaterial = nullptr;
		Preview->PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
}





#undef LOCTEXT_NAMESPACE

