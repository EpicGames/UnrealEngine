// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletalMesh/RefSkeletonPoser.h"
#include "Misc/ITransaction.h"
#include "Editor/EditorEngine.h"

extern UNREALED_API UEditorEngine* GEditor;

#include UE_INLINE_GENERATED_CPP_BY_NAME(RefSkeletonPoser)

#define LOCTEXT_NAMESPACE "RefSkeletonPoser"

URefSkeletonPoser::FPoseChange::FPoseChange(URefSkeletonPoser* InPoser)
{
	OldAdditiveTransforms = InPoser->AdditiveTransforms;
	NewAdditiveTransforms = OldAdditiveTransforms;
}

FString URefSkeletonPoser::FPoseChange::ToString() const
{
	return FString(TEXT("Pose Skeleton"));
}

void URefSkeletonPoser::FPoseChange::Apply(UObject* Object)
{
	if (URefSkeletonPoser* Poser = Cast<URefSkeletonPoser>(Object))
	{
		Poser->AdditiveTransforms = NewAdditiveTransforms;
		Poser->Invalidate();
	}
}

void URefSkeletonPoser::FPoseChange::Revert(UObject* Object)
{
	if (URefSkeletonPoser* Poser = CastChecked<URefSkeletonPoser>(Object))
	{
		Poser->AdditiveTransforms = OldAdditiveTransforms;
		Poser->Invalidate();
	}
}

void URefSkeletonPoser::FPoseChange::Close(URefSkeletonPoser* InPoser)
{
	NewAdditiveTransforms = InPoser->AdditiveTransforms;
}

void URefSkeletonPoser::SetRefSkeleton(const FReferenceSkeleton& InRefSkeleton)
{
	RefSkeleton = InRefSkeleton;
	
	Transforms = RefSkeleton.GetRefBonePose();
	TransformCached = TBitArray<>(false, RefSkeleton.GetNum());
	
	RefSkeleton.GetBoneAbsoluteTransforms(RefTransforms);

	AdditiveTransforms.Reset();
}

const FReferenceSkeleton& URefSkeletonPoser::GetRefSkeleton() const
{
	return RefSkeleton;
}

void URefSkeletonPoser::ModifyBoneAdditiveTransform(int32 BoneIndex, TFunctionRef<void(FTransform&)> InModifyFunc)
{
	FTransform& TransformRef = AdditiveTransforms.FindOrAdd(BoneIndex);

	FTransform NewTransform = TransformRef;
	InModifyFunc(NewTransform);
	

	if (!TransformRef.Equals(NewTransform))
	{
		TransformRef = NewTransform;
		bPoseChanged = true;
	}
	Invalidate(BoneIndex);
}

void URefSkeletonPoser::ClearBoneAdditiveTransform(int32 BoneIndex)
{
	if (AdditiveTransforms.Contains(BoneIndex))
	{
		AdditiveTransforms.Remove(BoneIndex);
		bPoseChanged = true;
		Invalidate(BoneIndex);
	}
}

void URefSkeletonPoser::ClearAllBoneAdditiveTransforms()
{
	if (!AdditiveTransforms.IsEmpty())
	{
		AdditiveTransforms.Reset();
		bPoseChanged = true;
		Invalidate();
	}
}

TOptional<FTransform> URefSkeletonPoser::GetBoneAdditiveTransform(int32 BoneIndex)
{
	if (FTransform* Transform = AdditiveTransforms.Find(BoneIndex))
	{
		return *Transform;
	}
			
	return {};
}

const TArray<FTransform>& URefSkeletonPoser::GetComponentSpaceTransforms()
{
	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
	{
		GetComponentSpaceTransform(BoneIndex);
	}
			
	return Transforms;
}

const FTransform& URefSkeletonPoser::GetComponentSpaceTransform(const uint32 Index)
{
	if (!Transforms.IsValidIndex(Index))
	{
		return FTransform::Identity;
	}

	if (TransformCached[Index])
	{
		return Transforms[Index];
	}

	if (FTransform* Transform = AdditiveTransforms.Find(Index))
	{
		Transforms[Index] = *Transform * Transforms[Index];
	}

	const int32 ParentIndex = RefSkeleton.GetParentIndex(Index);
	if (ParentIndex > INDEX_NONE)
	{
		Transforms[Index] *= GetComponentSpaceTransform(ParentIndex);
	}

	TransformCached[Index] = true;
	return Transforms[Index];
}

void URefSkeletonPoser::Invalidate(const uint32 Index)
{
	if (!TransformCached.IsValidIndex(Index))
	{
		Transforms = RefSkeleton.GetRefBonePose();
		TransformCached.Init(false, Transforms.Num());
		return;
	}
			
	Transforms[Index] = RefSkeleton.GetRefBonePose()[Index];
	TransformCached[Index] = false;

	TArray<int32> Children; RefSkeleton.GetDirectChildBones(Index, Children);
	
	for (const int32 ChildIndex: Children)
	{
		Invalidate(ChildIndex);
	}
}

const TArray<FTransform>& URefSkeletonPoser::GetComponentSpaceTransformsRefPose()
{
	return RefTransforms;
}

void URefSkeletonPoser::BeginPoseChange()
{
	GEditor->BeginTransaction(LOCTEXT("ChangePose", "Change Pose"));
	ActivePoseChange.Reset(new FPoseChange(this));
	bPoseChanged = false;
}

void URefSkeletonPoser::EndPoseChange()
{
	if (bPoseChanged)
	{
		ActivePoseChange->Close(this);
		GUndo->StoreUndo(this, MoveTemp(ActivePoseChange));
	}
	ActivePoseChange.Reset();
	GEditor->EndTransaction();
}

bool URefSkeletonPoser::IsRecordingPoseChange()
{
	return ActivePoseChange.IsValid();
}


#undef LOCTEXT_NAMESPACE
