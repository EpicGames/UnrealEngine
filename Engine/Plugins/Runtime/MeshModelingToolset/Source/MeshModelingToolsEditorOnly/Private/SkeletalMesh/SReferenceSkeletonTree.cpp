// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMesh/SReferenceSkeletonTree.h"

#include "SkeletonModifier.h"
#include "SkeletalMesh/SkeletonEditingTool.h"
#include "SkeletonClipboard.h"

#include "SPositiveActionButton.h"

#include "Misc/TextFilterExpressionEvaluator.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor.h"
#include "Algo/RemoveIf.h"

#include "Misc/ITransaction.h"

#define LOCTEXT_NAMESPACE "SReferenceSkeletonTree"

namespace ReferenceSkeletonTreeLocals
{

FSkeletonModifierChange::FSkeletonModifierChange(const USkeletonModifier* InModifier)
	: FCommandChange()
	, PreChangeSkeleton(InModifier->GetReferenceSkeleton())
	, PreBoneTracker(InModifier->GetBoneIndexTracker())
	, PostChangeSkeleton(InModifier->GetReferenceSkeleton())
	, PostBoneTracker(InModifier->GetBoneIndexTracker())
{}

void FSkeletonModifierChange::StoreSkeleton(const USkeletonModifier* InModifier)
{
	PostChangeSkeleton = InModifier->GetReferenceSkeleton();
	PostBoneTracker = InModifier->GetBoneIndexTracker();
}

void FSkeletonModifierChange::Apply(UObject* Object)
{ // redo
	USkeletonModifier* Modifier = CastChecked<USkeletonModifier>(Object);
	Modifier->ExternalUpdate(PostChangeSkeleton, PostBoneTracker);
}

void FSkeletonModifierChange::Revert(UObject* Object)
{ // undo
	USkeletonModifier* Modifier = CastChecked<USkeletonModifier>(Object);
	Modifier->ExternalUpdate(PreChangeSkeleton, PreBoneTracker);
}

/** The actual tree view for this widget */
class SReferenceSkeletonTreeView 
	: public STreeView<TSharedPtr<FBoneElement>>
{
public:
	/** Stores sparse item infos */
	void StoreSparseItemInfos()
	{
		CachedSparseItemInfos = STreeView<TSharedPtr<FBoneElement>>::SparseItemInfos;
	}
	
	/** Restores sparse item infos from last known infos */
	void RestoreSparseItemInfos(const TArray<TSharedPtr<FBoneElement>> TreeItems)
	{
		for (auto It = STreeView<TSharedPtr<FBoneElement>>::SparseItemInfos.CreateIterator(); It; ++It)
		{
			if (!TreeItems.Contains(It->Key))
			{
				It.RemoveCurrent();
			}
		}
		
		for (auto It = CachedSparseItemInfos.CreateIterator(); It; ++It)
		{
			const TSharedPtr<FBoneElement>* TreeItemPtr = TreeItems.FindByPredicate(
				[It](const TSharedPtr<FBoneElement>& TreeItem)
				{
					return It->Key->BoneName == TreeItem->BoneName;
				});
				
			if (TreeItemPtr)
			{
				STreeView<TSharedPtr<FBoneElement>>::SparseItemInfos.Add(*TreeItemPtr, It->Value);
			}
		}
	}
	
private:
	/** Holds the last known sparse item infos */
	STreeView<TSharedPtr<FBoneElement>>::TSparseItemMap CachedSparseItemInfos;
};
}

FBoneElement::FBoneElement(const FName& InBoneName, TWeakObjectPtr<USkeletonModifier> InModifier)
	: BoneName(InBoneName)
	, WeakModifier(InModifier)
{}

void FBoneElement::RequestRename(const ReferenceSkeletonTreeLocals::EBoneRenameType InRenameType) const
{
	OnRenameRequested.ExecuteIfBound(InRenameType);
}

SBoneItem::~SBoneItem()
{
	if (TSharedPtr<FBoneElement> BoneElement = WeakTreeElement.IsValid() ? WeakTreeElement.Pin() : nullptr)
	{
		BoneElement->OnRenameRequested.Unbind();
	}
}

void SBoneItem::Construct(const FArguments& InArgs)
{
	// cf. FSkeletonTreeBoneItem::GenerateWidgetForNameColumn
	
	WeakTreeElement = InArgs._TreeElement;
	WeakModifier = InArgs._WeakModifier;
	OnBoneElementNameCommitted = InArgs._Delegates.OnBoneElementNameCommitted;

	const TSharedPtr<FBoneElement> TreeElement = WeakTreeElement.Pin();

	const FSlateBrush* Brush = FAppStyle::Get().GetBrush("SkeletonTree.Bone");

	TSharedPtr<SHorizontalBox> RowBox;

	ChildSlot
	[
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 2.0f))
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.Image(Brush)
			.ColorAndOpacity_Static(&SBoneItem::GetTextColor, InArgs._IsSelected)
		]
	];

	RowBox->AddSlot()
    .AutoWidth()
	.Padding(4, 0, 0, 0)
    .VAlign(VAlign_Center)
    [
	    SAssignNew(InlineWidget, SInlineEditableTextBlock)
		.Text(this, &SBoneItem::GetName)
		// .Font(TextFont)
	    .IsReadOnly(InArgs._IsReadOnly)
		.ColorAndOpacity_Static(&SBoneItem::GetTextColor, InArgs._IsSelected)
		.MaximumLength(NAME_SIZE-1)
		.OnVerifyTextChanged(this, &SBoneItem::OnVerifyNameChanged)
		.OnTextCommitted(this, &SBoneItem::OnTextCommitted)
		.MultiLine(false)
    	.OnEnterEditingMode(this, &SBoneItem::OnEnterEditingMode)
		.OnExitEditingMode(this, &SBoneItem::OnExitEditingMode)
    	.ToolTipText_Lambda([this]()
		{
    		if (bEditing)
    		{
    			return LOCTEXT("BoneItem_RenameTooltip",
    				"use $name to add a prefix/suffix or /find/replace/ to replace a substring\n"
    				" - \"spine_$name\" to add \"spine_\" as prefix.\n"
    				" - \"$name_left\" to add \"_left\" as suffix.\n"
    				" - \"/neck_/tail_/\" to replace \"neck_\" by \"tail_\".\n");	
    		}
    		return FText::GetEmpty();
		})
    ];

	if (InlineWidget)
	{
		TreeElement->OnRenameRequested.BindRaw(this, &SBoneItem::HandleOnRenameRequested);	
	}
}

FSlateColor SBoneItem::GetTextColor(FIsSelected InIsSelected)
{
	const bool bIsSelected = InIsSelected.IsBound() ? InIsSelected.Execute() : false;
	if (bIsSelected)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundInverted");
	}
	
	return FSlateColor::UseForeground();
}

void SBoneItem::HandleOnRenameRequested(const ReferenceSkeletonTreeLocals::EBoneRenameType InRenameType)
{
	OptRenameType = InRenameType;
	if (InlineWidget)
	{
		InlineWidget->EnterEditingMode();
	}
}

bool SBoneItem::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage) const
{
	if (!WeakModifier.IsValid())
	{
		return false;
	}

	const TSharedPtr<FBoneElement> BoneElementPtr = WeakTreeElement.Pin();
	const FReferenceSkeleton& ReferenceSkeleton = WeakModifier->GetReferenceSkeleton();

	const FName NewName = FName(InText.ToString());
	int32 BoneIndex = ReferenceSkeleton.FindRawBoneIndex(NewName);
	if (BoneIndex != INDEX_NONE)
	{
		// changing case is allowed 
		if (!ReferenceSkeleton.GetBoneName(BoneIndex).IsEqual(NewName, ENameCase::CaseSensitive))
		{
			BoneIndex = INDEX_NONE;
		}
	}
	
	return BoneIndex == INDEX_NONE;
}

void SBoneItem::OnTextCommitted(const FText& Text, ETextCommit::Type TextCommitType)
{
	if (WeakTreeElement.IsValid())
	{
		OnBoneElementNameCommitted.ExecuteIfBound(Text, TextCommitType, WeakTreeElement.Pin().ToSharedRef());
	}
}

void SBoneItem::OnEnterEditingMode()
{
	bEditing = true;
}

void SBoneItem::OnExitEditingMode()
{
	bEditing = false;
	OptRenameType.Reset();
}

FText SBoneItem::GetName() const
{
	TSharedPtr<FBoneElement> BoneElement = WeakTreeElement.IsValid() ? WeakTreeElement.Pin() : nullptr;
	ensure(BoneElement);
	
	if (bEditing && OptRenameType)
	{
		switch (*OptRenameType)
		{
		case ReferenceSkeletonTreeLocals::EBoneRenameType::Rename:
			return FText::FromName(BoneElement->BoneName);
		case ReferenceSkeletonTreeLocals::EBoneRenameType::SearchAndReplace:
			return FText::FromString(TEXT("/to_search/to_replace/"));
		case ReferenceSkeletonTreeLocals::EBoneRenameType::AddPrefix:
			return FText::FromString(TEXT("prefix_$name"));
		case ReferenceSkeletonTreeLocals::EBoneRenameType::AddSuffix:
			return FText::FromString(TEXT("$name_suffix"));
		default:
			return FText::FromName(BoneElement->BoneName);
		}
	}
	
	return FText::FromName(WeakTreeElement.Pin()->BoneName);
}

void SReferenceSkeletonRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
{
	WeakTreeElement = InArgs._TreeElement;
	WeakModifier = InArgs._WeakModifier;
	Delegates = InArgs._Delegates;
	bIsReadOnly = InArgs._IsReadOnly;

	const FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.OnDragDetected(InArgs._Delegates.OnDragDetected)
		.OnCanAcceptDrop(InArgs._Delegates.OnCanAcceptDrop)
		.OnAcceptDrop(InArgs._Delegates.OnAcceptDrop)
		.Style( FAppStyle::Get(), "TableView.AlternatingRow" );
	
	SMultiColumnTableRow::Construct(Args , InOwnerTable);
}

TSharedRef<SWidget> SReferenceSkeletonRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SHorizontalBox> Widget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SExpanderArrow, SharedThis(this) )
				.ShouldDrawWires(true)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBoneItem)
				.WeakModifier(WeakModifier)
				.TreeElement(WeakTreeElement)
				.Delegates(Delegates)
				.IsSelected(this, &SReferenceSkeletonRow::IsSelected)
				.IsReadOnly(bIsReadOnly)
		];

	if (Delegates.OnGetRowWidget.IsBound())
	{
		Widget->InsertSlot(0)
		.AutoWidth()
		[
			Delegates.OnGetRowWidget.Execute(WeakTreeElement.Pin()->BoneName)	
		];
	}

	return Widget;
}

TSharedRef<FBoneItemDragDropOp> FBoneItemDragDropOp::New(TArrayView<const TSharedPtr<FBoneElement>> InElements)
{
	TSharedRef<FBoneItemDragDropOp> Operation = MakeShared<FBoneItemDragDropOp>();

	for (const TSharedPtr<FBoneElement>& BoneElement: InElements)
	{
		const bool bContainsParent = InElements.Contains(BoneElement->Parent);
		if (!bContainsParent)
		{
			Operation->Elements.Add(BoneElement);
		}
	}
	
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FBoneItemDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
        .Visibility(EVisibility::Visible)
        .BorderImage(FAppStyle::GetBrush("Menu.Background"))
        [
            SNew(STextBlock)
            .Text_Lambda([this]()
            {
	            return !Elements.IsEmpty() && Elements[0].IsValid() ? FText::FromName(Elements[0].Pin()->BoneName) : FText::GetEmpty();
            })
        ];
}

SReferenceSkeletonTree::SReferenceSkeletonTree()
{}

SReferenceSkeletonTree::~SReferenceSkeletonTree()
{
	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SReferenceSkeletonTree::Construct(const FArguments& InArgs)
{
	checkf(InArgs._Modifier.IsValid(), TEXT("Modifier is a required argument for SReferenceSkeletonTree"));
	Modifier = InArgs._Modifier;
	bIsReadOnly = Modifier->IsReadOnly();

	Delegates = InArgs._Delegates;
	Delegates.OnCanAcceptDrop = !Delegates.OnCanAcceptDrop.IsBound() ? FOnRefSkeletonTreeCanAcceptDrop::CreateSP(this, &SReferenceSkeletonTree::OnCanAcceptDrop) : Delegates.OnCanAcceptDrop;
	Delegates.OnAcceptDrop = !Delegates.OnAcceptDrop.IsBound() ? FOnRefSkeletonTreeAcceptDrop::CreateSP(this, &SReferenceSkeletonTree::OnAcceptDrop) : Delegates.OnAcceptDrop;
	Delegates.OnDragDetected = !Delegates.OnDragDetected.IsBound() ? FOnDragDetected::CreateSP(this, &SReferenceSkeletonTree::OnDragDetected) : Delegates.OnDragDetected;
	Delegates.OnBoneRenamed = !Delegates.OnBoneRenamed.IsBound() ? FOnBoneRenamed::CreateSP(this, &SReferenceSkeletonTree::OnBoneRenamed) : Delegates.OnBoneRenamed;	
	if (!Delegates.OnBoneElementNameCommitted.IsBound())
	{
		Delegates.OnBoneElementNameCommitted = FRefSkeletonTreeDelegates::FOnBoneElementNameCommitted::CreateSP(this, &SReferenceSkeletonTree::OnBoneElementNameCommitted);
	}

	CommandList = MakeShared<FUICommandList>();
	BindCommands();


	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));

	
	ChildSlot
	[
		SNew( SOverlay )
		+SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.f, 2.f))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(6.f, 0.0))
				[
					SNew(SPositiveActionButton)
					.OnGetMenuContent(this, &SReferenceSkeletonTree::CreateAddNewMenu)
					.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
					.Visibility(this, &SReferenceSkeletonTree::GetAddNewButtonVisibility)
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SSearchBox)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged( this, &SReferenceSkeletonTree::OnFilterTextChanged )
					.HintText(LOCTEXT( "SearchBoxHint", "Search Reference Skeleton Tree..."))
				]
			]
			
			+SVerticalBox::Slot()
			.Padding(0.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
				[
					SAssignNew(TreeView, SReferenceSkeletonTreeView)
					.TreeItemsSource(&RootElements)
					.SelectionMode(ESelectionMode::Multi)
					.AllowInvisibleItemSelection(true)
					.OnGenerateRow_Lambda( [this](
						TSharedPtr<FBoneElement> InItem,
						const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
					{
						return SNew(SReferenceSkeletonRow, OwnerTable)
							.WeakModifier(Modifier)
							.TreeElement(InItem)
							.Delegates(Delegates)
							.IsReadOnly(bIsReadOnly);
					})
					.OnGetChildren(this, &SReferenceSkeletonTree::HandleGetChildrenForTree)
					.OnSelectionChanged(this, &SReferenceSkeletonTree::OnSelectionChanged)
					.OnContextMenuOpening(this, &SReferenceSkeletonTree::CreateContextMenu)
					.OnMouseButtonDoubleClick(this, &SReferenceSkeletonTree::OnItemDoubleClicked)
					.OnSetExpansionRecursive(this, &SReferenceSkeletonTree::OnSetExpansionRecursive)
					.HighlightParentNodesForSelection(true)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column("Name")
						.ShouldGenerateWidget(true)
						.DefaultLabel(LOCTEXT("BoneNameLabel", "Name"))
						.FillWidth(0.5f)
					)
				]
			]
		]
    ];

	static constexpr bool IsInitialSetup = true;
	RefreshTreeView(IsInitialSetup);

	GEditor->RegisterForUndo(this);
}

void SReferenceSkeletonTree::PostUndo(bool bSuccess)
{
	RefreshTreeView(true);
}

void SReferenceSkeletonTree::PostRedo(bool bSuccess)
{
	RefreshTreeView(true);
}

void SReferenceSkeletonTree::AddItemToSelection(const TSharedPtr<FBoneElement>& InItem)
{
	TreeView->SetItemSelection(InItem, true, ESelectInfo::Direct);
	TreeView->RequestScrollIntoView(InItem);
}

void SReferenceSkeletonTree::RemoveItemFromSelection(const TSharedPtr<FBoneElement>& InItem)
{
	TreeView->SetItemSelection(InItem, false, ESelectInfo::Direct);
}

void SReferenceSkeletonTree::ReplaceItemInSelection(const FText& InOldName, const FText& InNewName)
{
	const FName OldName(*InOldName.ToString());
	const FName NewName(*InNewName.ToString());
	
	for (const TSharedPtr<FBoneElement>& Item : AllElements)
	{
		// remove old selection
		if (Item->BoneName.IsEqual(OldName))
		{
			TreeView->SetItemSelection(Item, false, ESelectInfo::Direct);
		}
		// add new selection
		if (Item->BoneName.IsEqual(NewName))
		{
			TreeView->SetItemSelection(Item, true, ESelectInfo::Direct);
		}
	}
}

TArray<TSharedPtr<FBoneElement>> SReferenceSkeletonTree::GetSelectedItems() const
{
	return TreeView->GetSelectedItems();
}

bool SReferenceSkeletonTree::HasSelectedItems() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

void SReferenceSkeletonTree::BindCommands()
{
	if (!bIsReadOnly)
	{
		const FSkeletonEditingToolCommands& Commands = FSkeletonEditingToolCommands::Get();
 
		CommandList->MapAction(Commands.NewBone,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleNewBone),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanAddNewBone));
		
		CommandList->MapAction(Commands.RemoveBone,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleDeleteBone),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanDeleteBone));
	 
		CommandList->MapAction(Commands.RenameBone,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleRenameBone, ReferenceSkeletonTreeLocals::EBoneRenameType::Rename),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanRenameBone));

		CommandList->MapAction(Commands.SearchAndReplace,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleRenameBone, ReferenceSkeletonTreeLocals::EBoneRenameType::SearchAndReplace),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanRenameBone));

		CommandList->MapAction(Commands.AddPrefix,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleRenameBone, ReferenceSkeletonTreeLocals::EBoneRenameType::AddPrefix),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanRenameBone));

		CommandList->MapAction(Commands.AddSuffix,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleRenameBone, ReferenceSkeletonTreeLocals::EBoneRenameType::AddSuffix),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanRenameBone));
		
		CommandList->MapAction(Commands.UnParentBone,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleUnParentBone),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanUnParentBone));

		CommandList->MapAction(Commands.CopyBones,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleCopyBones),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanCopyBones));
		
		constexpr bool bPasteUsingGlobalTransforms = true;
		CommandList->MapAction(Commands.PasteBonesLocal,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandlePasteBones, !bPasteUsingGlobalTransforms),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanPasteBones));
		
		CommandList->MapAction(Commands.PasteBonesGlobal,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandlePasteBones, bPasteUsingGlobalTransforms),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanPasteBones));
		
		CommandList->MapAction(Commands.DuplicateBones,
			FExecuteAction::CreateSP(this, &SReferenceSkeletonTree::HandleDuplicateBones),
			FCanExecuteAction::CreateSP(this, &SReferenceSkeletonTree::CanDuplicateBones));	
	}


	if (Delegates.OnExtendCommandList.IsBound())
	{
		Delegates.OnExtendCommandList.Execute(CommandList);
	}
}

void SReferenceSkeletonTree::HandleNewBone()
{
	if (!Modifier.IsValid())
	{
		return;
	}

	BeginChange();
	
	TArray<FName> BoneNames;
	GetSelectedBoneNames(BoneNames);
	
	const FName ParentName = BoneNames.IsEmpty() ? NAME_None : BoneNames[0];

	static const FName DefaultName("joint");
	const FName BoneName = Modifier->GetUniqueName(DefaultName);
	const bool bAdded = Modifier->AddBone(BoneName, ParentName, FTransform::Identity);
	if (bAdded)
	{
		RefreshTreeView(true);
		if (Notifier.IsValid())
		{
			Notifier->Notify( {BoneName}, ESkeletalMeshNotifyType::BonesAdded);
		}
		EndChange();
	}
	else
	{
		CancelChange();
	}
}

bool SReferenceSkeletonTree::CanAddNewBone() const
{
	if (bIsReadOnly)
	{
		return false;
	}
	
	return true;
}

void SReferenceSkeletonTree::HandleDeleteBone()
{
	if (!Modifier.IsValid())
	{
		return;
	}

	BeginChange();
	
	TArray<FName> BoneNames;
	GetSelectedBoneNames(BoneNames);
	
	const bool bRemoved = Modifier->RemoveBones(BoneNames, true);
	if (bRemoved)
	{
		RefreshTreeView(true);
		if (Notifier.IsValid())
		{
			Notifier->Notify(BoneNames, ESkeletalMeshNotifyType::BonesRemoved);
		}
		EndChange();
	}
	else
	{
		CancelChange();
	}
}

bool SReferenceSkeletonTree::CanDeleteBone() const
{
	if (bIsReadOnly)
	{
		return false;
	}
	
	return HasSelectedItems();
}

void SReferenceSkeletonTree::HandleUnParentBone()
{
	if (!Modifier.IsValid())
	{
		return;
	}

	BeginChange();
	
	TArray<FName> BoneNames;
	GetSelectedBoneNames(BoneNames);

	static const TArray<FName> Dummy;
	const bool bUnParented = Modifier->ParentBones(BoneNames, Dummy);
	if (bUnParented)
	{
		RefreshTreeView(true);
		SelectItemFromNames(BoneNames);
		if (Notifier.IsValid())
		{
			Notifier->Notify(BoneNames, ESkeletalMeshNotifyType::HierarchyChanged);
		}
		EndChange();
	}
	else
	{
		CancelChange();
	}
}

bool SReferenceSkeletonTree::CanUnParentBone() const
{
	if (bIsReadOnly)
	{
		return false;
	}
	
	return HasSelectedItems();
}

void SReferenceSkeletonTree::HandleCopyBones() const
{
	if (!ensure(Modifier.IsValid()))
	{
		return;
	}

	TArray<FName> SelectedBones;
	GetSelectedBoneNames(SelectedBones);

	if (!SelectedBones.IsEmpty())
	{
		SkeletonClipboard::CopyBonesToClipboard(*Modifier.Get(), SelectedBones);
	}
}

bool SReferenceSkeletonTree::CanCopyBones() const
{
	if (bIsReadOnly)
	{
		return false;
	}

	return Modifier.IsValid() && HasSelectedItems();
}

void SReferenceSkeletonTree::HandlePasteBones(const bool bUsingGlobalTransforms)
{
	if (!ensure(Modifier.IsValid()))
	{
		return;
	}

	TArray<FName> BoneNames;
	GetSelectedBoneNames(BoneNames);

	const FName DefaultParent = BoneNames.IsEmpty() ? NAME_None : BoneNames[0];  

	BeginChange();
	
	const TArray<FName> NewBones = SkeletonClipboard::PasteBonesFromClipboard(*Modifier.Get(), DefaultParent, bUsingGlobalTransforms);
	if (NewBones.IsEmpty())
	{
		CancelChange();
		return;
	}

	static constexpr bool bRebuildAll = true;
	RefreshTreeView(bRebuildAll);
	static constexpr bool bFrameSelection = true;
	SelectItemFromNames(NewBones, bFrameSelection);

	if (Notifier.IsValid())
	{
		Notifier->Notify(NewBones, ESkeletalMeshNotifyType::HierarchyChanged);
		Notifier->Notify(NewBones, ESkeletalMeshNotifyType::BonesSelected);
	}

	EndChange();
}

bool SReferenceSkeletonTree::CanPasteBones() const
{
	if (bIsReadOnly)
	{
		return false;
	}
	
	return Modifier.IsValid() && SkeletonClipboard::IsBoneClipboardValid();
}

void SReferenceSkeletonTree::HandleDuplicateBones()
{
	if (!ensure(Modifier.IsValid()))
	{
		return;
	}

	BeginChange();

	TArray<FName> SelectedBones;
	GetSelectedBoneNames(SelectedBones);
	
	constexpr bool bUsingGlobalTransforms = false;
	const TArray<FName> NewBones = SkeletonClipboard::ClipboardDuplicateBones(*Modifier.Get(), SelectedBones);
	if (NewBones.IsEmpty())
	{
		CancelChange();
		return;
	}

	static constexpr bool bRebuildAll = true;
	RefreshTreeView(bRebuildAll);
	static constexpr bool bFrameSelection = true;
	SelectItemFromNames(NewBones, bFrameSelection);

	if (Notifier.IsValid())
	{
		Notifier->Notify(NewBones, ESkeletalMeshNotifyType::HierarchyChanged);
		Notifier->Notify(NewBones, ESkeletalMeshNotifyType::BonesSelected);
	}

	EndChange();
}

bool SReferenceSkeletonTree::CanDuplicateBones() const
{
	return CanCopyBones();
}

void SReferenceSkeletonTree::GetSelectedBoneNames(TArray<FName>& OutSelectedBoneNames) const
{
	OutSelectedBoneNames.Reset();
	const TArray<TSharedPtr<FBoneElement>> BoneElements = TreeView->GetSelectedItems();
	OutSelectedBoneNames.Reserve(BoneElements.Num());

	Algo::Transform(BoneElements, OutSelectedBoneNames, [](const TSharedPtr<FBoneElement>& BoneElement)
	{
		return BoneElement->BoneName;
	});
}

void SReferenceSkeletonTree::SelectItemFromNames(const TArray<FName>& InBoneNames, bool bFrameSelection)
{
	for (const TSharedPtr<FBoneElement>& Item : AllElements)
	{
		const bool bSelect = InBoneNames.Contains(Item->BoneName);
		TreeView->SetItemSelection(Item, bSelect, ESelectInfo::Direct);

		if (bFrameSelection && bSelect)
		{
			TreeView->RequestScrollIntoView(Item);
			bFrameSelection = false;
		}
	}
}

TSharedPtr<ISkeletalMeshNotifier> SReferenceSkeletonTree::GetNotifier()
{
	if (!Notifier)
	{
		Notifier = MakeShared<FReferenceSkeletonWidgetNotifier>(SharedThis(this));
	}
	
	return Notifier;
}

void SReferenceSkeletonTree::HandleRenameBone(const ReferenceSkeletonTreeLocals::EBoneRenameType InRenameType) const
{
	TArray<TSharedPtr<FBoneElement>> BoneElements = TreeView->GetSelectedItems();
	if (BoneElements.IsEmpty())
	{
		return;
	}
	BoneElements[0]->RequestRename(InRenameType);
}

bool SReferenceSkeletonTree::CanRenameBone() const
{
	if (bIsReadOnly)
	{
		return false;
	}
	
	return HasSelectedItems();
}

void SReferenceSkeletonTree::OnBoneRenamed(const FName InOldName, const FName InNewName) const
{
	if (!Modifier.IsValid())
	{
		return;
	}

	if (Notifier.IsValid())
	{
		Notifier->Notify( {InNewName}, ESkeletalMeshNotifyType::BonesRenamed);
	}
}

void SReferenceSkeletonTree::OnBoneElementNameCommitted(const FText& InText, ETextCommit::Type InCommitType, TSharedRef<FBoneElement> BoneElement)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		return;
	}
	
	TArray<FName> BoneNames;
	GetSelectedBoneNames(BoneNames);

	// If the selection does not contain the bone name, the user committed by changing the selection.
	// Handle the case by reanming the element which invoked this commit and not the selection.
	if (!BoneNames.Contains(BoneElement->BoneName))
	{
		BoneNames = { BoneElement->BoneName };
	}

	static const FString NamePattern(TEXT("$name"));
	static constexpr TCHAR Slash(TEXT('/'));
	
	const FString NameStr = InText.ToString();
	const FName NewName = FName(NameStr);

	// wrong input
	if (BoneNames.IsEmpty() || NewName == NAME_None || NameStr == NamePattern)
	{
		return;
	}
	
	// same name
	if (BoneNames[0].IsEqual(NewName, ENameCase::CaseSensitive))
	{
		return;
	}

	// look for prefix / suffix pattern 
	const bool bAddPrefixSuffix = NameStr.Contains(NamePattern);
	
	// look for find & replace pattern 
	int32 SlashIndex[3] = { INDEX_NONE, INDEX_NONE, INDEX_NONE };
	for (int32 CharIndex = 0, NumSlash = 0; NumSlash < 3 && CharIndex < NameStr.Len(); ++CharIndex)
	{
		if (NameStr[CharIndex] == Slash)
		{
			SlashIndex[NumSlash++] = CharIndex;
		}
	}

	const bool bFindReplace =
		SlashIndex[0] == 0 && SlashIndex[2] == NameStr.Len()-1 && // starts and ends with /
		SlashIndex[0] != SlashIndex[1] && SlashIndex[0] != SlashIndex[2]; // all three indices are different

	if (bFindReplace)
	{
		if (bAddPrefixSuffix || (SlashIndex[1] - SlashIndex[0]) < 2)
		{
			// error
			return;
		}
	}
	
	TArray<int32> BoneIndices;
	const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
	Algo::Transform(BoneNames, BoneIndices, [&](const FName& BoneName)
	{
		return ReferenceSkeleton.FindRawBoneIndex(BoneName);
	});

	TArray<FName> NewNames;

	if (bFindReplace)
	{
		const FString Find = FString::ConstructFromPtrSize(&NameStr[SlashIndex[0]+1], SlashIndex[1]-SlashIndex[0]-1);
		const FString Replace = FString::ConstructFromPtrSize(&NameStr[SlashIndex[1]+1], SlashIndex[2]-SlashIndex[1]-1);
		
		NewNames.Reserve(BoneNames.Num());
		for (int32 Index = 0; Index < BoneNames.Num(); Index++)
		{
			FString NewBoneNameStr = BoneNames[Index].ToString().Replace(*Find, *Replace, ESearchCase::CaseSensitive);
			NewNames.Add(*NewBoneNameStr);
		} 
	}
	else
	{
		FString Prefix, Suffix;
		if (bAddPrefixSuffix && NameStr.Split(NamePattern, &Prefix, &Suffix))
		{
			NewNames.Reserve(BoneNames.Num());
			for (int32 Index = 0; Index < BoneNames.Num(); Index++)
			{
				FString NewBoneNameStr = FString::Printf( TEXT("%s%s%s"), *Prefix, *BoneNames[Index].ToString(), *Suffix);
				NewNames.Add(*NewBoneNameStr);
			}
		}
		else
		{
			NewNames.Init(NewName, BoneNames.Num());
		}
	}
	
	BeginChange();

	const bool bBoneRenamed = Modifier->RenameBones(BoneNames, NewNames);
	if (bBoneRenamed)
	{
		const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRawRefBoneInfo();
		for (int32 Index = 0; Index < BoneIndices.Num(); Index++)
		{
			if (BoneIndices[Index] != INDEX_NONE)
			{
				const FName OldName = BoneNames[Index];
				BoneNames[Index] = BoneInfos[BoneIndices[Index]].Name;
				const int32 ItemIndex = AllElements.IndexOfByPredicate([OldName](const TSharedPtr<FBoneElement>& Item)
				{
					return Item->BoneName == OldName;
				});

				if (ItemIndex != INDEX_NONE)
				{
					AllElements[ItemIndex]->BoneName = BoneNames[Index];
				}
			}
		}
		
		if (Notifier.IsValid())
		{
			Notifier->Notify(BoneNames, ESkeletalMeshNotifyType::BonesRenamed);
		}
		
		EndChange();
	}
	else
	{
		CancelChange();	
	}
}

void SReferenceSkeletonTree::OnFilterTextChanged(const FText& SearchText)
{
	TextFilter->SetFilterText(SearchText);
	RefreshTreeView();
}

void SReferenceSkeletonTree::RefreshTreeView(bool IsInitialSetup /*=false*/)
{
	if (!Modifier.IsValid())
	{
		return;
	}

	TArray<FName> PreviouslySelectedBones;
	GetSelectedBoneNames(PreviouslySelectedBones);

	TreeView->StoreSparseItemInfos();
	
	const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
	const TArray<FMeshBoneInfo>& BoneInfos = RefSkeleton.GetRawRefBoneInfo();
	
	// reset all tree items
	RootElements.Reset();
	AllElements.Reset();

	// validate we have a skeleton to load
	if (BoneInfos.IsEmpty())
	{
		TreeView->RequestTreeRefresh();
		return;
	}
	
	// record bone element indices
	TMap<FName, int32> BoneTreeElementIndices;

	auto FilterString = [this](const FString& StringToTest) ->bool
	{
		return TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(StringToTest));
	};

	TMap<int32, int32> PostponedParent;
	
	// create all bone elements
	for (int32 BoneIndex=0; BoneIndex < BoneInfos.Num(); ++BoneIndex)
	{
		const FName BoneName = BoneInfos[BoneIndex].Name;
		TSharedPtr<FBoneElement> BoneElement = MakeShared<FBoneElement>(BoneName, Modifier);
		const int32 BoneElementIndex = AllElements.Add(BoneElement);
		BoneTreeElementIndices.Add(BoneName, BoneElementIndex);

		// store pointer to parent (if there is one)
		const int32 ParentIndex = BoneInfos[BoneIndex].ParentIndex;
		if (ParentIndex != INDEX_NONE)
		{
			// get parent tree element
			const FName ParentBoneName = BoneInfos[ParentIndex].Name;

			const int32* FoundParent = BoneTreeElementIndices.Find(ParentBoneName);
			if (FoundParent)
			{ // set parent info directly
				const TSharedPtr<FBoneElement> ParentBoneTreeElement = AllElements[*FoundParent];
				BoneElement->UnFilteredParent = ParentBoneTreeElement;
			}
			else
			{ // postpone as the parent might not have been added to AllElements / BoneTreeElementIndices yet
				PostponedParent.Emplace(BoneElementIndex, ParentIndex);
			}
		}
		
		// apply text filter to bones
		if (!(TextFilter->GetFilterText().IsEmpty() || FilterString(BoneName.ToString())))
		{
			BoneElement->bIsHidden = true;
		}
	}

	// store pointer to parents that were postponed 
	for (const auto & [BoneElementIndex, ParentIndex] : PostponedParent)
	{
		const FName ParentBoneName = BoneInfos[ParentIndex].Name;
		const int32* FoundParent = BoneTreeElementIndices.Find(ParentBoneName);
		if (ensure(FoundParent))
		{
			const TSharedPtr<FBoneElement> BoneElement = AllElements[BoneElementIndex];
			const TSharedPtr<FBoneElement> ParentBoneTreeElement = AllElements[*FoundParent];
			BoneElement->UnFilteredParent = ParentBoneTreeElement;
		}
	}
	
	// resolve parent/children pointers on all tree elements, taking into consideration the filter options
	// (elements are parented to their nearest non-hidden/filtered parent element)
	for (TSharedPtr<FBoneElement>& Element : AllElements)
	{
		if (Element->bIsHidden)
		{
			continue;
		}
		
		// find first parent that is not filtered
		TSharedPtr<FBoneElement> ParentElement = Element->UnFilteredParent;
		while (true)
		{
			if (!ParentElement.IsValid())
			{
				break;
			}

			if (!ParentElement->bIsHidden)
			{
				break;
			}

			ParentElement = ParentElement->UnFilteredParent;
		}

		if (ParentElement.IsValid())
		{
			// store pointer to child on parent
			ParentElement->Children.Add(Element);
			// store pointer to parent on child
			Element->Parent = ParentElement;
		}
		else
		{
			// has no parent, store a root element
			RootElements.Add(Element);
		}
	}

	// expand all elements upon the initial construction of the tree
	for (const TSharedPtr<FBoneElement>& RootElement : RootElements)
	{
		SetExpansionRecursive(RootElement, false, true);
	}
	
	TreeView->RequestTreeRefresh();
	
	TreeView->RestoreSparseItemInfos(AllElements);

	if (!PreviouslySelectedBones.IsEmpty())
	{
		SelectItemFromNames(PreviouslySelectedBones);
	}
}

void SReferenceSkeletonTree::HandleGetChildrenForTree(
	TSharedPtr<FBoneElement> InItem,
	TArray<TSharedPtr<FBoneElement>>& OutChildren)
{
	OutChildren = InItem.Get()->Children;
}

void SReferenceSkeletonTree::OnSelectionChanged(TSharedPtr<FBoneElement> InItem, ESelectInfo::Type InSelectInfo)
{
	if (InSelectInfo == ESelectInfo::Direct)
	{
		return;
	}
	
	if (!InItem)
	{
		if (Notifier.IsValid())
		{
			Notifier->Notify({}, ESkeletalMeshNotifyType::BonesSelected);
		}
		return;
	}
	
	TArray<FName> BoneNames;
	GetSelectedBoneNames(BoneNames);
	
	if (Notifier.IsValid())
	{
		Notifier->Notify(BoneNames, ESkeletalMeshNotifyType::BonesSelected);
	}
}

TSharedRef<SWidget> SReferenceSkeletonTree::CreateAddNewMenu()
{
	const FSkeletonEditingToolCommands& Commands = FSkeletonEditingToolCommands::Get();

	static constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, CommandList);
	
	MenuBuilder.BeginSection("NewBone", LOCTEXT("AddNewBoneOperations", "Bones"));
	MenuBuilder.AddMenuEntry(Commands.NewBone);
	MenuBuilder.EndSection();
		
	return MenuBuilder.MakeWidget();
}

EVisibility SReferenceSkeletonTree::GetAddNewButtonVisibility() const
{
	return bIsReadOnly ? EVisibility::Collapsed : EVisibility::Visible;
}

TSharedPtr<SWidget> SReferenceSkeletonTree::CreateContextMenu()
{
	static constexpr bool bCloseAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, CommandList);

	
	const TArray<TSharedPtr<FBoneElement>> SelectedItems = GetSelectedItems();
	
	bool bHasValidMenuOptions = false;
	if (!bIsReadOnly)
	{
		const FSkeletonEditingToolCommands& Commands = FSkeletonEditingToolCommands::Get();
		bHasValidMenuOptions = true;
		if (SelectedItems.IsEmpty())
		{
			
			MenuBuilder.BeginSection("NewBone", LOCTEXT("AddNewBoneOperations", "Bones"));
	
			MenuBuilder.AddMenuEntry(Commands.NewBone);

			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("CopyPasteBones", LOCTEXT("CopyPasteBonesOperations", "Copy & Paste"));

			MenuBuilder.AddMenuEntry(Commands.CopyBones);
			MenuBuilder.AddMenuEntry(Commands.PasteBonesLocal);
			MenuBuilder.AddMenuEntry(Commands.PasteBonesGlobal);

			MenuBuilder.EndSection();
		}
		else
		{
			MenuBuilder.BeginSection("EditBones", LOCTEXT("EditBonesOperations", "Bones"));
			{
				MenuBuilder.AddMenuEntry(Commands.NewBone);
				MenuBuilder.AddMenuEntry(Commands.RemoveBone);
				MenuBuilder.AddMenuEntry(Commands.UnParentBone);
			}
			MenuBuilder.EndSection();
		
			MenuBuilder.BeginSection("RenameBones", LOCTEXT("RenameBonesOperations", "Rename"));
			{
				MenuBuilder.AddMenuEntry(Commands.RenameBone);
				MenuBuilder.AddMenuEntry(Commands.SearchAndReplace);
				MenuBuilder.AddMenuEntry(Commands.AddPrefix);
				MenuBuilder.AddMenuEntry(Commands.AddSuffix);
			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("CopyPasteBones", LOCTEXT("CopyPasteBonesOperations", "Copy & Paste"));
			{
				MenuBuilder.AddMenuEntry(Commands.CopyBones);
				MenuBuilder.AddMenuEntry(Commands.PasteBonesLocal);
				MenuBuilder.AddMenuEntry(Commands.PasteBonesGlobal);
				MenuBuilder.AddMenuEntry(Commands.DuplicateBones);
			}
			MenuBuilder.EndSection();
		}	
	}

	if (Delegates.OnExtendContextMenu.IsBound())
	{
		if (Delegates.OnExtendContextMenu.Execute( MenuBuilder))
		{
			bHasValidMenuOptions = true;
		}
	}
	
	if (bHasValidMenuOptions)
	{
		return MenuBuilder.MakeWidget();
	}
	
	return nullptr;
}

void SReferenceSkeletonTree::OnItemDoubleClicked(TSharedPtr<FBoneElement> InItem)
{
	InItem->RequestRename(ReferenceSkeletonTreeLocals::EBoneRenameType::Rename);
}

void SReferenceSkeletonTree::OnSetExpansionRecursive(TSharedPtr<FBoneElement> InItem, bool bShouldBeExpanded)
{
	SetExpansionRecursive(InItem, false, bShouldBeExpanded);
}

void SReferenceSkeletonTree::SetExpansionRecursive(
	TSharedPtr<FBoneElement> InElement,
	bool bTowardsParent,
    bool bShouldBeExpanded)
{
	TreeView->SetItemExpansion(InElement, bShouldBeExpanded);
    
    if (bTowardsParent)
    {
    	if (InElement->Parent.Get())
    	{
    		SetExpansionRecursive(InElement->Parent, bTowardsParent, bShouldBeExpanded);
    	}
    }
    else
    {
    	for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
    	{
    		SetExpansionRecursive(InElement->Children[ChildIndex], bTowardsParent, bShouldBeExpanded);
    	}
    }
}

FReply SReferenceSkeletonTree::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (!Modifier.IsValid())
	{
		return FReply::Handled();
	}
	
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

FReply SReferenceSkeletonTree::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TArray<TSharedPtr<FBoneElement>> BoneElements = GetSelectedItems();
	if (BoneElements.IsEmpty())
	{
		return FReply::Unhandled();
	}
	
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedRef<FBoneItemDragDropOp> DragDropOp = FBoneItemDragDropOp::New(BoneElements);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}
	
	return FReply::Unhandled();
}

TOptional<EItemDropZone> SReferenceSkeletonTree::OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FBoneElement> TargetItem)
{
	if (bIsReadOnly)
	{
		return TOptional<EItemDropZone>();
		
	}
	TOptional<EItemDropZone> ReturnedDropZone;

	const TSharedPtr<FBoneItemDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FBoneItemDragDropOp>();
	if (DragDropOp.IsValid())
	{
		ReturnedDropZone = EItemDropZone::OntoItem;
	}

	return ReturnedDropZone;
}

FReply SReferenceSkeletonTree::OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FBoneElement> TargetItem)
{
	const TSharedPtr<FBoneItemDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FBoneItemDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!Modifier.IsValid())
	{
		return FReply::Handled();
	}
	
	TArray<FName> BoneNames;
	BoneNames.Reserve(DragDropOp->Elements.Num());
	for (const TWeakPtr<FBoneElement>& BoneElement: DragDropOp->Elements)
	{
		if (BoneElement.IsValid())
		{
			BoneNames.Add(BoneElement.Pin()->BoneName);
		}
	}
	const FName ParentName = TargetItem->BoneName;

	BeginChange();
	
	const bool bParented = Modifier->ParentBones(BoneNames, {ParentName});
	if (bParented)
	{
		RefreshTreeView(true);
		SelectItemFromNames(BoneNames);
		if (Notifier.IsValid())
		{
			Notifier->Notify( BoneNames, ESkeletalMeshNotifyType::HierarchyChanged);
			Notifier->Notify( BoneNames, ESkeletalMeshNotifyType::BonesSelected);
		}
		EndChange();
	}
	else
	{
		CancelChange();
	}
	
	return FReply::Handled();
}

FReferenceSkeletonWidgetNotifier::FReferenceSkeletonWidgetNotifier(TSharedRef<SReferenceSkeletonTree> InWidget)
	: ISkeletalMeshNotifier()
	, Tree(InWidget)
{}

void FReferenceSkeletonWidgetNotifier::HandleNotification(const TArray<FName>& InBoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	if (Notifying() || !Tree.IsValid())
	{
		return;
	}

	TSharedPtr<SReferenceSkeletonTree> TreePtr = Tree.Pin();
	switch (InNotifyType)
	{
		case ESkeletalMeshNotifyType::BonesAdded:
			TreePtr->RefreshTreeView();
			break;
		case ESkeletalMeshNotifyType::BonesRemoved:
			TreePtr->RefreshTreeView();
			break;
		case ESkeletalMeshNotifyType::BonesMoved:
			break;
		case ESkeletalMeshNotifyType::BonesSelected:
			{
				static constexpr bool bFrameSelection = true;
				TreePtr->SelectItemFromNames(InBoneNames, bFrameSelection);
				break;
			}
		case ESkeletalMeshNotifyType::BonesRenamed:
			TreePtr->RefreshTreeView();
			TreePtr->SelectItemFromNames(InBoneNames);
			break;
		case ESkeletalMeshNotifyType::HierarchyChanged:
			{
				TArray<FName> BoneNames;
				TreePtr->GetSelectedBoneNames(BoneNames);
				TreePtr->RefreshTreeView();
				TreePtr->SelectItemFromNames(BoneNames);
			}
			break;
	}
}

void SReferenceSkeletonTree::BeginChange()
{
	ensure( ActiveChange == nullptr );
	ActiveChange = MakeUnique<ReferenceSkeletonTreeLocals::FSkeletonModifierChange>(Modifier.Get()); 
}

void SReferenceSkeletonTree::EndChange()
{
	if (!ActiveChange.IsValid())
	{
		return;
	}

	ActiveChange->StoreSkeleton(Modifier.Get());
	
	GEditor->BeginTransaction(LOCTEXT("ModifyRefSkeleton", "Modify Reference Skeleton"));
	GUndo->StoreUndo(Modifier.Get(), MoveTemp(ActiveChange));
	GEditor->EndTransaction();
}

void SReferenceSkeletonTree::CancelChange()
{
	ActiveChange.Reset();
}

#undef LOCTEXT_NAMESPACE
