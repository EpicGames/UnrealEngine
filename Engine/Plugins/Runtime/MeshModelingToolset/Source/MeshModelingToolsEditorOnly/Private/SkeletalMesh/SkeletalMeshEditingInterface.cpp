// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMesh/SkeletalMeshEditingInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshEditingInterface)

TSharedPtr<ISkeletalMeshNotifier> ISkeletalMeshEditingInterface::GetNotifier()
{
	if (!Notifier)
	{
		Notifier = MakeShared<FSkeletalMeshToolNotifier>(TWeakInterfacePtr<ISkeletalMeshEditingInterface>(this));
	}
	
	return Notifier;
}



bool ISkeletalMeshEditingInterface::NeedsNotification() const
{
	return Notifier && Notifier->Delegate().IsBound(); 
}

bool ISkeletalMeshEditingInterface::CanModifySkeleton() const
{
	return false;
}

TSharedPtr<SWidget> ISkeletalMeshEditingInterface::GetCustomSkeletonTreeWidget(TSharedPtr<ISkeletalMeshEditorBinding> EditorBinding)
{
	return nullptr;
}

FSkeletalMeshToolNotifier::FSkeletalMeshToolNotifier(TWeakInterfacePtr<ISkeletalMeshEditingInterface> InInterface)
	: ISkeletalMeshNotifier()
	, Interface(InInterface)
{}

void FSkeletalMeshToolNotifier::HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	if (Interface.IsValid())
	{
		Interface->HandleSkeletalMeshModified(BoneNames, InNotifyType);
	}
}
