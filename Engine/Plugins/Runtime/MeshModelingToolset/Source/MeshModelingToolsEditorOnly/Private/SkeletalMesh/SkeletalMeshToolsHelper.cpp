// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletalMesh/SkeletalMeshToolsHelper.h"

#include "BoneWeights.h"
#include "SkeletalMeshAttributes.h"
#include "Components/BaseDynamicMeshComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Parameterization/MeshPlanarSymmetry.h"
#include "Async/ParallelFor.h"

void SkeletalMeshToolsHelper::SetupPreviewTangentMode(UDynamicMeshComponent* Component)
{
	using namespace UE::Geometry;

	if (!Component)
	{
		return;
	}

	const FDynamicMesh3* Mesh = Component->GetMesh();
	if (Mesh && Mesh->Attributes() && Mesh->Attributes()->HasTangentSpace())
	{
		// HasTangentSpace only checks overlay existence. The MeshDescription tangent
		// attribute is always present and gets copied through the converter even when the
		// artist didn't export tangents — in that case the overlay holds all-zero vectors.
		// Spot-check the first valid element: non-zero => real imported tangents (leave
		// at Default); zero => junk, fall through to AutoCalculated.
		const FDynamicMeshNormalOverlay* TangentOverlay = Mesh->Attributes()->PrimaryTangents();
		if (TangentOverlay && TangentOverlay->ElementCount() > 0)
		{
			for (int32 ElementID = 0; ElementID <= TangentOverlay->MaxElementID(); ++ElementID)
			{
				if (TangentOverlay->IsElement(ElementID))
				{
					if (!TangentOverlay->GetElement(ElementID).IsNearlyZero())
					{
						return;
					}
					break;
				}
			}
		}
	}

	Component->SetTangentsType(EDynamicMeshComponentTangentsMode::AutoCalculated);
}

void SkeletalMeshToolsHelper::GetUnposedMesh(
	TFunctionRef<void(FVertInfo, const FVector&)> WriteFunc,
	TFunctionRef<FVector(int32)> GetPosedVertexFunc,
	const FDynamicMesh3& SourceMesh,
	const TArray<FMatrix>& BoneMatrices,
	FName SkinWeightProfile,
	const TMap<FName, float>& MorphTargetWeights,
	const TArray<int32>& VertArray
	)
{

	using namespace UE::Geometry;
	using namespace UE::AnimationCore;

	SkinWeightProfile = SkinWeightProfile == NAME_None ? FSkeletalMeshAttributes::DefaultSkinWeightProfileName : SkinWeightProfile;
	FDynamicMeshVertexSkinWeightsAttribute* SkinWeightAttribute = SourceMesh.Attributes()->GetSkinWeightsAttribute(SkinWeightProfile);

	bool bHasVertSelection = VertArray.Num() > 0;
	int32 NumToProcess = bHasVertSelection ? VertArray.Num() : SourceMesh.MaxVertexID();
	
	ParallelFor(NumToProcess, [&](int32 Index)
		{
			int32 VertID = bHasVertSelection ? VertArray[Index] : Index;
			
			if (!SourceMesh.IsVertex(VertID))
			{
				return;
			}
				
			FBoneWeights BoneWeights;
			SkinWeightAttribute->GetValue(VertID, BoneWeights);
			
			FMatrix SkinMatrix = FMatrix(FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector, FVector::ZeroVector);
			SkinMatrix.M[3][3] = 0.0f;

			for (const FBoneWeight& BW : BoneWeights)
			{
				SkinMatrix += BoneMatrices[BW.GetBoneIndex()] * BW.GetWeight();
			}
			FVector PosedVertPos = GetPosedVertexFunc(VertID);
				
			FVector UnposedVertPos = SkinMatrix.Inverse().TransformPosition(PosedVertPos);

			for (const TPair<FName, float>& MorphTargetWeight : MorphTargetWeights)
			{
				if (!FMath::IsNearlyZero(MorphTargetWeight.Value))
				{
					const FDynamicMeshMorphTargetAttribute* MorphTargetAttribute =
						SourceMesh.Attributes()->GetMorphTargetAttribute(MorphTargetWeight.Key);

					if (ensure(MorphTargetAttribute))
					{
						FVector Delta;
						MorphTargetAttribute->GetValue(VertID, Delta);
				
						UnposedVertPos += MorphTargetWeight.Value * Delta * (-1.0f);
					}
				}
			}

			FVertInfo Info = {
				bHasVertSelection ? Index : INDEX_NONE,
				VertID,
			};
			
			WriteFunc(Info, UnposedVertPos);
		});
}

void SkeletalMeshToolsHelper::GetPosedMesh(
	TFunctionRef<void(FVertInfo, const FVector&)> WriteFunc, 
	const FDynamicMesh3& SourceMesh,
	const TArray<FMatrix>& BoneMatrices,
	FName SkinWeightProfile,
	const TMap<FName, float>& MorphTargetWeights,
	const TArray<int32>& VertArray
	)
{
	using namespace UE::Geometry;
	using namespace UE::AnimationCore;

	SkinWeightProfile = SkinWeightProfile == NAME_None
		? FSkeletalMeshAttributes::DefaultSkinWeightProfileName
		: SkinWeightProfile;

	// Tolerate malformed input from procedural / blueprint-generated meshes:
	//  - missing attribute set                  -> no morph or skin work; vertex stays at input position
	//  - missing skin weight attribute          -> no skin step; vertex stays at morph-deformed input
	//  - empty BoneWeights for a vertex         -> no skin step; vertex stays at morph-deformed input
	//  - bone index out of range                -> that weight is skipped, others still applied
	//  - total applied weight is zero           -> no skin transform (avoids collapsing to origin)
	//  - missing morph target attribute         -> that morph delta is skipped
	const FDynamicMeshAttributeSet* AttributeSet = SourceMesh.Attributes();
	FDynamicMeshVertexSkinWeightsAttribute* SkinWeightAttribute = AttributeSet
		? AttributeSet->GetSkinWeightsAttribute(SkinWeightProfile)
		: nullptr;

	const int32 NumBoneMatrices = BoneMatrices.Num();
	const bool bHasVertSelection = VertArray.Num() > 0;
	const int32 NumToProcess = bHasVertSelection ? VertArray.Num() : SourceMesh.MaxVertexID();

	ParallelFor(NumToProcess, [&](int32 Index)
		{
			const int32 VertID = bHasVertSelection ? VertArray[Index] : Index;
			if (!SourceMesh.IsVertex(VertID))
			{
				return;
			}

			FVector PosedVertPos = SourceMesh.GetVertex(VertID);

			if (AttributeSet)
			{
				for (const TPair<FName, float>& MorphTargetWeight : MorphTargetWeights)
				{
					if (FMath::IsNearlyZero(MorphTargetWeight.Value))
					{
						continue;
					}
					const FDynamicMeshMorphTargetAttribute* MorphTargetAttribute
						= AttributeSet->GetMorphTargetAttribute(MorphTargetWeight.Key);
					if (MorphTargetAttribute)
					{
						FVector Delta;
						MorphTargetAttribute->GetValue(VertID, Delta);
						PosedVertPos += MorphTargetWeight.Value * Delta;
					}
				}
			}

			if (SkinWeightAttribute)
			{
				FBoneWeights BoneWeights;
				SkinWeightAttribute->GetValue(VertID, BoneWeights);

				if (BoneWeights.Num() > 0)
				{
					FMatrix SkinMatrix(FVector::ZeroVector, FVector::ZeroVector,
						FVector::ZeroVector, FVector::ZeroVector);
					SkinMatrix.M[3][3] = 0.0f;

					float TotalAppliedWeight = 0.0f;
					for (const FBoneWeight& BoneWeight : BoneWeights)
					{
						const int32 BoneIndex = BoneWeight.GetBoneIndex();
						if (BoneIndex >= 0 && BoneIndex < NumBoneMatrices)
						{
							const float Weight = BoneWeight.GetWeight();
							SkinMatrix += BoneMatrices[BoneIndex] * Weight;
							TotalAppliedWeight += Weight;
						}
					}

					if (TotalAppliedWeight > UE_SMALL_NUMBER)
					{
						PosedVertPos = SkinMatrix.TransformPosition(PosedVertPos);
					}
				}
			}

			FVertInfo Info = {
				bHasVertSelection ? Index : INDEX_NONE,
				VertID,
			};
			WriteFunc(Info, PosedVertPos);
		});
}

TArray<FMatrix> SkeletalMeshToolsHelper::ComputeBoneMatrices(const TArray<FTransform>& ComponentSpaceTransformsRefPose,
	const TArray<FTransform>& ComponentSpaceTransforms)
{
	check(ComponentSpaceTransformsRefPose.Num() == ComponentSpaceTransforms.Num());
	
	TArray<FMatrix> BoneMatrices;
	BoneMatrices.SetNumUninitialized(ComponentSpaceTransforms.Num());
	for (int32 BoneIndex = 0; BoneIndex < ComponentSpaceTransforms.Num(); ++BoneIndex)
	{
		BoneMatrices[BoneIndex] =
				ComponentSpaceTransformsRefPose[BoneIndex].ToMatrixWithScale().Inverse() *
				ComponentSpaceTransforms[BoneIndex].ToMatrixWithScale();
	}

	return BoneMatrices;
}

void SkeletalMeshToolsHelper::FPoseChangeDetector::CheckPose(const TArray<FTransform>& ComponentSpaceTransforms,
	const TMap<FName, float>& MorphTargetWeights)
{
	auto HasPoseChanged = [&]()
		{
			if (ComponentSpaceTransforms.Num() != PreviousComponentSpaceTransforms.Num())
			{
				return true;
			}
				
			for (int32 BoneIndex=0; BoneIndex< ComponentSpaceTransforms.Num(); ++BoneIndex)
			{
				const FTransform& CurrentBoneTransform = ComponentSpaceTransforms[BoneIndex];
				const FTransform& PrevBoneTransform = PreviousComponentSpaceTransforms[BoneIndex];
				if (!CurrentBoneTransform.Equals(PrevBoneTransform))
				{
					return true;
				}
			}
				
			if (!MorphTargetWeights.OrderIndependentCompareEqual(PreviousMorphTargetWeights))
			{
				return true;
			}
				
			return false;
		};

	const bool bPoseChanged = HasPoseChanged();
	bool bNotify = bPoseChanged;
	if (bPoseChanged)
	{
		if (State == PoseStoppedChanging)
		{
			State = PoseJustChanged;
		}
		else if (State == PoseJustChanged)
		{
			State = PoseChanged;
		}
	}
	else
	{
		if (State != PoseStoppedChanging)
		{
			State = PoseStoppedChanging;
			bNotify = true;
		}
	}

	if (bNotify)
	{
		if (PreviousComponentSpaceTransforms.IsEmpty())
		{
			PreviousComponentSpaceTransforms = ComponentSpaceTransforms;
			PreviousMorphTargetWeights = MorphTargetWeights;
		}
	
		FPayload Payload = {
			State,
			ComponentSpaceTransforms,
			MorphTargetWeights,
			PreviousComponentSpaceTransforms,
			PreviousMorphTargetWeights
		};
	
		Notifier.Broadcast(Payload);	
	}

	if (bPoseChanged)
	{
		PreviousComponentSpaceTransforms = ComponentSpaceTransforms;
		PreviousMorphTargetWeights = MorphTargetWeights;	
	}
}

SkeletalMeshToolsHelper::FPoseChangeDetector::FNotifier& SkeletalMeshToolsHelper::FPoseChangeDetector::GetNotifier() { return Notifier; }


FString SkeletalMeshToolsHelper::FMeshMorphTargetChange::ToString() const
{
	return TEXT("Edited Morph Target");
}

void SkeletalMeshToolsHelper::FMeshMorphTargetChange::ApplyChangeToMesh(UE::Geometry::FDynamicMesh3* Mesh, bool bRevert) const
{
	using namespace UE::Geometry;

	FDynamicMeshMorphTargetAttribute* MorphTargetAttribute = Mesh->Attributes()->GetMorphTargetAttribute(MorphTargetName);
	if (!MorphTargetAttribute)
	{
		return;
	}

	const TArray<FVector>& Deltas = bRevert ? OldDeltas : NewDeltas;
	ParallelFor(Vertices.Num(), [&](int32 Index)
	{
		MorphTargetAttribute->SetValue(Vertices[Index], Deltas[Index]);
	});
}


namespace SkeletalMeshToolsHelper::Private
{
	static void SnapshotMorphDeltas(
		const UE::Geometry::FDynamicMesh3& Mesh,
		const UE::Geometry::FDynamicMeshMorphTargetAttribute& Attr,
		TArray<FVector>& OutDeltas)
	{
		OutDeltas.SetNumUninitialized(Mesh.MaxVertexID());
		ParallelFor(Mesh.MaxVertexID(), [&](int32 VID)
		{
			if (Mesh.IsVertex(VID))
			{
				Attr.GetValue(VID, OutDeltas[VID]);
			}
		});
	}

	// Diffs the post-edit attribute against the pre-edit snapshot to fill OutChange. Lets the
	// per-vertex algorithms above stay verbatim instead of having to capture before/after inline.
	static void CollectChangedDeltas(
		const UE::Geometry::FDynamicMesh3& Mesh,
		const UE::Geometry::FDynamicMeshMorphTargetAttribute& Attr,
		const TArray<FVector>& OldDeltas,
		FName MorphTargetName,
		SkeletalMeshToolsHelper::FMeshMorphTargetChange& OutChange)
	{
		OutChange.MorphTargetName = MorphTargetName;

		for (int32 VID : Mesh.VertexIndicesItr())
		{
			FVector NewDelta;
			Attr.GetValue(VID, NewDelta);
			if (!NewDelta.Equals(OldDeltas[VID]))
			{
				OutChange.Vertices.Add(VID);
				OutChange.OldDeltas.Add(OldDeltas[VID]);
				OutChange.NewDeltas.Add(NewDelta);
			}
		}
	}
}

void SkeletalMeshToolsHelper::MirrorMorphTargetOnMesh(
	UE::Geometry::FDynamicMesh3& Mesh,
	FName MorphTargetName,
	const UE::Geometry::FMeshPlanarSymmetry& Symmetry,
	FMeshMorphTargetChange* OutDeltaChange)
{
	using namespace UE::Geometry;

	FDynamicMeshMorphTargetAttribute* SourceMorphTargetAttribute = Mesh.Attributes()->GetMorphTargetAttribute(MorphTargetName);
	if (!SourceMorphTargetAttribute)
	{
		return;
	}

	TArray<FVector> OldDeltas;
	if (OutDeltaChange)
	{
		Private::SnapshotMorphDeltas(Mesh, *SourceMorphTargetAttribute, OldDeltas);
	}

	ParallelFor(Mesh.MaxVertexID(), [&](int32 VertID)
	{
		if (!Mesh.IsVertex(VertID))
		{
			return;
		}

		FVector Position = Mesh.GetVertex(VertID);

		FVector Delta;
		SourceMorphTargetAttribute->GetValue(VertID, Delta);

		constexpr bool bForceSameSizeWithGaps = true;
		TArray<int32> MirroredVertID;
		Symmetry.GetMirrorVertexROI({VertID}, MirroredVertID, bForceSameSizeWithGaps);

		// INDEX_NONE means either an on-plane vert or an unmatched vert with no mirror pair.
		// On-plane gets its delta snapped to the plane; unmatched is left alone (the snap is a no-op).
		if (MirroredVertID[0] == INDEX_NONE)
		{
			TArray<FVector> ConstrainedPosition;
			ConstrainedPosition.Add(Position + Delta);
			Symmetry.ApplySymmetryPlaneConstraints({VertID}, ConstrainedPosition);

			Delta = ConstrainedPosition[0] - Position;
			SourceMorphTargetAttribute->SetValue(VertID, Delta);
		}
		else if (Position.X >= 0)
		{
			// Source side: write the reflected delta to the mirror vertex.
			FVector MirroredDelta = Symmetry.GetMirroredAxis(Delta);
			SourceMorphTargetAttribute->SetValue(MirroredVertID[0], MirroredDelta);
		}
	});

	if (OutDeltaChange)
	{
		Private::CollectChangedDeltas(Mesh, *SourceMorphTargetAttribute, OldDeltas, MorphTargetName, *OutDeltaChange);
	}
}
