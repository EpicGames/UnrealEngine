// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletalMesh/SkeletonEditingTool.h"

#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "SkeletalDebugRendering.h"
#include "ToolTargetManager.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "UnrealClient.h"
#include "HitProxies.h"
#include "SkeletonClipboard.h"
#include "ToolSetupUtil.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseGizmos/GizmoViewContext.h"
#include "DynamicMesh/MeshNormals.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Preferences/PersonaOptions.h"
#include "DynamicSubmesh3.h"
#include "Selection/PolygonSelectionMechanic.h"

#include "SkeletalMesh/SkeletonTransformProxy.h"
#include "SkeletalMesh/SReferenceSkeletonTree.h"
#include "SkeletalMesh/SkeletalMeshModelingToolsEditorSettings.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "Components/DynamicMeshComponent.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "TargetInterfaces/SkeletonCommitter.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonEditingTool)

#define LOCTEXT_NAMESPACE "USkeletonEditingTool"

IMPLEMENT_HIT_PROXY(HBoneHitProxy, HHitProxy)

namespace SkeletonEditingTool
{
	
FRefSkeletonChange::FRefSkeletonChange(const USkeletonEditingTool* InTool)
	: FToolCommandChange()
	, PreChangeSkeleton(InTool->Modifier->GetReferenceSkeleton())
	, PreBoneTracker(InTool->Modifier->GetBoneIndexTracker())
	, PostChangeSkeleton(InTool->Modifier->GetReferenceSkeleton())
	, PostBoneTracker(InTool->Modifier->GetBoneIndexTracker())
{}

void FRefSkeletonChange::StoreSkeleton(const USkeletonEditingTool* InTool)
{
	PostChangeSkeleton = InTool->Modifier->GetReferenceSkeleton();
	PostBoneTracker = InTool->Modifier->GetBoneIndexTracker();
}

void FRefSkeletonChange::Apply(UObject* Object)
{ // redo
	USkeletonEditingTool* Tool = CastChecked<USkeletonEditingTool>(Object);
	Tool->Modifier->ExternalUpdate(PostChangeSkeleton, PostBoneTracker);
	Tool->UpdateGizmo();
}

void FRefSkeletonChange::Revert(UObject* Object)
{ // undo
	USkeletonEditingTool* Tool = CastChecked<USkeletonEditingTool>(Object);
	Tool->Modifier->ExternalUpdate(PreChangeSkeleton, PreBoneTracker);
	Tool->UpdateGizmo();
}

}


FSkeletonEditingToolCommands::FSkeletonEditingToolCommands()
	: TCommands<FSkeletonEditingToolCommands>
	(
		TEXT("SkeletonEditingTool"),
		NSLOCTEXT("Contexts", "SkeletonEditingToolCommands", "Skeleton Editing Tool Commands"),
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{
}

void FSkeletonEditingToolCommands::RegisterCommands()
{
	UI_COMMAND(NewBone, "New Bone", "Add new Bone.", EUserInterfaceActionType::Button, FInputChord(EKeys::N, EModifierKey::Control));
	UI_COMMAND(RemoveBone, "Remove Bone", "Remove selected bone(s).", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete), FInputChord(EKeys::BackSpace));
	UI_COMMAND(UnParentBone, "Unparent Bone", "Unparent selected bone(s).", EUserInterfaceActionType::Button, FInputChord(EKeys::P, EModifierKey::Shift));

	UI_COMMAND(RenameBone, "Rename Bone", "Rename the selected bone.", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(SearchAndReplace, "Search and Replace", "Search and Replace the selected bone(s) name(s).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddPrefix, "Add Prefix", "Add a prefix to the selected bone(s) name(s).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddSuffix, "Add Suffix", "Add a suffix to the selected bone(s) name(s).", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(CopyBones, "Copy Bone(s)", "Copy selected bone(s).", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control));
	UI_COMMAND(PasteBonesLocal, "Paste Bone(s) Local", "Paste bone(s) using the local transform of the copied bones.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control));
	UI_COMMAND(PasteBonesGlobal, "Paste Bone(s) Global", "Paste bone(s) using the global transform of the copied bones.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control | EModifierKey::Shift));
	
	UI_COMMAND(CopyBoneTransforms, "Copy Bone Transform(s)", "Copy selected bone(s) transforms.", EUserInterfaceActionType::Button, FInputChord(EKeys::C, EModifierKey::Control | EModifierKey::Shift));
	UI_COMMAND(PasteBoneTransformsLocal, "Paste Bone Transform(s) Local", "Paste local bone transform(s) to the selected bones.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control | EModifierKey::Alt));
	UI_COMMAND(PasteBoneTransformsGlobal, "Paste Bone Transform(s) Global", "Paste global bone transfrom(s) to the selected bones.", EUserInterfaceActionType::Button, FInputChord(EKeys::V, EModifierKey::Control | EModifierKey::Shift | EModifierKey::Alt));

	UI_COMMAND(DuplicateBones, "Duplicate Bone(s)", "Duplicate selected bone(s).", EUserInterfaceActionType::Button, FInputChord(EKeys::D, EModifierKey::Control));

	UI_COMMAND(ImportBonesFromAssignedSkeleton, "Import Bones from Assigned Skeleton",
		"Adds bones present in the assigned Skeleton asset but missing in this mesh's skeleton, "
		"merging hierarchies with the assigned skeleton's root taking precedence.",
		EUserInterfaceActionType::Button, FInputChord());

	// DEPRECATED 5.8
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UI_COMMAND(PasteBones, "Paste Bone(s) DEPRECATED", "Paste selected bone(s) DEPRECATED, instead use PasteBonesLocal.", EUserInterfaceActionType::Button, FInputChord());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


/**
 * USkeletonEditingToolBuilder
 */

bool USkeletonEditingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

const FToolTargetTypeRequirements& USkeletonEditingToolBuilder::GetTargetRequirements() const
{
	static const FToolTargetTypeRequirements TypeRequirements({
			UPrimitiveComponentBackedTarget::StaticClass(),
			USkeletonProvider::StaticClass(),
			USkeletonCommitter::StaticClass()
			});
	return TypeRequirements;
}

UInteractiveTool* USkeletonEditingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	USkeletonEditingTool* NewTool = NewObject<USkeletonEditingTool>(SceneState.ToolManager);
	
	UToolTarget* Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTarget(Target);
	NewTool->Init(SceneState);
	
	return NewTool;
}

/**
 * USkeletonEditingTool
 */

void USkeletonEditingTool::Init(const FToolBuilderState& InSceneState)
{
	TargetWorld = InSceneState.World;

	const UContextObjectStore* ContextObjectStore = InSceneState.ToolManager->GetContextObjectStore();
	ViewContext = ContextObjectStore->FindContext<UGizmoViewContext>();
	GizmoContext = ContextObjectStore->FindContext<USkeletalMeshGizmoContextObjectBase>();
	EditorContext = ContextObjectStore->FindContext<USkeletalMeshEditorContextObjectBase>();
}

void USkeletonEditingTool::Setup()
{
	Super::Setup();

	const IPrimitiveComponentBackedTarget* PrimitiveComponentTarget = Cast<IPrimitiveComponentBackedTarget>(Target);
	ISkeletonCommitter* SkeletonCommitter = CastChecked<ISkeletonCommitter>(Target);
	UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(PrimitiveComponentTarget->GetOwnerComponent());

	if (ISkeletalMeshBackedTarget* SkeletalMeshBackedTarget = Cast<ISkeletalMeshBackedTarget>(Target))
	{
		// Used to get bone draw size, optional
		WeakMesh = SkeletalMeshBackedTarget->GetSkeletalMesh();
	}
	
	SetupModifier(SkeletonCommitter);
	SetupPreviewMesh();
	SetupProperties();
	SetupBehaviors();
	SetupGizmo(Component);
	SetupWatchers();
	SetupComponentsSelection();
	
	if (EditorContext.IsValid())
	{
		EditorContext->HideSkeleton();
		EditorContext->BindTo(this);
	}
}

void USkeletonEditingTool::SetupModifier(ISkeletonCommitter* InCommitter)
{
	Modifier = NewObject<USkeletonModifier>(this);
	InCommitter->SetupSkeletonModifier(Modifier);
}

void USkeletonEditingTool::SetupPreviewMesh()
{
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->bBuildSpatialDataStructure = true;
	PreviewMesh->CreateInWorld(TargetWorld.Get(), FTransform::Identity);

	PreviewMesh->SetTransform(UE::ToolTarget::GetLocalToWorldTransform(Target));

	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, Target);

	// Use isolated submesh if isolation was active, otherwise use the full mesh
	{
		FDynamicMesh3 FullMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);
		bool bShouldUseFullMesh = true;
		if (EditorContext.IsValid())
		{
			const TArray<int32>& IsolatedTriangles = EditorContext->GetIsolatedTriangles();
			if (!IsolatedTriangles.IsEmpty())
			{
				UE::Geometry::FDynamicSubmesh3 Submesh(&FullMesh, IsolatedTriangles);
				PreviewMesh->ReplaceMesh(Submesh.GetSubmesh());
				bShouldUseFullMesh = false;
			}
		}

		if (bShouldUseFullMesh)
		{
			PreviewMesh->ReplaceMesh(MoveTemp(FullMesh));
		}
	}

	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent()));


	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	PreviewMesh->SetMaterials(MaterialSet.Materials);
	
	// configure secondary render material for selected triangles
	UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor::Yellow, GetToolManager());
	if (SelectionMaterial != nullptr)
	{
		PreviewMesh->SetSecondaryRenderMaterial(SelectionMaterial);
	}

	// FIXME: This setting really belongs on the underlying mesh.
	UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent());
	DynamicMeshComponent->SetVertexColorSpaceTransformMode(EDynamicMeshVertexColorTransformMode::LinearToSRGB);

	// hide the skeletal mesh component
	UE::ToolTarget::HideSourceObject(Target);
}

void USkeletonEditingTool::SetupProperties()
{
	ToolPropertyObjects.Add(this);
		
	ProjectionProperties = NewObject<UProjectionProperties>();
	ProjectionProperties->Initialize(this, PreviewMesh);
	ProjectionProperties->RestoreProperties(this);
	AddToolPropertySource(ProjectionProperties);

	MirroringProperties = NewObject<UMirroringProperties>();
	MirroringProperties->Initialize(this);
	MirroringProperties->RestoreProperties(this);
	AddToolPropertySource(MirroringProperties);

	OrientingProperties = NewObject<UOrientingProperties>();
	OrientingProperties->Initialize(this);
	OrientingProperties->RestoreProperties(this);
	AddToolPropertySource(OrientingProperties);

	Properties = NewObject<USkeletonEditingProperties>();
	Properties->Initialize(this);
	Properties->Name = GetCurrentBone();
	Properties->RestoreProperties(this);
	AddToolPropertySource(Properties);
}

void USkeletonEditingTool::SetupBehaviors()
{
	UClickDragInputBehavior* ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
	ClickDragBehavior->Initialize(this);
	ClickDragBehavior->Modifiers.RegisterModifier(AddToSelectionModifier, FInputDeviceState::IsShiftKeyDown);
	ClickDragBehavior->Modifiers.RegisterModifier(ToggleSelectionModifier, FInputDeviceState::IsCtrlKeyDown);
	
	ClickDragBehavior->SetAllowsCaptureStealing(TAttribute<bool>::CreateLambda([this]
	{
		return Operation == EEditingOperation::Select;
	}));
	AddInputBehavior(ClickDragBehavior);
}

TScriptInterface<ITransformGizmoSource> USkeletonEditingTool::BuildTransformSource() 
{
	return nullptr;
}

void USkeletonEditingTool::SetupGizmo(USceneComponent* InComponent)
{
	if (!GizmoContext.IsValid() || !InComponent)
	{
		return;
	}

	// create state target
	UGizmoLambdaStateTarget* NewTarget = NewObject<UGizmoLambdaStateTarget>(this);
	NewTarget->BeginUpdateFunction = [this]()
	{
		if (GizmoWrapper && GizmoWrapper->CanInteract())
		{
			TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
			BeginChange();
		}
	};
	NewTarget->EndUpdateFunction = [this]()
	{
		if (GizmoWrapper && GizmoWrapper->CanInteract())
		{
			TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
			EndChange();

			const IToolsContextQueriesAPI* ToolsContextQueries = GetToolManager()->GetPairedGizmoManager()->GetContextQueriesAPI();
			check(ToolsContextQueries);
			if (ToolsContextQueries->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::World)
			{
				UpdateGizmo();
			}
		}
	};

	// create gizmo
	GizmoWrapper = GizmoContext->GetNewWrapper(GetToolManager(), this, NewTarget, BuildTransformSource());
	if (GizmoWrapper)
	{
		GizmoWrapper->Component = InComponent;
		GizmoWrapper->Initialize();
	}

	// bind coordinate system change
	IToolsContextQueriesAPI* ToolsContextQueries = GetToolManager()->GetPairedGizmoManager()->GetContextQueriesAPI();
	CoordinateSystemWatcher.Initialize(
		[ToolsContextQueries]()
		{
			check(ToolsContextQueries);
			return ToolsContextQueries->GetCurrentCoordinateSystem();
		},
		[this](EToolContextCoordinateSystem)
		{
			UpdateGizmo();
		},
		ToolsContextQueries->GetCurrentCoordinateSystem()
	);
}

void USkeletonEditingTool::SetupWatchers()
{
	SelectionWatcher.Initialize(
		[this]()
		{
			return this->Selection;
		},
		[this](TArray<FName> InBoneNames)
		{
			UpdateGizmo();
			if (NeedsNotification())
			{
				GetNotifier()->Notify(InBoneNames, ESkeletalMeshNotifyType::BonesSelected);
			}
		},
		this->Selection
	);
}

void USkeletonEditingTool::SetupComponentsSelection()
{
	using namespace UE::Geometry;
	
	if (!PreviewMesh)
	{
		return;
	}
	
	// set up vertex selection mechanic
	SelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);
	SelectionMechanic->Setup(this);
	SelectionMechanic->SetIsEnabled(false);
	// the property is disabled and managed thru customization
	SetToolPropertySourceEnabled(SelectionMechanic->Properties, false);
	SelectionMechanic->SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType::OnRelease);
	SelectionMechanic->Properties->bDisplayPolygroupReliantControls = false;

	SelectionMechanic->Properties->bSelectVertices = false;
	SelectionMechanic->Properties->bSelectEdges = false;
	SelectionMechanic->Properties->bSelectFaces = true;
	SelectionMechanic->SetShowSelectableCorners(true);

	// adjust selection rendering for this context
	SelectionMechanic->PolyEdgesRenderer.PointColor = FLinearColor(0.78f, 0.f, 0.78f);
	SelectionMechanic->PolyEdgesRenderer.PointSize = 5.0f;
	SelectionMechanic->PolyEdgesRenderer.DepthBias = 0.01f;
	
	SelectionMechanic->HilightRenderer.PointColor = FLinearColor::Red;
	SelectionMechanic->HilightRenderer.PointSize = 10.0f;
	
	SelectionMechanic->SelectionRenderer.LineThickness = 0.0f;
	SelectionMechanic->SelectionRenderer.DepthBias = 0.01f;
	SelectionMechanic->SelectionRenderer.PointColor = FLinearColor::Yellow;
	SelectionMechanic->SelectionRenderer.PointSize = 5.0f;
	SelectionMechanic->SetShowEdges(false);
	
	// initialize the polygon selection mechanic
	PreviewMesh->ProcessMesh([this](const FDynamicMesh3& Mesh)
	{
		Topology = MakeUnique<FTriangleGroupTopology>(&Mesh, true);
		SelectionMechanic->Initialize(&Mesh,
			FTransform::Identity,
			TargetWorld.Get(),
			Topology.Get(),
			[this]() { return PreviewMesh->GetSpatial(); });
	});

	// triangles rendering
	PreviewMesh->EnableSecondaryTriangleBuffers([this](const FDynamicMesh3* Mesh, int32 TriangleID)
	{
		return	Properties->bEnableComponentSelection &&
				SelectionMechanic->Properties->bSelectFaces &&
				SelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, Topology.Get(), TriangleID);
	});
	SelectionMechanic->OnSelectionChanged.AddWeakLambda(this, [this]()
	{
		PreviewMesh->FastNotifySecondaryTrianglesChanged();
	});
	SelectionMechanic->OnFaceSelectionPreviewChanged.AddWeakLambda(this, [this]()
	{
		PreviewMesh->FastNotifySecondaryTrianglesChanged();
	});
}

void USkeletonEditingTool::UpdateGizmo() const
{	
	if (!GizmoWrapper)
	{
		return;
	}

	GizmoWrapper->Initialize();

	const bool bUseGizmo = !Properties->bEnableComponentSelection && (Operation == EEditingOperation::Select || Operation == EEditingOperation::Transform); 
	if (Selection.IsEmpty() || !bUseGizmo)
	{
		return;
	}

	const IToolsContextQueriesAPI* ToolsContextQueries = GetToolManager()->GetPairedGizmoManager()->GetContextQueriesAPI();
	const bool bWorld = ToolsContextQueries->GetCurrentCoordinateSystem() == EToolContextCoordinateSystem::World;
	
	const TArray<int32> BoneIndexes = GetSelectedBoneIndexes();
	FTransform InitTransform = Modifier->GetTransform(BoneIndexes.Last(), true);
	if (bWorld)
	{
		InitTransform = FTransform(FQuat::Identity, InitTransform.GetTranslation(), InitTransform.GetScale3D());
	}
	GizmoWrapper->Initialize(InitTransform, ToolsContextQueries->GetCurrentCoordinateSystem());

	const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRawRefBoneInfo();
	for (const int32 BoneIndex: BoneIndexes)
	{
		if (BoneInfos.IsValidIndex(BoneIndex))
		{
			const FName BoneName = BoneInfos[BoneIndex].Name;
			const int32 ParentBoneIndex = BoneInfos[BoneIndex].ParentIndex;
			const FTransform& ParentGlobal = Modifier->GetTransform(ParentBoneIndex, true);
		
			GizmoWrapper->HandleBoneTransform(
				[this, BoneIndex, bWorld]()
				{
					return Modifier->GetTransform(BoneIndex, bWorld);
				},
				[this, BoneName, ParentGlobal, bWorld](const FTransform& InNewTransform)
				{
					if (bWorld)
					{
						const FTransform NewLocal = InNewTransform.GetRelativeTransform(ParentGlobal);
						return Modifier->SetBoneTransform(BoneName, NewLocal, Properties->bUpdateChildren);
					}
					return Modifier->SetBoneTransform(BoneName, InNewTransform, Properties->bUpdateChildren);
				});
		}
	}
}


void USkeletonEditingTool::Shutdown(EToolShutdownType ShutdownType)
{
	SkeletonTreeNotifierBindScope.Reset();
	
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("SkeletonEditingTool", "Commit Skeleton Editing"));
		ISkeletonCommitter* SkeletonCommitter = CastChecked<ISkeletonCommitter>(Target);
		SkeletonCommitter->CommitSkeletonModifier(Modifier);
		GetToolManager()->EndUndoTransaction();

		// to force to refresh the tree
		if (NeedsNotification())
		{
			GetNotifier()->Notify({}, ESkeletalMeshNotifyType::BonesAdded);
		}
	}
	
	Super::Shutdown(ShutdownType);

	if (SelectionMechanic)
	{
		SelectionMechanic->Properties->SaveProperties(this);
		SelectionMechanic->Shutdown();
		SelectionMechanic = nullptr;
	}
	
	// remove preview mesh
	if (PreviewMesh != nullptr)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
		PreviewMesh = nullptr;
	}

	// show the skeletal mesh component
	UE::ToolTarget::ShowSourceObject(Target);

	// save properties	
	Properties->SaveProperties(this);
	ProjectionProperties->SaveProperties(this);
	MirroringProperties->SaveProperties(this);
	OrientingProperties->SaveProperties(this);

	// clear gizmo
	if (GizmoWrapper)
	{
		GizmoWrapper->Clear();
	}
	
	if (EditorContext.IsValid())
	{
		EditorContext->ShowSkeleton();
		EditorContext->UnbindFrom(this);
	}
}

void USkeletonEditingTool::RegisterActions(FInteractiveToolActionSet& InOutActionSet)
{
	Super::RegisterActions(InOutActionSet);

	int32 ActionId = static_cast<int32>(EStandardToolActions::BaseClientDefinedActionID) + 400;
	auto GetActionId = [&ActionId]
	{
		return ActionId++;
	};

	RegisterCreateAction(InOutActionSet, GetActionId());
	RegisterDeleteAction(InOutActionSet, GetActionId());
	RegisterSelectAction(InOutActionSet, GetActionId());
	RegisterParentAction(InOutActionSet, GetActionId());
	RegisterUnParentAction(InOutActionSet, GetActionId());
	RegisterCopyAction(InOutActionSet, GetActionId());
	RegisterPasteBonesLocalAction(InOutActionSet, GetActionId());
	RegisterPasteBonesGlobalAction(InOutActionSet, GetActionId());
	RegisterDuplicateAction(InOutActionSet, GetActionId());
	RegisterCopyBoneTransformsAction(InOutActionSet, GetActionId());
	RegisterPasteBoneTransformsLocalAction(InOutActionSet, GetActionId());    
	RegisterPasteBoneTransformsGlobalAction(InOutActionSet, GetActionId());   
	RegisterSelectComponentsAction(InOutActionSet, GetActionId());
	RegisterSelectionFilterCyclingAction(InOutActionSet, GetActionId());
	RegisterSnapAction(InOutActionSet, GetActionId());
}

namespace SkeletonEditingTool::Private
{
	namespace Commands
	{
		constexpr const TCHAR* CreateNewBone               = TEXT("CreateNewBone");
		constexpr const TCHAR* DeleteSelectedBones         = TEXT("DeleteSelectedBones");
		constexpr const TCHAR* SelectBones                 = TEXT("SelectBones");
		constexpr const TCHAR* ParentBones                 = TEXT("ParentBones");
		constexpr const TCHAR* UnparentBones               = TEXT("UnparentBones");
		constexpr const TCHAR* CopyBones                   = TEXT("CopyBones");
		constexpr const TCHAR* PasteBonesLocal             = TEXT("PasteBonesLocal");
		constexpr const TCHAR* PasteBonesGlobal            = TEXT("PasteBonesGlobal");
		constexpr const TCHAR* DuplicateBones              = TEXT("DuplicateBones");
		constexpr const TCHAR* CopyBoneTransforms          = TEXT("CopyBoneTransforms");
		constexpr const TCHAR* PasteBoneTransformsLocal    = TEXT("PasteBoneTransformsLocal");
		constexpr const TCHAR* PasteBoneTransformsGlobal   = TEXT("PasteBoneTransformsGlobal");
		constexpr const TCHAR* SelectComponents            = TEXT("SelectComponents");
		constexpr const TCHAR* ComponentCycling            = TEXT("ComponentCycling");
		constexpr const TCHAR* SnapBone                    = TEXT("SnapBone");
	}

	constexpr const TCHAR* const HotkeyHintCommands[] = {
		Commands::CreateNewBone, Commands::DeleteSelectedBones, Commands::SelectBones, Commands::UnparentBones,
		Commands::CopyBones, Commands::PasteBonesLocal, Commands::PasteBonesGlobal, Commands::DuplicateBones,
	};
}

void USkeletonEditingTool::RegisterCreateAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::CreateNewBone,
		LOCTEXT("CreateNewBone", "Create New Bone"),
		LOCTEXT("CreateNewBoneDesc", "Create New Bone"),
		EModifierKey::None, EKeys::N,
		[this]()
		{
			Operation = EEditingOperation::Create;
			UpdateGizmo();
			GetToolManager()->DisplayMessage(LOCTEXT("Create", "Click & Drag to place a new bone."), EToolMessageLevel::UserNotification);
		});
}

void USkeletonEditingTool::RegisterDeleteAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::DeleteSelectedBones,
		LOCTEXT("DeleteSelectedBones", "Delete Selected Bone(s)"),
		LOCTEXT("DeleteSelectedBonesDesc", "Delete Selected Bone(s)"),
		EModifierKey::None, EKeys::Delete,
		[this]()
		{
			RemoveBones();
		});
}

void USkeletonEditingTool::RegisterSelectAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::SelectBones,
		LOCTEXT("SelectBone", "Select Bone"),
		LOCTEXT("SelectDesc", "Select Bone"),
		EModifierKey::None, EKeys::Escape,
		[this]()
		{
			if (Operation != EEditingOperation::Select)
			{
				Operation = EEditingOperation::Select;
				UpdateGizmo();
				GetToolManager()->DisplayMessage(LOCTEXT("Select", "Click on a bone to select it."), EToolMessageLevel::UserNotification);
			}
		});
}

void USkeletonEditingTool::RegisterParentAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::ParentBones,
		LOCTEXT("ParentBones", "Parent Bones"),
		LOCTEXT("ParentBonesDesc", "Parent Bones"),
		EModifierKey::None, EKeys::B, // FIXME find another shortcut
		[this]()
		{
			Operation = EEditingOperation::Parent;
			UpdateGizmo();
			GetToolManager()->DisplayMessage(LOCTEXT("Parent", "Click on a bone to be set as the new parent."), EToolMessageLevel::UserNotification);
		});
}

void USkeletonEditingTool::RegisterUnParentAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::UnparentBones,
		LOCTEXT("UnparentBones", "Unparent Bones"),
		LOCTEXT("UnparentBonesDesc", "Unparent Bones"),
		EModifierKey::Shift, EKeys::P,
		[this]()
		{
			UnParentBones();
			UpdateGizmo();
		});
}

void USkeletonEditingTool::RegisterCopyAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::CopyBones,
		LOCTEXT("CopyBones", "Copy Bone(s)"),
		LOCTEXT("CopyBonesDesc", "Copy Bone(s)"),
		EModifierKey::Control, EKeys::C,
		[this]()
		{
			if (!Modifier)
			{
				return;
			}
			
			SkeletonClipboard::CopyBonesToClipboard(*Modifier.Get(), Selection);
		});
}

void USkeletonEditingTool::RegisterPasteBonesLocalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::PasteBonesLocal,
		LOCTEXT("PasteBonesLocal", "Paste Bone(s) Local"),
		LOCTEXT("PasteBonesDesc", "Paste selected bone(s) using the local transform of the copied bones."),
		EModifierKey::Control, EKeys::V,
		[this]()
		{
			constexpr bool bUsingGlobalTransforms = false;
			PerformPasteBonesAction(bUsingGlobalTransforms);
		});
}

void USkeletonEditingTool::RegisterPasteBonesGlobalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::PasteBonesGlobal,
		LOCTEXT("PasteBonesGlobal", "Paste Bone(s) Global"),
		LOCTEXT("PasteBonesGlobalDesc", "Paste selected bone(s) using the global transform of the copied bones"),
		EModifierKey::Control | EModifierKey::Shift, EKeys::V, 
		[this]()
		{
			constexpr bool bUsingGlobalTransforms = true;
			PerformPasteBonesAction(bUsingGlobalTransforms);
		});
}

void USkeletonEditingTool::RegisterDuplicateAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::DuplicateBones,
		LOCTEXT("DuplicateBones", "Duplicate Bone(s)"),
		LOCTEXT("DuplicateBonesDesc", "Duplicate Bone(s)"),
		EModifierKey::Control, EKeys::D,
		[this]()
		{
			if (Selection.IsEmpty())
			{
				return;
			}

			TGuardValue OperationGuard(Operation, EEditingOperation::Create);
			BeginChange();
				
			const TArray<FName> NewBones = SkeletonClipboard::ClipboardDuplicateBones(*Modifier.Get(), Selection);
			if (!NewBones.IsEmpty())
			{
				Selection = NewBones;

				if (NeedsNotification())
				{
					GetNotifier()->Notify(NewBones, ESkeletalMeshNotifyType::HierarchyChanged);
					GetNotifier()->Notify(NewBones, ESkeletalMeshNotifyType::BonesSelected);
				}

				EndChange();
				return;
			}
			
			CancelChange();
		});
}

void USkeletonEditingTool::RegisterCopyBoneTransformsAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::CopyBoneTransforms,
		LOCTEXT("CopyBoneTransforms", "Copy Bone Transforms(s)"),
		LOCTEXT("CopyBonesTransformsDesc", "Copy Bone Transforms(s)"),
		EModifierKey::Control | EModifierKey::Shift, EKeys::C,
		[this]()
		{
			if (!Modifier)
			{
				return;
			}
			
			SkeletonClipboard::CopyBoneTransformsToClipboard(*Modifier.Get(), Selection);
		});
}

void USkeletonEditingTool::RegisterPasteBoneTransformsLocalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::PasteBoneTransformsLocal,
		LOCTEXT("PasteBoneTransformsLocal", "Paste Bone Transform(s) Local"),
		LOCTEXT("PasteBoneTransformsLocalDesc", "Paste local bone transforms(s) to the selected bones"),
		EModifierKey::Control | EModifierKey::Alt, EKeys::V, 
		[this]()
		{
			constexpr bool bUsingGlobalTransforms = false;
			PerformPasteBoneTransformsAction(bUsingGlobalTransforms);
		});	
}

void USkeletonEditingTool::RegisterPasteBoneTransformsGlobalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::PasteBoneTransformsGlobal,
	LOCTEXT("PasteBoneTransformsGlobal", "Paste Bone Transform(s) Global"),
	LOCTEXT("PasteBoneTransformsGlobalDesc", "Paste global bone transfroms(s) to the selected bones"),
	EModifierKey::Control | EModifierKey::Shift | EModifierKey::Alt, EKeys::V, 
	[this]()
	{
		constexpr bool bUsingGlobalTransforms = true;
		PerformPasteBoneTransformsAction(bUsingGlobalTransforms);
	});
}

void USkeletonEditingTool::RegisterSelectComponentsAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId,
		Commands::SelectComponents,
		LOCTEXT("SelectComponents", "Select Components"),
		LOCTEXT("SelectComponentsDesc", "Select Components"),
		EModifierKey::None, EKeys::T,
		[this]()
		{
			Properties->bEnableComponentSelection = !Properties->bEnableComponentSelection;
			PreviewMesh->FastNotifySecondaryTrianglesChanged();
			UpdateGizmo();
		});
}

void USkeletonEditingTool::RegisterSelectionFilterCyclingAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	auto UpdateSelect = [this](bool bSelectVertices, bool bSelectEdges, bool bSelectFaces)
	{
		const bool bChangingFace = SelectionMechanic->Properties->bSelectFaces != bSelectFaces;

		SelectionMechanic->Properties->bSelectVertices = bSelectVertices;
		SelectionMechanic->Properties->bSelectEdges = bSelectEdges;
		SelectionMechanic->Properties->bSelectFaces = bSelectFaces;
		SelectionMechanic->SetShowSelectableCorners(bSelectVertices);

		if (bChangingFace)
		{
			PreviewMesh->FastNotifySecondaryTrianglesChanged();
		}
	};
	
	InOutActionSet.RegisterAction(this, InActionId,
		Commands::ComponentCycling,
		LOCTEXT("ComponentCycling", "Cycle Selection Filter"),
		LOCTEXT("ComponentCyclingDesc", "Cycle between vertex, edge, face selection"),
		EModifierKey::None, EKeys::Y,
		[this, UpdateSelect]()
		{
			if (SelectionMechanic->Properties->bSelectVertices)
			{
				return UpdateSelect(false, true, false);
			}
			
			if (SelectionMechanic->Properties->bSelectEdges)
			{
				return UpdateSelect(false, false, true);
			}

			if (SelectionMechanic->Properties->bSelectFaces)
			{
				return UpdateSelect(true, false, false);
			}
			
			UpdateSelect(true, false, false);
		});
}

void USkeletonEditingTool::RegisterSnapAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId)
{
	using namespace SkeletonEditingTool::Private;
	InOutActionSet.RegisterAction(this, InActionId, Commands::SnapBone,
		LOCTEXT("SnapBone", "Snap Bone"),
		LOCTEXT("SnapBoneBonesDesc", "Snap Bone"),
		EModifierKey::None, EKeys::V,
		[this]()
		{
			if (Properties->bEnableComponentSelection)
			{
				SnapBoneToComponentSelection(Operation == EEditingOperation::Create);
				UpdateGizmo();
			}
		});
}

void USkeletonEditingTool::PerformPasteBonesAction(const bool bUsingGlobalTransforms)
{
	if (!Modifier ||
		!SkeletonClipboard::IsBoneClipboardValid())
	{
		return;
	}
	
	const TGuardValue OperationGuard(Operation, EEditingOperation::Create);
	BeginChange();
	
	const FName DefaultParent = Selection.IsEmpty() ? NAME_None : Selection[0];
	const TArray<FName> NewBones = SkeletonClipboard::PasteBonesFromClipboard(*Modifier.Get(), DefaultParent, bUsingGlobalTransforms);
	if (!NewBones.IsEmpty())
	{
		Selection = NewBones;

		if (NeedsNotification())
		{
			GetNotifier()->Notify(NewBones, ESkeletalMeshNotifyType::HierarchyChanged);
			GetNotifier()->Notify(NewBones, ESkeletalMeshNotifyType::BonesSelected);
		}

		EndChange();
	}
			
	CancelChange();
}

void USkeletonEditingTool::PerformPasteBoneTransformsAction(const bool bUsingGlobalTransforms)
{
	if (!Modifier ||
		!SkeletonClipboard::IsBoneTransformClipboardValid())
	{
		return;
	}
	
	const SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags PasteFlags = 
	[bUsingGlobalTransforms, this]()
		{
			SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags Flags = SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags::None;
			if (bUsingGlobalTransforms)
			{	
				EnumAddFlags(Flags, SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags::UsingGlobalTransforms);
			}
			if (Properties && Properties->bUpdateChildren)
			{
				EnumAddFlags(Flags, SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags::UpdateChildren);
			}
			return Flags;
		}();
	
	const TGuardValue OperationGuard(Operation, EEditingOperation::Create);
	BeginChange();
	
	const TArray<FName> AffectedBones = SkeletonClipboard::PasteBoneTransformsFromClipboard(*Modifier.Get(), Selection, PasteFlags);
	if (!AffectedBones.IsEmpty())
	{
		if (NeedsNotification())
		{
			GetNotifier()->Notify(AffectedBones, ESkeletalMeshNotifyType::HierarchyChanged);
			GetNotifier()->Notify(AffectedBones, ESkeletalMeshNotifyType::BonesMoved);
		}

		EndChange();
	}
			
	CancelChange();
}

TArray<int32> USkeletonEditingTool::GetSelectedComponents() const
{
	TArray<int32> IDs;

	PreviewMesh->ProcessMesh([this, &IDs](const FDynamicMesh3& Mesh)
	{
		const FGroupTopologySelection& TopologySelection = SelectionMechanic->GetActiveSelection();
		if (SelectionMechanic->Properties->bSelectVertices)
		{
			Algo::CopyIf(TopologySelection.SelectedCornerIDs, IDs, [&Mesh](int ID)
			{
				return Mesh.IsVertex(ID);	
			});
		}
		else if (SelectionMechanic->Properties->bSelectEdges)
		{
			Algo::CopyIf(TopologySelection.SelectedEdgeIDs, IDs, [&Mesh](int ID)
			{
				return Mesh.IsEdge(ID);	
			});
		}
		else if (SelectionMechanic->Properties->bSelectFaces)
		{
			Algo::CopyIf(TopologySelection.SelectedGroupIDs, IDs, [&Mesh](int ID)
			{
				return Mesh.IsTriangle(ID);	
			});
		}
	});
	
	return MoveTemp(IDs);
}

void USkeletonEditingTool::CreateNewBone()
{
	if (Operation != EEditingOperation::Create)
	{
		return;
	}

	BeginChange();

	static const FName DefaultName("joint");
	const FName BoneName = Modifier->GetUniqueName(DefaultName);
	const FName ParentName = Selection.IsEmpty() ? NAME_None : Selection.Last(); 
	const bool bBoneAdded = Modifier->AddBone(BoneName, ParentName, Properties->Transform);
	if (bBoneAdded)
	{
		if (NeedsNotification())
		{
			GetNotifier()->Notify({BoneName}, ESkeletalMeshNotifyType::BonesAdded);
		}
	
		Selection = {BoneName};
		Properties->Name = BoneName;

		return;
	}

	CancelChange();
}

void USkeletonEditingTool::MirrorBones()
{
	TGuardValue OperationGuard(Operation, EEditingOperation::Mirror);
	BeginChange();

	const bool bBonesMirrored = Modifier->MirrorBones(GetSelection(), MirroringProperties->Options);
	if (bBonesMirrored)
	{
		if (NeedsNotification())
		{
			GetNotifier()->Notify({}, ESkeletalMeshNotifyType::HierarchyChanged);
		}
	
		EndChange();
		return;		
	}
	
	CancelChange();
}

void USkeletonEditingTool::RemoveBones()
{
	const TArray<FName> BonesToRemove = GetSelection();
	
	TGuardValue OperationGuard(Operation, EEditingOperation::Remove);
	BeginChange();

	const bool bBonesRemoved = Modifier->RemoveBones(BonesToRemove, true);
	if (bBonesRemoved)
	{
		Selection.RemoveAll([&](const FName& BoneName)
		{
			return BonesToRemove.Contains(BoneName);
		});
		
		if (NeedsNotification())
		{
			GetNotifier()->Notify(BonesToRemove, ESkeletalMeshNotifyType::BonesRemoved);
		}
		
		EndChange();
		return;
	}

	CancelChange();
}

void USkeletonEditingTool::UnParentBones()
{
	static const TArray<FName> Dummy;

	TGuardValue OperationGuard(Operation, EEditingOperation::Parent);
	BeginChange();

	const bool bBonesUnParented = Modifier->ParentBones(GetSelection(), Dummy);
	if (bBonesUnParented)
	{
		GetToolManager()->DisplayMessage(LOCTEXT("Unparent", "Selected bones have been unparented."), EToolMessageLevel::UserNotification);

		if (NeedsNotification())
		{
			GetNotifier()->Notify({}, ESkeletalMeshNotifyType::HierarchyChanged);
		}
		
		EndChange();
		return;
	}
	
	CancelChange();
}

void USkeletonEditingTool::SnapBoneToComponentSelection(const bool bCreate)
{
	const TArray<int32> IDs = GetSelectedComponents();
	if (IDs.IsEmpty())
	{
		return;
	}

	const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
	const TArray<FName> Bones = GetSelection();

	if (!bCreate)
	{
		const bool bHasValidBone = Bones.ContainsByPredicate([&](const FName& InBoneName)
	   {
		   return RefSkeleton.FindRawBoneIndex(InBoneName) > INDEX_NONE;
	   });

		if (!bHasValidBone)
		{
			return;
		}
	}

	const int32 BoneIndex = RefSkeleton.FindRawBoneIndex(GetCurrentBone());
	const int32 ParentBoneIndex = bCreate ? BoneIndex : RefSkeleton.GetRawParentIndex(BoneIndex);
	
	FTransform NewTransform = ComputeTransformFromComponents(IDs);
	const FTransform& ParentGlobal = Modifier->GetTransform(ParentBoneIndex, true);
	NewTransform = NewTransform.GetRelativeTransform(ParentGlobal);

	bool bSuccess = false;
	if (bCreate)
	{
		TGuardValue OperationGuard(Operation, EEditingOperation::Create);
		BeginChange();

		static const FName DefaultName("joint");
		const FName BoneName = Modifier->GetUniqueName(DefaultName);
		const FName ParentName = Selection.IsEmpty() ? NAME_None : Selection.Last(); 
		bSuccess = Modifier->AddBone(BoneName, ParentName, NewTransform);
		if (bSuccess)
		{
			if (NeedsNotification())
			{
				GetNotifier()->Notify({BoneName}, ESkeletalMeshNotifyType::BonesAdded);
			}
	
			Selection = {BoneName};
			Properties->Name = BoneName;
			EndChange();
		}
	}
	else
	{
		TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
		BeginChange();

		bSuccess = Modifier->SetBoneTransform(GetCurrentBone(), NewTransform, Properties->bUpdateChildren);
		if (bSuccess)
		{
			EndChange();
		}
	}

	if (bSuccess)
	{
		Properties->Transform = NewTransform;
		SelectionMechanic->ClearSelection();
		UpdateGizmo();
	}
	else
	{
		CancelChange();	
	}
}

void USkeletonEditingTool::ParentBones(const FName& InParentName)
{
	if (Operation != EEditingOperation::Parent)
	{
		return;
	}

	BeginChange();
	const bool bBonesParented = Modifier->ParentBones(GetSelection(), {InParentName});
	if (bBonesParented)
	{
		if (NeedsNotification())
		{
			GetNotifier()->Notify({}, ESkeletalMeshNotifyType::HierarchyChanged);
		}
		
		EndChange();
	}
	else
	{
		CancelChange();
	}
	Operation = EEditingOperation::Select;
}

TOptional<FName> USkeletonEditingTool::GetBoneName(HHitProxy* InHitProxy)
{
	if (const HBoneHitProxy* BoneProxy = HitProxyCast<HBoneHitProxy>(InHitProxy))
	{
		return BoneProxy->BoneName;
	}
	return {};
}

void USkeletonEditingTool::MoveBones()
{
	const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();

	const TArray<FName> Bones = GetSelection();
	const bool bHasValidBone = Bones.ContainsByPredicate([&](const FName& InBoneName)
	{
		return RefSkeleton.FindRawBoneIndex(InBoneName) > INDEX_NONE;
	});

	if (!bHasValidBone)
	{
		return;
	}

	TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
	BeginChange();

	const bool bBonesMoved = Modifier->SetBoneTransform(Bones[0], Properties->Transform, Properties->bUpdateChildren);
	if (bBonesMoved)
	{
		// if (NeedsNotification())
		// {
		// 	GetNotifier()->Notify(Bones, ESkeletalMeshNotifyType::BonesMoved);
		// }
		
		EndChange();
		return;
	}

	CancelChange();
}

void USkeletonEditingTool::RenameBones()
{
	const TArray<int32> BoneIndices = GetSelectedBoneIndexes();
	if (BoneIndices.IsEmpty() || Properties->Name == NAME_None)
	{
		return;
	}
	
	const FName CurrentBone = GetCurrentBone();
	if (BoneIndices.Num() == 1 && CurrentBone == Properties->Name)
	{
		return;
	}

	TGuardValue OperationGuard(Operation, EEditingOperation::Rename);
	BeginChange();

	TArray<FName> ReversedSelection = Selection;
	Algo::Reverse(ReversedSelection);

	TArray<FName> NewNames;
	NewNames.Init(Properties->Name, Selection.Num());

	const bool bBoneRenamed = Modifier->RenameBones(ReversedSelection, NewNames);
	if (bBoneRenamed)
	{
		// update selection with new names
		const TArray<FMeshBoneInfo>& BoneInfos = Modifier->GetReferenceSkeleton().GetRawRefBoneInfo();
		for (int32 Index = 0; Index < BoneIndices.Num(); Index++)
		{
			if (BoneIndices[Index] != INDEX_NONE)
			{
				Selection[Index] = BoneInfos[BoneIndices[Index]].Name;
			}
		}

		const FName NewName = GetCurrentBone();
		Properties->Name = NewName;
		
		if (NeedsNotification())
		{
			GetNotifier()->Notify( {NewName}, ESkeletalMeshNotifyType::BonesRenamed);
		}
		
		EndChange();
		return;
	}

	CancelChange();
}

namespace SkeletonEditingTool
{
	using namespace UE::Geometry;
	
	// this is a helper class to compute a transform from a Components selection.
	// NOTE: the code bellow assumes that the ids are safe to use with Mesh
	// so make sure they are tested before calling any function in here.
	
	struct FTransformFromMesh
	{
		FTransformFromMesh() = delete;
		FTransformFromMesh(const FDynamicMesh3& InMesh)
			: Mesh(InMesh)
		{}
		
		FTransform GetVertexTransform(const int32 InId) const
		{
			static constexpr bool bFrameNormalY = false;
			FVector3d Normal = FMeshNormals::ComputeVertexNormal(Mesh, InId);
			const FFrame3d VertexFrame = Mesh.GetVertexFrame(InId, bFrameNormalY, &Normal);
			return VertexFrame.ToFTransform();
		}

		FTransform GetEdgeTransform(const int32 InId) const
		{
			const FDynamicMesh3::FEdge& EdgeRef = Mesh.GetEdgeRef(InId);
			
			const FVector3d& V0 = Mesh.GetVertexRef(EdgeRef.Vert[0]);
			const FVector3d& V1 = Mesh.GetVertexRef(EdgeRef.Vert[1]);
			const FVector3d Centroid = (V0 + V1) * 0.5;
			
			const FVector3d Normal = Mesh.GetEdgeNormal(InId);
			const FVector3d X = (V1 - V0).GetSafeNormal();
			const FVector3d Y = Normal.Cross(X);
			const FQuat Rotation = FRotationMatrix::MakeFromXY(X, Y).ToQuat();

			return FTransform(Rotation, Centroid);
		}

		FTransform GetTriangleTransform(const int32 InId) const
		{
			return Mesh.GetTriFrame(InId, 0).ToFTransform();	
		}
		
		FTransform GetTwoVerticesTransform(const int32 InId0, const int32 InId1) const
		{
			// check if this is an edge
			auto GetEdgeId = [this, InId0, InId1]() -> int32
			{	
				for (const int32 EdgeId : Mesh.EdgeIndicesItr())
				{
					const FDynamicMesh3::FEdge& EdgeRef = Mesh.GetEdgeRef(EdgeId);
					if (EdgeRef.Vert.Contains(InId0) && EdgeRef.Vert.Contains(InId1))
					{
						return EdgeId;
					}
				}
				return INDEX_NONE;
			};

			const int32 EdgeId = GetEdgeId();
			if (EdgeId != INDEX_NONE)
			{
				return GetEdgeTransform(EdgeId);
			}

			const FVector3d& V0 = Mesh.GetVertexRef(InId0);
			const FVector3d& V1 = Mesh.GetVertexRef(InId1);
			const FVector3d Centroid = (V0 + V1) * 0.5;
			
			const FVector3d N0 = FMeshNormals::ComputeVertexNormal(Mesh, InId0);
			const FVector3d N1 = FMeshNormals::ComputeVertexNormal(Mesh, InId1);
			const FVector3d Normal = ((N0 + N1) * 0.5).GetSafeNormal();
			
			const FVector3d X = (V1 - V0).GetSafeNormal();
			const FVector3d Y = Normal.Cross(X);
			const FQuat Rotation = FRotationMatrix::MakeFromXY(X, Y).ToQuat();

			return FTransform(Rotation, Centroid);;
		}
		
		FTransform GetThreeVerticesTransform(const int32 InId0, const int32 InId1, const int32 InId2) const
		{
			// check if this is a triangle
			auto GetTriangleId = [this, InId0, InId1, InId2]() -> int32
			{
				for (const int32 TriangleID : Mesh.TriangleIndicesItr())
				{
					const FIndex3i& TriRef = Mesh.GetTriangleRef(TriangleID);
					if (TriRef.Contains(InId0) && TriRef.Contains(InId1) && TriRef.Contains(InId2))
					{
						return TriangleID;
					}
				}
				return INDEX_NONE;
			};

			const int32 TriangleID = GetTriangleId();
			if (TriangleID != INDEX_NONE)
			{
				return GetTriangleTransform(TriangleID);
			}

			const FVector3d& V0 = Mesh.GetVertexRef(InId0);
			const FVector3d& V1 = Mesh.GetVertexRef(InId1);
			const FVector3d& V2 = Mesh.GetVertexRef(InId2);
			const FVector3d Centroid = (V0 + V1 + V2) / 3.0;

			const FVector3d Normal = VectorUtil::Normal(V0, V1, V2);
			const FVector3d X = (V1 - V0).GetSafeNormal();
			const FVector3d Y = Normal.Cross(X);
			const FQuat Rotation = FRotationMatrix::MakeFromXY(X, Y).ToQuat();

			return FTransform(Rotation, Centroid);
		}

		FTransform GetVerticesTransform(const TArray<int32>& InVertexIDs) const
		{
			const int32 NumVertices = InVertexIDs.Num();
			switch(NumVertices)
			{
			case 0:
				return FTransform::Identity;
			case 1:
				return GetVertexTransform(InVertexIDs[0]);
			case 2:
				return GetTwoVerticesTransform(InVertexIDs[0], InVertexIDs[1]);
			case 3:
				return GetThreeVerticesTransform(InVertexIDs[0], InVertexIDs[1], InVertexIDs[2]);
			default:
				break;
			}

			// compute centroid and normal
			FVector Centroid(0.0), Normal(0.0);
			for (const int32 VertexID: InVertexIDs)
			{
				Centroid += Mesh.GetVertexRef(VertexID);
				Normal += FMeshNormals::ComputeVertexNormal(Mesh, VertexID);
			}
			
			const double InvNumVertices = 1.0 / static_cast<double>(NumVertices);
			Centroid *= InvNumVertices;
			Normal *= InvNumVertices;
			
			const FVector3d& V0 = Mesh.GetVertexRef(InVertexIDs[0]);
			FVector3d X = (Centroid - V0).GetSafeNormal();
			if (X.IsNearlyZero())
			{
				for (int32 VertexID = 0; VertexID < NumVertices && X.IsNearlyZero(); VertexID++)
				{
					X = (Centroid - Mesh.GetVertexRef(InVertexIDs[VertexID])).GetSafeNormal();
				}	
			}

			FQuat Rotation = FQuat::Identity;
			const bool bParallel = FMath::IsNearlyEqual(FMath::Abs(FVector::DotProduct(Normal, X)), 1.0);
			if (!bParallel)
			{
				const FVector3d Y = Normal.Cross(X);
				Rotation = FRotationMatrix::MakeFromXY(X, Y).ToQuat();
			}
			
			return FTransform(Rotation, Centroid);
		}
		
		FTransform GetEdgesTransform(const TArray<int32>& InEdgeIDs) const
		{
			switch (InEdgeIDs.Num())
			{
			case 0:
				return FTransform::Identity;
			case 1:
				return GetEdgeTransform(InEdgeIDs[0]);
			default:
				break;
			}

			FVector Centroid(0.0), Normal(0.0);
			for (const int32 EdgeID: InEdgeIDs)
			{
				const FDynamicMesh3::FEdge& EdgeRef = Mesh.GetEdgeRef(EdgeID);
				const FVector3d& V0 = Mesh.GetVertexRef(EdgeRef.Vert[0]);
				const FVector3d& V1 = Mesh.GetVertexRef(EdgeRef.Vert[1]);
				Centroid += ((V0 + V1) * 0.5);
				Normal += Mesh.GetEdgeNormal(EdgeID);
			}
			
			Centroid /= static_cast<double>(InEdgeIDs.Num());
			Normalize(Normal);

			const FDynamicMesh3::FEdge& EdgeRef = Mesh.GetEdgeRef(InEdgeIDs[0]);
			const FVector3d& V0 = Mesh.GetVertexRef(EdgeRef.Vert[0]);
			const FVector3d& V1 = Mesh.GetVertexRef(EdgeRef.Vert[1]);
			const FVector3d X = (((V0 + V1) * 0.5) - Centroid).GetSafeNormal();
			const FVector3d Y = Normal.Cross(X);
			const FQuat Rotation = FRotationMatrix::MakeFromXY(X, Y).ToQuat();

			return FTransform(Rotation, Centroid);
		}

		FTransform GetFacesTransform(const TArray<int32>& InTriIDs) const
		{
			switch (InTriIDs.Num())
			{
			case 0:
				return FTransform::Identity;
			case 1:
				return GetTriangleTransform(InTriIDs[0]);
			default:
				break;
			}

			FVector Centroid(0.0), Normal(0.0);
			for (const int32 TriID: InTriIDs)
			{
				Centroid += Mesh.GetTriCentroid(TriID);
				Normal += Mesh.GetTriNormal(TriID);
			}
			
			Centroid /= static_cast<double>(InTriIDs.Num());
			Normalize(Normal);

			const FVector3d X = (Mesh.GetTriCentroid(InTriIDs[0]) - Centroid).GetSafeNormal();
			const FVector3d Y = Normal.Cross(X);
			const FQuat Rotation = FRotationMatrix::MakeFromXY(X, Y).ToQuat();

			return FTransform(Rotation, Centroid);
		}

	private:
		const FDynamicMesh3& Mesh;
	};
}

FTransform USkeletonEditingTool::ComputeTransformFromComponents(const TArray<int32>& InIDs) const
{
	const int32 NumComponents = InIDs.Num();
	if (NumComponents == 0)
	{
		return FTransform::Identity;
	}

	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
	if (!ensure(Mesh))
	{
		return FTransform::Identity;
	}

	const SkeletonEditingTool::FTransformFromMesh TransformFromMesh(*Mesh);
	if (SelectionMechanic->Properties->bSelectVertices)
	{
		return TransformFromMesh.GetVerticesTransform(InIDs);
	}
	
	if (SelectionMechanic->Properties->bSelectEdges)
	{
		return TransformFromMesh.GetEdgesTransform(InIDs);
	}
	
	if (SelectionMechanic->Properties->bSelectFaces)
	{
		return TransformFromMesh.GetFacesTransform(InIDs);
	}

	return FTransform::Identity;
}

void USkeletonEditingTool::OnClickPress(const FInputDeviceRay& InPressPos)
{
	if (PendingClickFunction)
	{
		if (bDeferUntilFocused)
		{
			DeferredFocusFunction = PendingClickFunction;
			PendingClickFunction.Reset();
		}
		else
		{
			PendingClickFunction();
			PendingClickFunction.Reset();
		}
	}
	else
	{ // make sure that the PendingClickFunction handles BeginChange if it needs to
		BeginChange();
	}
}

void USkeletonEditingTool::OnClickDrag(const FInputDeviceRay& InDragPos)
{
	if (bDeferUntilFocused)
	{
		return;
	}
	
	if (Operation != EEditingOperation::Create)
	{
		return;
	}
	
	FVector HitPoint;
	if (ProjectionProperties->GetProjectionPoint(InDragPos, HitPoint))
	{
		const FTransform& ParentGlobal = Modifier->GetTransform(ParentIndex, true);
		Properties->Transform.SetLocation(ParentGlobal.InverseTransformPosition(HitPoint));

		if (!ActiveChange)
		{
			TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
			BeginChange();
		}

		const bool bBoneMoved = Modifier->SetBoneTransform(GetCurrentBone(), Properties->Transform, Properties->bUpdateChildren);
		if (!bBoneMoved)
		{
			CancelChange();
			return;
		}

		const bool bOrient = Operation == EEditingOperation::Create && OrientingProperties->bAutoOrient;
		if (bOrient && ParentIndex != INDEX_NONE)
		{
			const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
			const FName ParentName = RefSkeleton.GetRawRefBoneInfo()[ParentIndex].Name;
			Modifier->OrientBone(ParentName, OrientingProperties->Options);
		}
	}
}

void USkeletonEditingTool::OrientBones()
{
	const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();

	const TArray<FName> Bones = GetSelection();
	const bool bHasValidBone = Bones.ContainsByPredicate([&](const FName& InBoneName)
	{
		return RefSkeleton.FindRawBoneIndex(InBoneName) > INDEX_NONE;
	});

	if (!bHasValidBone)
	{
		return;
	}

	TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
	BeginChange();
	
	const bool bBoneOriented = Modifier->OrientBones(Bones, OrientingProperties->Options);
	if (bBoneOriented)
	{
		UpdateGizmo();
		// if (NeedsNotification())
		// {
		// 	GetNotifier()->Notify(Bones, ESkeletalMeshNotifyType::BonesMoved);
		// }
		
		EndChange();
		return;
	}

	CancelChange();
}

void USkeletonEditingTool::OnUpdateModifierState(int InModifierID, bool bIsOn)
{
	auto UpdateFlag = [this, bIsOn](EBoneSelectionMode InFlag)
	{
		if (bIsOn)
		{
			EnumAddFlags(SelectionMode, InFlag);
		}
		else
		{
			EnumRemoveFlags(SelectionMode, InFlag);
		}		
	};
	
	if (InModifierID == AddToSelectionModifier)
	{
		UpdateFlag(EBoneSelectionMode::Additive);
	}
	else if (InModifierID == ToggleSelectionModifier)
	{
		UpdateFlag(EBoneSelectionMode::Toggle);
	}
}

bool USkeletonEditingTool::CanModifySkeleton() const
{
	return true;
}

TSharedPtr<SWidget> USkeletonEditingTool::GetCustomSkeletonTreeWidget(TSharedPtr<ISkeletalMeshEditorBinding> EditorBinding)
{
	FRefSkeletonTreeDelegates SkeletonTreeDelegates;

	SkeletonTreeDelegates.OnExtendCommandList.BindLambda(
		[this](TSharedPtr<FUICommandList> InOutCommandList)
		{
			const FSkeletonEditingToolCommands& Commands = FSkeletonEditingToolCommands::Get();

			InOutCommandList->MapAction(Commands.CopyBoneTransforms,
				FExecuteAction::CreateUObject(this, &USkeletonEditingTool::HandleCopyBoneTransforms),
				FCanExecuteAction::CreateUObject(this, &USkeletonEditingTool::CanCopyBoneTransforms));
		
			constexpr bool bPasteUsingGlobalTransforms = true;
			InOutCommandList->MapAction(Commands.PasteBoneTransformsLocal,
				FExecuteAction::CreateUObject(this, &USkeletonEditingTool::HandlePasteBoneTransforms, !bPasteUsingGlobalTransforms),
				FCanExecuteAction::CreateUObject(this, &USkeletonEditingTool::CanPasteBoneTransforms));
		
			InOutCommandList->MapAction(Commands.PasteBoneTransformsGlobal,
				FExecuteAction::CreateUObject(this, &USkeletonEditingTool::HandlePasteBoneTransforms, bPasteUsingGlobalTransforms),
				FCanExecuteAction::CreateUObject(this, &USkeletonEditingTool::CanPasteBoneTransforms));

			InOutCommandList->MapAction(Commands.ImportBonesFromAssignedSkeleton,
				FExecuteAction::CreateUObject(this, &USkeletonEditingTool::HandleImportBonesFromAssignedSkeleton),
				FCanExecuteAction::CreateUObject(this, &USkeletonEditingTool::CanImportBonesFromAssignedSkeleton));
		});
		
	SkeletonTreeDelegates.OnExtendContextMenu.BindLambda([](FMenuBuilder& MenuBuilder)
		{
			// The transform section is added here as it relies on this tool's bUpdateChildren setting
			const FSkeletonEditingToolCommands& Commands = FSkeletonEditingToolCommands::Get();
			
			MenuBuilder.BeginSection("CopyPasteTransforms", LOCTEXT("CopyPasteTransformsSection", "Copy & Paste Transforms"));
			{
				MenuBuilder.AddMenuEntry(Commands.CopyBoneTransforms);
				MenuBuilder.AddMenuEntry(Commands.PasteBoneTransformsLocal);
				MenuBuilder.AddMenuEntry(Commands.PasteBoneTransformsGlobal);
			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("ImportSkeleton", LOCTEXT("ImportSkeletonSection", "Import"));
			{
				MenuBuilder.AddMenuEntry(Commands.ImportBonesFromAssignedSkeleton);
			}
			MenuBuilder.EndSection();

			return true;
		});

	SAssignNew(SkeletonTree, SReferenceSkeletonTree)
		.Modifier(Modifier)
		.Delegates(SkeletonTreeDelegates);

	SkeletonTreeNotifierBindScope.Reset(
		new FSkeletalMeshNotifierBindScope(
			EditorBinding->GetNotifier(),
			SkeletonTree->GetNotifier()));	
		
	TArray<FName> SelectedBones = EditorBinding->GetSelectedBones();
	SkeletonTree->GetNotifier()->HandleNotification(SelectedBones, ESkeletalMeshNotifyType::BonesSelected);		
	
	return SkeletonTree;
}

EEditingOperation USkeletonEditingTool::GetOperation() const
{
	return Operation;	
}

void USkeletonEditingTool::SetOperation(const EEditingOperation InOperation, const bool bUpdateGizmo)
{
	Operation = InOperation;

	if (bUpdateGizmo)
	{
		UpdateGizmo();
	}
}

bool USkeletonEditingTool::HasSelectedComponent() const
{
	return !SelectionMechanic->GetActiveSelection().IsEmpty();
}

bool USkeletonEditingTool::IsInputIsolationValidOnOutput() const
{
	return true;
}

bool USkeletonEditingTool::IsInputSelectionValidOnOutput()
{
	return true;
}

void USkeletonEditingTool::GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const
{
	for (const TCHAR* CommandName : SkeletonEditingTool::Private::HotkeyHintCommands)
	{
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ActionCommandsContextName, CommandName);
	}
}

void USkeletonEditingTool::SelectBone(const FName& InBoneName)
{
	TArray<FName> NewSelection = Selection;

	if (EnumHasAnyFlags(SelectionMode, EBoneSelectionMode::Additive))
	{
		if (InBoneName != NAME_None)
		{
			NewSelection.AddUnique(InBoneName);
		}		
	}
	else if (EnumHasAnyFlags(SelectionMode, EBoneSelectionMode::Toggle))
	{
		const bool bSelected = NewSelection.Contains(InBoneName);
		if (bSelected)
		{
			NewSelection.Remove(InBoneName);
		}
		else
		{
			NewSelection.Add(InBoneName);
		}
	}
	else
	{
		NewSelection.Empty();
		if (InBoneName != NAME_None)
		{
			NewSelection.Add(InBoneName);
		}
	}

	Selection = MoveTemp(NewSelection);
	NormalizeSelection();
}

void USkeletonEditingTool::NormalizeSelection()
{
	if (Selection.Num() < 2)
	{
		return;
	}
	
	const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();

	// ensure they are known and unique
	TArray<int32> Indexes;
	for (const FName& BoneName: Selection)
	{
		const int32 BoneIndex = ReferenceSkeleton.FindRawBoneIndex(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			Indexes.AddUnique(BoneIndex);
		}
	}

	if (Indexes.IsEmpty())
	{
		Selection.Reset();
		return;
	}

	// sort them so that parents are placed after one of their children 
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton.GetRawRefBoneInfo();
	Indexes.StableSort([&](const int32 Index0, const int32 Index1)
	{
		int32 P0 = BoneInfos[Index0].ParentIndex;
		while (P0 != INDEX_NONE)
		{
			if (P0 == Index1)
			{ // Index1 is a parent
				return true;
			}
			if (Indexes.Contains(P0))
			{ // parent is selected
				return true;
			}
			P0 = BoneInfos[P0].ParentIndex;
		}
		return false;
	});

	// transform back to names
	Selection.Reset();
	Algo::Transform(Indexes, Selection, [&](const int32 Index)
	{
		return BoneInfos[Index].Name;
	});	
}

void USkeletonEditingTool::OnClickRelease(const FInputDeviceRay& InReleasePos)
{
	if (PendingReleaseFunction)
	{
		PendingReleaseFunction();
		PendingReleaseFunction.Reset();
		return;
	}

	if (Operation == EEditingOperation::Create)
	{
		return EndChange();
	}
	
	TGuardValue OperationGuard(Operation, EEditingOperation::Transform);
	EndChange();
}

void USkeletonEditingTool::OnTerminateDragSequence()
{}
void USkeletonEditingTool::OnTick(float DeltaTime)
{
	const FViewport* Viewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	bDeferUntilFocused = Viewport && !Viewport->HasFocus();
	
	if (!bDeferUntilFocused && DeferredFocusFunction)
	{
		DeferredFocusFunction();
		DeferredFocusFunction.Reset();
	}
	
	SelectionWatcher.CheckAndUpdate();
	CoordinateSystemWatcher.CheckAndUpdate();

	PreviewMesh->EnableWireframe(Properties->bEnableComponentSelection);
	SelectionMechanic->SetIsEnabled(Properties->bEnableComponentSelection);
	SelectionMechanic->Tick(DeltaTime);
}

FInputRayHit USkeletonEditingTool::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	static const FInputRayHit InvalidRayHit;
	
	if (Properties->bEnableComponentSelection)
	{
		return InvalidRayHit;
	}
	
	PendingClickFunction.Reset();
	PendingReleaseFunction.Reset();
	DeferredFocusFunction.Reset();
	ParentIndex = INDEX_NONE;
	
	FViewport* Viewport = GetToolManager()->GetContextQueriesAPI()->GetFocusedViewport();
	if (!Viewport)
	{
		return InvalidRayHit;
	}

	bDeferUntilFocused = !Viewport->HasFocus();
	
	if (GizmoWrapper && GizmoWrapper->IsGizmoHit(InPressPos))
	{
		return InvalidRayHit;
	}
	
	auto PickBone = [&]() -> int32
	{
		if (HHitProxy* HitProxy = Viewport->GetHitProxy(InPressPos.ScreenPosition.X, InPressPos.ScreenPosition.Y))
		{
			if (TOptional<FName> OptBoneName = GetBoneName(HitProxy))
			{
				const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
				return ReferenceSkeleton.FindRawBoneIndex(*OptBoneName);
			}
		}
		return INDEX_NONE;
	};
	
	// pick bone in viewport
	const int32 BoneIndex = PickBone();

	// update projection properties
	const FVector GlobalPosition = Modifier->GetTransform(BoneIndex, true).GetTranslation();
	ProjectionProperties->UpdatePlane(*ViewContext, GlobalPosition);
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(ProjectionProperties->CameraState);
	
	// if we picked a new bone
	if (BoneIndex > INDEX_NONE)
	{
		// parent selection without changing the selection
		if (Operation == EEditingOperation::Parent)
		{
			const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
			ParentBones(ReferenceSkeleton.GetBoneName(BoneIndex));
			return InvalidRayHit;
		}
		
		// otherwise, update current selection
		TFunction<void()> SelectFunction = [this, BoneIndex]
		{
			const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
			SelectBone(ReferenceSkeleton.GetBoneName(BoneIndex));

			Properties->Name = GetCurrentBone();
			Properties->Transform = ReferenceSkeleton.GetRefBonePose()[BoneIndex];
			ParentIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
		}; 
		
		if (Operation == EEditingOperation::Select)
		{
			// When just doing a select operation, select occurs on mouse up.
			PendingClickFunction = []{};
			PendingReleaseFunction = SelectFunction;
		}
		else
		{
			PendingClickFunction = SelectFunction;
		}
	
		return FInputRayHit(0.0);
	}

	// if we didn't pick anything
	if (Operation == EEditingOperation::Select)
	{
		PendingClickFunction = []{ };
		PendingReleaseFunction = [this]
		{
			Selection.Empty();
			Properties->Name = GetCurrentBone();
		};
		
		return FInputRayHit(0.0);
	}

	// if we're in creation mode then create a new bone
	if (Operation == EEditingOperation::Create)
	{
		FVector HitPoint;
		if (ProjectionProperties->GetProjectionPoint(InPressPos, HitPoint))
		{
			// CurrentBone is gonna be the parent
			PendingClickFunction = [this, HitPoint]
			{
				const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
				ParentIndex = ReferenceSkeleton.FindRawBoneIndex(GetCurrentBone());
				const FTransform& ParentGlobalTransform = Modifier->GetTransform(ParentIndex, true);

				// Create the new bone under mouse
				Properties->Transform.SetLocation(ParentGlobalTransform.InverseTransformPosition(HitPoint));
				CreateNewBone();
			};
			
			return FInputRayHit(0.0);
		}
	}
	
	return InvalidRayHit;
}

void USkeletonEditingTool::HandleSkeletalMeshModified(const TArray<FName>& InBoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	if (GetNotifier()->Notifying())
	{
		return;
	}

	TArray<FName> BoneNames(InBoneNames);
	const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
	BoneNames.RemoveAll([&RefSkeleton](const FName& BoneName)
	{
		return RefSkeleton.FindRawBoneIndex(BoneName) == INDEX_NONE;
	});

	switch (InNotifyType)
	{
		case ESkeletalMeshNotifyType::BonesAdded:
			Selection = BoneNames;
			break;
		case ESkeletalMeshNotifyType::BonesRemoved:
			Selection.RemoveAll([&InBoneNames](const FName& BoneName)
			{
				return InBoneNames.Contains(BoneName);
			});
			break;
		case ESkeletalMeshNotifyType::BonesMoved:
			break;
		case ESkeletalMeshNotifyType::BonesSelected:
			Selection.Reset();
			Selection = BoneNames;
			break;
		case ESkeletalMeshNotifyType::BonesRenamed:
			Selection = BoneNames;
			break;
		case ESkeletalMeshNotifyType::HierarchyChanged:
			UpdateGizmo();
			break;
		default:
			break;
	}

	NormalizeSelection();
	
	Properties->Name = GetCurrentBone();
}

void USkeletonEditingTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	Super::DrawHUD(Canvas, RenderAPI);

	if (SelectionMechanic && Properties->bEnableComponentSelection)
	{
		SelectionMechanic->DrawHUD(Canvas, RenderAPI);
	}
}

void USkeletonEditingTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	// NOTE many things could be cached here and updated lazily
	if (!Target)
	{
		return;
	}
	
	const UPersonaOptions* PersonaOptions = GetDefault<UPersonaOptions>();
	static FSkelDebugDrawConfig DrawConfig;
	DrawConfig.BoneDrawMode = EBoneDrawMode::Type::All;
#if WITH_EDITORONLY_DATA
	DrawConfig.BoneDrawSize = WeakMesh.IsValid() ? WeakMesh->GetBoneDrawSize() : 1.f;
#else
	DrawConfig.BoneDrawSize = 1.f;
#endif
	DrawConfig.bAddHitProxy = true;
	DrawConfig.bForceDraw = false;
	DrawConfig.DefaultBoneColor = PersonaOptions->DefaultBoneColor;
	DrawConfig.AffectedBoneColor = PersonaOptions->AffectedBoneColor;
	DrawConfig.SelectedBoneColor = PersonaOptions->SelectedBoneColor;
	DrawConfig.ParentOfSelectedBoneColor = PersonaOptions->ParentOfSelectedBoneColor;
	
	if (const USkeletalMeshModelingToolsEditorSettings* const EditorSettings = GetDefault<USkeletalMeshModelingToolsEditorSettings>())
	{
		DrawConfig.AxisConfig.Length = EditorSettings->GetLocalRotationAxisLength();
		DrawConfig.AxisConfig.Thickness = EditorSettings->GetLocalRotationAxisThickness();
		DrawConfig.bDrawAllLocalRotationAxes = EditorSettings->GetShowAllLocalRotationAxes();
	}
	
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

	const IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	const FTransform ComponentTransform = TargetComponent->GetWorldTransform();
	
	const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();

	const int32 NumBones = RefSkeleton.GetRawBoneNum();
	TArray<TRefCountPtr<HHitProxy>> HitProxies; HitProxies.Reserve(NumBones);
	TArray<FBoneIndexType> RequiredBones; RequiredBones.AddUninitialized(NumBones);
	TArray<FTransform> WorldTransforms; WorldTransforms.AddUninitialized(NumBones);
	TArray<FLinearColor> BoneColors; BoneColors.AddUninitialized(NumBones);

	const bool bUseBoneColors = GetDefault<UPersonaOptions>()->bShowBoneColors;
	
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		const FTransform& BoneTransform = Modifier->GetTransform(Index, true);
		WorldTransforms[Index] = BoneTransform;
		RequiredBones[Index] = Index;
		BoneColors[Index] = bUseBoneColors ? SkeletalDebugRendering::GetSemiRandomColorForBone(Index) : DrawConfig.DefaultBoneColor;
		HitProxies.Add(new HBoneHitProxy(Index, RefSkeleton.GetBoneName(Index)));
	}

	SkeletalDebugRendering::DrawBones(
		PDI,
		ComponentTransform.GetLocation(),
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		GetSelectedBoneIndexes(),
		BoneColors,
		HitProxies,
		DrawConfig
	);

	if (SelectionMechanic && Properties->bEnableComponentSelection)
	{
		SelectionMechanic->Render(RenderAPI);
	}
}

void USkeletonEditingTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (PropertySet == SelectionMechanic->Properties && Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectVertices))
		{
			if (SelectionMechanic->Properties->bSelectVertices)
			{
				SelectionMechanic->Properties->bSelectEdges = SelectionMechanic->Properties->bSelectFaces = false;
			}
		}
		else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectEdges))
		{
			if (SelectionMechanic->Properties->bSelectEdges)
			{
				SelectionMechanic->Properties->bSelectVertices = SelectionMechanic->Properties->bSelectFaces = false;
			}
		}
		else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshTopologySelectionMechanicProperties, bSelectFaces))
		{
			if (SelectionMechanic->Properties->bSelectFaces)
			{
				SelectionMechanic->Properties->bSelectEdges = SelectionMechanic->Properties->bSelectVertices = false;
			}
		}
		SelectionMechanic->SetShowSelectableCorners(SelectionMechanic->Properties->bSelectVertices);
		PreviewMesh->FastNotifySecondaryTrianglesChanged();
	}
	Super::OnPropertyModified(PropertySet, Property);
}

FBox USkeletonEditingTool::GetWorldSpaceFocusBox()
{
	if (Properties->bEnableComponentSelection)
	{
		const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
		if (!ActiveSelection.IsEmpty())
		{
			static constexpr bool bWorld = true;
			const FAxisAlignedBox3d Bounds = SelectionMechanic->GetSelectionBounds(bWorld);
			return static_cast<FBox>(Bounds);
		}
	}
	
	const TArray<int32> BoneIndexes = GetSelectedBoneIndexes();
	if (!BoneIndexes.IsEmpty())
	{
		FBox Box(EForceInit::ForceInit);
		TSet<int32> AllChildren;

		const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
		for (const int32 BoneIndex: BoneIndexes)
		{
			Box += Modifier->GetTransform(BoneIndex, true).GetTranslation();

			// get direct children
			TArray<int32> Children;
			RefSkeleton.GetRawDirectChildBones(BoneIndex, Children);
			AllChildren.Append(Children);
		}

		for (const int32 ChildIndex: AllChildren)
		{
			Box += Modifier->GetTransform(ChildIndex, true).GetTranslation();
		}
		
		return Box;
	}

	if (PreviewMesh && PreviewMesh->GetActor())
	{
		return PreviewMesh->GetActor()->GetComponentsBoundingBox();
	}

	return USingleSelectionTool::GetWorldSpaceFocusBox();
}

const TArray<FName>& USkeletonEditingTool::GetSelection() const
{
	return Selection;
}

const FTransform& USkeletonEditingTool::GetTransform(const FName InBoneName, const bool bWorld) const
{
	if (Modifier)
	{
		const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
		return Modifier->GetTransform(RefSkeleton.FindRawBoneIndex(InBoneName), bWorld);
	}
	return FTransform::Identity;
}

void USkeletonEditingTool::SetTransforms(const TArray<FName>& InBones, const TArray<FTransform>& InTransforms, const bool bWorld) const
{
	const int32 NumBonesToMove = InBones.Num();
	if (NumBonesToMove == 0 || NumBonesToMove != InTransforms.Num())
	{
		return;
	}
	
	TArray<FTransform> NewTransforms = InTransforms;
	if (bWorld)
	{ // switch to local
		const FReferenceSkeleton& RefSkeleton = Modifier->GetReferenceSkeleton();
		const TArray<FMeshBoneInfo>& BoneInfos = RefSkeleton.GetRawRefBoneInfo();
		
		for (int32 Index = 0; Index < InBones.Num(); ++Index)
		{
			const int32 BoneIndex = RefSkeleton.FindRawBoneIndex(InBones[Index]);
			check(BoneIndex != INDEX_NONE);

			const int32 ParentBoneIndex = BoneInfos[BoneIndex].ParentIndex;
			const int32 ParentToBeSet = ParentBoneIndex != INDEX_NONE ?
				InBones.IndexOfByKey(BoneInfos[ParentBoneIndex].Name) : INDEX_NONE;

			const FTransform& ParentGlobal = ParentToBeSet != INDEX_NONE ?
				InTransforms[ParentToBeSet] : Modifier->GetTransform(ParentBoneIndex, true);
			
			NewTransforms[Index] = NewTransforms[Index].GetRelativeTransform(ParentGlobal);
		}
	}

	const bool bBonesMoved = Modifier->SetBonesTransforms(InBones, NewTransforms, Properties->bUpdateChildren);
	if (bBonesMoved)
	{
		UpdateGizmo();
	}
}

FName USkeletonEditingTool::GetCurrentBone() const
{
	return Selection.IsEmpty() ? NAME_None : Selection.Last(); 
}

void USkeletonEditingTool::HandleCopyBoneTransforms() const
{
	if (!ensure(SkeletonTree.IsValid() && Modifier))
	{
		return;
	}

	TArray<FName> SelectedBones;
	SkeletonTree->GetSelectedBoneNames(SelectedBones);

	if (!SelectedBones.IsEmpty())
	{
		SkeletonClipboard::CopyBoneTransformsToClipboard(*Modifier.Get(), SelectedBones);
	}
}

bool USkeletonEditingTool::CanCopyBoneTransforms() const
{
	if (!SkeletonTree.IsValid() ||
		!Modifier || 
		Modifier->IsReadOnly())
	{
		return false;
	}

	return SkeletonTree->HasSelectedItems();
}

void USkeletonEditingTool::HandlePasteBoneTransforms(const bool bUsingGlobalTransforms)
{
	if (!ensure(SkeletonTree.IsValid() && Modifier))
	{
		return;
	}

	TArray<FName> BoneNames;
	SkeletonTree->GetSelectedBoneNames(BoneNames);

	const SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags PasteFlags = 
		[bUsingGlobalTransforms, this]()
		{
			SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags Flags = SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags::None;
			if (bUsingGlobalTransforms)
			{	
				Flags |= SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags::UsingGlobalTransforms;
			}
			if (Properties && Properties->bUpdateChildren)
			{
				Flags |= SkeletonClipboard::EPasteBoneTransformsFromClipboardFlags::UpdateChildren;
			}

			return Flags;
		}();

	BeginChange();
	
	const TArray<FName> AffectedBones = SkeletonClipboard::PasteBoneTransformsFromClipboard(*Modifier.Get(), BoneNames, PasteFlags);
	if (AffectedBones.IsEmpty())
	{
		CancelChange();
		return;
	}
	
	if (const TSharedPtr<ISkeletalMeshNotifier> MeshNotifier = GetNotifier())
	{
		MeshNotifier->Notify(AffectedBones, ESkeletalMeshNotifyType::HierarchyChanged);
		MeshNotifier->Notify(AffectedBones, ESkeletalMeshNotifyType::BonesMoved);
	}
	UpdateGizmo();
	
	EndChange();
}

bool USkeletonEditingTool::CanPasteBoneTransforms() const
{
	if (!Modifier ||
		Modifier->IsReadOnly())
	{
		return false;
	}
	
	return SkeletonClipboard::IsBoneTransformClipboardValid();
}

bool USkeletonEditingTool::CanImportBonesFromAssignedSkeleton() const
{
	if (!Modifier ||
		Modifier->IsReadOnly())
	{
		return false;
	}

	const USkeletalMesh* Mesh = WeakMesh.Get();
	if (!Mesh)
	{
		return false;
	}

	const USkeleton* AssignedSkeleton = Mesh->GetSkeleton();
	return AssignedSkeleton != nullptr && AssignedSkeleton->GetReferenceSkeleton().GetRawBoneNum() > 0;
}

void USkeletonEditingTool::HandleImportBonesFromAssignedSkeleton()
{
	if (!ensure(Modifier && WeakMesh.IsValid()))
	{
		return;
	}

	const USkeleton* AssignedSkeleton = WeakMesh->GetSkeleton();
	if (!AssignedSkeleton)
	{
		return;
	}

	const FReferenceSkeleton& SourceReferenceSkeleton = AssignedSkeleton->GetReferenceSkeleton();
	const TArray<FMeshBoneInfo>& SourceBoneInfos = SourceReferenceSkeleton.GetRawRefBoneInfo();
	const TArray<FTransform>& SourceBonePose = SourceReferenceSkeleton.GetRawRefBonePose();
	if (SourceBoneInfos.IsEmpty())
	{
		return;
	}

	BeginChange();

	// Pass 1: root reconciliation. If the source root is missing from the mesh, insert it as a new
	// top-level bone and re-parent the existing mesh root underneath it, forcing the merged tree's
	// root to be the source skeleton's root.
	const FName SourceRootName = SourceBoneInfos[0].Name;
	{
		const FReferenceSkeleton& MeshReferenceSkeleton = Modifier->GetReferenceSkeleton();
		const FName MeshRootBefore = MeshReferenceSkeleton.GetRawBoneNum() > 0
			? MeshReferenceSkeleton.GetRawRefBoneInfo()[0].Name
			: NAME_None;
		const bool bMeshHasSourceRoot = MeshReferenceSkeleton.FindRawBoneIndex(SourceRootName) != INDEX_NONE;

		if (!bMeshHasSourceRoot)
		{
			Modifier->AddBone(SourceRootName, NAME_None, SourceBonePose[0]);
			if (MeshRootBefore != NAME_None && MeshRootBefore != SourceRootName)
			{
				Modifier->ParentBone(MeshRootBefore, SourceRootName);
			}
		}
	}

	// Pass 2: add bones missing in the mesh, in source hierarchy order (parent-before-child is an
	// invariant of FReferenceSkeleton, which is what guarantees each parent name resolves).
	TArray<FName> NamesToAdd;
	TArray<FName> ParentsToAdd;
	TArray<FTransform> LocalsToAdd;
	NamesToAdd.Reserve(SourceBoneInfos.Num());
	ParentsToAdd.Reserve(SourceBoneInfos.Num());
	LocalsToAdd.Reserve(SourceBoneInfos.Num());

	for (int32 SourceIndex = 0; SourceIndex < SourceBoneInfos.Num(); ++SourceIndex)
	{
		const FName BoneName = SourceBoneInfos[SourceIndex].Name;
		if (Modifier->GetReferenceSkeleton().FindRawBoneIndex(BoneName) != INDEX_NONE)
		{
			continue;
		}

		const int32 SourceParentIndex = SourceReferenceSkeleton.GetRawParentIndex(SourceIndex);
		const FName SourceParentName = SourceParentIndex == INDEX_NONE
			? NAME_None
			: SourceBoneInfos[SourceParentIndex].Name;

		NamesToAdd.Add(BoneName);
		ParentsToAdd.Add(SourceParentName);
		LocalsToAdd.Add(SourceBonePose[SourceIndex]);
	}
	if (!NamesToAdd.IsEmpty())
	{
		Modifier->AddBones(NamesToAdd, ParentsToAdd, LocalsToAdd);
	}

	// Pass 3: re-parent bones that exist in both skeletons but have a different parent in source.
	TArray<FName> ReparentNames;
	TArray<FName> ReparentParents;
	ReparentNames.Reserve(SourceBoneInfos.Num());
	ReparentParents.Reserve(SourceBoneInfos.Num());

	const FReferenceSkeleton& MeshReferenceSkeletonAfter = Modifier->GetReferenceSkeleton();
	for (int32 SourceIndex = 0; SourceIndex < SourceBoneInfos.Num(); ++SourceIndex)
	{
		const FName BoneName = SourceBoneInfos[SourceIndex].Name;
		const int32 MeshIndex = MeshReferenceSkeletonAfter.FindRawBoneIndex(BoneName);
		if (MeshIndex == INDEX_NONE)
		{
			continue;
		}

		const int32 SourceParentIndex = SourceReferenceSkeleton.GetRawParentIndex(SourceIndex);
		const FName SourceParentName = SourceParentIndex == INDEX_NONE
			? NAME_None
			: SourceBoneInfos[SourceParentIndex].Name;

		const int32 MeshParentIndex = MeshReferenceSkeletonAfter.GetRawParentIndex(MeshIndex);
		const FName MeshParentName = MeshParentIndex == INDEX_NONE
			? NAME_None
			: MeshReferenceSkeletonAfter.GetRawRefBoneInfo()[MeshParentIndex].Name;

		if (SourceParentName != MeshParentName)
		{
			ReparentNames.Add(BoneName);
			ReparentParents.Add(SourceParentName);
		}
	}
	if (!ReparentNames.IsEmpty())
	{
		Modifier->ParentBones(ReparentNames, ReparentParents);
	}

	if (NamesToAdd.IsEmpty() && ReparentNames.IsEmpty())
	{
		CancelChange();
		return;
	}

	if (const TSharedPtr<ISkeletalMeshNotifier> MeshNotifier = GetNotifier())
	{
		TArray<FName> AffectedBones = NamesToAdd;
		AffectedBones.Append(ReparentNames);
		MeshNotifier->Notify(AffectedBones, ESkeletalMeshNotifyType::HierarchyChanged);
	}

	EndChange();
}

TArray<int32> USkeletonEditingTool::GetSelectedBoneIndexes() const
{
	TArray<int32> Indexes;
	
	const FReferenceSkeleton& ReferenceSkeleton = Modifier->GetReferenceSkeleton();
	Algo::Transform(Selection, Indexes, [&](const FName& BoneName)
	{
		return ReferenceSkeleton.FindRawBoneIndex(BoneName);
	});
	
	return Indexes;
}

void USkeletonEditingTool::BeginChange()
{
	if (Operation == EEditingOperation::Select)
	{
		return;
	}
	
	ensure( ActiveChange == nullptr );
	ActiveChange = MakeUnique<SkeletonEditingTool::FRefSkeletonChange>(this); 
}

void USkeletonEditingTool::EndChange()
{
	if (!ActiveChange.IsValid())
	{
		return;
	}

	if (Operation == EEditingOperation::Select)
	{
		return CancelChange();
	}
	
	ActiveChange->StoreSkeleton(this);

	static const UEnum* OperationEnum = StaticEnum<EEditingOperation>();
	const FString OperationString = OperationEnum->GetNameStringByValue(static_cast<int64>(Operation));
	const FText TransactionDesc = FText::Format(LOCTEXT("RefSkeletonChanged", "Skeleton Edit - {0}"), FText::FromString(OperationString));
	
	UInteractiveToolManager* ToolManager = GetToolManager();
	ToolManager->BeginUndoTransaction(TransactionDesc);
	ToolManager->EmitObjectChange(this, MoveTemp(ActiveChange), TransactionDesc);
	ToolManager->EndUndoTransaction();
}

void USkeletonEditingTool::CancelChange()
{
	ActiveChange.Reset();
}

void USkeletonEditingProperties::Initialize(USkeletonEditingTool* ParentToolIn)
{
	ParentTool = ParentToolIn;
}

#if WITH_EDITOR
void USkeletonEditingProperties::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(USkeletonEditingProperties, Name))
		{
			ParentTool->RenameBones();
		}
	}
}
#endif

void UMirroringProperties::Initialize(USkeletonEditingTool* ParentToolIn)
{
	ParentTool = ParentToolIn;
}

void UMirroringProperties::MirrorBones()
{
	ParentTool->MirrorBones();
}

void UOrientingProperties::Initialize(USkeletonEditingTool* ParentToolIn)
{
	ParentTool = ParentToolIn;
}

void UOrientingProperties::OrientBones()
{
	ParentTool->OrientBones();
}

#if WITH_EDITOR

void UOrientingProperties::PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		auto CheckAxis = [&](const EOrientAxis& InRef, EOrientAxis& OutOther)
		{
			switch (InRef)
			{
			case EOrientAxis::PositiveX:
				if (OutOther == InRef || OutOther == EOrientAxis::NegativeX)
				{
					OutOther = EOrientAxis::PositiveY;
				}
				break;
			case EOrientAxis::PositiveY:
				if (OutOther == InRef || OutOther == EOrientAxis::NegativeY)
				{
					OutOther = EOrientAxis::PositiveZ;
				}
				break;
			case EOrientAxis::PositiveZ:
				if (OutOther == InRef || OutOther == EOrientAxis::NegativeZ)
				{
					OutOther = EOrientAxis::PositiveX;
				}
				break;
			case EOrientAxis::NegativeX:
				if (OutOther == InRef || OutOther == EOrientAxis::PositiveX)
				{
					OutOther = EOrientAxis::PositiveY;
				}
				break;
			case EOrientAxis::NegativeY:
				if (OutOther == InRef || OutOther == EOrientAxis::PositiveY)
				{
					OutOther = EOrientAxis::PositiveZ;
				}
				break;
			case EOrientAxis::NegativeZ:
				if (OutOther == InRef || OutOther == EOrientAxis::PositiveZ)
				{
					OutOther = EOrientAxis::PositiveX;
				}
				break;
			default:
				break;
			}
		};
		
		const FName PropertyName = PropertyChangedEvent.GetPropertyName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FOrientOptions, Primary))
		{
			CheckAxis(Options.Primary, Options.Secondary);
			return;
		}
		
		if (PropertyName == GET_MEMBER_NAME_CHECKED(FOrientOptions, Secondary))
		{
			CheckAxis(Options.Secondary, Options.Primary);
			return;
		}
	}
}

#endif

void UProjectionProperties::Initialize(USkeletonEditingTool* ParentToolIn, TObjectPtr<UPreviewMesh> InPreviewMesh) 
{
	ParentTool = ParentToolIn;
	PreviewMesh = InPreviewMesh;
}

void UProjectionProperties::UpdatePlane(const UGizmoViewContext& InViewContext, const FVector& InOrigin)
{
	PlaneNormal = -InViewContext.GetViewDirection();
	PlaneOrigin = InOrigin;
}

bool UProjectionProperties::GetProjectionPoint(const FInputDeviceRay& InRay, FVector& OutHitPoint) const
{
	const FRay& WorldRay = InRay.WorldRay;

	if (PreviewMesh.IsValid())
	{
		if (ProjectionType == EProjectionType::OnMesh)
		{
			FHitResult Hit;
			if (PreviewMesh->FindRayIntersection(WorldRay, Hit))
			{
				OutHitPoint = WorldRay.Origin + WorldRay.Direction * Hit.Distance;
				return true;
			}
		}

		if (ProjectionType == EProjectionType::WithinMesh)
		{
			using namespace UE::Geometry;

			if (FDynamicMeshAABBTree3* MeshAABBTree = PreviewMesh->GetSpatial())
			{
				using HitResult = MeshIntersection::FHitIntersectionResult;
				TArray<HitResult> Hits;
				
				if (MeshAABBTree->FindAllHitTriangles(WorldRay, Hits))
				{
					if (Hits.Num() == 1)
					{
						OutHitPoint = WorldRay.Origin + WorldRay.Direction * Hits[0].Distance;
						return true;						
					}

					// const double AverageDistance = Algo::Accumulate(Distances, 0.0) / static_cast<double>(Distances.Num());
					// const int32 Index0 = Distances.IndexOfByPredicate([AverageDistance](double Distance) {return Distance <= AverageDistance;} );
					// const int32 Index1 = Distances.IndexOfByPredicate([AverageDistance](double Distance) {return Distance >= AverageDistance;} );

					static constexpr int32 Index0 = 0;
					static constexpr int32 Index1 = 1;

					const double d0 = Hits[Index0].Distance;
					const double d1 = Hits[Index1].Distance;
					OutHitPoint = WorldRay.Origin + WorldRay.Direction * ((d0+d1)*0.5);
					return true;
				}
			}

			FHitResult Hit;
			if (PreviewMesh->FindRayIntersection(WorldRay, Hit))
			{
				OutHitPoint = WorldRay.Origin + WorldRay.Direction * Hit.Distance;
				return true;
			}
		}
	}
	
	// if ray is parallel to plane, nothing has been hit
	if (FMath::IsNearlyZero(FVector::DotProduct(PlaneNormal, WorldRay.Direction)))
	{
		return false;
	}

	const FPlane Plane(PlaneOrigin, PlaneNormal);
	
	if (CameraState.bIsOrthographic)
	{
		OutHitPoint = FVector::PointPlaneProject(WorldRay.Origin, Plane);
		return true;
	}
	
	const double HitDepth = FMath::RayPlaneIntersectionParam(WorldRay.Origin, WorldRay.Direction, Plane);
	if (HitDepth < 0.0)
	{
		return false;
	}

	OutHitPoint = WorldRay.Origin + WorldRay.Direction * HitDepth;
	
	return true;
}

#undef LOCTEXT_NAMESPACE
