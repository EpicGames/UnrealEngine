// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletalMesh/SkeletonTransformProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonTransformProxy)

FTransform USkeletonTransformProxy::GetTransform() const
{
	return SharedTransform;
}

void USkeletonTransformProxy::Initialize(const FTransform& InTransform, const EToolContextCoordinateSystem& InTransformMode)
{
	Objects.Reset();
	SharedTransform = InitialSharedTransform = InTransform;
	TransformMode = InTransformMode;
}

void USkeletonTransformProxy::UpdateSharedTransform()
{
	// nothing to do
}

void USkeletonTransformProxy::UpdateObjects()
{
	const FTransform OffsetTransform = SharedTransform.GetRelativeTransform(InitialSharedTransform);
	const FVector DeltaTranslation = OffsetTransform.GetTranslation();
	const FQuat DeltaRotation = OffsetTransform.GetRotation();
	const bool bApplyLocal = TransformMode == EToolContextCoordinateSystem::Local;
	
	for (FRelativeObject& Object : Objects)
	{
		if (bApplyLocal)
		{
			Object.SetTransformFunc(OffsetTransform * Object.StartTransform);
		}
		else
		{
			FTransform Transform = Object.StartTransform;
			Transform.SetRotation(DeltaRotation * Transform.GetRotation());
			Transform.SetTranslation(DeltaTranslation + Transform.GetTranslation());
			Object.SetTransformFunc(Transform);
		}
	}
}

bool USkeletonTransformProxy::IsValid() const
{
	return !Objects.IsEmpty();
}
