// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMesh/SkinWeightsBindingTool.h"

#include "BoneWeights.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "InteractiveToolManager.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MeshOpPreviewHelpers.h"
#include "SkeletalMeshAttributes.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "PrimitiveDrawingUtils.h"
#include "Spatial/MeshWindingNumberGrid.h"

#include "ModelingToolTargetUtil.h"
#include "SkinningOps/SkinBindingOp.h"
#include "Spatial/OccupancyGrid3.h"
#include "ContextObjectStore.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "Components/DynamicMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinWeightsBindingTool)


#define LOCTEXT_NAMESPACE "USkinWeightsBindingTool"


bool USkinWeightsBindingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, FToolTargetTypeRequirements()) == 1;
}


UMultiSelectionMeshEditingTool* USkinWeightsBindingToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	USkinWeightsBindingTool* Tool = NewObject<USkinWeightsBindingTool>(SceneState.ToolManager);
	Tool->Init(SceneState);
	return Tool;
}

const FToolTargetTypeRequirements& USkinWeightsBindingToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		USkeletonProvider::StaticClass()
		});
	return TypeRequirements;
}

USkinWeightsBindingTool::USkinWeightsBindingTool()
{
	Properties = CreateDefaultSubobject<USkinWeightsBindingToolProperties>(TEXT("SkinWeightsBindingProperties"));
	// CreateDefaultSubobject automatically sets RF_Transactional flag, we need to clear it so that undo/redo doesn't affect tool properties
	Properties->ClearFlags(RF_Transactional);
}


USkinWeightsBindingTool::~USkinWeightsBindingTool()
{
}

void USkinWeightsBindingTool::Init(const FToolBuilderState& InSceneState)
{
	const UContextObjectStore* ContextObjectStore = InSceneState.ToolManager->GetContextObjectStore();
	EditorContext = ContextObjectStore->FindContext<USkeletalMeshEditorContextObjectBase>();
}

void USkinWeightsBindingTool::Setup()
{
	Super::Setup();

	if (ensure(Properties))
	{
		Properties->RestoreProperties(this);
		
		if (Properties->CurrentBone != NAME_None)
		{
			const FName RestoredCurrentBone = Properties->CurrentBone;
			Properties->CurrentBone = NAME_None;
			PendingUpdateFunction = [this, RestoredCurrentBone]()
			{
				const int32 BoneIndex = ReferenceSkeleton.FindRawBoneIndex(RestoredCurrentBone);
				Properties->CurrentBone = BoneIndex != INDEX_NONE ? RestoredCurrentBone : NAME_None;
				UpdateVisualization();
			};
		}
	}

	if (!ensure(Targets.Num() > 0) || !ensure(Targets[0]))
	{
		return;
	}
	
	ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(Targets[0]);
	ReferenceSkeleton = SkeletonProvider->GetSkeleton();
	
	UE::ToolTarget::HideSourceObject(Targets[0]);
	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this, "Preview");
	Preview->Setup(GetTargetWorld(), this);
	Preview->SetIsMeshTopologyConstant(true, EMeshRenderAttributeFlags::VertexColors);
	Preview->OnMeshUpdated.AddLambda([this](UMeshOpPreviewWithBackgroundCompute* Compute)
	{
		UpdateVisualization();
	});
	
	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Targets[0]);
	Preview->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

	UMaterialInterface* VtxColorMaterial = GetToolManager()->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	Preview->OverrideMaterial = VtxColorMaterial;

	auto UpdateOccupancy = [this]()
	{
		Occupancy.Reset();
		if (Properties->BindingType == ESkinWeightsBindType::GeodesicVoxel && Properties->bDebugDraw)
		{
			Occupancy = MakeShared<UE::Geometry::FOccupancyGrid3>(*OriginalMesh, Properties->VoxelResolution);
		}
	};
	
	// setup watchers
	auto HandlePropertyChange = [this, UpdateOccupancy](const bool bUpdateOperator)
	{
		if (bUpdateOperator)
		{
			Preview->InvalidateResult();
			UpdateOccupancy();
		}
		UpdateVisualization();
	};

	Properties->WatchProperty(Properties->CurrentBone,
							  [HandlePropertyChange](FName) { HandlePropertyChange(false); });
	Properties->WatchProperty(Properties->BindingType,
							  [HandlePropertyChange](ESkinWeightsBindType) { HandlePropertyChange(true); });
	Properties->WatchProperty(Properties->Stiffness,
							  [HandlePropertyChange](float) { HandlePropertyChange(true); });
	Properties->WatchProperty(Properties->MaxInfluences,
							  [HandlePropertyChange](int32) { HandlePropertyChange(true); });
	Properties->WatchProperty(Properties->VoxelResolution,
							  [HandlePropertyChange](int32) { HandlePropertyChange(true); });

	EditedMeshDescription = MakeUnique<FMeshDescription>();
	*EditedMeshDescription = *UE::ToolTarget::GetMeshDescription(Targets[0]); 
	
	OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	*OriginalMesh = UE::ToolTarget::GetDynamicMeshCopy(Targets[0]);



	Preview->PreviewMesh->SetTransform((FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[0]));
	Preview->PreviewMesh->SetShadowsEnabled(false);
	Preview->PreviewMesh->UpdatePreview(OriginalMesh.Get());
	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(Preview->PreviewMesh->GetRootComponent()));

	UpdateOccupancy();
	
	UpdateVisualization(/*bForce=*/true);
	
	// add properties to GUI
	AddToolPropertySource(Properties);

	Preview->InvalidateResult();

	if (EditorContext.IsValid())
	{
		EditorContext->BindTo(this);
	}
	
	SetToolDisplayName(LOCTEXT("ToolName", "Bind Skin"));
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Creates a rigid binding for the skin weights."),
		EToolMessageLevel::UserNotification);
}


void USkinWeightsBindingTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Properties->SaveProperties(this);

	UE::ToolTarget::ShowSourceObject(Targets[0]);
	
	const FDynamicMeshOpResult Result = Preview->Shutdown();
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset(Result);
	}

	if (EditorContext.IsValid())
	{
		EditorContext->UnbindFrom(this);
	}
}


void USkinWeightsBindingTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);

	if (PendingUpdateFunction)
	{
		PendingUpdateFunction();
		PendingUpdateFunction.Reset();
	}
}

static void DrawBox(IToolsContextRenderAPI* RenderAPI, const FTransform& Transform, const FBox &Box, const FLinearColor &Color, float LineThickness)
{
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
	const float PDIScale = RenderAPI->GetCameraState().GetPDIScalingFactor();

	const FVector Corners[2] = {
		Transform.TransformPosition(Box.Min),
		Transform.TransformPosition(Box.Max)
	};

	static constexpr UE::Geometry::FVector3i Offsets[12][2] =
	{
		// Bottom
		{{0, 0, 0}, {1, 0, 0}},
		{{1, 0, 0}, {1, 1, 0}},
		{{1, 1, 0}, {0, 1, 0}},
		{{0, 1, 0}, {0, 0, 0}},
		
		// Top
		{{0, 0, 1}, {1, 0, 1}},
		{{1, 0, 1}, {1, 1, 1}},
		{{1, 1, 1}, {0, 1, 1}},
		{{0, 1, 1}, {0, 0, 1}},
		
		// Sides
		{{0, 0, 0}, {0, 0, 1}},
		{{1, 0, 0}, {1, 0, 1}},
		{{1, 1, 0}, {1, 1, 1}},
		{{0, 1, 0}, {0, 1, 1}},
	}; 

	for (int32 Index = 0; Index < 12; Index++)
	{
		const UE::Geometry::FVector3i* LineOffsets = Offsets[Index];
		FVector  A(Corners[LineOffsets[0].X].X, Corners[LineOffsets[0].Y].Y, Corners[LineOffsets[0].Z].Z);
		FVector  B(Corners[LineOffsets[1].X].X, Corners[LineOffsets[1].Y].Y, Corners[LineOffsets[1].Z].Z);
		
		PDI->DrawTranslucentLine(A, B, Color, 1, LineThickness * PDIScale);
	}
}


void USkinWeightsBindingTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	using namespace UE::Geometry;
	
	if (Occupancy && Properties->bDebugDraw)
	{
		constexpr bool bShowInterior = false;
		constexpr bool bShowBoundary = true;

		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		const FTransform Transform = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
		float PDIScale = RenderAPI->GetCameraState().GetPDIScalingFactor();
		const TDenseGrid3<FOccupancyGrid3::EDomain>& OccupancyGrid = Occupancy->GetOccupancyStateGrid(); 

		for (int32 Index = 0; Index < OccupancyGrid.Num(); Index++)
		{
			FVector3i OccupancyIndex(OccupancyGrid.GetCoords(Index));

			const FOccupancyGrid3::EDomain Domain = OccupancyGrid[OccupancyIndex];
			if (bShowBoundary && Domain == FOccupancyGrid3::EDomain::Boundary)
			{
				FBox Box{Occupancy->GetCellBoxFromIndex(OccupancyIndex)};
				DrawBox(RenderAPI, Transform, Box, FLinearColor(1.0, 1.0, 0.0, 0.5), 0.0f);
			}
		}
	}
	
}


bool USkinWeightsBindingTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidResult() && Properties->ConfirmCurrentSettings;
}

TUniquePtr<UE::Geometry::FDynamicMeshOperator> USkinWeightsBindingTool::MakeNewOperator()
{
	TUniquePtr<UE::Geometry::FSkinBindingOp> Op = MakeUnique<UE::Geometry::FSkinBindingOp>();

	Op->ProfileName = FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
	Op->BindType = static_cast<UE::Geometry::ESkinBindingType>(Properties->BindingType);
	Op->Stiffness = Properties->Stiffness;
	Op->MaxInfluences = Properties->MaxInfluences;
	Op->VoxelResolution = Properties->VoxelResolution;
	
	Op->OriginalMesh = OriginalMesh;
	Op->SetTransformHierarchyFromReferenceSkeleton(ReferenceSkeleton);

	const FTransform3d LocalToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
	Op->SetResultTransform(LocalToWorld);
	
	return Op;
}

bool USkinWeightsBindingTool::IsInputIsolationValidOnOutput() const
{
	return true;
}

bool USkinWeightsBindingTool::IsInputSelectionValidOnOutput()
{
	return true;
}

bool USkinWeightsBindingTool::UpdateSkinWeightsFromDynamicMesh(FDynamicMesh3& InResultMesh) const
{
	using namespace UE::AnimationCore;
	using namespace UE::Geometry;

	if (!InResultMesh.HasAttributes())
	{
		return false;
	}
	
	const FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = InResultMesh.Attributes()->GetSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
	if (!SkinWeights)
	{
		return false;
	}

	FSkeletalMeshAttributes MeshAttribs(*EditedMeshDescription);
	FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
	
	const int32 NumVertices = EditedMeshDescription->Vertices().Num();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
	{
		FBoneWeights Weights;
		SkinWeights->GetValue(VertexIndex, Weights);
		VertexSkinWeights.Set(FVertexID(VertexIndex), Weights);
	}

	return true;
}

void USkinWeightsBindingTool::GenerateAsset(const FDynamicMeshOpResult& Result)
{
	using namespace UE::Geometry;

	FDynamicMesh3* ResultMesh = Result.Mesh.Get();
	check(ResultMesh != nullptr);

	if (!UpdateSkinWeightsFromDynamicMesh(*ResultMesh))
	{
		return;
	}
	
	GetToolManager()->BeginUndoTransaction(LOCTEXT("SkinWeightsBindingToolTransactionName", "Create Rigid Binding"));

	UE::ToolTarget::CommitMeshDescriptionUpdate(Targets[0], EditedMeshDescription.Get());

	GetToolManager()->EndUndoTransaction();	
}

FVector4f USkinWeightsBindingTool::WeightToColor(float Value)
{
	Value = FMath::Clamp(Value, 0.0f, 1.0f);

	{
		// A close approximation of the skeletal mesh editor's bone weight ramp. 
		const FLinearColor HSV((1.0f - Value) * 285.0f, 100.0f, 85.0f);
		return UE::Geometry::ToVector4<float>(HSV.HSVToLinearRGB());
	}
}

void USkinWeightsBindingTool::UpdateVisualization(bool bInForce)
{
	using namespace UE::AnimationCore;
	using namespace UE::Geometry;

	if (bInForce || Preview->HaveValidNonEmptyResult())
	{
		const int32 RawBoneIndex = ReferenceSkeleton.FindRawBoneIndex(Properties->CurrentBone);
		const FBoneIndexType BoneIndex = static_cast<FBoneIndexType>(RawBoneIndex);

		// update mesh with new value colors
		Preview->PreviewMesh->EditMesh([&](FDynamicMesh3& InMesh)
		{
			const FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = InMesh.Attributes()->GetSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
			FDynamicMeshColorOverlay* ColorOverlay = InMesh.Attributes()->PrimaryColors();

			if (!ColorOverlay)
			{
				InMesh.EnableAttributes();
				InMesh.Attributes()->EnablePrimaryColors();
				// Create an overlay that has no split elements, init with zero value.
				ColorOverlay = InMesh.Attributes()->PrimaryColors();
				ColorOverlay->CreateFromPredicate([](int /*ParentVID*/, int /*TriIDA*/, int /*TriIDB*/){return true;}, 0.f);
			}

			static const FVector4f DefaultColor(WeightToColor(0.f));
			
			FBoneWeights BoneWeights;
			for (const int32 ElementId : ColorOverlay->ElementIndicesItr())
			{
				if (RawBoneIndex != INDEX_NONE)
				{
					const int32 VertexId = ColorOverlay->GetParentVertex(ElementId);
					SkinWeights->GetValue(VertexId, BoneWeights);

					float Weight = 0.0f;
					for (const FBoneWeight BoneWeight: BoneWeights)
					{
						if (BoneWeight.GetBoneIndex() == BoneIndex)
						{
							Weight = BoneWeight.GetWeight();
							break;
						}
					}
			
					const FVector4f Color(WeightToColor(Weight));
					ColorOverlay->SetElement(ElementId, Color);
				}
				else
				{
					ColorOverlay->SetElement(ElementId, DefaultColor);
				}
			}
		});
	}
}

void USkinWeightsBindingTool::HandleSkeletalMeshModified(const TArray<FName>& InBoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	switch (InNotifyType)
	{
	case ESkeletalMeshNotifyType::BonesAdded:
		break;
	case ESkeletalMeshNotifyType::BonesRemoved:
		break;
	case ESkeletalMeshNotifyType::BonesMoved:
		break;
	case ESkeletalMeshNotifyType::BonesSelected:
		{
			const int32 BoneIndex = InBoneNames.IsEmpty() ? INDEX_NONE : ReferenceSkeleton.FindRawBoneIndex(InBoneNames[0]);
			const FName NewSelectedBone = BoneIndex != INDEX_NONE ? InBoneNames[0] : NAME_None;
			PendingUpdateFunction = [this, NewSelectedBone]()
			{
				Properties->CurrentBone = NewSelectedBone;
				UpdateVisualization();
			};
		}
		break;
	case ESkeletalMeshNotifyType::BonesRenamed:
		break;
	case ESkeletalMeshNotifyType::HierarchyChanged:
        break;
	default:
		checkNoEntry();
	}
}

#undef LOCTEXT_NAMESPACE

// #pragma optimize( "", on )

