// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMesh/SkinWeightsPaintTool.h"

#include "Algo/AllOf.h"
#include "AssetViewerSettings.h"
#include "Framework/Application/SlateApplication.h"

#include "Math/UnrealMathUtility.h"
#include "Async/Async.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Engine/SkeletalMesh.h"
#include "Components/SkeletalMeshComponent.h"
#include "SkeletalDebugRendering.h"
#include "AnimationRuntime.h"
#include "Animation/MirrorDataTable.h"

#include "Editor/Persona/Public/IPersonaEditorModeManager.h"
#include "Editor/Persona/Public/PersonaModule.h"
#include "EditorViewportClient.h"
#include "Preferences/PersonaOptions.h"
#include "PreviewProfileController.h"
#include "BaseGizmos/BrushStampIndicator.h"
#include "BaseGizmos/VolumetricBrushStampIndicator.h"

#include "InteractiveGizmoManager.h"
#include "InteractiveToolManager.h"
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "ContextObjectStore.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "SkeletalMesh/SkeletalMeshToolsHelper.h"
#include "Components/DynamicMeshComponent.h"

#include "MeshDescription.h"
#include "MeshModelingToolsEditorOnly.h"
#include "DynamicSubmesh3.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/MeshAdapterUtil.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletonModifier.h"

#include "Operations/SmoothBoneWeights.h"
#include "Operations/TransferBoneWeights.h"

#include "Parameterization/MeshDijkstra.h"
#include "Spatial/PointHashGrid3.h"
#include "Parameterization/MeshLocalParam.h"
#include "Spatial/FastWinding.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "HAL/PlatformApplicationMisc.h"

#include "Serialization/JsonSerializable.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"

#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Images/SImage.h"
#include "Misc/MessageDialog.h"
#include "SkeletalMesh/SReferenceSkeletonTree.h"
#include "SkeletalMesh/SkinWeightsPaintToolPropertiesCustomVersion.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinWeightsPaintTool)

#define LOCTEXT_NAMESPACE "USkinWeightsPaintTool"

namespace UE::MeshModelingToolsEditorOnly::SelectionUtils
{
	/** Returns the geometry type that corresponds to a selection mode */
	UE::Geometry::EGeometryElementType GetGeometryElementTypeFromSelectionMode(const EComponentSelectionMode SelectionMode)
	{
		using namespace UE::Geometry;
		if (SelectionMode == EComponentSelectionMode::Vertices)
		{
			return EGeometryElementType::Vertex;
		}
		else if (SelectionMode == EComponentSelectionMode::Edges)
		{
			return EGeometryElementType::Edge;
		}
		else if (SelectionMode == EComponentSelectionMode::Faces)
		{
			return EGeometryElementType::Face;
		}
		else
		{
			ensureMsgf(0, TEXT("Unhandled selection mode"));
			return EGeometryElementType::Vertex;
		}
	}

	/** Util to convert selection from one selection mode to another */
	struct FWeightToolScopedConvertSelection
		: FNoncopyable
	{
		FWeightToolScopedConvertSelection(
			USkinWeightsPaintTool& WeightTool,
			const UE::Geometry::FDynamicMesh3& Mesh,
			const EComponentSelectionMode TargetSelectionMode)
		{
			using namespace UE::Geometry;

			UToolMeshSelector* MeshSelector = WeightTool.GetMainMeshSelector();
			SelectionMechanic = MeshSelector ? MeshSelector->GetSelectionMechanic() : nullptr;
			const USkinWeightsPaintToolProperties* ToolProperties = WeightTool.GetWeightToolProperties();

			if (ensure(ToolProperties && SelectionMechanic))
			{
				const EComponentSelectionMode SourceSelectionMode = ToolProperties->ComponentSelectionMode;
				if (SourceSelectionMode == TargetSelectionMode)
				{
					bPerformConversion = false;
					return;
				}

				FGeometrySelection SourceSelection;
				SourceSelection.TopologyType = EGeometryTopologyType::Triangle;
				SourceSelection.ElementType = GetGeometryElementTypeFromSelectionMode(SourceSelectionMode);
				
				SelectionMechanic->GetSelection_AsTriangleTopology(SourceSelection);
				
				const auto VerticesToEdges = [&Mesh](const FGeometrySelection& InVertexSelection)
					{
						FGeometrySelection Result;						
						Result.ElementType = EGeometryElementType::Edge;
						Result.TopologyType = EGeometryTopologyType::Triangle;

						for (const int EdgeID : Mesh.EdgeIndicesItr())
						{
							const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
							if (InVertexSelection.Selection.Contains(Edge.Vert.A) &&
								InVertexSelection.Selection.Contains(Edge.Vert.B))
							{
								const FMeshTriEdgeID TriEdgeID = Mesh.GetTriEdgeIDFromEdgeID(EdgeID);
								Result.Selection.Add(TriEdgeID.Encoded());
							}
						}
						
						return Result;
					};
					
				const auto VerticesToFaces = [&Mesh](const FGeometrySelection& InVertexSelection)
					{
						FGeometrySelection Result;
						Result.ElementType = EGeometryElementType::Face;
						Result.TopologyType = EGeometryTopologyType::Triangle;

						for (const int32& TriangleID : Mesh.TriangleIndicesItr())
						{
							const FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
							if (InVertexSelection.Selection.Contains(Triangle.A) &&
								InVertexSelection.Selection.Contains(Triangle.B) &&
								InVertexSelection.Selection.Contains(Triangle.C))
							{
								Result.Selection.Add(TriangleID);
							}
						}
						
						return Result;
					};
									
				const auto EdgesToVertices = [this, &Mesh](const FGeometrySelection& InVertexSelection)
					{
						FGeometrySelection Result;
						Result.ElementType = EGeometryElementType::Vertex;
						Result.TopologyType = EGeometryTopologyType::Triangle;

						for (const uint64 EdgeID : InVertexSelection.Selection)
						{
							const FMeshTriEdgeID TriEdgeID(EdgeID);	
							const int TriEdge = Mesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex);
							const FDynamicMesh3::FEdge Edge = Mesh.GetEdge(TriEdge);
							
							Result.Selection.Add(Edge.Vert.A);
							Result.Selection.Add(Edge.Vert.B);
						}
						
						return Result;
					};
					
				const auto FacesToVertices = [this, &Mesh](const FGeometrySelection& InVertexSelection)
					{
						FGeometrySelection Result;
						Result.ElementType = EGeometryElementType::Vertex;
						Result.TopologyType = EGeometryTopologyType::Triangle;

						for (const uint64 TriangleID : InVertexSelection.Selection)
						{
							const FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
							Result.Selection.Add(Triangle.A);
							Result.Selection.Add(Triangle.B);
							Result.Selection.Add(Triangle.C);
						}
						
						return Result;
					};
				
				if (SourceSelectionMode == EComponentSelectionMode::Vertices &&
					TargetSelectionMode == EComponentSelectionMode::Edges)
				{	
					TargetSelection = VerticesToEdges(SourceSelection);
				}
				else if (
					SourceSelectionMode == EComponentSelectionMode::Vertices &&
					TargetSelectionMode == EComponentSelectionMode::Faces)
				{	
					TargetSelection = VerticesToFaces(SourceSelection);
				}
				else if (
					SourceSelectionMode == EComponentSelectionMode::Edges &&
					TargetSelectionMode == EComponentSelectionMode::Vertices)
				{	
					TargetSelection = EdgesToVertices(SourceSelection);
				}
				else if (
					SourceSelectionMode == EComponentSelectionMode::Edges &&
					TargetSelectionMode == EComponentSelectionMode::Faces)
				{
					TargetSelection = EdgesToVertices(SourceSelection);
					TargetSelection = VerticesToFaces(TargetSelection);
				}
				else if (
					SourceSelectionMode == EComponentSelectionMode::Faces &&
					TargetSelectionMode == EComponentSelectionMode::Vertices)
				{	
					TargetSelection = FacesToVertices(SourceSelection);
				}
				else if (
					SourceSelectionMode == EComponentSelectionMode::Faces &&
					TargetSelectionMode == EComponentSelectionMode::Edges)
				{
					TargetSelection = FacesToVertices(SourceSelection);
					TargetSelection = VerticesToEdges(TargetSelection);
				}
				else
				{
					ensureMsgf(0, TEXT("Unhandled selection mode"));
				}
				
				SelectionMechanic->ClearSelection();
			}
		}
		
		~FWeightToolScopedConvertSelection()
		{
			if (bPerformConversion &&
				SelectionMechanic)
			{					
				// Restore selection
				constexpr bool bBroadcastSelectionChanged = true;
				SelectionMechanic->SetSelection_AsTriangleTopology(TargetSelection, bBroadcastSelectionChanged);
			}
		}
		
	private:
		bool bPerformConversion = true;
		
		UE::Geometry::FGeometrySelection TargetSelection;

		UPolygonSelectionMechanic* SelectionMechanic = nullptr;
	};
	
	enum class ESelectionRemapDirection : uint8
	{
		ToBaseMesh,
		ToSubmesh
	};

	/** Remap a selection using a submesh mapping. */
	static void RemapSelection(
		ESelectionRemapDirection Direction,
		const UE::Geometry::FGeometrySelection& InSelection,
		UE::Geometry::FGeometrySelection& OutSelection,
		const UE::Geometry::FDynamicSubmesh3& Submesh)
	{
		using namespace UE::Geometry;
		
		OutSelection.TopologyType = InSelection.TopologyType;
		OutSelection.ElementType = InSelection.ElementType;

		if (InSelection.ElementType == EGeometryElementType::Vertex)
		{
			Algo::Transform(InSelection.Selection, OutSelection.Selection,
				[&Submesh, Direction](const uint64 ElementID)
				{
					return Direction == ESelectionRemapDirection::ToBaseMesh
						? Submesh.MapVertexToBaseMesh(ElementID)
						: Submesh.MapVertexToSubmesh(ElementID);
				});
		}
		else if (InSelection.ElementType == EGeometryElementType::Edge)
		{
			for (const uint64 EdgeID : InSelection.Selection)
			{
				FMeshTriEdgeID TriEdgeID(EdgeID);
				TriEdgeID.TriangleID = Direction == ESelectionRemapDirection::ToBaseMesh
					? Submesh.MapTriangleToBaseMesh(TriEdgeID.TriangleID)
					: Submesh.MapTriangleToSubmesh(TriEdgeID.TriangleID);
				OutSelection.Selection.Add(TriEdgeID.Encoded());
			}
		}
		else if (InSelection.ElementType == EGeometryElementType::Face)
		{
			Algo::Transform(InSelection.Selection, OutSelection.Selection,
				[&Submesh, Direction](const uint64 Index)
				{
					return Direction == ESelectionRemapDirection::ToBaseMesh
						? Submesh.MapTriangleToBaseMesh(Index)
						: Submesh.MapTriangleToSubmesh(Index);
				});
		}
	}

	/**
	 * Retains the selection across isolation changes.
	 * Constructor: captures current selection and remaps to full mesh space.
	 * Destructor: remaps from full mesh space to whatever space the isolator is in at destruction time.
	 */
	struct FWeightToolScopedRetainSelection
		: FNoncopyable
	{
		FWeightToolScopedRetainSelection(
			UWeightToolSelectionIsolator& InIsolator)
			: Isolator(InIsolator)
		{
			using namespace UE::Geometry;

			USkinWeightsPaintTool& WeightTool = InIsolator.GetWeightTool();
			UToolMeshSelector* MeshSelector = WeightTool.GetMainMeshSelector();
			SelectionMechanic = MeshSelector ? MeshSelector->GetSelectionMechanic() : nullptr;
			const USkinWeightsPaintToolProperties* ToolProperties = WeightTool.GetWeightToolProperties();

			if (ToolProperties && SelectionMechanic)
			{
				const EComponentSelectionMode SelectionMode = ToolProperties->ComponentSelectionMode;
				const EGeometryElementType ElementType = GetGeometryElementTypeFromSelectionMode(SelectionMode);

				FGeometrySelection CurrentSelection;
				CurrentSelection.TopologyType = EGeometryTopologyType::Triangle;
				CurrentSelection.ElementType = ElementType;
				SelectionMechanic->GetSelection_AsTriangleTopology(CurrentSelection);

				// Remap to full mesh space if currently isolated
				if (Isolator.IsMeshIsolated())
				{
					RemapSelection(ESelectionRemapDirection::ToBaseMesh, CurrentSelection, FullMeshSelection, Isolator.GetPartialSubmesh());
				}
				else
				{
					FullMeshSelection = MoveTemp(CurrentSelection);
				}
			}
		}

		~FWeightToolScopedRetainSelection()
		{
			if (SelectionMechanic)
			{
				// Remap from full mesh space to current space
				if (Isolator.IsMeshIsolated())
				{
					UE::Geometry::FGeometrySelection RestoredSelection;
					RemapSelection(ESelectionRemapDirection::ToSubmesh, FullMeshSelection, RestoredSelection, Isolator.GetPartialSubmesh());
					SelectionMechanic->SetSelection_AsTriangleTopology(RestoredSelection);
				}
				else
				{
					SelectionMechanic->SetSelection_AsTriangleTopology(FullMeshSelection);
				}
			}
		}

	private:
		UE::Geometry::FGeometrySelection FullMeshSelection;
		UWeightToolSelectionIsolator& Isolator;
		UPolygonSelectionMechanic* SelectionMechanic = nullptr;
	};
}

using namespace SkinPaintTool;

static void ShowEditorMessage(ELogVerbosity::Type InMessageType, const FText& InMessage)
{
	FNotificationInfo Notification(InMessage);
	Notification.bUseSuccessFailIcons = true;
	Notification.ExpireDuration = 5.0f;

	SNotificationItem::ECompletionState State = SNotificationItem::CS_Success;

	switch(InMessageType)
	{
	case ELogVerbosity::Warning:
		UE_LOGF(LogMeshModelingToolsEditor, Warning, "%ls", *InMessage.ToString());
		break;
	case ELogVerbosity::Error:
		State = SNotificationItem::CS_Fail;
		UE_LOGF(LogMeshModelingToolsEditor, Error, "%ls", *InMessage.ToString());
		break;
	default:
		break; // don't log anything unless a warning or error
	}
	
	FSlateNotificationManager::Get().AddNotification(Notification)->SetCompletionState(State);
}

namespace SkinPaintTool
{
	
EMeshLODIdentifier GetLODId(const FName InLODName)
{
	static const TMap<FName, EMeshLODIdentifier> LODs({
		{"LOD0", EMeshLODIdentifier::LOD0},
		{"LOD1", EMeshLODIdentifier::LOD1},
		{"LOD2", EMeshLODIdentifier::LOD2},
		{"LOD3", EMeshLODIdentifier::LOD3},
		{"LOD4", EMeshLODIdentifier::LOD4},
		{"LOD5", EMeshLODIdentifier::LOD5},
		{"LOD6", EMeshLODIdentifier::LOD6},
		{"LOD7", EMeshLODIdentifier::LOD7},
		{"HiResSource", EMeshLODIdentifier::HiResSource},
		{"Default", EMeshLODIdentifier::Default},
		{"MaxQuality", EMeshLODIdentifier::MaxQuality}
	});

	const EMeshLODIdentifier* LODIdFound = LODs.Find(InLODName);
	return LODIdFound ? *LODIdFound : EMeshLODIdentifier::Default;
}

FName GetLODName(const EMeshLODIdentifier InLOD)
{
	static const TMap<EMeshLODIdentifier, FName> LODs({
		{EMeshLODIdentifier::LOD0, "LOD0"},
		{EMeshLODIdentifier::LOD1, "LOD1"},
		{EMeshLODIdentifier::LOD2, "LOD2"},
		{EMeshLODIdentifier::LOD3, "LOD3"},
		{EMeshLODIdentifier::LOD4, "LOD4"},
		{EMeshLODIdentifier::LOD5, "LOD5"},
		{EMeshLODIdentifier::LOD6, "LOD6"},
		{EMeshLODIdentifier::LOD7, "LOD7"},
		{EMeshLODIdentifier::HiResSource, "HiResSource"},
		{EMeshLODIdentifier::Default, "Default"},
		{EMeshLODIdentifier::MaxQuality, "MaxQuality"}
	});

	const FName* Name = LODs.Find(InLOD);
	return Name ? *Name : NAME_None;
}


const FName& CreateNewName()
{
	static const FName CreateNew("Create New...");
	return CreateNew;
}
	
const TCHAR* NewProfilePlaceHolderName()
{
	static const TCHAR* Name = TEXT("SkinWeightsPaintTool_NewProfilePlaceHolderName");
	return Name;
}
	
UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* GetOrCreateSkinWeightsAttribute(FDynamicMesh3& InMesh, const FName InProfileName)
{
	using namespace UE::Geometry;
	FDynamicMeshVertexSkinWeightsAttribute* Attribute = InMesh.Attributes()->GetSkinWeightsAttribute(InProfileName);
	if (Attribute == nullptr)
	{
		Attribute = new FDynamicMeshVertexSkinWeightsAttribute(&InMesh);
		InMesh.Attributes()->AttachSkinWeightsAttribute(InProfileName, Attribute);
	}
	return Attribute;
}

bool RenameSkinWeightsAttribute(FDynamicMesh3& InMesh, const FName InOldName, const FName InNewName)
{
	using namespace UE::Geometry;
	if (FDynamicMeshVertexSkinWeightsAttribute* Attribute = InMesh.Attributes()->GetSkinWeightsAttribute(InOldName))
	{
		FDynamicMeshVertexSkinWeightsAttribute* NewAttribute = GetOrCreateSkinWeightsAttribute(InMesh, InNewName);
		*NewAttribute = MoveTemp(*Attribute);

		InMesh.Attributes()->RemoveSkinWeightsAttribute(InOldName);
		return true;
	}

	return false;
}
	
}

// thread pool to use for async operations
static EAsyncExecution SkinPaintToolAsyncExecTarget = EAsyncExecution::ThreadPool;

// any weight below this value is ignored, since it won't be representable in unsigned 16-bit precision
constexpr float MinimumWeightThreshold = 1.0f / 65535.0f;

class FPaintToolWeightsDataSource : public UE::Geometry::TBoneWeightsDataSource<int32, float>
{
public:

	enum class EWeightSource : uint8 { PreChange, Current };

	struct FWeightSourceScope
	{
		FWeightSourceScope(FPaintToolWeightsDataSource& InDataSource, EWeightSource InTransientSource)
			: DataSource(InDataSource), PreviousSource(InDataSource.Source)
		{
			DataSource.Source = InTransientSource;
		}
		~FWeightSourceScope()
		{
			DataSource.Source = PreviousSource;
		}
		FWeightSourceScope(const FWeightSourceScope&) = delete;
		FWeightSourceScope& operator=(const FWeightSourceScope&) = delete;

	private:
		FPaintToolWeightsDataSource& DataSource;
		EWeightSource PreviousSource;
	};

	FPaintToolWeightsDataSource(const FSkinToolWeights* InWeights)
		: Weights(InWeights)
	{
		checkSlow(Weights);
	}

	virtual ~FPaintToolWeightsDataSource() = default;

	virtual int32 GetBoneNum(const int32 VertexID) override
	{
		return GetBuffer()[VertexID].Num();
	}

	virtual int32 GetBoneIndex(const int32 VertexID, const int32 Index) override
	{
		return GetBuffer()[VertexID][Index].BoneID;
	}

	virtual float GetBoneWeight(const int32 VertexID, const int32 Index) override
	{
		return GetBuffer()[VertexID][Index].Weight;
	}

	virtual float GetWeightOfBoneOnVertex(const int32 VertexID, const int32 BoneIndex) override
	{
		return Weights->GetWeightOfBoneOnVertex(BoneIndex, VertexID, GetBuffer());
	}

private:

	const TArray<VertexWeights>& GetBuffer() const
	{
		return Source == EWeightSource::Current ? Weights->CurrentWeights : Weights->PreChangeWeights;
	}

	const FSkinToolWeights* Weights = nullptr;
	EWeightSource Source = EWeightSource::PreChange;
};

void FDirectEditWeightState::Reset()
{
	bInTransaction = false;
	StartValue = CurrentValue = GetModeDefaultValue();
}

float FDirectEditWeightState::GetModeDefaultValue()
{
	static TMap<EWeightEditOperation, float> DefaultModeValues = {
		{EWeightEditOperation::Add, 0.0f},
		{EWeightEditOperation::Replace, .0f},
		{EWeightEditOperation::Multiply, 1.f},
		{EWeightEditOperation::Relax, 0.f}
	};

	return DefaultModeValues[EditMode];
}

float FDirectEditWeightState::GetModeMinValue()
{
	static TMap<EWeightEditOperation, float> MinModeValues = {
		{EWeightEditOperation::Add, -1.f},
		{EWeightEditOperation::Replace, 0.f},
		{EWeightEditOperation::Multiply, 0.f},
		{EWeightEditOperation::Relax, 0.f}
	};

	return MinModeValues[EditMode];
}

float FDirectEditWeightState::GetModeMaxValue()
{
	static TMap<EWeightEditOperation, float> MaxModeValues = {
		{EWeightEditOperation::Add, 1.f},
		{EWeightEditOperation::Replace, 1.f},
		{EWeightEditOperation::Multiply, 2.f},
		{EWeightEditOperation::Relax, 10.f}
	};

	return MaxModeValues[EditMode];
}

USkinWeightsPaintToolProperties::USkinWeightsPaintToolProperties()
	: CustomVersion(ESkinWeightsPaintToolPropertiesCustomVersion::NoVersion)
{
	BrushConfigs.Add(EWeightEditOperation::Add, &BrushConfigAdd);
	BrushConfigs.Add(EWeightEditOperation::Replace, &BrushConfigReplace);
	BrushConfigs.Add(EWeightEditOperation::Multiply, &BrushConfigMultiply);
	BrushConfigs.Add(EWeightEditOperation::Relax, &BrushConfigRelax);

	// Multiply's slider value is the literal multiplier (1.0 = identity); default to a
	// gentle shrink so the brush has a visible effect out of the box.
	BrushConfigMultiply.Strength = 0.9f;

	LoadConfig();

	if (LinearColorRamp.IsEmpty())
	{
		const TArray<FLinearColor> DefaultColorRamp
		{
			FLinearColor(0.8f, 0.4f, 0.8f),	// Purple
			FLinearColor(0.f, 0.f, 0.5f),	// Dark Blue
			FLinearColor(0.2f, 0.2f, 1.0f),	// Light Blue
			FLinearColor(0.f, 1.f, 0.f),	// Green
			FLinearColor(1.f, 1.f, 0.f), 	// Yellow
			FLinearColor(1.f, 0.65f, 0.f),	// Orange
			FLinearColor(1.f, 0.f, 0.f)		// Red
		};

		float Time = 0.f;
		const float Delta = 1.f / (DefaultColorRamp.Num() - 1);
		for (const FLinearColor& LinearColor : DefaultColorRamp)
		{
			constexpr bool bOnlyRGB = true;
			LinearColorRamp.SetColorAtTime(Time, LinearColor, bOnlyRGB);
			
			Time += Delta;
		}
	}
}

void USkinWeightsPaintToolProperties::RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier)
{
	Super::RestoreProperties(RestoreToTool, CacheIdentifier);

	// Upgrade SkinWeightsPaintToolProperties to use FLinearColorRamp instead of an array of linear colors
	if (CustomVersion < ESkinWeightsPaintToolPropertiesCustomVersion::SkinWeightPaintToolUseLinearColorRamp)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS		
		if (!ColorRamp.IsEmpty())
		{
			LinearColorRamp.Reset();
		
			constexpr bool bOnlyRGB = true;
			LinearColorRamp.SetColorAtTime(0.f, ColorRamp[0], bOnlyRGB);
			
			const float DeltaTime = ColorRamp.Num() > 1 ? 1.f / (ColorRamp.Num() - 1) : 0.f;
			for (int32 RampStepIndex = 1; RampStepIndex < ColorRamp.Num() - 1; RampStepIndex++)
			{
				if (!FMath::Lerp(ColorRamp[RampStepIndex - 1], ColorRamp[RampStepIndex + 1], .5f).Equals(ColorRamp[RampStepIndex]))
				{
					// SetColorAtTime doesn't adopt alpha 1, hence force it here
					const FLinearColor Color = FLinearColor(ColorRamp[RampStepIndex]).CopyWithNewOpacity(1.f);
					LinearColorRamp.SetColorAtTime(DeltaTime * RampStepIndex, Color, bOnlyRGB);
				}
			}

			LinearColorRamp.SetColorAtTime(1.f, FLinearColor(ColorRamp.Last()).CopyWithNewOpacity(1.f), bOnlyRGB);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CustomVersion = ESkinWeightsPaintToolPropertiesCustomVersion::SkinWeightPaintToolUseLinearColorRamp;
		SaveProperties(RestoreToTool, CacheIdentifier);
		SaveConfig();
	}

	// Clamp restored per-mode Strength values to the slider's [0,1] range. Older configs
	// could hold values outside this range (e.g. BrushConfigRelax.Strength = 5 from the
	// pre-fix 0-10 slider) and would otherwise leave the slider stuck at its new max
	// until the user touched it.
	if (CustomVersion < ESkinWeightsPaintToolPropertiesCustomVersion::SkinWeightPaintToolClampStrengthToUnit)
	{
		BrushConfigAdd.Strength = FMath::Clamp(BrushConfigAdd.Strength, 0.f, 1.f);
		BrushConfigReplace.Strength = FMath::Clamp(BrushConfigReplace.Strength, 0.f, 1.f);
		BrushConfigRelax.Strength = FMath::Clamp(BrushConfigRelax.Strength, 0.f, 1.f);
		BrushConfigMultiply.Strength = FMath::Clamp(BrushConfigMultiply.Strength, 0.f, 1.f);

		CustomVersion = ESkinWeightsPaintToolPropertiesCustomVersion::SkinWeightPaintToolClampStrengthToUnit;
		SaveProperties(RestoreToTool, CacheIdentifier);
		SaveConfig();
	}

	// FalloffMode is always shared across every per-mode config. Radius is shared only when
	// the user has the Sync toggle on; with it off, each mode's stored radius is honoured.
	const FSkinWeightBrushConfig& Active = GetBrushConfig();
	if (bSyncBrushRadiusAcrossModes)
	{
		SetSharedRadius(Active.Radius);
	}
	SetSharedFalloffMode(Active.FalloffMode);
}

namespace SkinWeightLayer
{

TArray<FName> GetLODs(UToolTarget* InTarget)
{
    static TArray<FName> Dummy;

    if (!InTarget)
    {
    	return Dummy;
    }

    bool bOutSupportsLODs = false;
	constexpr bool bOnlyReturnDefaultLOD = false;
	// NOTE: currently auto-generated LODs do not have USkeletalMesh::SourceModels and so the returned MeshDescription will be null
	// for this reason, we do not allow transferring weights to/from auto-generated LODs
	constexpr bool bExcludeAutoGeneratedLODs = true;
    const TArray<EMeshLODIdentifier> LODIDSs = UE::ToolTarget::GetMeshDescriptionLODs(InTarget, bOutSupportsLODs, bOnlyReturnDefaultLOD, bExcludeAutoGeneratedLODs);
    if (!ensure(bOutSupportsLODs))
    {
    	return Dummy;
    }

    TArray<FName> LODs;
    LODs.Reserve(LODIDSs.Num());
    for (const EMeshLODIdentifier LODId: LODIDSs)
    {
    	const FName LODName = GetLODName(LODId);
    	if (LODName != NAME_None)
    	{
    		LODs.Add(LODName);
    	}
    }
    ensure(!LODs.IsEmpty());
    
    return LODs;
}

TArray<FName> GetProfilesFromToolTarget(UToolTarget* InTarget, const FName InLOD)
{
	if (InTarget)
	{
		const EMeshLODIdentifier LODId = GetLODId(InLOD);

		// offer a fast path if target is a dynamic mesh component to avoid conversion to mesh description
		if (UDynamicMeshComponentToolTarget* DynamicMeshToolTarget = Cast<UDynamicMeshComponentToolTarget>(InTarget))
		{
			TArray<FName> Profiles;
			DynamicMeshToolTarget->GetDynamicMesh().Attributes()->GetSkinWeightsAttributes().GenerateKeyArray(Profiles);
			return Profiles;	
		}
		
		if (IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(InTarget))
		{
			const FMeshDescription* MeshDescription = nullptr;
			if (MeshDescriptionProvider->SupportsLODs())
			{
				const FGetMeshParameters Params(true, LODId);
				MeshDescription = UE::ToolTarget::GetMeshDescription(InTarget, Params);	
			}
			else
			{
				MeshDescription = UE::ToolTarget::GetMeshDescription(InTarget);		
			}
		
			if (MeshDescription)
			{
				const FSkeletalMeshConstAttributes MeshAttribs(*MeshDescription);
				return MeshAttribs.GetSkinWeightProfileNames();
			}	
		}
		else if (IDynamicMeshProvider* DynamicMeshProvider = Cast<IDynamicMeshProvider>(InTarget))
		{
			TArray<FName> Profiles;
			DynamicMeshProvider->GetDynamicMesh().Attributes()->GetSkinWeightsAttributes().GenerateKeyArray(Profiles);
			return Profiles;
		}
	}
	
	static const TArray EmptyProfiles = {FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName, CreateNewName()};	
	return EmptyProfiles;
};

}


TArray<FName> USkinWeightsPaintToolProperties::GetTargetSkinWeightProfilesFunc() const
{
	TArray<FName> Profiles = SkinWeightLayer::GetProfilesFromToolTarget(WeightTool->GetTarget(), GetLODName(WeightTool->GetEditingLOD()));
	Profiles.Add(CreateNewName());
	return Profiles;
}

TArray<FName> USkinWeightsPaintToolProperties::GetSourceLODsFunc() const
{
	return SkinWeightLayer::GetLODs(WeightTool->GetWeightTransferManager()->GetTarget());
}

TArray<FName> USkinWeightsPaintToolProperties::GetSourceSkinWeightProfilesFunc() const
{
	UToolTarget* SourceToolTarget = WeightTool->GetWeightTransferManager()->GetTarget();
	return SkinWeightLayer::GetProfilesFromToolTarget(SourceToolTarget, SourceLOD);
}

void UWeightToolTransferManager::InitialSetup(USkinWeightsPaintTool* InWeightTool, FEditorViewportClient* InViewportClient)
{
	WeightTool = InWeightTool;

	// always reset back to target selection
	WeightTool->GetWeightToolProperties()->MeshSelectMode = EMeshTransferOption::Target;

	// create the mesh selector and run initial setup
	// NOTE: currently this must happen inside the Setup of a UInteractiveTool so that input is routed to the selection mechanic
	MeshSelector = NewObject<UToolMeshSelector>(this);
	auto DoNothingOnSelection = [](){};
	MeshSelector->InitialSetup(WeightTool->GetTargetWorld(), WeightTool.Get(), DoNothingOnSelection);
}

void UWeightToolTransferManager::SetSourceMesh(USkeletalMesh* InSkeletalMesh)
{
	USkinWeightsPaintToolProperties* ToolProperties = WeightTool->GetWeightToolProperties();
	if (!InSkeletalMesh || InSkeletalMesh != SourceSkeletalMesh)
	{
		ToolProperties->SourceSkinWeightProfile = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;
		ToolProperties->SourceLOD = "LOD0";
	}
		
	SourceSkeletalMesh = InSkeletalMesh;
	
	// reset to prepare for new mesh (or possibly no mesh)
	{
		if (SourcePreviewMesh)
		{
			SourcePreviewMesh->SetVisible(false);
			SourcePreviewMesh->Disconnect();
			SourcePreviewMesh = nullptr;
		}

		if (SourceTarget)
		{
			SourceTarget = nullptr;
		}

		if (MeshSelector)
		{
			MeshSelector->SetIsEnabled(false);
		}
	}
	
	if (!InSkeletalMesh)
	{
		WeightTool->UpdateSelectorState();
		return;
	}
	
	// create the preview mesh (this creates the skeletal mesh component in the world)
	SourcePreviewMesh = NewObject<UPreviewMesh>(this);
	SourcePreviewMesh->CreateInWorld(WeightTool->GetTargetWorld(), FTransform::Identity);

	// create a new tool target for this mesh
	SourceTarget = WeightTool->GetTargetManager()->BuildTarget(InSkeletalMesh, FToolTargetTypeRequirements());
	
	// move source mesh beside the main mesh (to the left in screen space)
	{
		UToolTarget* MainTarget = WeightTool->GetTarget();
		if (!MainTarget)
		{
			return;
		}

		IPrimitiveComponentBackedTarget* PrimitiveComponentTarget = CastChecked<IPrimitiveComponentBackedTarget>(MainTarget);
		const UPrimitiveComponent* PrimitiveComponent = PrimitiveComponentTarget->GetOwnerComponent();
		if (!PrimitiveComponent)
		{
			return;
		}
		
		FTransform Transform = UE::ToolTarget::GetLocalToWorldTransform(MainTarget);
		const FBoxSphereBounds TargetBounds = PrimitiveComponent->CalcBounds(Transform);
		const FBoxSphereBounds SourceBounds = InSkeletalMesh->GetBounds();
		FVector Location = Transform.GetLocation();
		Location.X -= TargetBounds.GetBoxExtrema(1).X;
		Location.X -= 1.1 * SourceBounds.GetBoxExtrema(1).X;
		Transform.SetLocation(Location);
		WeightTool->GetWeightToolProperties()->SourcePreviewOffset = Transform;
		SourcePreviewMesh->SetTransform(Transform);
	}

	// replace the dynamic mesh contained in the preview mesh
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(SourcePreviewMesh, SourceTarget);
	const EMeshLODIdentifier SourceLODId = GetLODId(ToolProperties->SourceLOD);
	const FGetMeshParameters SourceParams(true, SourceLODId);
	SourcePreviewMesh->ReplaceMesh(UE::ToolTarget::GetDynamicMeshCopy(SourceTarget, SourceParams));
	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(SourcePreviewMesh->GetRootComponent()));

	// setup materials and visibility of the preview mesh
	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(SourceTarget);
	SourcePreviewMesh->SetMaterials(MaterialSet.Materials);

	// setup the mesh selection for the source
	MeshSelector->SetMesh(SourcePreviewMesh, ToolProperties->SourcePreviewOffset);
	MeshSelector->SetIsEnabled(ToolProperties->EditingMode == EWeightEditMode::Mesh);
	MeshSelector->SetComponentSelectionMode(ToolProperties->ComponentSelectionMode);

	WeightTool->UpdateSelectorState();
}

void UWeightToolTransferManager::Shutdown()
{
	SetSourceMesh(nullptr);
	
	if (MeshSelector)
	{
		MeshSelector->Shutdown();
	}

	MeshSelector = nullptr;
	SourcePreviewMesh = nullptr;
	SourceTarget = nullptr;
}

void UWeightToolTransferManager::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (MeshSelector)
	{
		MeshSelector->Render(RenderAPI);	
	}
}

void UWeightToolTransferManager::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (MeshSelector)
	{
		MeshSelector->DrawHUD(Canvas, RenderAPI);
	}
}

void UWeightToolTransferManager::TransferWeights()
{
	if (!ensure(SourceTarget))
	{
		// to transfer weights from another mesh we need a source mesh
		// (UI should prevent us from getting here)
		const FText NotificationText = LOCTEXT("NoSourceTarget", "No source skeletal mesh specified. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	USkinWeightsPaintToolProperties* ToolProperties = WeightTool->GetWeightToolProperties();

	ISkeletalMeshBackedTarget* SkeletalMeshBackedTarget = Cast<ISkeletalMeshBackedTarget>(WeightTool->GetTarget()); 
	const USkeletalMesh* TargetSkeletalMesh = SkeletalMeshBackedTarget ? SkeletalMeshBackedTarget->GetSkeletalMesh() : nullptr;
	const bool bSameMesh = SourceSkeletalMesh == TargetSkeletalMesh;
	const bool bSameLOD = WeightTool->GetEditingLOD() == GetLODId(ToolProperties->SourceLOD);
	const bool bSameProfile = WeightTool->GetActiveSkinWeightProfileName() == ToolProperties->SourceSkinWeightProfile;
	const bool bVerticesSelectedOnTarget = ToolProperties->EditingMode == EWeightEditMode::Mesh && !WeightTool->GetMainMeshSelector()->GetSelectedVertices().IsEmpty();
	const bool bVerticesSelectedOnSource = ToolProperties->EditingMode == EWeightEditMode::Mesh && !MeshSelector->GetSelectedVertices().IsEmpty();
	const bool bHasAnySelectedVertices = bVerticesSelectedOnSource || bVerticesSelectedOnTarget;
	const bool bOneMeshHasSelectedVertices = bHasAnySelectedVertices && !(bVerticesSelectedOnSource && bVerticesSelectedOnTarget);
	
	// cannot transfer between same mesh/LOD/profile without selection (identical weights)
	if (bSameMesh && bSameLOD && bSameProfile && !bHasAnySelectedVertices)
	{
		const FText NotificationText = LOCTEXT("IdenticalSourceAndTarget", "Cannot copy weights from the same mesh, LOD and profile without anything selected. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	// can weights be transferred copying the attribute directly?
	if (bSameMesh && bSameLOD && !bSameProfile && bOneMeshHasSelectedVertices)
	{
		TransferWeightsFromSameMeshAndLOD();
	}
	else
	{
		TransferWeightsFromOtherMeshOrSubset();
	}
}

void UWeightToolTransferManager::TransferWeightsFromOtherMeshOrSubset()
{
	using UE::Geometry::FDynamicMeshAttributeSet;
	using UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute;
	using UE::Geometry::FTransferBoneWeights;
	using UE::AnimationCore::FBoneWeights;

	if (!ensure(SourceTarget))
	{
		return;
	}

	USkinWeightsPaintToolProperties* ToolProperties = WeightTool->GetWeightToolProperties();

	// get LOD IDs
	const EMeshLODIdentifier TargetLODId = WeightTool->GetEditingLOD();
	const EMeshLODIdentifier SourceLODId = GetLODId(ToolProperties->SourceLOD);

	// get selection
	TArray<int32> SourceTrianglesToIsolate;
	MeshSelector->GetSelectedTriangles(SourceTrianglesToIsolate);
	
	TArray<int> TargetSelectedVertices = WeightTool->GetMainMeshSelector()->GetSelectedVertices();
	
	// if transferring between the same mesh, ensure that the LODs are different and that the transfer is done on a subset
	ISkeletalMeshBackedTarget* SkeletalMeshBackedTarget = Cast<ISkeletalMeshBackedTarget>(WeightTool->GetTarget()); 
	const USkeletalMesh* TargetSkeletalMesh = SkeletalMeshBackedTarget ? SkeletalMeshBackedTarget->GetSkeletalMesh() : nullptr;
	const bool bSameMeshAndLOD = SourceSkeletalMesh == TargetSkeletalMesh && TargetLODId == SourceLODId;
	if (bSameMeshAndLOD)
	{
		if (SourceTrianglesToIsolate.IsEmpty() && TargetSelectedVertices.IsEmpty())
		{
			const FText NotificationText = LOCTEXT("SameLODAndNoSelection", "Cannot copy weights between the same LOD on the same mesh without anything selected. No weights were transferred.");
			ShowEditorMessage(ELogVerbosity::Error, NotificationText);
			return;
		}
	}

	FDynamicMesh3& EditedMesh = WeightTool->GetMesh();
	// get target dynamic mesh
	FDynamicMesh3 TargetMesh = EditedMesh;
	
	// get the source dynamic mesh and validate it
	FDynamicMesh3 TmpSourceCopy;
	auto GetSourceMesh = [this, bSameMeshAndLOD, SourceLODId, &TmpSourceCopy, &EditedMesh](const TArray<int32>& ToIsolate) -> const FDynamicMesh3*
	{
		// use the current EditedMesh to get the current data without having to commit
		if (bSameMeshAndLOD)
		{
			if (ToIsolate.IsEmpty())
			{
				return &EditedMesh;
			}
			
			// create a sub-mesh from the selected triangles to filter the transfer so that it only copies from the selected components
			const UE::Geometry::FDynamicSubmesh3 PartialSubMesh(&EditedMesh, ToIsolate);
			TmpSourceCopy = PartialSubMesh.GetSubmesh();
			// By default, submesh only have matching attributes enabled, so we need to explicitly copy the ones we need
			FDynamicMeshAttributeSet* SourceAttributes = TmpSourceCopy.Attributes();
			SourceAttributes->CopyBoneAttributes(*EditedMesh.Attributes());
			return &TmpSourceCopy;
		}

		// otherwise, get the corresponding DynamicMesh for that LOD
		const FGetMeshParameters SourceParams(true, SourceLODId);
		FDynamicMesh3 SourceMeshOrig = UE::ToolTarget::GetDynamicMeshCopy(SourceTarget, SourceParams);
		
		// create a sub-mesh from the selected triangles to filter the transfer so that it only copies from the selected components
		if (!ToIsolate.IsEmpty())
		{
			const UE::Geometry::FDynamicSubmesh3 PartialSubMesh(&SourceMeshOrig, ToIsolate);
			TmpSourceCopy = PartialSubMesh.GetSubmesh();
			// By default, submesh only have matching attributes enabled, so we need to explicitly copy the ones we need
			FDynamicMeshAttributeSet* SourceAttributes = TmpSourceCopy.Attributes();
			SourceAttributes->CopyBoneAttributes(*SourceMeshOrig.Attributes());
		}
		else
		{
			TmpSourceCopy = MoveTemp(SourceMeshOrig);
		}
		return &TmpSourceCopy;
	};
	
	const FDynamicMesh3* SourceMesh = GetSourceMesh(SourceTrianglesToIsolate);
	if (!ensure(SourceMesh))
	{
		const FText NotificationText = LOCTEXT("NoSourceMesh", "Cannot retrieve any source mesh form the current properties.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}
	
	if (!SourceMesh->HasAttributes() || !SourceMesh->Attributes()->HasBones())
	{
		const FText NotificationText = LOCTEXT("NoWeightsFoundInTransfer", "No skin weights were found in the source skeletal mesh. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}
	if (SourceMesh->Attributes()->GetNumBones() == 0)
	{
		const FText NotificationText = LOCTEXT("NoBonesFoundInTransfer", "No bones were found in the source skeletal mesh. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	FTransferBoneWeights TransferBoneWeights(SourceMesh, ToolProperties->SourceSkinWeightProfile);
	TransferBoneWeights.TransferMethod = FTransferBoneWeights::ETransferBoneWeightsMethod::InpaintWeights;
	
	FDynamicMeshAttributeSet* TargetAttributes = TargetMesh.Attributes();
	if (!TargetAttributes->HasBones())
	{
		TargetAttributes->CopyBoneAttributes(*SourceMesh->Attributes());
	}
	

	// NOTE should we expose all the options?
	// 	TransferBoneWeights.NormalThreshold;
	// 	TransferBoneWeights.SearchRadius
	// 	TransferBoneWeights.NumSmoothingIterations;
	// 	TransferBoneWeights.SmoothingStrength;
	// 	TransferBoneWeights.LayeredMeshSupport;
	// 	TransferBoneWeights.ForceInpaintWeightMapName;
	
	if (TransferBoneWeights.Validate() != UE::Geometry::EOperationValidationResult::Ok)
	{
		const FText NotificationText = LOCTEXT("TransferWeightsNotValid", "Transfer weights operation unable to validate meshes. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	const FName TargetProfile = WeightTool->GetActiveSkinWeightProfileName();
	if (!TransferBoneWeights.TransferWeightsToMesh(TargetMesh, TargetProfile))
	{
		const FText NotificationText = LOCTEXT("TransferWeightsUnknownIssue", "Transfer weights operation encountered an unknown issue. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	FDynamicMeshVertexSkinWeightsAttribute* TransferredSkinWeights = nullptr;
	TransferredSkinWeights = TargetAttributes->GetSkinWeightsAttribute(TargetProfile);

	// apply the weight changes as a transaction
	ApplyTranferredWeightsAsTransaction(TransferredSkinWeights, TargetSelectedVertices, TargetMesh);
}

void UWeightToolTransferManager::TransferWeightsFromSameMeshAndLOD()
{
	if (!ensure(SourceTarget))
	{
		return;
	}
	
	using UE::Geometry::FDynamicMeshAttributeSet;
	using UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute;
	using UE::AnimationCore::FBoneWeights;
	
	const USkinWeightsPaintToolProperties* ToolProperties = WeightTool->GetWeightToolProperties();
	
	// get target dynamic mesh
	const FName TargetProfile = WeightTool->GetActiveSkinWeightProfileName();
	FDynamicMesh3 TargetMesh = WeightTool->GetMesh();
	
	const FDynamicMeshAttributeSet* TargetAttributes = TargetMesh.Attributes();
	FDynamicMeshVertexSkinWeightsAttribute* TransferredSkinWeights = TargetAttributes->GetSkinWeightsAttribute(TargetProfile);
		
	const FDynamicMeshVertexSkinWeightsAttribute* SourceAttributes = TargetAttributes->GetSkinWeightsAttribute(ToolProperties->SourceSkinWeightProfile);
	check(SourceAttributes != nullptr);

	// this function assumes we are transferring between: Same Mesh, Same LOD, DIFFERENT Profile so attributes must not be identical.
	if (!ensure(SourceAttributes != TransferredSkinWeights))
	{
		const FText NotificationText = LOCTEXT("TransferBetweenSame", "Cannot transfer between same LOD & profile. No weights were transferred.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	// get subset of vertices to transfer weights on (from either source or target)
	TArray<int32> VertexSubset = MeshSelector->GetSelectedVertices();
	if (VertexSubset.IsEmpty())
	{
		// get vertex selection from the target mesh instead
		VertexSubset = WeightTool->GetMainMeshSelector()->GetSelectedVertices();
	}

	if (VertexSubset.IsEmpty())
	{
		// copy weights for ALL vertices in the mesh
		TransferredSkinWeights->Copy(*SourceAttributes);
	}
	else
	{
		// copy vertex weights for a subset of vertices
		for (const int32 VertexID: VertexSubset)
		{
			FBoneWeights BoneWeights;
			SourceAttributes->GetValue(VertexID, BoneWeights);
			TransferredSkinWeights->SetValue(VertexID, BoneWeights);
		}
	}

	// apply the weight changes as a transaction
	ApplyTranferredWeightsAsTransaction(TransferredSkinWeights, VertexSubset, TargetMesh);
}

bool UWeightToolTransferManager::CanTransferWeights() const
{
	return SourceTarget && WeightTool->GetTarget();
}

void UWeightToolTransferManager::OnPropertyModified(const USkinWeightsPaintToolProperties* WeightToolProperties, const FProperty* ModifiedProperty)
{
	if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, SourceSkeletalMesh))
	{
		SetSourceMesh(WeightToolProperties->SourceSkeletalMesh.Get());
	}

	if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, SourceLOD))
	{
		if (SourcePreviewMesh)
		{
			// reapply mesh (will use the new LOD)
			SetSourceMesh(SourceSkeletalMesh);
		}
	}

	if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, SourcePreviewOffset))
	{
		if (SourcePreviewMesh)
		{
			SourcePreviewMesh->SetTransform(WeightToolProperties->SourcePreviewOffset);
			MeshSelector->SetTransform(WeightToolProperties->SourcePreviewOffset);
		}
	}
}

void UWeightToolTransferManager::ApplyTranferredWeightsAsTransaction(
	const UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* InTransferredSkinWeights,
	const TArray<int32>& InVertexSubset,
	const FDynamicMesh3& InTargetMesh)
{
	if (!WeightTool->ShouldProceedWithAllBonesUnlocked())
	{
		return;
	}
	
	check(InTransferredSkinWeights);
	
	// weight edits for transaction
	FMultiBoneWeightEdits WeightEdits;

	// get the weight data (used for making edits)
	FSkinToolWeights& Weights = WeightTool->GetWeights();
	
	// spin through all the transferred skin weights and record a weight edit to apply as a transaction
	static constexpr float ZeroWeight = 0.f;
	const bool bUseSubset = !InVertexSubset.IsEmpty();

	const UWeightToolSelectionIsolator* Isolator = WeightTool->GetSelectionIsolator();
	const bool bIsMeshIsolated = Isolator->IsMeshIsolated();
	
	// InVertexSubset and Weights.PreChangeWeights/CurrentWeights reflect the current state of the mesh (partial or not) being edited
	// so the target is one or the other.
	// the full mesh is still needed to be able to get the original VertexID in order to get the transferred weights.  
	const FDynamicMesh3& TargetMesh = bIsMeshIsolated ? Isolator->GetPartialMesh() : InTargetMesh;

	int32 NumVertices = bUseSubset ? InVertexSubset.Num() : TargetMesh.MaxVertexID();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
	{
		int32 VertexID = INDEX_NONE;
		if (bUseSubset)
		{
			VertexID = InVertexSubset[VertexIndex];
		}
		else if (TargetMesh.IsVertex(VertexIndex))
		{
			VertexID = VertexIndex;
		}
		
		if (VertexID == INDEX_NONE)
		{
			continue;
		}

		// remove all weight on vertex
		const VertexWeights& VertexBoneWeights = Weights.PreChangeWeights[VertexID];
		if (!VertexBoneWeights.IsEmpty())
		{
			for (const FVertexBoneWeight& BoneWeight : VertexBoneWeights)
			{
				// when transferring weights we do prune influences because we would prefer the results to be identical to the source
				constexpr bool bPruneInfluence = true;
				WeightEdits.MergeSingleEdit(BoneWeight.BoneID, VertexID, ZeroWeight, bPruneInfluence, Weights.PreChangeWeights);
			}
		}
		else
		{
			// in the unlikely event that the target vertex has no weight, "fake" remove it from the root so that undo will put it back there
			constexpr bool bPruneInfluence = false;
			constexpr BoneIndex RootBoneIndex = 0;
			WeightEdits.MergeSingleEdit(RootBoneIndex, VertexID, ZeroWeight, bPruneInfluence, Weights.PreChangeWeights);
		}

		// map from partial to cleaned mesh (if isolated) 
		const int32 FullVertexID = bIsMeshIsolated ? Isolator->PartialToFullMeshVertexIndex(VertexID) : VertexID;


		// update with new weight
		UE::AnimationCore::FBoneWeights TransferredBoneWeights;
		InTransferredSkinWeights->GetValue(FullVertexID, TransferredBoneWeights);
		for (const UE::AnimationCore::FBoneWeight& BoneWeight: TransferredBoneWeights)
		{
			const int32 BoneIndex = BoneWeight.GetBoneIndex();					
			const float NewWeight = BoneWeight.GetWeight();
			constexpr bool bPruneInfluence = false;
			WeightEdits.MergeSingleEdit(BoneIndex, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
	}

	// apply the changes as a transaction
	const FText TransactionLabel = LOCTEXT("TransferWeightsChange", "Transfer skin weights.");
	WeightTool->ApplyWeightEditsAsTransaction(WeightEdits, TransactionLabel);
	
	// put the mesh back in its current pose
	Weights.Deformer.SetAllVerticesToBeUpdated();

	// notify user that weights were transferred.
	const FText NotificationText = LOCTEXT("WeightsTransferred", "Skin weights transferred.");
	ShowEditorMessage(ELogVerbosity::Log, NotificationText);
}

FName USkinWeightsPaintToolProperties::GetDesiredSkinWeightProfileForTool() const
{
	return bShowNewProfileName ? NewProfilePlaceHolderName() : SkinWeightProfileOption;
}

FSkinWeightBrushConfig& USkinWeightsPaintToolProperties::GetBrushConfig()
{
	return *BrushConfigs[BrushMode];
}

void FMultiBoneWeightEdits::MergeSingleEdit(
	const int32 BoneIndex,
	const int32 VertexID,
	const float NewWeight,
	bool bPruneInfluence,
	const TArray<VertexWeights>& InPreChangeWeights)
{
	if (!ensure(BoneIndex!=INDEX_NONE))
	{
		return;
	}
	
	if (bPruneInfluence)
	{
		// should never be pruning an influence while also trying to add weight to it
		if (!ensure(FMath::IsNearlyEqual(NewWeight, 0.f)))
		{
			return;
		}
	}

	// get the old weight of this influence and check whether it was already influences this vertex
	float OldWeight = 0.f;
	bool bWasAlreadyAnInfluence = false;
	const VertexWeights& VertexWeights = InPreChangeWeights[VertexID];
	for (const FVertexBoneWeight& BoneWeight : VertexWeights)
	{
		if (BoneWeight.BoneID == BoneIndex)
		{
			OldWeight = BoneWeight.Weight;
			bWasAlreadyAnInfluence = true;
			break;
		}
	}
	
	// record a modification of this vertex weight. Pruned influences are recorded
	// only in VerticesRemovedFrom below — keeping them out of NewWeights/OldWeights
	// stops ApplyEditsToCurrentWeights' scratchpad fold from re-adding the bone at
	// weight 0 after the strip-pruned pass has just removed it.
	FSingleBoneWeightEdits& BoneWeightEdit = PerBoneWeightEdits.FindOrAdd(BoneIndex);
	BoneWeightEdit.BoneIndex = BoneIndex;
	if (!bPruneInfluence)
	{
		BoneWeightEdit.NewWeights.Add(VertexID, NewWeight);
		BoneWeightEdit.OldWeights.FindOrAdd(VertexID, OldWeight);
	}

	// record when an influence is REMOVED (unless it was not connected to the vertex)
	if (bPruneInfluence && bWasAlreadyAnInfluence)
	{
		// record that this influence was REMOVED from this vertex
		BoneWeightEdit.VerticesRemovedFrom.AddUnique(VertexID);
		// remove from the AddedTo list in the off chance that vertex was added by a prior edit (last edit takes precedence)
		BoneWeightEdit.VerticesAddedTo.Remove(VertexID);
	}

	// record when an influence is ADDED
	if (!bPruneInfluence && !bWasAlreadyAnInfluence)
	{
		// record that this influence was ADDED to this vertex
		BoneWeightEdit.VerticesAddedTo.AddUnique(VertexID);
		// remove from the pruned list in the off chance that vertex was pruned by a prior edit (last edit takes precedence)
		BoneWeightEdit.VerticesRemovedFrom.Remove(VertexID);
	}
}

void FMultiBoneWeightEdits::MergeEdits(const FSingleBoneWeightEdits& BoneWeightEdits)
{
	ensure(BoneWeightEdits.BoneIndex!=INDEX_NONE);
	
	// make sure bone has an entry in the map of weight edits
	const int32 BoneIndex = BoneWeightEdits.BoneIndex;
	PerBoneWeightEdits.FindOrAdd(BoneIndex);
	PerBoneWeightEdits[BoneIndex].BoneIndex = BoneIndex;
	
	for (const TTuple<int32, float>& NewWeight : BoneWeightEdits.NewWeights)
	{
		int32 VertexIndex = NewWeight.Key;
		PerBoneWeightEdits[BoneIndex].NewWeights.Add(VertexIndex, NewWeight.Value);
		PerBoneWeightEdits[BoneIndex].OldWeights.FindOrAdd(VertexIndex, BoneWeightEdits.OldWeights[VertexIndex]);
	}
}

void FSkinToolDeformer::Initialize(const FReferenceSkeleton& InRefSkeleton, const TArray<FTransform>& PoseComponentSpace, const FDynamicMesh3& InMesh)
{
	// get all bone transforms in the reference pose store a copy in component space
	RefSkeleton = InRefSkeleton;
	const TArray<FTransform> &LocalSpaceBoneTransforms = RefSkeleton.GetRefBonePose();
	const int32 NumBones = LocalSpaceBoneTransforms.Num();
	InvCSRefPoseTransforms.SetNumUninitialized(NumBones);
	for (int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
	{
		const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
		const FTransform& LocalTransform = LocalSpaceBoneTransforms[BoneIndex];
		if (ParentBoneIndex != INDEX_NONE)
		{
			InvCSRefPoseTransforms[BoneIndex] = LocalTransform * InvCSRefPoseTransforms[ParentBoneIndex];
		}
		else
		{
			InvCSRefPoseTransforms[BoneIndex] = LocalTransform;
		}
	}
	
	for (int32 BoneIndex=0; BoneIndex<NumBones; ++BoneIndex)
	{
		// pre-invert the transforms so we don't have to at runtime
		InvCSRefPoseTransforms[BoneIndex] = InvCSRefPoseTransforms[BoneIndex].Inverse();

		// store map of bone indices to bone names
		FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
		BoneNames.Add(BoneName);
		BoneNameToIndexMap.Add(BoneName, BoneIndex);
	}

	// store reference pose vertex positions
	Mesh = &InMesh;
	
	RefPoseVertexPositions.SetNumUninitialized(Mesh->MaxVertexID());
	for (int32 VertexIndex : Mesh->VertexIndicesItr())
	{
		RefPoseVertexPositions[VertexIndex] = Mesh->GetVertex(VertexIndex);
	}

	// set all vertices to be updated on first tick
	SetAllVerticesToBeUpdated();

	// record "prev" bone transforms to detect change in pose
	PreviousPoseComponentSpace = PoseComponentSpace;
}

void FSkinToolDeformer::SetAllVerticesToBeUpdated()
{
	VerticesWithModifiedWeights.Reset();
	for (int32 VertID : Mesh->VertexIndicesItr())
	{
		VerticesWithModifiedWeights.Add(VertID);
	}
}

void FSkinToolDeformer::SetToRefPose(USkinWeightsPaintTool* Tool)
{
	// get ref pose
	const TArray<FTransform>& RefPoseLocalSpace = RefSkeleton.GetRefBonePose();
	// convert to global space and store in current pose
	FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RefPoseLocalSpace, RefPoseComponentSpace);
	// update mesh to new pose
	UpdateVertexDeformation(Tool, RefPoseComponentSpace);
}

void FSkinToolDeformer::UpdateVertexDeformation(
	USkinWeightsPaintTool* Tool,
	const TArray<FTransform>& PoseComponentSpace)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateDeformationTotal);

	// if no weights have been modified, we must check for a modified pose which requires re-calculation of skinning
	if (VerticesWithModifiedWeights.IsEmpty())
	{
		for (int32 BoneIndex=0; BoneIndex<PoseComponentSpace.Num(); ++BoneIndex)
		{
			if (!Tool->Weights.IsBoneWeighted[BoneIndex])
			{
				continue;
			}
			
			const FTransform& CurrentBoneTransform = PoseComponentSpace[BoneIndex];
			const FTransform& PrevBoneTransform = PreviousPoseComponentSpace[BoneIndex];
			if (!CurrentBoneTransform.Equals(PrevBoneTransform))
			{
				SetAllVerticesToBeUpdated();
				break;
			}
		}
	}

	if (VerticesWithModifiedWeights.IsEmpty())
	{
		return;
	}
	
	// update vertex positions
	UPreviewMesh* PreviewMesh = Tool->PreviewMesh;
	const TArray<VertexWeights>& CurrentWeights = Tool->Weights.CurrentWeights;
	PreviewMesh->DeferredEditMesh([this, &CurrentWeights, &PoseComponentSpace](FDynamicMesh3& InOutMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateDeformation);
		const TArray<int32> VertexIndices = VerticesWithModifiedWeights.Array();
		
		ParallelFor( VerticesWithModifiedWeights.Num(), [this, &VertexIndices, &InOutMesh, &PoseComponentSpace, &CurrentWeights](int32 Index)
		{
			const int32 VertexID = VertexIndices[Index];
			FVector VertexNewPosition = FVector::ZeroVector;
			const VertexWeights& VertexPerBoneData = CurrentWeights[VertexID];
			for (const FVertexBoneWeight& VertexData : VertexPerBoneData)
			{
				if (!ensure(VertexData.BoneID != INDEX_NONE))
				{
					continue;
				}
				const FTransform& CurrentTransform = PoseComponentSpace[VertexData.BoneID];
				VertexNewPosition += CurrentTransform.TransformPosition(VertexData.VertexInBoneSpace) * VertexData.Weight;
			}
			
			InOutMesh.SetVertex(VertexID, VertexNewPosition, false);
		});
	}, false);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::Positions, false);
	// Refresh polygroup-edges wireframe so its scene proxy follows the new vertex positions.
	if (Tool->MeshElementsDisplay)
	{
		Tool->MeshElementsDisplay->NotifyMeshChanged();
	}

	// what mode are we in?
	const EWeightEditMode EditingMode = Tool->WeightToolProperties->EditingMode;
	
	// update data structures used by the brush mode	
	if (EditingMode == EWeightEditMode::Brush)
	{
		// update vertex acceleration structure
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateVertexOctree);
			Tool->VerticesOctree->RemoveVertices(VerticesWithModifiedWeights);
			Tool->VerticesOctree->InsertVertices(VerticesWithModifiedWeights);
		}
		
		// update triangle acceleration structure
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateTriangleOctree);

			// create list of triangles that were affected by the vertices that were deformed
			TArray<int32>& AffectedTriangles = Tool->TrianglesToReinsert; // reusable buffer of triangles to update
			{
				AffectedTriangles.Reset();

				// reinsert all triangles containing an updated vertex
				const FDynamicMesh3* DynamicMesh = PreviewMesh->GetMesh();
				for (const int32 TriangleID : DynamicMesh->TriangleIndicesItr())
				{
					UE::Geometry::FIndex3i TriVerts = DynamicMesh->GetTriangle(TriangleID);
					bool bIsTriangleAffected = VerticesWithModifiedWeights.Contains(TriVerts[0]);
					bIsTriangleAffected = VerticesWithModifiedWeights.Contains(TriVerts[1]) ? true : bIsTriangleAffected;
					bIsTriangleAffected = VerticesWithModifiedWeights.Contains(TriVerts[2]) ? true : bIsTriangleAffected;
					if (bIsTriangleAffected)
					{
						AffectedTriangles.Add(TriangleID);
					}
				}
			}

			// ensure previous async update is finished before queuing the next one...
			Tool->TriangleOctreeFuture.Wait();
		
			// asynchronously update the octree, this normally finishes well before the next update
			// but in the unlikely event that it does not, it would result in a frame where the paint brush
			// is not perfectly aligned with the mesh; not a deal breaker.
			UE::Geometry::FDynamicMeshOctree3& OctreeToUpdate = *Tool->TrianglesOctree;
			Tool->TriangleOctreeFuture = Async(SkinPaintToolAsyncExecTarget, [&OctreeToUpdate, &AffectedTriangles]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::TriangleOctreeReinsert);	
				OctreeToUpdate.ReinsertTriangles(AffectedTriangles);
			});
		}
	}

	// update data structures used by the selection mode
	if (EditingMode == EWeightEditMode::Mesh)
	{
		// update AABB Tree for vertex selection
		TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateAABBTree);
		Tool->MeshSelector->UpdateAfterMeshDeformation();
	}

	// empty queue of vertices to update
	VerticesWithModifiedWeights.Reset();

	// record the skeleton state we used to update the deformations
	PreviousPoseComponentSpace = PoseComponentSpace;
}

void FSkinToolDeformer::SetVertexNeedsUpdated(int32 VertexIndex)
{
	VerticesWithModifiedWeights.Add(VertexIndex);
}

void FSkinToolWeights::InitializeSkinWeights(
	USkinWeightsPaintTool* InTool,
	const FDynamicMesh3& Mesh
	)
{
	static constexpr int32 RootBoneIndex = 0;
	static constexpr float FullWeight = 1.f;

	Tool = InTool;
	ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(Tool->GetTarget());

	// initialize deformer data
	Deformer.Initialize(SkeletonProvider->GetSkeleton(), Tool->GetComponentSpaceBoneTransforms(), Mesh);

	// initialize current weights (using compact format: num_verts * max_influences)
	using namespace UE::Geometry;
	using namespace UE::AnimationCore;

	int32 MaxVertexIndex = Mesh.MaxVertexID();
	TMap<BoneIndex, float> NormalizedWeights;
	CurrentWeights.SetNum(MaxVertexIndex);

	if (FDynamicMeshVertexSkinWeightsAttribute* VertexSkinWeights = Mesh.Attributes()->GetSkinWeightsAttribute(Profile))
	{
		for (int32 VertexIndex : Mesh.VertexIndicesItr())
		{
			// we have to normalize here because there are edges cases where skeletal meshes are loaded with non-normalized weights.
			// even when/if we fix these, it's best to be 100% sure that things are normalized since weight data can be generated by tools we don't control
			NormalizedWeights.Reset();
			int32 NumInfluences = 0;
			FBoneWeights BoneWeights;
			VertexSkinWeights->GetValue(VertexIndex, BoneWeights);
			for (const FBoneWeight BoneWeight : BoneWeights)
			{
				if (!ensure(NumInfluences < MAX_TOTAL_INFLUENCES))
				{
					// cut-off weights if they exceed max influences (this shouldn't happen)
					break;
				}
				NormalizedWeights.Add(BoneWeight.GetBoneIndex(), BoneWeight.GetWeight());
				++NumInfluences;
			}
			USkinWeightsPaintTool::NormalizeWeightMap(NormalizedWeights);

			// if there are no bone weights, default to root bone 
			if (NumInfluences == 0)
			{
				const FVector& RefPoseVertexPosition = Deformer.RefPoseVertexPositions[VertexIndex];
				const FTransform& InvRefPoseTransform = Deformer.InvCSRefPoseTransforms[RootBoneIndex];
				const FVector& BoneLocalPositionInRefPose = InvRefPoseTransform.TransformPosition(RefPoseVertexPosition);
				CurrentWeights[VertexIndex].Emplace(RootBoneIndex, BoneLocalPositionInRefPose, FullWeight);
				continue;
			}

			// load into the main weights data structure
			for (TTuple<BoneIndex, float>& SingleBoneWeight : NormalizedWeights)
			{
				BoneIndex BoneIndex = SingleBoneWeight.Key;
				const float Weight = SingleBoneWeight.Value;
				if (!ensure(Deformer.InvCSRefPoseTransforms.IsValidIndex(BoneIndex)))
				{
					UE_LOGF(LogMeshModelingToolsEditor, Warning, "InitializeSkinWeights: Invalid bone index provided (%d); falling back to 0 (root) as bone index.", BoneIndex);
					BoneIndex = 0;
				}
				const FVector& RefPoseVertexPosition = Deformer.RefPoseVertexPositions[VertexIndex];
				const FTransform& InvRefPoseTransform = Deformer.InvCSRefPoseTransforms[BoneIndex];
				const FVector& BoneLocalPositionInRefPose = InvRefPoseTransform.TransformPosition(RefPoseVertexPosition);
				CurrentWeights[VertexIndex].Emplace(BoneIndex, BoneLocalPositionInRefPose, Weight);
			}
		}
	}
	else
	{
		UE_LOGF(LogMeshModelingToolsEditor, Warning, "FSkinToolWeights::InitializeSkinWeights : failed to find skin weights on the target mesh, all initial weights will be empty")
	}
	
	// maintain duplicate weight map
	PreChangeWeights = CurrentWeights;

	// maintain relax-per stroke map
	MaxFalloffPerVertexThisStroke.SetNumZeroed(MaxVertexIndex);

	// Weight-edit scratch buffers used by ApplyEditsToCurrentWeights; sized to vertex count
	// and reused between calls (the pad and the tracker are cleared in place per stamp).
	PerWeightEditScratchPad.SetNum(MaxVertexIndex);
	PerWeightEditVertexChangeTracker.Init(MaxVertexIndex);

	// maintain bool-per-bone if weighted or not
	IsBoneWeighted.Init(false, Deformer.BoneNames.Num());
	for (const VertexWeights& VertexData : CurrentWeights)
	{
		for (const FVertexBoneWeight& VertexBoneData : VertexData)
		{
			if (VertexBoneData.Weight > BoneWeightThreshold)
			{
				IsBoneWeighted[VertexBoneData.BoneID] = true;
			}
		}
	}
}

bool FSkinToolWeights::IsInitialized() const
{
	return Tool.IsValid();
}

void FSkinToolWeights::CreateWeightEditForVertex(
	const int32 BoneToHoldIndex,
	const int32 VertexID,
	float NewWeightValue,
	FMultiBoneWeightEdits& WeightEdits)
{
	const TSet<int32>& LockedBoneIndices = Tool->GetFinalLockedBoneIndices();
	// this operation should never prune weights
	constexpr bool bPruneInfluence = false;

	if (LockedBoneIndices.Contains(BoneToHoldIndex))
	{
		return;
	}


	// calculate the sum of all the weights on this vertex (not including the one we currently applied)
	TArray<int32> RecordedBonesOnVertex;
	TArray<float> ValuesToNormalize;
	float Total = 0.0f;
	float TotalLockedWeights = 0.0f;
	const VertexWeights& VertexData = PreChangeWeights[VertexID];
	for (const FVertexBoneWeight& VertexBoneData : VertexData)
	{
		if (VertexBoneData.BoneID == BoneToHoldIndex)
		{
			continue;
		}

		if (!ensure(VertexBoneData.BoneID != INDEX_NONE))
		{
			continue;
		}

		if (!LockedBoneIndices.Contains(VertexBoneData.BoneID))
		{
			RecordedBonesOnVertex.Add(VertexBoneData.BoneID);
			ValuesToNormalize.Add(VertexBoneData.Weight);
			Total += VertexBoneData.Weight;
		}
		else
		{
			TotalLockedWeights += VertexBoneData.Weight;
		}
	}

	float MaxNewWeight = 1.f - TotalLockedWeights;
	// clamp new weight
	NewWeightValue = FMath::Clamp(NewWeightValue, 0.0f, MaxNewWeight);
	
	// assigning full weight to this vertex?
	if (FMath::IsNearlyEqual(NewWeightValue, MaxNewWeight))
	{
		// in this case normalization is trivial, just assign the full weight directly and zero all others
		const float FullWeight = MaxNewWeight;
		WeightEdits.MergeSingleEdit(BoneToHoldIndex, VertexID, FullWeight, bPruneInfluence, PreChangeWeights);

		// zero all others
		for (int32 i=0; i<ValuesToNormalize.Num(); ++i)
		{
			const int32 BoneIndex = RecordedBonesOnVertex[i];
			constexpr float NewWeight = 0.f;
			WeightEdits.MergeSingleEdit(BoneIndex, VertexID, NewWeight, bPruneInfluence, PreChangeWeights);
		}

		return;
	}

	// do any other influences have any weight on this vertex?
	//
	// In the case that:
	// 1. user applied any weight < 1 to this vertex AND
	// 2. there are NO other weights on this vertex
	// then we need to decide where to put the remaining influence...
	//
	// the logic here attempts to find a reasonable and "least surprising" place to put the remaining weight based on artist feedback
	const bool bVertexHasNoOtherWeightedInfluences = Total <= MinimumWeightThreshold;
	if (bVertexHasNoOtherWeightedInfluences)
	{
		// does this vertex have any other recorded influences on it?
		// a "recorded" influence here is one that used to have weight, but no longer does
		if (!RecordedBonesOnVertex.IsEmpty())
		{
			// this vertex:
			// 1. was previously weighted to other influences
			// 2. has subsequently had all other weight removed
			// In this case, we evenly split the remaining weight among the recorded influences

			// distribute remaining weight evenly over other recorded influences
			const float WeightToDistribute = (MaxNewWeight - NewWeightValue) / RecordedBonesOnVertex.Num();
			for (int32 i=0; i<ValuesToNormalize.Num(); ++i)
			{
				const int32 BoneIndex = RecordedBonesOnVertex[i];
				const float NewWeight = WeightToDistribute;
				WeightEdits.MergeSingleEdit(BoneIndex, VertexID, NewWeight, bPruneInfluence, PreChangeWeights);
			}

			// set current bone value to user assigned weight
			WeightEdits.MergeSingleEdit(BoneToHoldIndex, VertexID, NewWeightValue, bPruneInfluence, PreChangeWeights);
		}
		else
		{
			// this vertex:
			// 1. has no other recorded influences
			// 2. user is assigning PARTIAL weight to it (less than 1.0)
			// so in this case we push the remaining weight onto the PARENT bone
				
			// assign remaining weight to the parent
			const int32 ParentBoneIndex = GetParentBoneToWeightTo(BoneToHoldIndex);
			if (ParentBoneIndex == BoneToHoldIndex || Tool->GetFinalLockedBoneIndices().Contains(ParentBoneIndex))
			{
				// was unable to find parent OR child bone!
				// this could only happen if user is trying to remove weight from the ONLY bone in the whole skeleton
				// in this case just assign the full weight to the bone (there's no other valid configuration)
				// this is a "do nothing" operation, but at least it generates an undo transaction to let user know the input was received
				const float FullWeight = MaxNewWeight;
				WeightEdits.MergeSingleEdit(BoneToHoldIndex, VertexID, FullWeight, bPruneInfluence, PreChangeWeights);
			}
			else
			{
				// assign remaining weight to parent
				const float NewParentWeight = MaxNewWeight - NewWeightValue;
				WeightEdits.MergeSingleEdit(ParentBoneIndex, VertexID, NewParentWeight, bPruneInfluence, PreChangeWeights);
				// and assign user requested weight to the current bone
				WeightEdits.MergeSingleEdit(BoneToHoldIndex, VertexID, NewWeightValue, bPruneInfluence, PreChangeWeights);
			}
		}
		
		return;
	}

	// a normal weight edit where we assign the weight as requested
	// and split the remainder amongst the recorded influences
	{
		// calculate amount we have to spread across the other bones affecting this vertex
		const float AvailableTotal = MaxNewWeight - NewWeightValue;

		// normalize weights into available space not set by current bone
		for (int32 i=0; i<ValuesToNormalize.Num(); ++i)
		{
			float NormalizedValue = 0.f;
			if (AvailableTotal > MinimumWeightThreshold)
			{
				if (Total > KINDA_SMALL_NUMBER)
				{
					NormalizedValue = (ValuesToNormalize[i] / Total) * AvailableTotal;	
				}
				else
				{
					NormalizedValue = AvailableTotal / ValuesToNormalize.Num();
				}
			}
			const int32 BoneIndex = RecordedBonesOnVertex[i];
			const float NewWeight = NormalizedValue;
			WeightEdits.MergeSingleEdit(BoneIndex, VertexID, NewWeight, bPruneInfluence, PreChangeWeights);
		}

		// record current bone edit
		WeightEdits.MergeSingleEdit(BoneToHoldIndex, VertexID, NewWeightValue, bPruneInfluence, PreChangeWeights);
	}
}

void FSkinToolWeights::ApplyCurrentWeightsToMesh(FDynamicMesh3& Mesh, TFunction<int32(int32)> WeightsIndexToMeshIndex)
{
	if (!IsInitialized())
	{
		return;
	}
	
	using namespace UE::Geometry;
	FDynamicMeshVertexSkinWeightsAttribute* VertexWeightAttrs = Mesh.Attributes()->GetSkinWeightsAttribute(Profile);
	
	UE::AnimationCore::FBoneWeightsSettings Settings;
	Settings.SetNormalizeType(UE::AnimationCore::EBoneWeightNormalizeType::None);

	TArray<UE::AnimationCore::FBoneWeight> BoneWeightsToApply;
	BoneWeightsToApply.Reserve(UE::AnimationCore::MaxInlineBoneWeightCount);

	if (!WeightsIndexToMeshIndex)
	{
		if (!ensure(CurrentWeights.Num() == Mesh.MaxVertexID()))
		{
			// weights are out of sync with mesh you're trying to apply them to
			return;
		}
	}

	for (int32 WeightsIndex = 0; WeightsIndex < CurrentWeights.Num(); WeightsIndex++)
	{
		BoneWeightsToApply.Reset();

		const VertexWeights& VertexWeights = CurrentWeights[WeightsIndex];
		for (const FVertexBoneWeight& SingleBoneWeight : VertexWeights)
		{
			if (!ensure(SingleBoneWeight.BoneID != INDEX_NONE))
			{
				continue;
			}
			BoneWeightsToApply.Add(UE::AnimationCore::FBoneWeight(SingleBoneWeight.BoneID, SingleBoneWeight.Weight));
		}

		int32 VertexIndex = WeightsIndex;
		if (WeightsIndexToMeshIndex)
		{
			VertexIndex = WeightsIndexToMeshIndex(WeightsIndex);
		}
		VertexWeightAttrs->SetValue(VertexIndex, UE::AnimationCore::FBoneWeights::Create(BoneWeightsToApply, Settings));
	}
}

float FSkinToolWeights::GetWeightOfBoneOnVertex(
	const int32 BoneIndex,
	const int32 VertexID,
	const TArray<VertexWeights>& InVertexWeights)
{
	const VertexWeights& VertexWeights = InVertexWeights[VertexID];
	for (const FVertexBoneWeight& BoneWeight : VertexWeights)
	{
		if (BoneWeight.BoneID == BoneIndex)
		{
			return BoneWeight.Weight;
		}
	}
	
	return 0.f;
}

void FSkinToolWeights::RemoveInfluenceFromVertex(
	const VertexIndex InVertexID,
	const BoneIndex InBoneID,
	TArray<VertexWeights>& InOutVertexWeights)
{
	// should never be pruning a vertex that doesn't exist
	if (!ensure(InOutVertexWeights.IsValidIndex(InVertexID)))
	{
		return;
	}
	
	VertexWeights& SingleVertexWeights = InOutVertexWeights[InVertexID];
	const int32 IndexOfBoneInVertex = SingleVertexWeights.IndexOfByPredicate([&InBoneID](const FVertexBoneWeight& CurrentVertexWeight)
	{
		return CurrentVertexWeight.BoneID == InBoneID;
	});
	
	// can't prune an influence that doesn't exist on a vertex
	// this may happen if the calling code already pruned the influence to avoid normalization weights
	if (!ensure(IndexOfBoneInVertex != INDEX_NONE))
	{
		return;
	}
	
	SingleVertexWeights.RemoveAt(IndexOfBoneInVertex);
}

void FSkinToolWeights::SyncWeightBuffers()
{
	PreChangeWeights = CurrentWeights;

	for (int32 i=0; i<MaxFalloffPerVertexThisStroke.Num(); ++i)
	{
		MaxFalloffPerVertexThisStroke[i] = 0.f;
	}
}

float FSkinToolWeights::SetCurrentFalloffAndGetMaxFalloffThisStroke(int32 VertexID, float CurrentStrength)
{
	float& MaxFalloffThisStroke = MaxFalloffPerVertexThisStroke[VertexID];
	if (MaxFalloffThisStroke < CurrentStrength)
	{
		MaxFalloffThisStroke = CurrentStrength;
	}
	return MaxFalloffThisStroke;
}

void FSkinToolWeights::ApplyEditsToCurrentWeights(const FMultiBoneWeightEdits& Edits, FVertexChangeTracker& InOutVertexChangeTracker)
{
	// Strip pruned influences first so the scratch pad seeds from a post-removal CurrentWeights.
	for (const TTuple<BoneIndex, FSingleBoneWeightEdits>& BoneWeightEdits : Edits.PerBoneWeightEdits)
	{
		BoneIndex InfluenceToRemove = BoneWeightEdits.Key;
		for (const VertexIndex VertexID : BoneWeightEdits.Value.VerticesRemovedFrom)
		{
			RemoveInfluenceFromVertex(VertexID, InfluenceToRemove, CurrentWeights);
		}
	}

	// Pass 1: fold each (bone, vertex, weight) edit into the per-vertex scratchpad.
	for (const TTuple<BoneIndex, FSingleBoneWeightEdits>& BoneEdits : Edits.PerBoneWeightEdits)
	{
		const FSingleBoneWeightEdits& WeightEdits = BoneEdits.Value;
		const int32 EditedBoneIndex = WeightEdits.BoneIndex;
		check(EditedBoneIndex != INDEX_NONE);

		for (const TTuple<int32, float>& NewWeight : WeightEdits.NewWeights)
		{
			const int32 VertexID = NewWeight.Key;
			const float Weight = NewWeight.Value;

			UnlimitedVertexWeights& VertexWeights = PerWeightEditScratchPad[VertexID];
			if (PerWeightEditVertexChangeTracker.Mark(VertexID))
			{
				// First touch this stamp: seed from CurrentWeights so subsequent edits
				// merge against the post-strip baseline.
				VertexWeights.Reserve(CurrentWeights[VertexID].Num());
				for (const FVertexBoneWeight& BW : CurrentWeights[VertexID])
				{
					VertexWeights.Emplace(BW.BoneID, BW.Weight);
				}
			}

			bool bFoundExisting = false;
			for (BoneWeight& Pair : VertexWeights)
			{
				if (Pair.Key == EditedBoneIndex)
				{
					Pair.Value = Weight;
					bFoundExisting = true;
					break;
				}
			}
			if (!bFoundExisting)
			{
				VertexWeights.Emplace(EditedBoneIndex, Weight);
			}
		}
	}

	// Pass 2: sort+truncate (conditional), renormalize (conditional), patch back into CurrentWeights.
	constexpr float NormalizationTolerance = 1.e-4f;
	for (const int32 VertexID : PerWeightEditVertexChangeTracker.Vertices)
	{
		UnlimitedVertexWeights& VertexWeights = PerWeightEditScratchPad[VertexID];

		const bool bWasTruncated = VertexWeights.Num() > UE::AnimationCore::MaxInlineBoneWeightCount;
		if (bWasTruncated)
		{
			// Descending sort so SetNum drops the smallest weights from the tail; zero-weight
			// influences sink first.
			VertexWeights.Sort([](const BoneWeight& A, const BoneWeight& B)
			{
				return A.Value > B.Value;
			});
			VertexWeights.SetNum(UE::AnimationCore::MaxInlineBoneWeightCount, EAllowShrinking::No);
		}

		// Truncation above is the only path that legitimately denormalizes VertexWeights —
		// sort + SetNum drops the smallest weights from the tail. Every other entry point
		// either comes from CreateWeightEditForVertex or PruneWeights, both of which produce
		// NewWeights summing to 1 - TotalLockedWeights; the strip-pruned-influences pass +
		// survivor fold preserves that invariant.
		float Total = 0.f;
		for (const BoneWeight& Pair : VertexWeights)
		{
			Total += Pair.Value;
		}

		auto Renormalize = [&VertexWeights, Total]()
		{
			if (Total > KINDA_SMALL_NUMBER)
			{
				for (BoneWeight& Pair : VertexWeights)
				{
					Pair.Value /= Total;
				}
			}
		};

		if (bWasTruncated)
		{
			// Truncation dropped weight off the tail — renormalize the survivors.
			Renormalize();
		}
		else if (!ensure(FMath::IsNearlyEqual(Total, 1.0f, NormalizationTolerance)))
		{
			// Upstream edit producer emitted denormalized weights. Surface via ensure
			// (catches the bug in dev), then recover so users don't see broken weights.
			Renormalize();
		}

		// Patch in place so surviving bones keep their cached VertexInBoneSpace.
		SkinPaintTool::VertexWeights& Dest = CurrentWeights[VertexID];
		for (int32 i = Dest.Num() - 1; i >= 0; --i)
		{
			const BoneIndex Bone = Dest[i].BoneID;
			const bool bStillPresent = VertexWeights.ContainsByPredicate(
				[Bone](const BoneWeight& Pair) { return Pair.Key == Bone; });
			if (!bStillPresent)
			{
				Dest.RemoveAt(i);
			}
		}
		for (const BoneWeight& Pair : VertexWeights)
		{
			const int32 ExistingIdx = Dest.IndexOfByPredicate(
				[&Pair](const FVertexBoneWeight& BW) { return BW.BoneID == Pair.Key; });
			if (ExistingIdx != INDEX_NONE)
			{
				Dest[ExistingIdx].Weight = Pair.Value;
			}
			else
			{
				const FVector PosLocalToBone = Deformer.InvCSRefPoseTransforms[Pair.Key].TransformPosition(
					Deformer.RefPoseVertexPositions[VertexID]);
				Dest.Emplace(Pair.Key, PosLocalToBone, Pair.Value);
			}
		}
		Deformer.SetVertexNeedsUpdated(VertexID);

		VertexWeights.Reset();
		InOutVertexChangeTracker.Mark(VertexID);
	}

	PerWeightEditVertexChangeTracker.Clear();

	for (const TTuple<BoneIndex, FSingleBoneWeightEdits>& BoneWeightEdits : Edits.PerBoneWeightEdits)
	{
		UpdateIsBoneWeighted(BoneWeightEdits.Key);
	}
}

void FSkinToolWeights::UpdateIsBoneWeighted(BoneIndex BoneToUpdate)
{
	IsBoneWeighted[BoneToUpdate] = false;
	for (const VertexWeights& VertexData : CurrentWeights)
	{
		for (const FVertexBoneWeight& VertexBoneData : VertexData)
		{
			if (VertexBoneData.BoneID == BoneToUpdate && VertexBoneData.Weight > UE::AnimationCore::BoneWeightThreshold)
			{
				IsBoneWeighted[BoneToUpdate] = true;
				break;
			}
		}
		if (IsBoneWeighted[BoneToUpdate])
		{
			break;
		}
	}
}

BoneIndex FSkinToolWeights::GetParentBoneToWeightTo(BoneIndex ChildBone)
{
	int32 ParentBoneIndex = Deformer.RefSkeleton.GetParentIndex(ChildBone);

	// are we at the root? (no parent)
	if (ParentBoneIndex == INDEX_NONE)
	{
		ParentBoneIndex = 0; // fallback to root
		
		// in this case return the first child bone, if there is one
		// NOTE: this allows the user to forcibly remove all weight on the root bone, without having another recorded influence on it
		TArray<int32> RootsChildren;
		Deformer.RefSkeleton.GetDirectChildBones(0, RootsChildren);
		if (!RootsChildren.IsEmpty())
		{
			ParentBoneIndex = RootsChildren[0];
		}
	}

	return ParentBoneIndex;
}

void FMeshSkinWeightsChange::RestoreVertexFromSnapshot(
	USkinWeightsPaintTool* Tool,
	const int32 VertexID,
	const VertexWeights& Snapshot)
{
	FSkinToolWeights& Weights = Tool->Weights;
	Weights.CurrentWeights[VertexID] = Snapshot;
	Weights.PreChangeWeights[VertexID] = Snapshot;
	Weights.Deformer.SetVertexNeedsUpdated(VertexID);
	Tool->PerColorUpdateVertexChangeTracker.Mark(VertexID);
}

void FMeshSkinWeightsChange::Apply(UObject* Object)
{
	USkinWeightsPaintTool* Tool = CastChecked<USkinWeightsPaintTool>(Object);
	Tool->ExternalUpdateSkinWeightLayer(SkinWeightProfile);

	const TArray<int32>& ChangedVertices = PerTransactionVertexChangeTracker.Vertices;
	check(ChangedVertices.Num() == AfterStates.Num());
	for (int32 i = 0; i < ChangedVertices.Num(); ++i)
	{
		RestoreVertexFromSnapshot(Tool, ChangedVertices[i], AfterStates[i]);
	}

	for (const BoneIndex Bone : ChangedBones)
	{
		Tool->Weights.UpdateIsBoneWeighted(Bone);
	}

	Tool->OnWeightsChanged.Broadcast();
}

void FMeshSkinWeightsChange::Revert(UObject* Object)
{
	USkinWeightsPaintTool* Tool = CastChecked<USkinWeightsPaintTool>(Object);
	Tool->ExternalUpdateSkinWeightLayer(SkinWeightProfile);

	const TArray<int32>& ChangedVertices = PerTransactionVertexChangeTracker.Vertices;
	check(ChangedVertices.Num() == BeforeStates.Num());
	for (int32 i = 0; i < ChangedVertices.Num(); ++i)
	{
		RestoreVertexFromSnapshot(Tool, ChangedVertices[i], BeforeStates[i]);
	}

	for (const BoneIndex Bone : ChangedBones)
	{
		Tool->Weights.UpdateIsBoneWeighted(Bone);
	}

	Tool->OnWeightsChanged.Broadcast();
}

void FMeshSkinWeightsChange::StoreMultipleWeightEdits(const FMultiBoneWeightEdits& WeightEdits)
{
	for (const TTuple<BoneIndex, FSingleBoneWeightEdits>& Pair : WeightEdits.PerBoneWeightEdits)
	{
		ChangedBones.Add(Pair.Key);
		for (const TTuple<int32, float>& NewWeight : Pair.Value.NewWeights)
		{
			PerTransactionVertexChangeTracker.Mark(NewWeight.Key);
		}
		for (const int32 VertexID : Pair.Value.VerticesAddedTo)
		{
			PerTransactionVertexChangeTracker.Mark(VertexID);
		}
		for (const int32 VertexID : Pair.Value.VerticesRemovedFrom)
		{
			PerTransactionVertexChangeTracker.Mark(VertexID);
		}
	}
}

void FMeshSkinWeightsChange::Close(USkinWeightsPaintTool* Tool)
{
	const FSkinToolWeights& Weights = Tool->Weights;
	const TArray<int32>& ChangedVertices = PerTransactionVertexChangeTracker.Vertices;
	BeforeStates.Reset(ChangedVertices.Num());
	AfterStates.Reset(ChangedVertices.Num());
	for (const int32 VertexID : ChangedVertices)
	{
		BeforeStates.Add(Weights.PreChangeWeights[VertexID]);
		AfterStates.Add(Weights.CurrentWeights[VertexID]);
	}
	// Mask is no longer needed once Close has frozen the Vertices list — Apply/Revert
	// only walk Vertices. Free the mask memory; keep Vertices for replay.
	PerTransactionVertexChangeTracker.Mask.Empty();
}

FSkinWeightsPaintToolCommands::FSkinWeightsPaintToolCommands()
	: TCommands<FSkinWeightsPaintToolCommands>
	(
		TEXT("SkinWeightsPaintTool"),
		NSLOCTEXT("Contexts", "SkinWeightsPaintToolCommands", "Skin Weight Paint Tool Commands"),
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{
}

void FSkinWeightsPaintToolCommands::RegisterCommands()
{
	UI_COMMAND(IsolateSelectedBones, "Isolate Selected (Unlock)", "Isolate selected bones by locking all other bones, Skin weights for locked bones cannot be changed", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleSelectedBonesLockState, "Toggle Selected Lock State", "Toggle the lock state for selected bones, Skin weights for locked bones cannot be changed", EUserInterfaceActionType::Button, FInputChord(EKeys::L));
	
	UI_COMMAND(LockSelectedBones, "Lock Selected", "Lock selected bones, Skin weights for locked bones cannot be changed", EUserInterfaceActionType::Button, FInputChord());	
	UI_COMMAND(UnlockSelectedBones, "Unlock Selected", "Unlock selected bones, Skin weights for locked bones cannot be changed", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(LockAllBones, "Lock All", "Lock all bones, Skin weights for locked bones cannot be changed", EUserInterfaceActionType::Button, FInputChord());	
	UI_COMMAND(UnlockAllBones, "Unlock All", "Unlock all bones, Skin weights for locked bones cannot be changed", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(BypassLocks, "Bypass Locks", "Disable skin weight locking, existing locks are preserved but disabled", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::L));
	
}

/*
 * ToolBuilder
 */

UMeshSurfacePointTool* USkinWeightsPaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	USkinWeightsPaintTool* Tool = NewObject<USkinWeightsPaintTool>(SceneState.ToolManager);
	Tool->Init(SceneState);
	return Tool;
}

const FToolTargetTypeRequirements& USkinWeightsPaintToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		USkeletonProvider::StaticClass(),
		});
	return TypeRequirements;
}

void USkinWeightsPaintTool::Init(const FToolBuilderState& InSceneState)
{
	const UContextObjectStore* ContextObjectStore = InSceneState.ToolManager->GetContextObjectStore();
	EditorContext = ContextObjectStore->FindContext<USkeletalMeshEditorContextObjectBase>();
	PersonaModeManagerContext = ContextObjectStore->FindContext<UPersonaEditorModeManagerContext>();
	TargetManager = InSceneState.TargetManager;
}

void USkinWeightsPaintTool::Setup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::Setup);
	
	UDynamicMeshBrushTool::Setup();

	ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(Target);
	const FReferenceSkeleton& RefSkeleton = SkeletonProvider->GetSkeleton();

	IDynamicMeshProvider* DynamicMeshProvider = CastChecked<IDynamicMeshProvider>(Target);
	EditedMesh = DynamicMeshProvider->GetDynamicMesh();
	
	// create a custom set of properties inheriting from the base tool properties
	WeightToolProperties = NewObject<USkinWeightsPaintToolProperties>(this);

	// Per-mesh cache key — mirrors UMorphTargetMaskTool (MorphTargetMaskTool.cpp:74-85).
	{
		FString AssetPath = TEXT("UnknownAsset");
		if (const ISkeletalMeshBackedTarget* SkeletalMeshBackedTarget = Cast<ISkeletalMeshBackedTarget>(Target))
		{
			if (const USkeletalMesh* SkeletalMesh = SkeletalMeshBackedTarget->GetSkeletalMesh())
			{
				AssetPath = SkeletalMesh->GetPathName();
			}
		}
		WeightToolProperties->PropertySetCacheIdentifier = MoveTemp(AssetPath);
	}
	WeightToolProperties->RestoreProperties(this, WeightToolProperties->PropertySetCacheIdentifier);
	WeightToolProperties->WeightTool = this;
	WeightToolProperties->bSpecifyRadius = true;
	// watch for skin weight layer changes
	WeightToolProperties->SkinWeightProfileOption = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;
	int32 WatcherIndex = WeightToolProperties->WatchProperty(WeightToolProperties->SkinWeightProfileOption, [this](FName) { OnActiveSkinWeightProfileChanged(); });
	WeightToolProperties->SilentUpdateWatcherAtIndex(WatcherIndex);
	
	WeightToolProperties->NewSkinWeightProfile = GetValidNewProfileName(WeightToolProperties->NewSkinWeightProfile);
	WatcherIndex = WeightToolProperties->WatchProperty(WeightToolProperties->NewSkinWeightProfile, [this](FName) { OnNewSkinWeightProfileChanged(); });
	WeightToolProperties->SilentUpdateWatcherAtIndex(WatcherIndex);
	
	WeightToolProperties->SourceSkeletalMesh = nullptr;
    WeightToolProperties->SourcePreviewOffset = FTransform::Identity;
	
	ActiveSkinWeightProfileName = WeightToolProperties->GetDesiredSkinWeightProfileForTool();
	
	// listen to linear color ramp changes
	WeightToolProperties->LinearColorRamp.OnColorCurveChangedDelegate.AddUObject(this, &USkinWeightsPaintTool::OnLinearColorRampChanged);

	// replace the base brush properties
	ReplaceToolPropertySource(BrushProperties, WeightToolProperties);
	BrushProperties = WeightToolProperties;
	// brush render customization
	BrushStampIndicator->bScaleNormalByStrength = true;
	BrushStampIndicator->SecondaryLineThickness = 1.0f;
	BrushStampIndicator->SecondaryLineColor = FLinearColor::Yellow;
	RecalculateBrushRadius();

	// default to the root bone as current bone
	PendingCurrentBone = CurrentBone = RefSkeleton.IsValidIndex(0)? RefSkeleton.GetBoneName(0) : NAME_None;

	// configure preview mesh
	SkeletalMeshToolsHelper::SetupPreviewTangentMode(Cast<UDynamicMeshComponent>(PreviewMesh->GetRootComponent()));
	PreviewMesh->SetShadowsEnabled(false);

	// create the transfer manager
	TransferManager = NewObject<UWeightToolTransferManager>();
	TransferManager->InitialSetup(this, GetViewportClient());

	// create the isolated selection manager
	SelectionIsolator = NewObject<UWeightToolSelectionIsolator>();
	SelectionIsolator->InitialSetup(this);

	// setup selection for the main mesh
	MeshSelector = NewObject<UToolMeshSelector>(this);
	auto OnSelectionChangedLambda = [this](){OnSelectionChanged.Broadcast();};
	MeshSelector->InitialSetup(TargetWorld.Get(), this, OnSelectionChangedLambda);

	SkeletonReader = NewObject<USkeletonModifier>();
	SkeletonReader->SetReferenceSkeleton(RefSkeleton);
	SkeletonReader->SetReadOnly(true);

	// Retained-mode visualization dedicated to polygroup boundary edges.
	// Created before the isolation init below so UpdateCurrentlyEditedMesh's NotifyMeshChanged
	// hook has a valid visualizer to talk to from the first call.
	MeshElementsDisplay = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsDisplay->CreateInWorld(TargetWorld.Get(), PreviewMesh->GetTransform());
	if (ensure(MeshElementsDisplay->Settings))
	{
		// Only polygroup-boundary edges are rendered; the tool's "show polygroup edges" config
		// toggle drives the visualizer's overall bVisible.
		MeshElementsDisplay->Settings->ShowAllElements(false);
		MeshElementsDisplay->Settings->bShowGroupBoundaries = true;

		MeshElementsDisplay->Settings->bVisible           = WeightToolProperties->bShowGroupBoundaries;
		MeshElementsDisplay->Settings->GroupBoundaryColor = WeightToolProperties->GroupBoundaryColor;
		MeshElementsDisplay->Settings->ThicknessScale     = WeightToolProperties->GroupBoundaryThickness;
	}
	MeshElementsDisplay->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc)
	{
		if (PreviewMesh) { PreviewMesh->ProcessMesh(ProcessFunc); }
	});
	
	// run all initialization for mesh/weights.
	
	// seed the isolator and initialize on the submesh if default isolation is provided
	const TArray<int32>& DefaultIsolationTriangles = GetDefaultIsolationTriangles();
	if (!DefaultIsolationTriangles.IsEmpty())
	{
		// Then isolate to external boundary
		SelectionIsolator->SetTrianglesToIsolate(DefaultIsolationTriangles);
		SelectionIsolator->UpdateIsolatedSelection();
	}
	else
	{
		UpdateCurrentlyEditedMesh(EditedMesh);
	}

	// Best-effort recovery of the per-mesh lock cache against the now-initialized
	// BoneNameToIndexMap. Names that no longer resolve (renamed/removed bones) drop
	// silently — under-locking beats mis-locking a name collision.
	LockedBoneIndices.Reset();
	for (const FName& BoneName : WeightToolProperties->LockedBoneNames)
	{
		const BoneIndex Index = GetBoneIndexFromName(BoneName);
		if (Index != INDEX_NONE)
		{
			LockedBoneIndices.Add(Index);
		}
	}

	// bind the skeletal mesh editor context
	if (EditorContext.IsValid())
	{
		EditorContext->BindTo(this);
	}

	// trigger last used mode
	ToggleEditingMode();

	// modify viewport render settings to optimize for painting weights
	FPreviewProfileController PreviewProfileController;
	PreviewProfileToRestore = PreviewProfileController.GetActiveProfile();
	PreviewProfileController.SetActiveProfile(UDefaultEditorProfiles::EditingProfileName.ToString());

	// set focus to viewport so brush hotkey works
	SetFocusInViewport();
	
	// inform user of tool keys
	// TODO talk with UX team about viewport overlay to show hotkeys
	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartSkinWeightsPaint", "Paint per-bone skin weights. [ and ] change brush size, Ctrl to Erase/Subtract, Shift to Smooth"),
		EToolMessageLevel::UserNotification);
}

void USkinWeightsPaintTool::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	Super::DrawHUD(Canvas, RenderAPI);

	if (MeshSelector)
	{
		MeshSelector->DrawHUD(Canvas, RenderAPI);
	}
			
	GetWeightTransferManager()->DrawHUD(Canvas, RenderAPI);
}

void USkinWeightsPaintTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	switch (WeightToolProperties->EditingMode)
	{
	case EWeightEditMode::Brush:
		Super::Render(RenderAPI);
	case EWeightEditMode::Mesh:
		{
			if (MeshSelector)
			{
				MeshSelector->Render(RenderAPI);	
			}

			GetWeightTransferManager()->Render(RenderAPI);
			
			break;
		}
	default:
		return;
	}
}

FName USkinWeightsPaintTool::GetActiveSkinWeightProfileName() const
{
	return ActiveSkinWeightProfileName;
}

namespace SkinWeightsPaintTool::Private
{
	const FString BaseVolumetricBrushIndicatorGizmoType = TEXT("VolumetricBrushIndicatorGizmoType");
}

void USkinWeightsPaintTool::SetupBrushStampIndicator()
{
	if (!BrushStampIndicator)
	{
		// register and spawn brush indicator gizmo
		GetToolManager()->GetPairedGizmoManager()->RegisterGizmoType(SkinWeightsPaintTool::Private::BaseVolumetricBrushIndicatorGizmoType, NewObject<UVolumetricBrushStampIndicatorBuilder>());
		BrushStampIndicator = GetToolManager()->GetPairedGizmoManager()->CreateGizmo<UVolumetricBrushStampIndicator>(SkinWeightsPaintTool::Private::BaseVolumetricBrushIndicatorGizmoType, FString(), this);
		if (UVolumetricBrushStampIndicator* VolumetricBrushIndicator = Cast<UVolumetricBrushStampIndicator>(BrushStampIndicator))
		{
			VolumetricBrushIndicator->MakeBrushIndicatorMesh(this, GetTargetWorld());

			// Keep the volumetric brush rendered with its normal material when the viewport is in a
			// wireframe view mode (Wireframe / LitWireframe / etc.) - otherwise the brush sphere
			// renders as wireframe lines and obscures the brush footprint while painting.
			if (UPreviewMesh* BrushIndicatorPreviewMesh = VolumetricBrushIndicator->GetPreviewMesh())
			{
				BrushIndicatorPreviewMesh->SetOverrideWireframeRenderMaterial(
					ToolSetupUtil::GetDefaultBrushVolumeMaterial(GetToolManager()));
			}
		}
	}
}

void USkinWeightsPaintTool::UpdateBrushStampIndicator()
{
	Super::UpdateBrushStampIndicator();
}

void USkinWeightsPaintTool::ShutdownBrushStampIndicator()
{
	if (BrushStampIndicator)
	{
		GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(BrushStampIndicator);
		BrushStampIndicator = nullptr;
		GetToolManager()->GetPairedGizmoManager()->DeregisterGizmoType(SkinWeightsPaintTool::Private::BaseVolumetricBrushIndicatorGizmoType);
	}
}

void USkinWeightsPaintTool::UpdateBrushIndicators()
{
	const bool bUseVolumnetricBrush = WeightToolProperties ? (WeightToolProperties->GetBrushConfig().FalloffMode == EWeightBrushFalloffMode::Volume) : false;
	const bool bIsInBrushMode = WeightToolProperties ? (WeightToolProperties->EditingMode == EWeightEditMode::Brush) : false;

	if (BrushStampIndicator)
	{
		BrushStampIndicator->bVisible = (bIsInBrushMode && !bUseVolumnetricBrush);
		if (UVolumetricBrushStampIndicator* VolumetricBrushIndicator = Cast<UVolumetricBrushStampIndicator>(BrushStampIndicator))
		{
			VolumetricBrushIndicator->SetVolumetricBrushVisible(bIsInBrushMode && bUseVolumnetricBrush);
		}
	}
}

FBox USkinWeightsPaintTool::GetWorldSpaceFocusBox()
{
	if (!WeightToolProperties)
	{
		return PreviewMesh->GetActor()->GetComponentsBoundingBox();
	}
	
	// 1. Prioritize Brush & Vertex modes
	switch (WeightToolProperties->EditingMode)
	{
	case EWeightEditMode::Brush:
			{
				const FVector Radius(CurrentBrushRadius);
				return FBox(LastBrushStamp.WorldPosition - Radius, LastBrushStamp.WorldPosition + Radius);
			}
	case EWeightEditMode::Mesh:
		{
			static const TArray<int32> Dummy;
			const TArray<int32>& SelectedVertices = MeshSelector ? MeshSelector->GetSelectedVertices() : Dummy;
			if (!SelectedVertices.IsEmpty())
			{
				const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
				const FTransform3d Transform(PreviewMesh->GetTransform());
				FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
				for (const int32 VertexID : SelectedVertices)
				{
					Bounds.Contain(Transform.TransformPosition(Mesh->GetVertex(VertexID)));
				}
				return static_cast<FBox>(Bounds);
			}
		}
		break;
	case EWeightEditMode::Bones:
	default:
		break;
	}

	// 2. Fallback on framing selected bones (if there are any)
	// TODO, there are several places in the engine that frame bone selections. Let's consolidate this logic.
	if (!SelectedBoneIndices.IsEmpty())
	{
		const FReferenceSkeleton& RefSkeleton = Weights.Deformer.RefSkeleton;
		const TArray<FTransform>& CurrentBoneTransforms = GetComponentSpaceBoneTransforms();
		if (!CurrentBoneTransforms.IsEmpty())
		{
			FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
			for (const int32 BoneIndex : SelectedBoneIndices)
			{
				// add bone position and position of all direct children to the frame bounds
				const FVector BonePosition = CurrentBoneTransforms[BoneIndex].GetLocation();
				Bounds.Contain(BonePosition);
				TArray<int32> ChildrenIndices;
				RefSkeleton.GetDirectChildBones(BoneIndex, ChildrenIndices);
				if (ChildrenIndices.IsEmpty())
				{
					constexpr float SingleBoneSize = 10.f;
					FVector BoneOffset = FVector(SingleBoneSize, SingleBoneSize, SingleBoneSize);
					Bounds.Contain(BonePosition + BoneOffset);
					Bounds.Contain(BonePosition - BoneOffset);
				}
				else
				{
					for (const int32 ChildIndex : ChildrenIndices)
					{
						Bounds.Contain(CurrentBoneTransforms[ChildIndex].GetLocation());
					}
				}	
			}
			if (Bounds.MaxDim() > FMathf::ZeroTolerance)
			{
				return static_cast<FBox>(Bounds);
			}
		}
	}

	// 3. Finally, fallback on component bounds if nothing else is selected
	static constexpr bool bNonColliding = true;
	FBox PreviewBox = PreviewMesh->GetActor()->GetComponentsBoundingBox(bNonColliding);
	// expand the bounds by the source transfer mesh (if there is one and it's visible)
	if (UPreviewMesh* SourcePreviewMesh = GetWeightTransferManager()->GetPreviewMesh())
	{
		if (AActor* SourceActor = SourcePreviewMesh->GetActor())
		{
			PreviewBox += SourceActor->GetComponentsBoundingBox(bNonColliding);
		}
	}

	return PreviewBox;
}

void USkinWeightsPaintTool::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	// toggle Relax mode on while shift key is held, then swap back to prior mode on release
	if (ModifierID == ShiftModifier)
	{
		if (bIsOn)
		{
			// when shift key is pressed
			if (!bShiftToggle)
			{
				WeightToolProperties->PriorBrushMode = WeightToolProperties->BrushMode;
				WeightToolProperties->SetBrushMode(EWeightEditOperation::Relax);
			}
		}
		else
		{
			// when shift key is released
			if (bShiftToggle)
			{
				WeightToolProperties->SetBrushMode(WeightToolProperties->PriorBrushMode);
			}
		}
	}

	Super::OnUpdateModifierState(ModifierID, bIsOn);
}

FInputRayHit USkinWeightsPaintTool::CanBeginClickDragSequence(const FInputDeviceRay& InPressPos)
{
	// NOTE: this function is only overridden to prevent left-click fly camera behavior while brushing
	// this should eventually be removed once we have a clear way of disabling the fly-cam mode
	
	if (WeightToolProperties->EditingMode != EWeightEditMode::Brush)
	{
		return FInputRayHit(); // allow other behaviors to capture mouse while not brushing
	}
	
	const FInputRayHit Hit = Super::CanBeginClickDragSequence(InPressPos);
	if (Hit.bHit)
	{
		return Hit;
	}

	// always return a hit so we always capture and prevent accidental camera movement
	return FInputRayHit(TNumericLimits<float>::Max());
}

void USkinWeightsPaintTool::OnTick(float DeltaTime)
{
	if (SelectionIsolator)
	{
		SelectionIsolator->UpdateIsolatedSelection();
	}

	if (MeshElementsDisplay)
	{
		// Manually tick watchers for since it is not a registered property set
		MeshElementsDisplay->Settings->CheckAndUpdateWatched();
		MeshElementsDisplay->OnTick(DeltaTime);
	}
	
	if (bStampPending)
	{
		ApplyStamp(LastStamp);
		bStampPending = false;
	}

	if (PendingCurrentBone.IsSet())
	{
		UpdateCurrentBone(*PendingCurrentBone);
		PendingCurrentBone.Reset();
	}

	if (bVertexColorsNeedUpdated)
	{
		UpdateVertexColorForAllVertices();
		bVertexColorsNeedUpdated = false;
	}

	if (bConfigDirty)
	{
		if (WeightToolProperties)
		{
			WeightToolProperties->SaveConfig();
		}
		bConfigDirty = false;
	}

	if (!PerColorUpdateVertexChangeTracker.IsEmpty())
	{
		UpdateVertexColorForSubsetOfVertices();
		PerColorUpdateVertexChangeTracker.Clear();
	}

	// sparsely updates vertex positions (only on vertices with modified weights)
	Weights.Deformer.UpdateVertexDeformation(this, GetComponentSpaceBoneTransforms());
}

void USkinWeightsPaintTool::OnLinearColorRampChanged(TArray<FRichCurve*> ChangedCurves)
{
	bVertexColorsNeedUpdated = true;
	bConfigDirty = true;
}

void USkinWeightsPaintTool::UpdateCurrentlyEditedMesh(const FDynamicMesh3& InDynamicMesh)
{
	// Should always have a valid profile before initialization
	check(IsProfileValid(GetActiveSkinWeightProfileName()))
	
	// update the preview mesh in the viewport
	PreviewMesh->ReplaceMesh(InDynamicMesh);

	// Rebuild the element-visualizer wireframe against the new mesh topology.
	if (MeshElementsDisplay)
	{
		MeshElementsDisplay->NotifyMeshChanged();
	}
	PreviewMesh->EditMesh([](FDynamicMesh3& Mesh)
	{
		Mesh.EnableAttributes();
		Mesh.Attributes()->DisablePrimaryColors();
		Mesh.Attributes()->EnablePrimaryColors();
		Mesh.Attributes()->PrimaryColors()->CreateFromPredicate([](int ParentVID, int TriIDA, int TriIDB){return true;}, 0.f);
	});
	SetDisplayVertexColors(WeightToolProperties->ColorMode != EWeightColorMode::FullMaterial);
	
	// update vertices & triangle octrees (this must be done after PreviewMesh has been updated)
	InitializeOctrees();

	IPrimitiveComponentBackedTarget* PrimitiveComponentTarget = CastChecked<IPrimitiveComponentBackedTarget>(GetTarget());
	
	// update the mesh selection mechanic (this must be done after PreviewMesh has been updated)
	MeshSelector->SetMesh(PreviewMesh, PrimitiveComponentTarget->GetOwnerComponent()->GetComponentTransform());

	// update weights
	Weights = FSkinToolWeights();
	Weights.Profile = GetActiveSkinWeightProfileName();
	Weights.InitializeSkinWeights(this, InDynamicMesh);
	// Size the per-tick color tracker to the new mesh; the previous mesh's tracker
	// may have been smaller (or larger, with isolation).
	PerColorUpdateVertexChangeTracker.Init(InDynamicMesh.MaxVertexID());
	bVertexColorsNeedUpdated = true;
	
	// update smooth operator (this must be done after PreviewMesh & Weights have been updated)
	InitializeSmoothWeightsOperator(InDynamicMesh);

	// after any mesh change, the mirror tables will need rebuilt next time mirroring is used
	MirrorData.SetNeedsReinitialized();
}

void USkinWeightsPaintToolProperties::SetComponentMode(EComponentSelectionMode InComponentMode)
{
	// DEPRECATED 5.8
	constexpr bool bConvertSelection = false;
	SetComponentSelectionMode(InComponentMode, bConvertSelection);
}

void USkinWeightsPaintToolProperties::SetComponentSelectionMode(const EComponentSelectionMode InComponentMode, const bool bInConvertSelection)
{
	using namespace UE::MeshModelingToolsEditorOnly::SelectionUtils;
	
	if (!WeightTool)
	{
		return;
	}

	TSharedPtr<FWeightToolScopedConvertSelection> ConvertSelection;
	if (bInConvertSelection)
	{
		const UPreviewMesh* PreviewMesh = WeightTool->GetPreviewMesh();
		const FDynamicMesh3* DynamicMesh = PreviewMesh ? PreviewMesh->GetMesh() : nullptr;
		if (DynamicMesh)
		{
			ConvertSelection = MakeShared<FWeightToolScopedConvertSelection>(*WeightTool, *DynamicMesh, InComponentMode);
		}
	}
	else
	{
		UToolMeshSelector* MeshSelector = WeightTool->GetMainMeshSelector();
		UPolygonSelectionMechanic* SelectionMechanic = MeshSelector ? MeshSelector->GetSelectionMechanic() : nullptr;
		if (SelectionMechanic)
		{	
			SelectionMechanic->ClearSelection();
		}
	}

	ComponentSelectionMode = InComponentMode;
	
	WeightTool->UpdateSelectorState();
	WeightTool->SetFocusInViewport();
}

void USkinWeightsPaintToolProperties::SetFalloffMode(EWeightBrushFalloffMode InFalloffMode)
{
	SetSharedFalloffMode(InFalloffMode);
	SaveConfig();

	WeightTool->UpdateBrushIndicators();
	WeightTool->SetFocusInViewport();
}

void USkinWeightsPaintToolProperties::SetSharedRadius(const float InRadius)
{
	// Radius behaves as a shared brush field — propagate to every per-mode config so a
	// later SetBrushMode pull sees the same value regardless of which mode is active.
	BrushConfigAdd.Radius = InRadius;
	BrushConfigReplace.Radius = InRadius;
	BrushConfigMultiply.Radius = InRadius;
	BrushConfigRelax.Radius = InRadius;
}

void USkinWeightsPaintToolProperties::SetSharedFalloffMode(const EWeightBrushFalloffMode InFalloffMode)
{
	// FalloffMode behaves as a shared brush field — propagate to every per-mode config.
	BrushConfigAdd.FalloffMode = InFalloffMode;
	BrushConfigReplace.FalloffMode = InFalloffMode;
	BrushConfigMultiply.FalloffMode = InFalloffMode;
	BrushConfigRelax.FalloffMode = InFalloffMode;
}

void USkinWeightsPaintToolProperties::SetColorMode(EWeightColorMode InColorMode)
{
	ColorMode = InColorMode;
	WeightTool->SetDisplayVertexColors(ColorMode!=EWeightColorMode::FullMaterial);
	WeightTool->SetFocusInViewport();
}

void USkinWeightsPaintToolProperties::SetBrushMode(EWeightEditOperation InBrushMode)
{
	BrushMode = InBrushMode;

	// sync base tool settings with the mode specific saved values
	// these are the source of truth for the base class viewport rendering of brush
	// (Radius and FalloffMode are stored per-mode for config compat but kept in lockstep
	// across all modes via SetSharedRadius / SetSharedFalloffMode, so this pull is consistent)
	BrushRadius = GetBrushConfig().Radius;
	BrushStrength = GetBrushConfig().Strength;
	BrushFalloffAmount = GetBrushConfig().Falloff;

	WeightTool->UpdateBrushIndicators();
	WeightTool->SetFocusInViewport();
}

void FIsolateSelectionChange::Apply(UObject* Object)
{
	USkinWeightsPaintTool* Tool = Cast<USkinWeightsPaintTool>(Object);
	if (Tool)
	{
		Tool->GetSelectionIsolator()->SetTrianglesToIsolate(IsolatedTrianglesAfter);
	}	
}

void FIsolateSelectionChange::Revert(UObject* Object)
{
	USkinWeightsPaintTool* Tool = Cast<USkinWeightsPaintTool>(Object);
	if (Tool)
	{
		Tool->GetSelectionIsolator()->SetTrianglesToIsolate(IsolatedTrianglesBefore);
	}
}

FString FIsolateSelectionChange::ToString() const
{
	return FToolCommandChange::ToString();
}

EMeshLODIdentifier USkinWeightsPaintTool::GetEditingLOD()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetEditingLOD();
	}
	
	return EMeshLODIdentifier::LOD0;
}

const TArray<FTransform>& USkinWeightsPaintTool::GetComponentSpaceBoneTransforms()
{
	if (EditorContext.IsValid())
	{
		return EditorContext->GetComponentSpaceBoneTransforms(Target);
	}

	if (DefaultComponentSpaceBoneTransforms.IsEmpty())
	{
		ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(GetTarget());
		SkeletonProvider->GetSkeleton().GetBoneAbsoluteTransforms(DefaultComponentSpaceBoneTransforms);
	}
		
	return DefaultComponentSpaceBoneTransforms;
}

void USkinWeightsPaintTool::ToggleBoneManipulation(bool bEnable)
{
	if (EditorContext.IsValid())
	{
		EditorContext->ToggleBoneManipulation(bEnable);
	}
}


bool USkinWeightsPaintTool::HitTest(const FRay& Ray, FHitResult& OutHit)
{
	const bool bIsInBrushMode = WeightToolProperties->EditingMode == EWeightEditMode::Brush;
	if (!bIsInBrushMode)
	{
		return false;
	}
	
	// do not query the triangle octree until all async ops are finished
	TriangleOctreeFuture.Wait();
	
	// put ray in local space of skeletal mesh component
	// currently no way to transform skeletal meshes in the editor,
	// but at some point in the future we may add the ability to move parts around
	const IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	const FTransform3d CurTargetTransform(TargetComponent->GetWorldTransform());
	FRay3d LocalRay(
		CurTargetTransform.InverseTransformPosition((FVector3d)Ray.Origin),
		CurTargetTransform.InverseTransformVector((FVector3d)Ray.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);
	
	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();

	FViewCameraState StateOut;
	GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(StateOut);
	FVector3d LocalEyePosition(CurTargetTransform.InverseTransformPosition((FVector3d)StateOut.Position));

	int32 TriID = IndexConstants::InvalidID;

	if (BrushProperties && BrushProperties->bHitBackFaces)
	{
		TriID = TrianglesOctree->FindNearestHitObject(LocalRay);
	}
	else
	{
		TriID = TrianglesOctree->FindNearestHitObject(
			LocalRay,
			[this, Mesh, &LocalEyePosition](int TriangleID)
			{
				FVector3d Normal, Centroid;
				double Area;
				Mesh->GetTriInfo(TriangleID, Normal, Area, Centroid);
				return Normal.Dot((Centroid - LocalEyePosition)) < 0;
			});
	}

	if (TriID != IndexConstants::InvalidID)
	{	
		FastTriWinding::FTriangle3d Triangle;
		Mesh->GetTriVertices(TriID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
		UE::Geometry::FIntrRay3Triangle3d Query(LocalRay, Triangle);
		Query.Find();

		StampLocalPos = LocalRay.PointAt(Query.RayParameter);
		TriangleUnderStamp = TriID;

		OutHit.FaceIndex = TriID;
		OutHit.Distance = Query.RayParameter;
		OutHit.Normal = CurTargetTransform.TransformVector(Mesh->GetTriNormal(TriID));
		OutHit.ImpactPoint = CurTargetTransform.TransformPosition(StampLocalPos);
		return true;
	}
	
	return false;
}

void USkinWeightsPaintTool::OnBeginDrag(const FRay& WorldRay)
{
	UDynamicMeshBrushTool::OnBeginDrag(WorldRay);

	bInvertStroke = GetCtrlToggle();
	BeginChange();
	StartStamp = UBaseBrushTool::LastBrushStamp;
	LastStamp = StartStamp;
	bStampPending = true;
	LastStampWorldPosition.Reset();
	LastStampWorldRadius = 0.0;
	LongTransactions.Open(LOCTEXT("PaintWeightChange", "Paint skin weights."), GetToolManager());
}

void USkinWeightsPaintTool::OnUpdateDrag(const FRay& WorldRay)
{
	UDynamicMeshBrushTool::OnUpdateDrag(WorldRay);
	
	LastStamp = UBaseBrushTool::LastBrushStamp;
	bStampPending = true;
}

void USkinWeightsPaintTool::OnEndDrag(const FRay& Ray)
{
	UDynamicMeshBrushTool::OnEndDrag(Ray);

	bInvertStroke = false;
	bStampPending = false;

	if (ActiveChange)
	{
		// close change, record transaction
		const FText TransactionLabel = LOCTEXT("PaintWeightChange", "Paint skin weights.");
		EndChange(TransactionLabel);
		LongTransactions.Close(GetToolManager());
	}
}

bool USkinWeightsPaintTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	UDynamicMeshBrushTool::OnUpdateHover(DevicePos);
	return true;
}

double USkinWeightsPaintTool::EstimateMaximumTargetDimension()
{
	if (const IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target))
	{
		const UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(TargetComponent->GetOwnerComponent());
		return Component->CalcBounds(FTransform::Identity).SphereRadius * 2.0f;
	}
	
	return Super::EstimateMaximumTargetDimension();
}

void USkinWeightsPaintTool::CalculateVertexROI(
	const FBrushStampData& InStamp,
	TArray<VertexIndex>& OutVertexIDs,
	TArray<float>& OutVertexFalloffs)
{
	using namespace UE::Geometry;

	TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::CalculateVertexROI);

	// Accumulating Relax bypasses the per-stroke max-falloff clamp so stamps compound;
	// other modes keep the clamp so lower-falloff re-stamps can't dial back a higher one.
	const bool bIsAccumulatingRelax =
		WeightToolProperties->BrushMode == EWeightEditOperation::Relax
		&& WeightToolProperties->RelaxBrushAdvancedConfig.bAccumulate;

	auto DistanceToFalloff = [this, bIsAccumulatingRelax](int32 InVertexID, float InDistanceSq) -> float
	{
		const float CurrentFalloff = CalculateBrushFalloff(FMath::Sqrt(InDistanceSq));
		return bIsAccumulatingRelax
			? CurrentFalloff
			: Weights.SetCurrentFalloffAndGetMaxFalloffThisStroke(InVertexID, CurrentFalloff);
	};
	
	if (WeightToolProperties->GetBrushConfig().FalloffMode == EWeightBrushFalloffMode::Volume)
	{
		const IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
		const FTransform3d Transform(TargetComponent->GetWorldTransform());
		const FVector3d StampPosLocal = Transform.InverseTransformPosition(InStamp.WorldPosition);
		const float RadiusSqr = CurrentBrushRadius * CurrentBrushRadius;
		const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
		const FAxisAlignedBox3d QueryBox(StampPosLocal, CurrentBrushRadius);
		VerticesOctree->RangeQuery(QueryBox,
			[&](int32 VertexID) { return FVector3d::DistSquared(Mesh->GetVertex(VertexID), StampPosLocal) < RadiusSqr; },
			OutVertexIDs);

		OutVertexFalloffs.Reserve(OutVertexIDs.Num());
		for (const int32 VertexID : OutVertexIDs)
		{
			const float DistSq = FVector3d::DistSquared(Mesh->GetVertex(VertexID), StampPosLocal);

			OutVertexFalloffs.Add(DistanceToFalloff(VertexID, DistSq));
		}
		
		return;
	}

	if (WeightToolProperties->GetBrushConfig().FalloffMode == EWeightBrushFalloffMode::Surface)
	{
		// create the ExpMap generator, computes vertex polar coordinates in a plane tangent to the surface
		const FDynamicMesh3* Mesh = PreviewMesh->GetPreviewDynamicMesh();
		FFrame3d SeedFrame = Mesh->GetTriFrame(TriangleUnderStamp);
		
		const FTransform3d MeshWorldTransform(PreviewMesh->GetTransform());
		SeedFrame.Origin = MeshWorldTransform.InverseTransformPosition(FVector3d(InStamp.WorldPosition));
		
		TMeshLocalParam<FDynamicMesh3> Param(Mesh);
		// Planar projection collapses opposite sides of a sharp fold onto the same UV magnitude, which
		// lets the brush pick up vertices on the far side of the fold (e.g. back of a tilted-up neck).
		// ExponentialMapUpwindAvg propagates UVs along the surface so magnitude tracks geodesic distance.
		Param.ParamMode = ELocalParamTypes::ExponentialMapUpwindAvg;
		const FIndex3i TriVerts = Mesh->GetTriangle(TriangleUnderStamp);
		Param.ComputeToMaxDistance(SeedFrame, TriVerts, InStamp.Radius * 1.5f);
		// store vertices under the brush and their distances from the stamp
		const float StampRadSq = FMath::Pow(InStamp.Radius, 2);
		for (int32 VertexID : Mesh->VertexIndicesItr())
		{
			if (!Param.HasUV(VertexID))
			{
				continue;
			}
			
			FVector2d UV = Param.GetUV(VertexID);
			const float DistSq = UV.SizeSquared();
			if (DistSq >= StampRadSq)
			{
				continue;
			}

			OutVertexFalloffs.Add(DistanceToFalloff(VertexID, DistSq));
			OutVertexIDs.Add(VertexID);
		}
		
		return;
	}
	
	checkNoEntry();
}

FVector4f USkinWeightsPaintTool::GetColorOfVertex(VertexIndex InVertexIndex, BoneIndex InCurrentBoneIndex) const
{
	switch (WeightToolProperties->ColorMode)
	{
	case EWeightColorMode::Greyscale:
		{
			if (InCurrentBoneIndex == INDEX_NONE)
			{
				return FLinearColor::Black; // with no bone selected, all vertices are drawn black
			}
			const float Value = Weights.GetWeightOfBoneOnVertex(InCurrentBoneIndex, InVertexIndex, Weights.CurrentWeights);
			return FMath::Lerp(FLinearColor::Black, FLinearColor::White, Value);
		}
	case EWeightColorMode::Ramp:
		{
			if (InCurrentBoneIndex == INDEX_NONE)
			{
				return FLinearColor::Black; // with no bone selected, all vertices are drawn black
			}

			// get weight value
			float Value = Weights.GetWeightOfBoneOnVertex(InCurrentBoneIndex, InVertexIndex, Weights.CurrentWeights);
			
			return WeightToolProperties->LinearColorRamp.GetLinearColorValue(Value);
		}
	case EWeightColorMode::BoneColors:
		{
			FVector4f Color = FVector4f::Zero();
			const VertexWeights& VertexWeights = Weights.CurrentWeights[InVertexIndex];
			for (const FVertexBoneWeight& BoneWeight : VertexWeights)
			{
				if (BoneWeight.Weight < KINDA_SMALL_NUMBER)
				{
					continue;
				}
				
				const float Value = InCurrentBoneIndex == BoneWeight.BoneID ? 1.0f: 0.6f;
				constexpr float Saturation = 0.75f;
				const FLinearColor BoneColor = SkeletalDebugRendering::GetSemiRandomColorForBone(BoneWeight.BoneID, Value, Saturation);
				Color = FMath::Lerp(Color, BoneColor, BoneWeight.Weight);
			}
			return Color;
		}
	case EWeightColorMode::FullMaterial:
		return FLinearColor::White;
	default:
		checkNoEntry();
		return FLinearColor::Black;
	}	
}


void USkinWeightsPaintTool::UpdateVertexColorForAllVertices()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateVertexColors);
	
	const int32 CurrentBoneIndex = GetBoneIndexFromName(CurrentBone);
	
	// update mesh with new value colors
	PreviewMesh->DeferredEditMesh([this, &CurrentBoneIndex](FDynamicMesh3& Mesh)
	{
		UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();
		for (const int32 ElementId : ColorOverlay->ElementIndicesItr())
		{
			const int32 VertexID = ColorOverlay->GetParentVertex(ElementId);	
			ColorOverlay->SetElement(ElementId, GetColorOfVertex(VertexID, CurrentBoneIndex));
		}
		
	}, false);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::VertexColors, false);
}

void USkinWeightsPaintTool::UpdateVertexColorForSubsetOfVertices()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::UpdateVertexColors);
	
	PreviewMesh->DeferredEditMesh([this](FDynamicMesh3& Mesh)
		{
			if (CurrentBone == NAME_None)
			{
				
			}
			TArray<int> ElementIds;
			UE::Geometry::FDynamicMeshColorOverlay* ColorOverlay = Mesh.Attributes()->PrimaryColors();
			const int32 CurrentBoneIndex = GetBoneIndexFromName(CurrentBone);
			for (const int32 VertexID : PerColorUpdateVertexChangeTracker.Vertices)
			{
				FVector4f NewColor(GetColorOfVertex(VertexID, CurrentBoneIndex));
				ColorOverlay->GetVertexElements(VertexID, ElementIds);
				for (const int32 ElementId : ElementIds)
				{
					ColorOverlay->SetElement(ElementId, NewColor);
				}
				ElementIds.Reset();
			}
			
		}, false);
	PreviewMesh->NotifyDeferredEditCompleted(UPreviewMesh::ERenderUpdateMode::FastUpdate, EMeshRenderAttributeFlags::VertexColors, false);
}

float USkinWeightsPaintTool::CalculateBrushFalloff(float Distance) const
{
	const float f = FMathd::Clamp(1.f - BrushProperties->BrushFalloffAmount, 0.f, 1.f);
	float d = Distance / CurrentBrushRadius;
	double w = 1;
	if (d > f)
	{
		d = FMathd::Clamp((d - f) / (1.0 - f), 0.0, 1.0);
		w = (1.0 - d * d);
		w = w * w * w;
	}
	return w;
}

bool USkinWeightsPaintTool::IsInputIsolationValidOnOutput() const
{
	return true;
}

bool USkinWeightsPaintTool::IsInputSelectionValidOnOutput()
{
	return true;
}

namespace SkinWeightsPaintTool::Private
{
	namespace Commands
	{
		constexpr const TCHAR* ToggleSelectedLockState = TEXT("Toggle Selected Lock State");
		constexpr const TCHAR* ToggleBypassLocks       = TEXT("Toggle Bypass Locks");
	}

	constexpr const TCHAR* const HotkeyHintCommands[] = {
		Commands::ToggleSelectedLockState,
		Commands::ToggleBypassLocks,
	};
}

void USkinWeightsPaintTool::GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const
{
	for (const TCHAR* CommandName : SkinWeightsPaintTool::Private::HotkeyHintCommands)
	{
		IHotkeyHintProvider::TryAppendCommandHint(OutHints, ActionCommandsContextName, CommandName);
	}

	// B+drag is owned by BrushEditBehavior (UBaseBrushTool::BrushEditBehavior), not a FUICommandInfo.
	OutHints.Add({ LOCTEXT("HintBrushSize", "Brush Size"), LOCTEXT("HintBrushSizeChord", "B + drag left/right") });
	OutHints.Add({ LOCTEXT("HintBrushStrength", "Brush Strength"), LOCTEXT("HintBrushStrengthChord", "B + drag up/down") });
	OutHints.Add({ LOCTEXT("HintSmoothBrush", "Smooth Brush"), LOCTEXT("HintSmoothBrushChord", "Shift") });
	OutHints.Add({ LOCTEXT("HintInvertBrush", "Invert Brush"), LOCTEXT("HintInvertBrushChord", "Ctrl") });
}

void USkinWeightsPaintTool::RegisterActions(FInteractiveToolActionSet& ActionSet)
{
	using namespace SkinWeightsPaintTool::Private;

	Super::RegisterActions(ActionSet);
	
	
	int32 ActionId = static_cast<int32>(EStandardToolActions::BaseClientDefinedActionID) + 400;
	auto GetActionId = [&ActionId]
		{
			return ActionId++;
		};

	ActionSet.RegisterAction(this, GetActionId(), Commands::ToggleSelectedLockState,
		LOCTEXT("ToggleSelectedLockState", "Toggle Selected Lock State"),
		LOCTEXT("ToggleSelectedLockStateDesc", "Toggle the lock state for selected bone(s)"),
		EModifierKey::None, EKeys::L,
		[this]()
		{
			ToggleSelectedBonesLockState();
			GetToolManager()->DisplayMessage(LOCTEXT("ToggleLockMessage", "Toggled the lock state for selected bone(s) "), EToolMessageLevel::UserNotification);
		});

	ActionSet.RegisterAction(this, GetActionId(), Commands::ToggleBypassLocks,
		LOCTEXT("ToggleBypassLock", "Toggle Bypass Locks"),
		LOCTEXT("ToggleBypassLockDesc", "Toggle Bypass Locks"),
		EModifierKey::Alt, EKeys::L,
		[this]()
		{
			ToggleBypassLocks();
			GetToolManager()->DisplayMessage(LOCTEXT("ToggleBypassLockMessage", "Toggled Lock Bypass Mode"), EToolMessageLevel::UserNotification);
		});
}

void USkinWeightsPaintTool::ApplyStamp(const FBrushStampData& Stamp)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::ApplyStamp);

	// must select a bone to paint in all modes EXCEPT relax-mode which operates on ALL bones
	const bool bIsInRelaxMode = WeightToolProperties->BrushMode == EWeightEditOperation::Relax;
	if (!bIsInRelaxMode && CurrentBone == NAME_None)
	{
		return;
	}

	// Dwell throttle for accumulating Relax. Mirrors UMeshSculptToolBase::UpdateStampPendingState
	// (5% of combined radii in world space).
	if (bIsInRelaxMode && WeightToolProperties->RelaxBrushAdvancedConfig.bAccumulate)
	{
		const FVector3d CurrentWorldPos(Stamp.WorldPosition);
		const double CurrentBrushWorldRadius = CurrentBrushRadius;
		if (LastStampWorldPosition.IsSet())
		{
			constexpr double SpacingFraction = 0.05;
			const double Spacing = SpacingFraction * (LastStampWorldRadius + CurrentBrushWorldRadius);
			if (FVector3d::Distance(CurrentWorldPos, LastStampWorldPosition.GetValue()) < Spacing)
			{
				return; // too close to the last accepted stamp — skip
			}
		}
		LastStampWorldPosition = CurrentWorldPos;
		LastStampWorldRadius = CurrentBrushWorldRadius;
	}

	// get the vertices under the brush, and their squared distances to the brush center
	// when using "Volume" brush, distances are straight line
	// when using "Surface" brush, distances are geodesics
	TArray<int32> VerticesInStamp;
	TArray<float> VertexFalloffs;
	CalculateVertexROI(Stamp, VerticesInStamp, VertexFalloffs);

	// gather sparse set of modifications made from this stamp, these edits are merged throughout
	// the lifetime of a single brush stroke in the "ActiveChange" allowing for undo/redo
	FMultiBoneWeightEdits WeightEditsFromStamp;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::EditWeightOfVerticesInStamp);
		// generate a weight edit from this stamp (includes modifications caused by normalization)
		if (WeightToolProperties->BrushMode == EWeightEditOperation::Relax)
		{
			// One smoothing pass per stamp; wiggling supplies the iteration count.
			const float UseStrength = CalculateBrushStrengthToUse(EWeightEditOperation::Relax);
			const bool bUsePreChangeWeight = !WeightToolProperties->RelaxBrushAdvancedConfig.bAccumulate;
			CreateWeightEditsToRelaxVertices(VerticesInStamp, VertexFalloffs, UseStrength, bUsePreChangeWeight, WeightEditsFromStamp);
		}
		else
		{
			// edit weight; either by "Add", "Remove", "Replace", "Multiply"
			const float UseStrength = CalculateBrushStrengthToUse(WeightToolProperties->BrushMode);
			const int32 CurrentBoneIndex = GetCurrentBoneIndex();
			CreateWeightEditsForVertices(
				WeightToolProperties->BrushMode,
				CurrentBoneIndex,
				VerticesInStamp,
				VertexFalloffs,
				UseStrength,
				WeightEditsFromStamp);
		}
	}

	// apply weight edits to the mesh without closing the transaction
	ApplyWeightEditsWithoutTransaction(WeightEditsFromStamp);
}

float USkinWeightsPaintTool::CalculateBrushStrengthToUse(EWeightEditOperation EditMode) const
{
	float UseStrength = BrushProperties->BrushStrength;

	// invert brush strength differently depending on brush mode
	switch (EditMode)
	{
	case EWeightEditOperation::Add:
		{
			UseStrength *= bInvertStroke ? -1.0f : 1.0f;
			break;
		}
	case EWeightEditOperation::Replace:
		{
			UseStrength = bInvertStroke ? 1.0f - UseStrength : UseStrength;
			break;
		}
	case EWeightEditOperation::Multiply:
		{
			// Ctrl shifts the multiplier from 'x' (raw attenuate, slider in [0,1])
			// to '1 + x' (amplify, effective range [1,2]).
			UseStrength = bInvertStroke ? (1.0f + UseStrength) : UseStrength;
			break;
		}
	case EWeightEditOperation::Relax:
		{
			// Invert is intentionally not supported for Relax — sharpening skin weights
			// would leave the [0,1] simplex and isn't a useful brush operation. The previous
			// formula (1 - UseStrength) also produced negative lerp factors when Strength > 1,
			// causing unbounded oscillation across strokes.
			break;
		}
	default:
		checkNoEntry();
	}

	return UseStrength;
}

void USkinWeightsPaintTool::CreateWeightEditsForVertices(
	EWeightEditOperation EditOperation,
	const BoneIndex Bone,
	const TArray<int32>& VertexIndices,
	const TArray<float>& VertexFalloffs,
	const float InValue,
	FMultiBoneWeightEdits& InOutWeightEdits)
{
	// spin through the vertices in the stamp and store new weight values in NewValuesFromStamp
	// afterwards, these values are normalized while taking into consideration the user's desired changes
	const int32 NumVerticesInStamp = VertexIndices.Num();
	for (int32 Index = 0; Index < NumVerticesInStamp; ++Index)
	{
		const int32 VertexID = VertexIndices[Index];
		const float UseFalloff = VertexFalloffs.IsValidIndex(Index) ? VertexFalloffs[Index] : 1.f;
		const float ValueBeforeStroke = Weights.GetWeightOfBoneOnVertex(Bone, VertexID, Weights.PreChangeWeights);

		// calculate new weight value
		float NewValueAfterStamp = ValueBeforeStroke;
		switch (EditOperation)
		{
		case EWeightEditOperation::Add:
			{
				NewValueAfterStamp = ValueBeforeStroke + (InValue * UseFalloff);
				break;
			}
		case EWeightEditOperation::Replace:
			{
				NewValueAfterStamp = FMath::Lerp(ValueBeforeStroke, InValue, UseFalloff);
				break;
			}
		case EWeightEditOperation::Multiply:
			{
				const float DeltaFromThisStamp = ((ValueBeforeStroke * InValue) - ValueBeforeStroke) * UseFalloff;
				NewValueAfterStamp = ValueBeforeStroke + DeltaFromThisStamp;
				break;
			}
		case EWeightEditOperation::RelativeScale:
			{
				// LERP the weight from it's current value towards 1 (for positive values) or towards 0 (for negative values)
				if (InValue >= 0.f)
				{
					NewValueAfterStamp = FMath::Lerp(ValueBeforeStroke, 1.0f, FMath::Abs(InValue) * UseFalloff);
				}
				else
				{
					NewValueAfterStamp = FMath::Lerp(ValueBeforeStroke, 0.0f, FMath::Abs(InValue) * UseFalloff);
				}
				break;
			}
		default:
			// relax operation not supported by this function, use RelaxWeightOnVertices()
			checkNoEntry();
		}

		// normalize the values across all bones affecting this vertex, and record the bone edits
		// normalization is done while holding all weights on the current bone constant so that user edits are not overwritten
		Weights.CreateWeightEditForVertex(
			Bone,
			VertexID,
			NewValueAfterStamp,
			InOutWeightEdits);
	}
}

void USkinWeightsPaintTool::CreateWeightEditsToRelaxVertices(
	TArray<int32> VertexIndices,
	TArray<float> VertexFalloffs,
	const float Strength,
	const bool bUsePreChangeWeight,
	FMultiBoneWeightEdits& InOutWeightEdits)
{
	if (!ensure(SmoothWeightsOp) || !ensure(SmoothWeightsDataSource))
	{
		return;
	}

	FPaintToolWeightsDataSource* DataSource = static_cast<FPaintToolWeightsDataSource*>(SmoothWeightsDataSource.Get());
	const FPaintToolWeightsDataSource::EWeightSource TransientSource = bUsePreChangeWeight
		? FPaintToolWeightsDataSource::EWeightSource::PreChange
		: FPaintToolWeightsDataSource::EWeightSource::Current;
	const FPaintToolWeightsDataSource::FWeightSourceScope SourceScope(*DataSource, TransientSource);

	for (int32 VertexIndex = 0; VertexIndex < VertexIndices.Num(); ++VertexIndex)
	{
		const int32 VertexID = VertexIndices[VertexIndex];
		const float UseFalloff = VertexFalloffs.IsValidIndex(VertexIndex)
			? VertexFalloffs[VertexIndex] * Strength
			: Strength;

		TMap<int32, float> FinalWeights;
		const bool bSmoothSuccess = SmoothWeightsOp->SmoothWeightsAtVertex(VertexID, UseFalloff, GetFinalLockedBoneIndices(), FinalWeights);
		if (!ensure(bSmoothSuccess))
		{
			continue;
		}

		// apply weight edits
		for (const TTuple<BoneIndex, float>& FinalWeight : FinalWeights)
		{
			// record an edit for this vertex, for this bone
			const int32 BoneIndex = FinalWeight.Key;
			const float RawWeight = FinalWeight.Value;
			// The smoothing operator is expected to keep results on the [0,1] simplex.
			// Ensure here so out-of-range values are surfaced during development; clamp
			// after so a one-off stale value (e.g. left over in PreChangeWeights from an
			// earlier buggy stroke) doesn't corrupt CurrentWeights at runtime.
			ensureMsgf(RawWeight >= 0.f && RawWeight <= 1.f,
				TEXT("Smoothing produced out-of-range weight %f for bone %d on vertex %d."),
				RawWeight, BoneIndex, VertexID);
			const float NewWeight = FMath::Clamp(RawWeight, 0.f, 1.f);
			constexpr bool bPruneInfluence = false;
			InOutWeightEdits.MergeSingleEdit(BoneIndex, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
	}
}

void USkinWeightsPaintTool::InitializeOctrees()
{
	if (!ensure(PreviewMesh && PreviewMesh->GetMesh()))
	{
		return;
	}

	// build octree for vertices
	VerticesOctree = MakeUnique<DynamicVerticesOctree>();
	VerticesOctree->Initialize(PreviewMesh->GetMesh(), true);

	// build octree for triangles
	TrianglesOctree = MakeUnique<DynamicTrianglesOctree>();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::InitTriangleOctree);
		
		TriangleOctreeFuture = Async(SkinPaintToolAsyncExecTarget, [&]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SkinTool::InitTriangleOctreeRun);
			TrianglesOctree->Initialize(PreviewMesh->GetMesh());
		});
	}
}

void USkinWeightsPaintTool::InitializeSmoothWeightsOperator(const FDynamicMesh3& RestPoseMesh)
{
	// NOTE: this could probably be initialized lazily as it's only used with the relax brush
	SmoothWeightsDataSource = MakeUnique<FPaintToolWeightsDataSource>(&Weights);
	SmoothWeightsOp = MakeUnique<UE::Geometry::TSmoothBoneWeights<int32, float>>(&RestPoseMesh, SmoothWeightsDataSource.Get());
	SmoothWeightsOp->MinimumWeightThreshold = MinimumWeightThreshold;
	SmoothWeightsOp->EdgeWeightMethod = UE::Geometry::ESmoothBoneWeightsEdgeWeightMethod::CotanWeights;
}

void USkinWeightsPaintTool::ApplyWeightEditsWithoutTransaction(const FMultiBoneWeightEdits& WeightEdits)
{
	Weights.ApplyEditsToCurrentWeights(WeightEdits, PerColorUpdateVertexChangeTracker);
	ActiveChange->StoreMultipleWeightEdits(WeightEdits);
}

void USkinWeightsPaintTool::ApplyWeightEditsAsTransaction(const FMultiBoneWeightEdits& WeightEdits, const FText& TransactionLabel)
{
	// clear the active change to start a new one
	BeginChange();
	// apply the weights
	ApplyWeightEditsWithoutTransaction(WeightEdits);
	// store active change in the transaction buffer and sync weight buffers
	EndChange(TransactionLabel);
}

void USkinWeightsPaintTool::UpdateCurrentBone(const FName& BoneName)
{
	CurrentBone = BoneName;
	bVertexColorsNeedUpdated = true;
	OnSelectionChanged.Broadcast();
}

BoneIndex USkinWeightsPaintTool::GetBoneIndexFromName(const FName BoneName) const
{
	if (BoneName == NAME_None)
	{
		return INDEX_NONE;		
	}
	const BoneIndex* Found = Weights.Deformer.BoneNameToIndexMap.Find(BoneName);
	return Found ? *Found : INDEX_NONE;
}

bool USkinWeightsPaintTool::HasSelectedBone() const
{
	return !SelectedBoneIndices.IsEmpty();
}

void USkinWeightsPaintTool::SetFocusInViewport() const
{
	if (PersonaModeManagerContext.IsValid())
	{
		PersonaModeManagerContext->SetFocusInViewport();	
	}
}

void USkinWeightsPaintTool::OnShutdown(EToolShutdownType ShutdownType)
{
	// apply changes to tool target
	if (ShutdownType == EToolShutdownType::Accept)
	{
		SelectionIsolator->ApplyWeightsToFullMesh();

		bool bAskToSaveNewProfile = false;
		if (IsProfileValid(NewProfilePlaceHolderName()))
		{
			// Rename the new profile to user-supplied name, if a new profile was created 
			RenameSkinWeightsAttribute(EditedMesh, NewProfilePlaceHolderName(), WeightToolProperties->NewSkinWeightProfile);
			if (ActiveSkinWeightProfileName == NewProfilePlaceHolderName())
			{
				ActiveSkinWeightProfileName = WeightToolProperties->NewSkinWeightProfile;	
			}
			else
			{
				// User is not editing the new profile when clicking accept, there is a chance that the profile was created unintentionally
				bAskToSaveNewProfile = true;
			}
		}
		
		if (bAskToSaveNewProfile)
		{
			EAppReturnType::Type Reply = FMessageDialog::Open(EAppMsgType::YesNo,
					FText::Format(
						LOCTEXT("Prompt_SkinWeightsPaintTool_SaveNewSkinWeightProfile",
							"Save new Skin Weight Profile \"{0}\"? \nSelect No to discard it."),
						FText::FromName(WeightToolProperties->NewSkinWeightProfile)));

			if (Reply == EAppReturnType::No)
			{
				EditedMesh.Attributes()->RemoveSkinWeightsAttribute(WeightToolProperties->NewSkinWeightProfile);
			}
		}
		
		// this block bakes the modified DynamicMeshComponent back into the StaticMeshComponent inside an undo transaction
		GetToolManager()->BeginUndoTransaction(LOCTEXT("SkinWeightsPaintTool", "Paint Skin Weights"));
		
		IDynamicMeshCommitter* DynamicMeshCommitter = CastChecked<IDynamicMeshCommitter>(Target);
		DynamicMeshCommitter->CommitDynamicMesh(EditedMesh);
		
		GetToolManager()->EndUndoTransaction();
	
		// Refresh the skeleton tree
		GetNotifier()->Notify({}, ESkeletalMeshNotifyType::HierarchyChanged);
	}
	
	SkeletonTreeNotifierBindScope.Reset();
	
	if (ensure(WeightToolProperties))
	{
		WeightToolProperties->LinearColorRamp.OnColorCurveChangedDelegate.RemoveAll(this);
	}

	// Snapshot the lock set into the persisted name list on non-Cancel. On Cancel,
	// leave LockedBoneNames as the values restored at Setup so the cache rolls back.
	if (ShutdownType != EToolShutdownType::Cancel)
	{
		WeightToolProperties->LockedBoneNames.Reset(LockedBoneIndices.Num());
		for (const BoneIndex Index : LockedBoneIndices)
		{
			const FName Name = GetBoneNameFromIndex(Index);
			if (Name != NAME_None)
			{
				WeightToolProperties->LockedBoneNames.Add(Name);
			}
		}
	}

	WeightToolProperties->SaveProperties(this, WeightToolProperties->PropertySetCacheIdentifier);
	RemoveToolPropertySource(WeightToolProperties);

	// shutdown polygon selection mechanic
	if (MeshSelector)
	{
		MeshSelector->Shutdown();
	}

	if (MeshElementsDisplay)
	{
		MeshElementsDisplay->Disconnect();
		MeshElementsDisplay = nullptr;
	}
	
	// restore viewport show flags and preview settings
	FPreviewProfileController PreviewProfileController;
	PreviewProfileController.SetActiveProfile(PreviewProfileToRestore);

	if (EditorContext.IsValid())
	{
		EditorContext->UnbindFrom(this);
	}

	ToggleBoneManipulation(false);

	GetWeightTransferManager()->Shutdown();
}

FEditorViewportClient* USkinWeightsPaintTool::GetViewportClient() const
{
	FEditorViewportClient* ViewportClient = nullptr;
	if (const UPersonaEditorModeManagerContext* ModeManagerContext = PersonaModeManagerContext.Get())
	{
		if (const IPersonaEditorModeManager* PersonaEditorModeManager = ModeManagerContext->GetPersonaEditorModeManager())
		{
			ViewportClient = PersonaEditorModeManager->GetHoveredViewportClient();
			if (!ViewportClient)
			{
				ViewportClient = PersonaEditorModeManager->GetFocusedViewportClient();
			}
		}
	}

	ensure(ViewportClient);
	return ViewportClient;
}

USkinWeightsPaintToolProperties* USkinWeightsPaintTool::GetWeightToolProperties() const
{
	return WeightToolProperties;
}

const TArray<int32>& USkinWeightsPaintTool::GetDefaultIsolationTriangles() const
{
	static const TArray<int32> Empty;
	USkeletalMeshEditorContextObjectBase* Context = GetToolManager()->GetContextObjectStore()->FindContext<USkeletalMeshEditorContextObjectBase>();
	if (Context)
	{
		return Context->GetIsolatedTriangles();
	}
	return Empty;
}

const TSet<int32>& USkinWeightsPaintTool::GetFinalLockedBoneIndices()
{
	if (IsBypassingAllLocks())
	{
		static TSet<int32> EmptySet;
		return EmptySet;
	}
	
	return GetRawLockedBoneIndices();	
}

const TSet<int32>& USkinWeightsPaintTool::GetRawLockedBoneIndices()
{
	return LockedBoneIndices;
}

bool USkinWeightsPaintTool::IsBypassingAllLocks() const
{
	return bBypassLocks;
}

void USkinWeightsPaintTool::BeginChange()
{
	const FName SkinProfile = GetActiveSkinWeightProfileName();
	ActiveChange = MakeUnique<FMeshSkinWeightsChange>(SkinProfile, Weights.PreChangeWeights.Num());
}

void USkinWeightsPaintTool::EndChange(const FText& TransactionLabel)
{
	if (!ensureMsgf(ActiveChange.IsValid(), TEXT("USkinWeightsPaintTool::EndChange called without a matching BeginChange (ActiveChange is null).")))
	{
		return;
	}

	// Close before Sync — Sync overwrites PreChangeWeights and Close needs it for Before snapshots.
	ActiveChange->Close(this);
	Weights.SyncWeightBuffers();
	
	// record transaction
	GetToolManager()->BeginUndoTransaction(TransactionLabel);
	GetToolManager()->EmitObjectChange(this, MoveTemp(ActiveChange), TransactionLabel);
	GetToolManager()->EndUndoTransaction();

	// notify dependent systems
	OnWeightsChanged.Broadcast();
}

void USkinWeightsPaintTool::ExternalUpdateSkinWeightLayer(const FName InSkinWeightProfile)
{
	enum class ESkinWeightChangeState
	{
		SkinProfile,
		None
	} State = ESkinWeightChangeState::None;
	
	// Trigger skin weight profile switch to show the correct profile during undo/redo
	if (InSkinWeightProfile != GetActiveSkinWeightProfileName())
	{
		if (InSkinWeightProfile == NewProfilePlaceHolderName())
		{
			WeightToolProperties->SkinWeightProfileOption = CreateNewName();
			WeightToolProperties->bShowNewProfileName = true;
		}
		else
		{
			WeightToolProperties->SkinWeightProfileOption = InSkinWeightProfile;
			WeightToolProperties->bShowNewProfileName = false;
		}
		
		State = ESkinWeightChangeState::SkinProfile;
	}

	switch (State)
	{
	case ESkinWeightChangeState::SkinProfile:
		return OnActiveSkinWeightProfileChanged();
	case ESkinWeightChangeState::None:
		default:
		break;
	}
}

void FSkinMirrorData::EnsureMirrorDataIsUpdated(
    const TArray<FName>& BoneNames,
    const TMap<FName, BoneIndex>& BoneNameToIndexMap,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FVector>& RefPoseVertices,
	EAxis::Type InMirrorAxis,
	EMirrorDirection InMirrorDirection)
{
	if (bIsInitialized && InMirrorAxis == Axis && InMirrorDirection==Direction)
	{
		// already initialized, just re-use cached data
		return;
	}

	// need to re-initialize
	bIsInitialized = false;
	Axis = InMirrorAxis;
	Direction = InMirrorDirection;
	BoneMap.Reset();
	VertexMap.Reset();
	
	// build bone map for mirroring
	// TODO, provide some way to edit the mirror bone mapping, either by providing a UMirrorDataTable input or editing directly in the hierarchy view.
	for (FName BoneName : BoneNames)
	{
		FName MirroredBoneName = UMirrorDataTable::FindBestMirroredBone(BoneName, RefSkeleton, Axis);

		int32 BoneIndex = BoneNameToIndexMap[BoneName];
		int32 MirroredBoneIndex = BoneNameToIndexMap[MirroredBoneName];
		BoneMap.Add(BoneIndex, MirroredBoneIndex);
		
		// debug view bone mapping
		//UE_LOGF(LogTemp, Log, "Bone    : %ls", *BoneName.ToString());
		//UE_LOGF(LogTemp, Log, "Mirrored: %ls", *MirroredBoneName.ToString());
		//UE_LOGF(LogTemp, Log, "-------");
	}

	// build a spatial hash grid
	constexpr float HashGridCellSize = 2.0f; // the length of the cell size in the point hash grid
	UE::Geometry::TPointHashGrid3f<int32> VertHash(HashGridCellSize, INDEX_NONE);
	VertHash.Reserve(RefPoseVertices.Num());
	for (int32 VertexID = 0; VertexID < RefPoseVertices.Num(); ++VertexID)
	{
		VertHash.InsertPointUnsafe(VertexID, static_cast<FVector3f>(RefPoseVertices[VertexID]));
	}
	
	// generate a map of point IDs on the target side, to their equivalent vertex ID on the source side 
	for (int32 TargetVertexID = 0; TargetVertexID < RefPoseVertices.Num(); ++TargetVertexID)
	{
		const FVector& TargetPosition = RefPoseVertices[TargetVertexID];

		// we only generate the mirror map for vertices on the target side of the mirror plane
		// if the mirror plane is flipped, the map must be regenerated because there is no guarantee that the mirror map is symmetrical
		if (!IsPointOnTargetMirrorSide(TargetPosition))
		{
			continue;
		}

		// flip position across the mirror axis
		FVector3f MirroredPosition = FVector3f(TargetPosition);
		MirroredPosition[Axis-1] *= -1.f;

		// query spatial hash near mirrored position, gradually increasing search radius until at least 1 point is found
		TPair<int32, double> ClosestMirroredPoint = {INDEX_NONE, TNumericLimits<double>::Max()};
		float SearchRadius = HashGridCellSize;
		while(ClosestMirroredPoint.Key == INDEX_NONE)
		{
			ClosestMirroredPoint = VertHash.FindNearestInRadius(
				MirroredPosition,
				SearchRadius,
				[&RefPoseVertices, MirroredPosition](int32 VID)
				{
					return FVector3f::DistSquared(FVector3f(RefPoseVertices[VID]), MirroredPosition);
				});
			
			SearchRadius += HashGridCellSize;

			// forcibly break out if search radius gets bigger than the maximum search radius
			static float MaxSearchRadius = 15.f; // TODO we may want to expose this value to the user...
			if (SearchRadius >= MaxSearchRadius)
			{
				break;
			}
		}
		
		// disallow copying from vertices that are on the target side of the mirror plane
		if (ClosestMirroredPoint.Key != INDEX_NONE)
		{
			const FVector& SourcePointPosition = RefPoseVertices[ClosestMirroredPoint.Key];
			if (IsPointOnTargetMirrorSide(SourcePointPosition))
			{
				ClosestMirroredPoint.Key = INDEX_NONE;	
			}
		}
		
		// record the mirrored vertex ID for this vertex (may be INDEX_NONE)
		VertexMap.FindOrAdd(TargetVertexID, ClosestMirroredPoint.Key); // (TO, FROM)
	}
	
	bIsInitialized = true;
}

const TMap<int32, int32>& FSkinMirrorData::GetVertexMap() const
{
	ensure(bIsInitialized);
	return VertexMap;
}

bool FSkinMirrorData::IsPointOnTargetMirrorSide(const FVector& InPoint) const
{
	if (Direction == EMirrorDirection::PositiveToNegative && InPoint[Axis-1] >= 0.f)
	{
		return false; // target is negative side, but point is on positive side
	}
	if (Direction == EMirrorDirection::NegativeToPositive && InPoint[Axis-1] <= 0.f)
	{
		return false; // target is positive side, but vertex is on negative side
	}

	return true;
}

void USkinWeightsPaintTool::MirrorWeights(EAxis::Type Axis, EMirrorDirection Direction)
{
	if (!ShouldProceedWithAllBonesUnlocked())
	{
		return;
	}
	
	check(Axis != EAxis::None);
	
	// get all ref pose vertices
	const TArray<FVector>& RefPoseVertices = Weights.Deformer.RefPoseVertexPositions;

	// refresh mirror tables (cached / lazy generated)
	MirrorData.EnsureMirrorDataIsUpdated(
		Weights.Deformer.BoneNames,
		Weights.Deformer.BoneNameToIndexMap,
		Weights.Deformer.RefSkeleton,
		RefPoseVertices,
		Axis,
		Direction);

	// get a reference to the mirror tables
	const TMap<BoneIndex, BoneIndex>& BoneMap = MirrorData.GetBoneMap();
	const TMap<VertexIndex, VertexIndex>& VertexMirrorMap = MirrorData.GetVertexMap(); // <Target, Source>

	// get the selected vertices
	const TArray<int32>& SelectedVertices = MeshSelector->GetSelectedVertices();
	// we need to convert selection to the equivalent target vertex indices (on the target side of the mirror plane)
	// if a vertex is already on the target side, great
	// if the user selected vertices on the source side, we convert them to the mirrored equivalent on the target side
	TSet<VertexIndex> TargetVertices;
	TArray<VertexIndex> MissingVertices;
	for (const VertexIndex SelectedVertex : SelectedVertices)
	{
		int32 TargetVertexID = INDEX_NONE;
		const bool bIsOnTargetSide = VertexMirrorMap.Contains(SelectedVertex);
		
		if (bIsOnTargetSide)
		{
			// vertex is located across the mirror plane (target side, to copy TO)
			TargetVertexID = SelectedVertex;
		}
		else
		{
			// vertex is located on the source side (to copy FROM), so we need to search for it's mirror target vertex
			for (const TPair<VertexIndex, VertexIndex>& ToFromPair : VertexMirrorMap)
			{
				if (ToFromPair.Value == SelectedVertex)
				{
					TargetVertexID = ToFromPair.Key;
					break;
				}
			}
		}

		// selected vertex did not have a mirrored equivalent
		if (TargetVertexID == INDEX_NONE)
		{
			if (bIsOnTargetSide)
			{
				MissingVertices.Add(TargetVertexID);
			}
			
			continue;
		}
		
		// add to the list of target vertices to set weights on
		TargetVertices.Add(TargetVertexID);
	}
	
	// spin through all target vertices to mirror and copy weights from source
	FMultiBoneWeightEdits WeightEditsFromMirroring;
	TMap<BoneIndex, float> NewBoneWeights;
	NewBoneWeights.Reserve(MAX_TOTAL_INFLUENCES);
	for (const VertexIndex TargetVertexID : TargetVertices)
	{
		const VertexIndex SourceVertexID = VertexMirrorMap[TargetVertexID];

		// store in the missing vertices array if no mirrored vertex was found
		if (SourceVertexID == INDEX_NONE)
		{
			MissingVertices.Add(TargetVertexID);
			continue;
		}

		// remove all weight on vertex
		for (const FVertexBoneWeight& TargetBoneWeight : Weights.PreChangeWeights[TargetVertexID])
		{
			static bool bPruneInfluence = true;
			static float NewWeight = 0.f;
			WeightEditsFromMirroring.MergeSingleEdit(TargetBoneWeight.BoneID, TargetVertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}

		// copy source weights, but with mirrored bones
		// NOTE: we have to normalize here because it's possible that while searching for mirrored bones, multiple source bones will be
		// mapped to the same target bone. When that happens, only the last applied weight on that bone would be recorded in an edit
		// If that happens, the final weight may not sum to 1.
		NewBoneWeights.Reset();
		for (const FVertexBoneWeight& SourceBoneWeight : Weights.PreChangeWeights[SourceVertexID])
		{
			const BoneIndex MirroredBoneIndex = BoneMap[SourceBoneWeight.BoneID];
			const float NewWeight = SourceBoneWeight.Weight;
			float& Weight = NewBoneWeights.FindOrAdd(MirroredBoneIndex, 0.0f);
			Weight += NewWeight;
		}
		TruncateWeightMap(NewBoneWeights);
		NormalizeWeightMap(NewBoneWeights);

		// apply weight edits
		for (const TPair<BoneIndex, float>& NewBoneWeight : NewBoneWeights)
		{
			const BoneIndex BoneID = NewBoneWeight.Key;
			const float NewWeight = NewBoneWeight.Value;
			static bool bPruneInfluence = false;
			WeightEditsFromMirroring.MergeSingleEdit(BoneID, TargetVertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
	}

	// apply the changes
	const FText TransactionLabel = LOCTEXT("MirrorWeightChange", "Mirror skin weights.");
	ApplyWeightEditsAsTransaction(WeightEditsFromMirroring, TransactionLabel);

	// warn if some vertices were not mirrored
	if (!MissingVertices.IsEmpty())
	{
		UE_LOGF(LogMeshModelingToolsEditor, Log, "Mirror Skin Weights: %d vertex weights were not mirrored because a vertex was not found close enough to the mirrored location.", MissingVertices.Num());
	}
}

void USkinWeightsPaintTool::EditWeightsOnVertices(
	BoneIndex Bone,
	const float Value,
	const int32 Iterations,
	EWeightEditOperation EditOperation,
	const TArray<VertexIndex>& VertexIndices,
	const bool bShouldTransact)
{
	if (!(Weights.Deformer.InvCSRefPoseTransforms.IsValidIndex(Bone)))
	{
		return;
	}

	const TArray<float> VertexFalloffs = {}; // no falloff in mesh-selection mode

	if (bShouldTransact)
	{
		BeginChange();
	}

	if (EditOperation == EWeightEditOperation::Relax)
	{
		// Mesh-mode Relax always compounds: each iteration reads the prior iteration's
		// output via CurrentWeights, independent of the brush's bAccumulate setting.
		constexpr bool bUsePreChangeWeight = false;
		for (int32 i = 0; i < Iterations; ++i)
		{
			FMultiBoneWeightEdits IterEdits;
			CreateWeightEditsToRelaxVertices(
				MeshSelector->GetSelectedVertices(),
				VertexFalloffs,
				Value,
				bUsePreChangeWeight,
				IterEdits);
			ApplyWeightEditsWithoutTransaction(IterEdits);
		}
	}
	else
	{
		FMultiBoneWeightEdits DirectWeightEdits;
		CreateWeightEditsForVertices(
			EditOperation,
			Bone,
			VertexIndices,
			VertexFalloffs,
			Value,
			DirectWeightEdits);
		ApplyWeightEditsWithoutTransaction(DirectWeightEdits);
	}

	if (bShouldTransact)
	{
		const FText TransactionLabel = LOCTEXT("EditWeightChange", "Edit skin weights directly.");
		EndChange(TransactionLabel);
	}
}

void USkinWeightsPaintTool::PruneWeights(float Threshold, const TArray<BoneIndex>& BonesToPrune)
{
	if (!ShouldProceedWithAllBonesUnlocked())
	{
		return;
	}
	
	// set weights below the given threshold to zero AND remove them as a recorded influence on that vertex
	FMultiBoneWeightEdits WeightEditsFromPrune;
	const TArray<VertexIndex>& VerticesToPrune = MeshSelector->GetSelectedVertices();
	for (const VertexIndex VertexID : VerticesToPrune)
	{
		const VertexWeights& VertexWeights = Weights.CurrentWeights[VertexID];

		// Track which bones are getting pruned so the renormalize pass below can skip them.
		TArray<BoneIndex, TInlineAllocator<UE::AnimationCore::MaxInlineBoneWeightCount>> PrunedBones;
		for (const FVertexBoneWeight& BoneWeight : VertexWeights)
		{
			if (BoneWeight.Weight < Threshold || BonesToPrune.Contains(BoneWeight.BoneID))
			{
				PrunedBones.Add(BoneWeight.BoneID);
				constexpr float NewWeight = 0.f;
				constexpr bool bPruneInfluence = true;
				WeightEditsFromPrune.MergeSingleEdit(BoneWeight.BoneID, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
			}
		}

		const int32 NumSurviving = VertexWeights.Num() - PrunedBones.Num();
		if (NumSurviving == 0)
		{
			// All influences pruned — dump full weight on root so the vertex stays normalized.
			constexpr BoneIndex RootBoneIndex = 0;
			constexpr bool bPruneInfluence = false;
			constexpr float NewWeight = 1.f;
			WeightEditsFromPrune.MergeSingleEdit(RootBoneIndex, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
		else
		{
			float TotalWeight = 0.f;
			for (const FVertexBoneWeight& BoneWeight : VertexWeights)
			{
				if (!PrunedBones.Contains(BoneWeight.BoneID))
				{
					TotalWeight += BoneWeight.Weight;
				}
			}

			// All survivors at zero → split evenly so the vertex has well-defined weights.
			const bool bNoOtherWeights = FMath::IsNearlyEqual(TotalWeight, 0.f);
			const float EvenlySplitWeight = 1.0f / NumSurviving;

			for (const FVertexBoneWeight& BoneWeight : VertexWeights)
			{
				if (PrunedBones.Contains(BoneWeight.BoneID))
				{
					continue;
				}
				constexpr bool bPruneInfluence = false;
				const float NewWeight = bNoOtherWeights ? EvenlySplitWeight : BoneWeight.Weight / TotalWeight;
				WeightEditsFromPrune.MergeSingleEdit(BoneWeight.BoneID, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
			}
		}
	}

	// apply the changes
	const FText TransactionLabel = LOCTEXT("PruneWeightValuesChange", "Prune skin weights.");
	ApplyWeightEditsAsTransaction(WeightEditsFromPrune, TransactionLabel);
}

void USkinWeightsPaintTool::AverageWeights(const float Strength)
{
	// if strength is zero, don't do anything
	if (FMath::IsNearlyEqual(Strength, 0.0f))
	{
		return;
	}
	
	// get vertices to edit weights on
	const TArray<VertexIndex>& VerticesToAverage = MeshSelector->GetSelectedVertices();
	TMap<BoneIndex, float> AccumulatedWeights;	
	AccumulateWeights(Weights.PreChangeWeights, VerticesToAverage, GetFinalLockedBoneIndices(), AccumulatedWeights);
	// store weight edits to apply averaging to selected vertices
	FMultiBoneWeightEdits WeightEditsFromAveraging;	
	for (const VertexIndex VertexID : VerticesToAverage)
	{
		float TotalLockedWeight = 0.0f;
		int32 NumLockedBones = 0;
		GetLockedWeight(Weights.PreChangeWeights, VertexID, GetFinalLockedBoneIndices(), TotalLockedWeight, NumLockedBones);
			
		TMap<BoneIndex, float> AveragedWeights = AccumulatedWeights;	
		TruncateWeightMap(AveragedWeights, NumLockedBones);
		NormalizeWeightMap(AveragedWeights, 1.f - TotalLockedWeight);	
			
		// FULLY apply averaged weights to vertices if strength is 1.0
		if (FMath::IsNearlyEqual(Strength, 1.0f))
		{
			// remove influences not a part of the average results
			for (const FVertexBoneWeight& BoneWeight : Weights.PreChangeWeights[VertexID])
			{
				if (!AveragedWeights.Contains(BoneWeight.BoneID) && !GetFinalLockedBoneIndices().Contains(BoneWeight.BoneID))
				{
					constexpr bool bPruneInfluence = false;
					constexpr float NewWeight = 0.f;
					WeightEditsFromAveraging.MergeSingleEdit(BoneWeight.BoneID, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
				}
			}

			// add influences from the averaging results
			for (const TTuple<BoneIndex, float>& AveragedWeight : AveragedWeights)
			{
				const BoneIndex IndexOfBone = AveragedWeight.Key;
				constexpr bool bPruneInfluence = false;
				const float NewWeight = AveragedWeight.Value;
				WeightEditsFromAveraging.MergeSingleEdit(IndexOfBone, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
			}
		}
		else
		{
			// blend averaged weight with the existing weight based on the strength value
			const float OldWeightStrength = 1.0f - Strength;
			const float NewWeightStrength = Strength;

			// storage for final blended weights on this vertex
			TMap<BoneIndex, float> BlendedWeights;

			// scale the existing weights by OldWeightStrength
			for (const FVertexBoneWeight& BoneWeight : Weights.PreChangeWeights[VertexID])
			{
				if (!GetFinalLockedBoneIndices().Contains(BoneWeight.BoneID))
				{
					BlendedWeights.Add(BoneWeight.BoneID, BoneWeight.Weight * OldWeightStrength);
				}
			}
			
			// accumulate existing weights with the scaled averaged weights
			for (const TTuple<BoneIndex, float>& AveragedWeight : AveragedWeights)
			{
				if (BlendedWeights.Contains(AveragedWeight.Key))
				{
					BlendedWeights[AveragedWeight.Key] += AveragedWeight.Value * NewWeightStrength;
				}
				else
				{
					BlendedWeights.Add(AveragedWeight.Key, AveragedWeight.Value * NewWeightStrength);
				}
			}

			// enforce max influences and normalize
			TruncateWeightMap(BlendedWeights, NumLockedBones);
			NormalizeWeightMap(BlendedWeights, 1.0f - TotalLockedWeight);
			
			// apply blended weights to this vertex
			for (const TTuple<BoneIndex, float>& BlendedWeight : BlendedWeights)
			{
				const BoneIndex BoneID = BlendedWeight.Key;
				constexpr bool bPruneInfluence = false;
				const float NewWeight = BlendedWeight.Value;
				WeightEditsFromAveraging.MergeSingleEdit(BoneID, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
			}
		}
	}

	// apply the changes
	const FText TransactionLabel = LOCTEXT("AverageWeightValuesChange", "Average skin weights.");
	ApplyWeightEditsAsTransaction(WeightEditsFromAveraging, TransactionLabel);
}

void USkinWeightsPaintTool::NormalizeWeights()
{
	// re-set a weight on each vertex to force normalization
	FMultiBoneWeightEdits WeightEditsFromNormalization;
	const TArray<VertexIndex> VerticesToNormalize = MeshSelector->GetSelectedVertices();
	for (const VertexIndex VertexID : VerticesToNormalize)
	{
		const VertexWeights& VertexWeights = Weights.CurrentWeights[VertexID];
		if (VertexWeights.IsEmpty())
		{
			// ALL influences have been pruned from vertex, so assign it to the root
			constexpr BoneIndex RootBoneIndex = 0;
			constexpr float FullWeight = 1.f;
			Weights.CreateWeightEditForVertex(RootBoneIndex, VertexID, FullWeight,WeightEditsFromNormalization);
		}
		else
		{
			// set first weight to current value, just to force re-normalization
			const FVertexBoneWeight& BoneWeight = VertexWeights[0];
			Weights.CreateWeightEditForVertex(BoneWeight.BoneID, VertexID, BoneWeight.Weight,WeightEditsFromNormalization);
		}
	}

	// apply the changes
	const FText TransactionLabel = LOCTEXT("NormalizeWeightValuesChange", "Normalize skin weights.");
	ApplyWeightEditsAsTransaction(WeightEditsFromNormalization, TransactionLabel);
}

void USkinWeightsPaintTool::HammerWeights()
{
	// get selected vertices
	const TArray<VertexIndex> SelectedVerts = MeshSelector->GetSelectedVertices();
	if (SelectedVerts.IsEmpty())
	{
		return;
	}

	if (!ShouldProceedWithAllBonesUnlocked())
	{
		return;
	}
	
	// reset mesh to ref pose so that Dijkstra path lengths are not deformed
	Weights.Deformer.SetToRefPose(this);
	
	// find 1-ring neighbors of the current selection, lets call these "Surrounding" vertices
	const FDynamicMesh3* Mesh = PreviewMesh->GetMesh();
	TSet<int32> SurroundingVertices;
	for (const int32 SelectedVertex : SelectedVerts)
	{
		for (const int32 NeighborIndex : Mesh->VtxVerticesItr(SelectedVertex))
		{
			if (!SelectedVerts.Contains(NeighborIndex))
			{
				SurroundingVertices.Add(NeighborIndex);
			}
		}
	}

	// seed a Dijkstra path finder with the surrounding vertices
	UE::Geometry::TMeshDijkstra<FDynamicMesh3> PathFinder(Mesh);
	TArray<UE::Geometry::TMeshDijkstra<FDynamicMesh3>::FSeedPoint> SeedPoints;
	for (const int32 SurroundingVertex : SurroundingVertices)
	{
		SeedPoints.Add({ SurroundingVertex, SurroundingVertex, 0 });
	}
	PathFinder.ComputeToMaxDistance(SeedPoints, TNumericLimits<double>::Max());

	// create set of weight edits that hammer the weights
	FMultiBoneWeightEdits HammerWeightEdits;
	
	// for each selected vertex, find the nearest surrounding vertex and copy it's weights
	TArray<int32> VertexPath;
	for (const int32 SelectedVertex : SelectedVerts)
	{
		// find the closest surrounding vertex to this selected vertex
		if (!PathFinder.FindPathToNearestSeed(SelectedVertex, VertexPath))
		{
			continue;
		}
		const int32 ClosestVertex = VertexPath.Last();

		// remove all current weights (pruning since this operation completely replaces the weight)
		for (const FVertexBoneWeight& BoneWeight : Weights.PreChangeWeights[SelectedVertex])
		{
			constexpr float NewWeight = 0.f;
			constexpr bool bPruneInfluence = true;
			HammerWeightEdits.MergeSingleEdit(BoneWeight.BoneID, SelectedVertex, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}

		// replace weights with values from the closest vertex
		for (const FVertexBoneWeight& BoneWeight : Weights.PreChangeWeights[ClosestVertex])
		{
			const float NewWeight = BoneWeight.Weight;
			constexpr bool bPruneInfluence = false;
			HammerWeightEdits.MergeSingleEdit(BoneWeight.BoneID, SelectedVertex, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
	}
	
	// apply the changes
	const FText TransactionLabel = LOCTEXT("HammerWeightsChange", "Hammer skin weights.");
	ApplyWeightEditsAsTransaction(HammerWeightEdits, TransactionLabel);

	// put the mesh back in it's current pose
	Weights.Deformer.SetAllVerticesToBeUpdated();
}

void USkinWeightsPaintTool::ClampInfluences(const int32 MaxInfluences)
{
	if (!ensure(MaxInfluences >= 1))
	{
		return;
	}

	// get all the selected vertices
	const TArray<VertexIndex>& VerticesToClamp = MeshSelector->GetSelectedVertices();
	if (VerticesToClamp.IsEmpty())
	{
		const FText NotificationText = FText::FromString(TEXT("No vertices were selected. No weights were clamped."));
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	if (!ShouldProceedWithAllBonesUnlocked())
	{
		return;
	}
	
	// create set of weight edits that clamp the smallest influences
	FMultiBoneWeightEdits ClampedWeightEdits;

	// clamp each vertex to MaxInfluences (discarding the smallest weights)
	for (const VertexIndex VertexID : VerticesToClamp)
	{
		if (Weights.PreChangeWeights[VertexID].Num() <= MaxInfluences)
		{
			continue;
		}
		
		VertexWeights WeightsToClamp = Weights.PreChangeWeights[VertexID];

		// sort in descending order by weight
		Algo::Sort(WeightsToClamp, [](const FVertexBoneWeight& A, const FVertexBoneWeight& B)
			{
				return A.Weight > B.Weight; 
			});
		// cull all the smallest weights up to MaxInfluences
		WeightsToClamp.SetNum(MaxInfluences);

		// normalize remaining influences
		float TotalWeight = 0.f;
		for (const FVertexBoneWeight& VertexWeight : WeightsToClamp)
		{
			TotalWeight += VertexWeight.Weight;
		}
		for (FVertexBoneWeight& VertexWeight : WeightsToClamp)
		{
			VertexWeight.Weight /= TotalWeight > SMALL_NUMBER ? TotalWeight : 1.f;
		}

		// remove all current weights (pruning since this operation completely replaces the weight)
		for (const FVertexBoneWeight& BoneWeight : Weights.PreChangeWeights[VertexID])
		{
			constexpr float NewWeight = 0.f;
			constexpr bool bPruneInfluence = true;
			ClampedWeightEdits.MergeSingleEdit(BoneWeight.BoneID, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}

		// replace weights with clamped and renormalized weights
		for (const FVertexBoneWeight& ClampedBoneWeight : WeightsToClamp)
		{
			const float NewWeight = ClampedBoneWeight.Weight;
			constexpr bool bPruneInfluence = false;
			ClampedWeightEdits.MergeSingleEdit(ClampedBoneWeight.BoneID, VertexID, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
	}
	
	// apply the changes
	const FText TransactionLabel = LOCTEXT("ClampInfluencesChange", "Clamped influences.");
	ApplyWeightEditsAsTransaction(ClampedWeightEdits, TransactionLabel);

	// update deformations
	Weights.Deformer.SetAllVerticesToBeUpdated();
}

const FString USkinWeightsPaintTool::CopyPasteWeightsIdentifier = TEXT("UNREAL_VERTEX_WEIGHTS:");

void USkinWeightsPaintTool::CopyWeights()
{
	// get all the selected vertices
	const TArray<VertexIndex>& VerticesToCopy = MeshSelector->GetSelectedVertices();
	if (VerticesToCopy.IsEmpty())
	{
		const FText NotificationText = FText::FromString(TEXT("No vertices were selected. No weights were copied to the clipboard."));
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}

	// get a map of the average weight of all the selected vertices (accumulating and normalizing produces an average)
	TMap<BoneIndex, float> WeightsToCopy;
	AccumulateWeights(Weights.PreChangeWeights, VerticesToCopy, {}, WeightsToCopy);
	TruncateWeightMap(WeightsToCopy);
	NormalizeWeightMap(WeightsToCopy);

	//
	// serialize and store in the clipboard
	//

	// create a JSON array to hold our pairs
	TArray<TSharedPtr<FJsonValue>> JSONArray;
	for (const TPair<BoneIndex, float>& BoneWeight : WeightsToCopy)
	{
		TSharedPtr<FJsonObject> WeightJSON = MakeShared<FJsonObject>();
		WeightJSON->SetStringField(TEXT("BoneName"), GetBoneNameFromIndex(BoneWeight.Key).ToString());
		WeightJSON->SetNumberField(TEXT("Weight"), BoneWeight.Value);
		JSONArray.Add(MakeShared<FJsonValueObject>(WeightJSON));
	}

	// convert JSON array to string
	FString JSONString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JSONString);
	FJsonSerializer::Serialize(JSONArray, Writer);

	// add a custom prefix to identify our data
	const FString ClipboardString = CopyPasteWeightsIdentifier + JSONString;
        
	// copy to clipboard
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);

	// notify user
	const FText NotificationText = FText::FromString("Copied weights to clipboard.");
	ShowEditorMessage(ELogVerbosity::Log, NotificationText);
}

void USkinWeightsPaintTool::PasteWeights()
{
	const TArray<VertexIndex>& VerticesToPasteOn = MeshSelector->GetSelectedVertices();
	if (VerticesToPasteOn.IsEmpty())
	{
		const FText NotificationText = FText::FromString("No vertices were selected. No weights were pasted.");
		ShowEditorMessage(ELogVerbosity::Error, NotificationText);
		return;
	}
	
	// get the clipboard content and check if it matches our format
	FString ClipboardContent;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardContent);
	if (!ClipboardContent.StartsWith(CopyPasteWeightsIdentifier))
	{
		const FText NotificationText = FText::FromString("Failed to paste vertex weights from clipboard. Expected header not found.");
		ShowEditorMessage(ELogVerbosity::Fatal, NotificationText);
		return;
	}

	if (!ShouldProceedWithAllBonesUnlocked())
	{
		return;
	}

	// deserialize the string into JSON and parse it as name/weight pairs
	TMap<BoneIndex, float> LoadedWeights;
	const FString JsonString = ClipboardContent.RightChop(CopyPasteWeightsIdentifier.Len());
	TSharedPtr<FJsonValue> JsonParsed;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
	bool bFoundImproperlyFormattedData = false;
	bool bFoundNonExistantBone = false;
	if (FJsonSerializer::Deserialize(Reader, JsonParsed) && JsonParsed.IsValid())
	{
		TArray<TSharedPtr<FJsonValue>> JsonArray = JsonParsed->AsArray();
		for (const TSharedPtr<FJsonValue>& JsonValue : JsonArray)
		{
			TSharedPtr<FJsonObject> JsonObject = JsonValue->AsObject();
			if (!JsonObject.IsValid())
			{
				continue;
			}
			FString NameString;
			double ValueAsDouble;
			if (!(JsonObject->TryGetStringField(TEXT("BoneName"), NameString) && JsonObject->TryGetNumberField(TEXT("Weight"), ValueAsDouble)))
			{
				bFoundImproperlyFormattedData = true;
				continue;
			}
			
			BoneIndex IndexOfBone = GetBoneIndexFromName(FName(NameString));
			float Weight = ValueAsDouble;
			if (IndexOfBone == INDEX_NONE)
			{
				bFoundNonExistantBone = true;
				UE_LOGF(LogMeshModelingToolsEditor, Warning, "Pasted weights referenced a missing bone: %ls", *NameString);
				continue;
			}

			// store the loaded weight value
			LoadedWeights.Add(IndexOfBone, Weight);
		}
	}

	if (bFoundNonExistantBone)
	{
		const FText NotificationText = FText::FromString("Pasted weights referenced a missing bone. See output for details.");
		ShowEditorMessage(ELogVerbosity::Warning, NotificationText);
	}
	
	if (bFoundImproperlyFormattedData)
	{
		const FText NotificationText = FText::FromString("Found improperly formatted data while pasting weights from clipboard. Expected array of (BoneName,Weight) pairs.");
		ShowEditorMessage(ELogVerbosity::Warning, NotificationText);
	}

	if (LoadedWeights.IsEmpty())
	{
		const FText NotificationText = FText::FromString("No weights were loaded from the clipboard. Paste aborted.");
		ShowEditorMessage(ELogVerbosity::Fatal, NotificationText);
		return;
	}

	// truncate and normalize (these weights could have come from anywhere)
	TruncateWeightMap(LoadedWeights);
	NormalizeWeightMap(LoadedWeights);

	// create set of weight edits that paste the weights
	FMultiBoneWeightEdits PasteWeightEdits;
	for (const int32 SelectedVertex : VerticesToPasteOn)
	{
		// remove all current weights
		for (const FVertexBoneWeight& BoneWeight : Weights.PreChangeWeights[SelectedVertex])
		{
			// when pasting weights, we want a complete replacement, so we prune everything first
			constexpr bool bPruneInfluence = true;
			constexpr float ZeroWeight = 0.f;
			PasteWeightEdits.MergeSingleEdit(BoneWeight.BoneID, SelectedVertex, ZeroWeight, bPruneInfluence, Weights.PreChangeWeights);
		}

		// add weights from clipboard
		for (const TPair<BoneIndex, float>& LoadedWeight : LoadedWeights)
		{
			const BoneIndex BoneID = LoadedWeight.Key;
			const float NewWeight = LoadedWeight.Value;
			constexpr bool bPruneInfluence = false;
			PasteWeightEdits.MergeSingleEdit(BoneID, SelectedVertex, NewWeight, bPruneInfluence, Weights.PreChangeWeights);
		}
	}
	
	// apply the changes
	const FText TransactionLabel = LOCTEXT("PasteWeightsChange", "Paste skin weights.");
	ApplyWeightEditsAsTransaction(PasteWeightEdits, TransactionLabel);

	// notify user
	const FText NotificationText = FText::FromString("Pasted weights.");
	ShowEditorMessage(ELogVerbosity::Log, NotificationText);
}

void USkinWeightsPaintTool::TruncateWeightMap(TMap<BoneIndex, float>& InOutWeights, int32 NumLockedBones)
{
	// sort influences by total weight
	InOutWeights.ValueSort([](const float& A, const float& B)
	{
		return A > B;
	});

	// truncate to MaxInfluences
	int32 Index = 0;
	for (TMap<BoneIndex, float>::TIterator It(InOutWeights); It; ++It)
	{
		if (Index >= MAX_TOTAL_INFLUENCES - NumLockedBones)
		{
			It.RemoveCurrent();
		}
		else
		{
			++Index;
		}
	}
}

void USkinWeightsPaintTool::NormalizeWeightMap(TMap<BoneIndex, float>& InOutWeights, float ExpectedSum)
{
	// normalize remaining influences
	float TotalWeight = 0.f;
	for (const TTuple<BoneIndex, float>& Weight : InOutWeights)
	{
		TotalWeight += Weight.Value;
	}
	
	for (TTuple<BoneIndex, float>& Weight : InOutWeights)
	{
		Weight.Value /= TotalWeight > SMALL_NUMBER ? TotalWeight : 1.f;
		Weight.Value *= ExpectedSum;
	}
}

void USkinWeightsPaintTool::AccumulateWeights(
	const TArray<SkinPaintTool::VertexWeights>& AllWeights,
	const TArray<VertexIndex>& VerticesToAccumulate,
	const TSet<BoneIndex>& ExcludedBones,
	TMap<BoneIndex, float>& OutAccumulatedWeights)
{
	for (const VertexIndex VertexID : VerticesToAccumulate)
	{
		for (const FVertexBoneWeight& BoneWeight : AllWeights[VertexID])
		{
			if (!ExcludedBones.Contains(BoneWeight.BoneID))
			{
				float& AccumulatedWeight = OutAccumulatedWeights.FindOrAdd(BoneWeight.BoneID);
				AccumulatedWeight += BoneWeight.Weight;
			}
		}
	}
}

void USkinWeightsPaintTool::GetLockedWeight(
	const TArray<VertexWeights>& AllWeights,
	VertexIndex InVertexID,
	const TSet<BoneIndex>& LockedBones,
	float& OutTotalLockedWeight,
	int32& OutNumLockedBones)
{
	OutTotalLockedWeight = 0;
	OutNumLockedBones = 0;
	
	for (const FVertexBoneWeight& BoneWeight : AllWeights[InVertexID])
	{
		if (LockedBones.Contains(BoneWeight.BoneID))
		{
			OutTotalLockedWeight += BoneWeight.Weight;
			OutNumLockedBones++;
		}
	}
}

void USkinWeightsPaintTool::HandleSkeletalMeshModified(const TArray<FName>& InBoneNames, const ESkeletalMeshNotifyType InNotifyType)
{
	switch (InNotifyType)
	{
	case ESkeletalMeshNotifyType::BonesAdded:
		break;
	case ESkeletalMeshNotifyType::BonesRemoved:
		break;
	case ESkeletalMeshNotifyType::BonesMoved:
		{
			// TODO update only vertices weighted to modified bones (AND CHILDREN!?)
			Weights.Deformer.SetAllVerticesToBeUpdated();
			break;	
		}
	case ESkeletalMeshNotifyType::BonesSelected:
		{
			// store selected bones
			SelectedBoneNames = InBoneNames;
			PendingCurrentBone = InBoneNames.IsEmpty() ? NAME_None : InBoneNames[0];

			// update selected bone indices from names
			SelectedBoneIndices.Reset();
			for (const FName SelectedBoneName : SelectedBoneNames)
			{
				SelectedBoneIndices.Add(GetBoneIndexFromName(SelectedBoneName));
			}
		}
		break;
	case ESkeletalMeshNotifyType::BonesRenamed:
		break;
	case ESkeletalMeshNotifyType::HierarchyChanged:
		break;
	default:
		checkNoEntry();
	}
}

TSharedPtr<SWidget> USkinWeightsPaintTool::GetCustomSkeletonTreeWidget(TSharedPtr<ISkeletalMeshEditorBinding> EditorBinding)
{
	if (!SkeletonReader)
	{
		return nullptr;
	}
	
	FRefSkeletonTreeDelegates Delegates;
	Delegates.OnGetRowWidget.BindUObject(this,&USkinWeightsPaintTool::OnGetLockWidget);
	Delegates.OnExtendCommandList.BindLambda([this]( TSharedPtr<FUICommandList> InOutCommandList)
		{
			FSkinWeightsPaintToolCommands Commands = FSkinWeightsPaintToolCommands::Get();
			InOutCommandList->MapAction( Commands.IsolateSelectedBones,
				FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::IsolateSelectedBones),
				FCanExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::HasSelectedBone));
			
			InOutCommandList->MapAction( Commands.ToggleSelectedBonesLockState,
				FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::ToggleSelectedBonesLockState),
				FCanExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::HasSelectedBone));
			
			InOutCommandList->MapAction( Commands.LockSelectedBones,
				FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::SetLockSelectedBones, true),
				FCanExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::HasSelectedBone));
			InOutCommandList->MapAction( Commands.UnlockSelectedBones,
				FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::SetLockSelectedBones, false),
				FCanExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::HasSelectedBone));
			
			InOutCommandList->MapAction( Commands.LockAllBones, FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::SetLockAllBones, true));
			InOutCommandList->MapAction( Commands.UnlockAllBones, FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::SetLockAllBones, false));
			
			InOutCommandList->MapAction( Commands.BypassLocks,
				FExecuteAction::CreateUObject(this, &USkinWeightsPaintTool::ToggleBypassLocks),
				FCanExecuteAction(),
				FIsActionChecked::CreateUObject(this, &USkinWeightsPaintTool::IsBypassingAllLocks));
		});
	Delegates.OnExtendContextMenu.BindLambda([](FMenuBuilder& MenuBuilder)
		{
			FSkinWeightsPaintToolCommands Commands = FSkinWeightsPaintToolCommands::Get();

			
			
			MenuBuilder.BeginSection("Common", LOCTEXT("CommonSection", "Common"));
			{
				MenuBuilder.AddMenuEntry(Commands.IsolateSelectedBones);
				MenuBuilder.AddMenuEntry(Commands.ToggleSelectedBonesLockState);
			}
			MenuBuilder.EndSection();
			
			MenuBuilder.BeginSection("Lock", LOCTEXT("LockSection", "Lock"));
			{
				MenuBuilder.AddMenuEntry(Commands.LockSelectedBones);
				MenuBuilder.AddMenuEntry(Commands.LockAllBones);
			}
			MenuBuilder.EndSection();
			
			MenuBuilder.BeginSection("Unlock", LOCTEXT("UnlockSection", "Unlock"));
			{
				MenuBuilder.AddMenuEntry(Commands.UnlockSelectedBones);
				MenuBuilder.AddMenuEntry(Commands.UnlockAllBones);
			}
			MenuBuilder.EndSection();
			
			MenuBuilder.BeginSection("Bypass", LOCTEXT("BypassSection", "Bypass"));
			{
				MenuBuilder.AddMenuEntry(Commands.BypassLocks);
			}
			MenuBuilder.EndSection();

			return true;
		});
	
	ToolSkeletonTree =
		SNew(SReferenceSkeletonTree)
		.Modifier(SkeletonReader)
		.Delegates(Delegates);

	SkeletonTreeNotifierBindScope.Reset(
		new FSkeletalMeshNotifierBindScope(
			EditorBinding->GetNotifier(),
			ToolSkeletonTree->GetNotifier()));	
		
	TArray<FName> SelectedBones = EditorBinding->GetSelectedBones();
	ToolSkeletonTree->GetNotifier()->HandleNotification(SelectedBones, ESkeletalMeshNotifyType::BonesSelected);		
	
	return ToolSkeletonTree;
}

TSharedRef<SWidget> USkinWeightsPaintTool::OnGetLockWidget(FName BoneName)
{
	int32 BoneIndex = GetBoneIndexFromName(BoneName);

	auto GetColorAndOpacity = [this, BoneIndex] ()
		{
			const bool bIsLockByUser = GetRawLockedBoneIndices().Contains(BoneIndex);
			const bool bIsHovered = LockButtons[BoneIndex]->IsHovered();

			FLinearColor OutColor = FLinearColor::White;
			if (IsBypassingAllLocks())
			{
				OutColor = FLinearColor::Green;
			}
			
			float Opacity = 0.0f;

			if (bIsLockByUser)
			{
				// Directly active, full opacity
				Opacity = 1.0f;
			}
			else if (bIsHovered)
			{
				// Mouse is over widget and it is not directly active.
				Opacity = .8f;
			}
			else 
			{
				// Not active in any way and mouse is not over widget or item.
				Opacity = 0.1f;
			}
			OutColor.A = Opacity;
			return OutColor;	
		};

	TSharedPtr<SButton> LockButton = SNew(SButton)
		.IsFocusable(false)
		.ButtonStyle(FAppStyle::Get(), "NoBorder")
		.ButtonColorAndOpacity_Lambda(GetColorAndOpacity)
		.OnClicked_Lambda([this, BoneIndex]()
		{
			if (LockedBoneIndices.Contains(BoneIndex))
			{
				SetLockBone(BoneIndex, false);
			}
			else
			{
				SetLockBone(BoneIndex, true);
			}
			return FReply::Handled();
		})
		.ContentPadding(1.f)
		.ToolTipText(LOCTEXT("BoneWeightLockToolTip", "Skin weights for locked bones cannot be changed"))
		[
			SNew(SImage)
			.ColorAndOpacity_Lambda(GetColorAndOpacity)
			.Image_Lambda([this]()
				{
					if (bBypassLocks)
					{
						return FAppStyle::GetBrush("Icons.Unlock");
					}
					return FAppStyle::GetBrush("Icons.Lock");
				})
		];

	LockButtons.Emplace(BoneIndex, LockButton);
	
	return LockButton.ToSharedRef();
}

void USkinWeightsPaintTool::SetLockAllBones(bool bLock)
{
	ISkeletonProvider* SkeletonProvider = CastChecked<ISkeletonProvider>(Target);
	const FReferenceSkeleton& RefSkeleton = SkeletonProvider->GetSkeleton();
	int32 NumBones = RefSkeleton.GetRawBoneNum();

	for (BoneIndex BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		SetLockBone(BoneIndex, bLock);
	}
}

void USkinWeightsPaintTool::SetLockSelectedBones(bool bLock)
{
	for (int32 BoneIndex : SelectedBoneIndices)
	{
		SetLockBone(BoneIndex, bLock);
	}
}

void USkinWeightsPaintTool::IsolateSelectedBones()
{
	SetLockAllBones(true);
	SetLockSelectedBones(false);
}

void USkinWeightsPaintTool::ToggleSelectedBonesLockState()
{
	for (int32 BoneIndex : SelectedBoneIndices)
	{
		if (GetRawLockedBoneIndices().Contains(BoneIndex))
		{
			SetLockBone(BoneIndex,false);
		}
		else
		{
			SetLockBone(BoneIndex, true);
		}
	}
}

void USkinWeightsPaintTool::ToggleBypassLocks()
{
	bBypassLocks = !bBypassLocks;
}

void USkinWeightsPaintTool::SetLockBone(BoneIndex InBoneIndex, bool bLock)
{
	if (bLock)
	{
		LockedBoneIndices.Add(InBoneIndex);
	}
	else
	{
		LockedBoneIndices.Remove(InBoneIndex);
	}	
}

bool USkinWeightsPaintTool::ShouldProceedWithAllBonesUnlocked()
{
	if (GetFinalLockedBoneIndices().IsEmpty())
	{
		return true;
	}

	FText Message = LOCTEXT("UnlockAllBonesDialog", "The current operation does not support skin weight locking, unlock weights for all bones and continue?");
	EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::OkCancel, Message);

	if (Response == EAppReturnType::Ok)
	{
		SetLockAllBones(false);	
	}
	
	return Response == EAppReturnType::Ok;
}

void USkinWeightsPaintTool::OnActiveSkinWeightProfileChanged()
{
	FScopedSuspendIsolation RetainIsolationScope(this);

	WeightToolProperties->bShowNewProfileName = WeightToolProperties->SkinWeightProfileOption == CreateNewName();

	FName DesiredSkinWeightProfileName = WeightToolProperties->GetDesiredSkinWeightProfileForTool();

	GetOrCreateSkinWeightsAttribute(EditedMesh, DesiredSkinWeightProfileName);
	
	check(IsProfileValid(DesiredSkinWeightProfileName));
	
	if ( DesiredSkinWeightProfileName == ActiveSkinWeightProfileName)
	{
		return;
	}

	ActiveSkinWeightProfileName = DesiredSkinWeightProfileName;

	// Re-init Weights with new skin profile
	Weights = FSkinToolWeights();
	Weights.Profile = ActiveSkinWeightProfileName;
	Weights.InitializeSkinWeights(this, EditedMesh);
	bVertexColorsNeedUpdated = true;	
}

void USkinWeightsPaintTool::OnNewSkinWeightProfileChanged()
{
	// We use NewProfilePlaceHolderName() to address the new profile while the tool is running
	// and only apply user-supplied new profile name to the mesh during shutdown. 
	// It is done this way because profile rename does not support yet support undo, while weight edits
	// can undo across profiles. This is not the most intuitive and could be refactored in the future.
	
	// Given this design, here we only need to make sure the new profile name is unique w.r.t 
	// existing profiles + our new profile placeholder name
	
	// Note: it is a bit silly that this might be called twice since we are changing the property
	// being watched.
	
	if (ensure(WeightToolProperties->bShowNewProfileName))
	{
		WeightToolProperties->NewSkinWeightProfile = GetValidNewProfileName(WeightToolProperties->NewSkinWeightProfile);
	}
}

bool USkinWeightsPaintTool::IsProfileValid(const FName InProfileName)
{
	return EditedMesh.Attributes()->HasSkinWeightsAttribute(InProfileName);
}

USkinWeightsPaintTool::FScopedSuspendIsolation::FScopedSuspendIsolation(USkinWeightsPaintTool* InTool)
	: Tool(InTool)
{
	// Restore to full mesh to add the new profile and reisolate later to default isolation state
	if (!Tool->GetSelectionIsolator()->IsMeshIsolated())
	{
		// apply previous changes
		Tool->GetWeights().ApplyCurrentWeightsToMesh(Tool->GetMesh());
	}
	else
	{
		SavedIsolatedTriangles = Tool->GetSelectionIsolator()->GetIsolatedTriangles();
		// Restore also applies weights
		Tool->GetSelectionIsolator()->RestoreFullMesh();
	}
}

USkinWeightsPaintTool::FScopedSuspendIsolation::~FScopedSuspendIsolation()
{
	if (!SavedIsolatedTriangles.IsEmpty())
	{
		// re-isolate to default isolation if present
		Tool->GetSelectionIsolator()->SetTrianglesToIsolate(SavedIsolatedTriangles);
		Tool->GetSelectionIsolator()->UpdateIsolatedSelection();
	}
}

FName USkinWeightsPaintTool::GetValidNewProfileName(const FName& InProposedName) const
{
	FName ValidName = InProposedName;
	TArray<FName> Profiles;
	EditedMesh.Attributes()->GetSkinWeightsAttributes().GenerateKeyArray(Profiles);
		
	while (Profiles.Contains(ValidName))
	{
		ValidName.SetNumber(ValidName.GetNumber() + 1);
	}	
	
	return ValidName;
}

void USkinWeightsPaintTool::ToggleEditingMode()
{
	Weights.Deformer.SetAllVerticesToBeUpdated();

	UpdateBrushIndicators();

	// toggle which mesh we're selecting and what components (vert/edge/face)
	UpdateSelectorState();

	ToggleBoneManipulation( WeightToolProperties->EditingMode == EWeightEditMode::Bones);

	SetFocusInViewport();
}

void USkinWeightsPaintTool::UpdateSelectorState() const
{
	const USkinWeightsPaintToolProperties* ToolProperties = GetWeightToolProperties();
	
	const bool bIsMeshEditing = ToolProperties->EditingMode == EWeightEditMode::Mesh;
	const bool bHasSourceMesh = TransferManager->GetPreviewMesh() != nullptr;
	const bool bWasSetToSelectSource = ToolProperties->MeshSelectMode == EMeshTransferOption::Source;
	const bool bEnableSourceMeshSelector = bIsMeshEditing && bWasSetToSelectSource && bHasSourceMesh;
	const bool bEnableTargetMeshSelector = bIsMeshEditing && !bWasSetToSelectSource;
	
	// source mesh selector
	UToolMeshSelector* SourceMeshSelector = TransferManager->GetMeshSelector();
	SourceMeshSelector->SetIsEnabled(bEnableSourceMeshSelector);
	
	// main mesh selector
	MeshSelector->SetIsEnabled(bEnableTargetMeshSelector);

	// update component mode
	MeshSelector->SetComponentSelectionMode(WeightToolProperties->ComponentSelectionMode);
	SourceMeshSelector->SetComponentSelectionMode(WeightToolProperties->ComponentSelectionMode);
}

UToolMeshSelector* USkinWeightsPaintTool::GetMainMeshSelector()
{
	return MeshSelector;
}

UToolMeshSelector* USkinWeightsPaintTool::GetActiveMeshSelector()
{
	if (WeightToolProperties->MeshSelectMode == EMeshTransferOption::Source)
	{
		return GetWeightTransferManager()->GetMeshSelector();
	}

	return MeshSelector;
}

void UWeightToolSelectionIsolator::InitialSetup(USkinWeightsPaintTool* InTool)
{
	WeightTool = InTool;
}

void UWeightToolSelectionIsolator::UpdateIsolatedSelection()
{
	// this is queued to run on Tick() because modifying the mesh from other threads can cause the tool's Render() to be out of sync
	if (bIsolatedMeshNeedsUpdated)
	{
		if (CurrentlyIsolatedTriangles.IsEmpty())
		{
			RestoreFullMesh();	
		}
		else
		{
			CreatePartialMesh();
		}
		
		bIsolatedMeshNeedsUpdated = false;
	}
}

bool UWeightToolSelectionIsolator::CanIsolate() const
{
	// Currently only support isolation when the mesh in default isolation state (either full mesh, or isolated to a set externally provided via context)
	// It also means it is possible for the isolator to start an isolation while already in a isolated state if the default is not full mesh
	return CurrentlyIsolatedTriangles.Num() == WeightTool->GetDefaultIsolationTriangles().Num();
}

bool UWeightToolSelectionIsolator::IsMeshIsolated() const
{
	return PartialSubMesh.GetBaseMesh() != nullptr;
}

void UWeightToolSelectionIsolator::IsolateSelectionAsTransaction()
{
	const FText& TransactionLabel = LOCTEXT("IsolateSelectTransaction", "Isolate Selection");

	TUniquePtr<FIsolateSelectionChange> ActiveChange = MakeUnique<FIsolateSelectionChange>();
	ActiveChange->IsolatedTrianglesBefore = GetIsolatedTriangles();
	WeightTool->GetMainMeshSelector()->GetSelectedTriangles(ActiveChange->IsolatedTrianglesAfter);
	
	// Selected triangles are in preview mesh space. If currently isolated, remap to EditedMesh (full mesh) space.
	if (IsMeshIsolated())
	{
		for (int32& TriID : ActiveChange->IsolatedTrianglesAfter)
		{
			TriID = PartialSubMesh.MapTriangleToBaseMesh(TriID);
		}
	}

	UpdateIsolatedSelection();
	SetTrianglesToIsolate(ActiveChange->IsolatedTrianglesAfter);
	
	UInteractiveToolManager* ToolManager = WeightTool->GetToolManager();
	ToolManager->BeginUndoTransaction(TransactionLabel);
	ToolManager->EmitObjectChange(WeightTool, MoveTemp(ActiveChange), TransactionLabel);
	ToolManager->EndUndoTransaction();
}

void UWeightToolSelectionIsolator::UnIsolateSelectionAsTransaction()
{
	const FText& TransactionLabel =  LOCTEXT("ShowAllTransaction", "Show All");

	TUniquePtr<FIsolateSelectionChange> ActiveChange = MakeUnique<FIsolateSelectionChange>();
	ActiveChange->IsolatedTrianglesBefore = GetIsolatedTriangles();
	// Restore to default isolation, which is either a valid set or empty(full mesh)
	ActiveChange->IsolatedTrianglesAfter = WeightTool->GetDefaultIsolationTriangles();

	UpdateIsolatedSelection();
	SetTrianglesToIsolate(ActiveChange->IsolatedTrianglesAfter);
	
	UInteractiveToolManager* ToolManager = WeightTool->GetToolManager();
	ToolManager->BeginUndoTransaction(TransactionLabel);
	ToolManager->EmitObjectChange(WeightTool, MoveTemp(ActiveChange), TransactionLabel);
	ToolManager->EndUndoTransaction();
}

void UWeightToolSelectionIsolator::SetTrianglesToIsolate(const TArray<int32>& TrianglesToIsolate)
{
	if (bIsolatedMeshNeedsUpdated)
	{
		// cannot queue up multiple changes
		// we must allow previous selection isolations to be applied to keep the weight buffer in-sync
		return;
	}
	
	// record the triangles we are isolating (if empty, this will restore the full mesh)
	CurrentlyIsolatedTriangles = TrianglesToIsolate;
	// queue an update for next tick
	bIsolatedMeshNeedsUpdated = true;
}

void UWeightToolSelectionIsolator::RestoreFullMesh()
{
	using namespace UE::MeshModelingToolsEditorOnly::SelectionUtils;

	if (!IsMeshIsolated())
	{
		// nothing hidden
		return;
	}

	FDynamicMesh3& FullMesh = WeightTool->GetMesh();

	// apply partial mesh weights to full mesh
	ApplyWeightsToFullMesh();

	// retain selection (a convenience for the user that allows for easily adjusting the isolation by going back/forth as needed)
	const FWeightToolScopedRetainSelection RetainSelectionScope(*this);
 
	// reinitialize with full mesh
	// this resizes Weights to the full mesh size
	WeightTool->UpdateCurrentlyEditedMesh(FullMesh);

	PartialSubMesh = {};
	CurrentlyIsolatedTriangles.Reset();
}

void UWeightToolSelectionIsolator::ApplyWeightsToFullMesh() const
{
	// get the weights
	FSkinToolWeights& Weights = WeightTool->GetWeights();
	FDynamicMesh3& FullMesh = WeightTool->GetMesh();
	
	if (IsMeshIsolated())
	{
		auto PartialToFull = [this](int32 Idx) { return PartialSubMesh.MapVertexToBaseMesh(Idx); };
		Weights.ApplyCurrentWeightsToMesh(FullMesh, PartialToFull);
	}
	else
	{
		Weights.ApplyCurrentWeightsToMesh(FullMesh);
	}
}

int32 UWeightToolSelectionIsolator::PartialToFullMeshVertexIndex(int32 PartialMeshVertexIndex) const
{
	if (!IsMeshIsolated())
	{
		return PartialMeshVertexIndex;
	}

	return PartialSubMesh.MapVertexToBaseMesh(PartialMeshVertexIndex);
}

int32 UWeightToolSelectionIsolator::FullToPartialMeshVertexIndex(int32 FullMeshVertexIndex) const
{
	if (!IsMeshIsolated())
	{
		return FullMeshVertexIndex;
	}

	return PartialSubMesh.MapVertexToSubmesh(FullMeshVertexIndex);
}

const FDynamicMesh3& UWeightToolSelectionIsolator::GetPartialMesh() const
{
	static const FDynamicMesh3 Dummy;
	return IsMeshIsolated() ? PartialSubMesh.GetSubmesh() : Dummy;
}

void UWeightToolSelectionIsolator::CreatePartialMesh()
{
	using namespace UE::MeshModelingToolsEditorOnly::SelectionUtils;

	UPolygonSelectionMechanic* SelectionMechanic = WeightTool->GetMainMeshSelector()->GetSelectionMechanic();
	if (!ensure(SelectionMechanic))
	{
		return;
	}

	if (!ensure(!CurrentlyIsolatedTriangles.IsEmpty()))
	{
		return;
	}

	ApplyWeightsToFullMesh();
	
	// retain selection — captures in full mesh space, restores in new submesh space
	const FWeightToolScopedRetainSelection RetainSelectionScope(*this);

	FDynamicMesh3& FullMesh = WeightTool->GetMesh();
	// create a partial sub-mesh from a subset of triangles on the current mesh
	PartialSubMesh = UE::Geometry::FDynamicSubmesh3(&FullMesh, CurrentlyIsolatedTriangles);

	// reinitialize all mesh data structures
	WeightTool->UpdateCurrentlyEditedMesh(PartialSubMesh.GetSubmesh());
}


bool USkinWeightsPaintTool::HasActiveSelectionOnMainMesh()
{
	if (!WeightToolProperties)
	{
		return false;
	}

	UToolMeshSelector* MainMeshSelector = GetMainMeshSelector();
	if (!MainMeshSelector)
	{
		return false;
	}
	
	const bool bSelectingTargetMesh = WeightToolProperties->MeshSelectMode == EMeshTransferOption::Target;
	const bool bTargetMeshHasSelection = GetMainMeshSelector()->IsAnyComponentSelected();
	return bSelectingTargetMesh && bTargetMeshHasSelection;
}

void USkinWeightsPaintTool::SelectAffected() const
{
	UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic();
	if (!ensure(SelectionMechanic))
	{
		return;
	}
	
	GetToolManager()->BeginUndoTransaction(LOCTEXT("AffectedSelectionChange", "Select Affected"));
	SelectionMechanic->BeginChange();
	
	// get all vertices affected by all selected bones
	TSet<int32> AffectedVertices;
	for (const BoneIndex SelectedBone : SelectedBoneIndices)
	{
		GetVerticesAffectedByBone(SelectedBone, AffectedVertices);
	}
	
	// create selection set
	FGroupTopologySelection Selection;

	// optionally add/remove/replace selection based on modifier key state
	const FGroupTopologySelection& CurrentSelection = SelectionMechanic->GetActiveSelection();
	
	const bool bIsShiftDown = FSlateApplication::IsInitialized() ? FSlateApplication::Get().GetModifierKeys().IsShiftDown() : false;
	const bool bIsControlDown = FSlateApplication::IsInitialized() ? FSlateApplication::Get().GetModifierKeys().IsControlDown() : false;
	if (bIsShiftDown)
	{
		// ADD to selection
		Selection.SelectedCornerIDs.Append(CurrentSelection.SelectedCornerIDs);
		Selection.SelectedCornerIDs.Append(AffectedVertices);
	}
	else if (bIsControlDown)
	{		
		// REMOVE from selection if all affected vertices are contained in the current selection
		// ADD to the selection otherwise
		const bool bRemoveFromSelection = Algo::AllOf(AffectedVertices, [&CurrentSelection](const int32 VertexIndex)
			{
				return CurrentSelection.SelectedCornerIDs.Contains(VertexIndex);
			});

		if (bRemoveFromSelection)
		{
			Selection.SelectedCornerIDs = CurrentSelection.SelectedCornerIDs.Difference(AffectedVertices);
		}
		else
		{
			Selection.SelectedCornerIDs.Append(CurrentSelection.SelectedCornerIDs);
			Selection.SelectedCornerIDs.Append(AffectedVertices);
		}
	}
	else
	{
		// REPLACE selection
		Selection.SelectedCornerIDs = MoveTemp(AffectedVertices);
	}
	
	// select vertices
	constexpr bool bBroadcast = true;
	SelectionMechanic->SetSelection(Selection, bBroadcast);
	SelectionMechanic->EndChangeAndEmitIfModified();
	GetToolManager()->EndUndoTransaction();
}

void USkinWeightsPaintTool::SelectByInfluenceCount(const int32 MinInfluences) const
{
	UPolygonSelectionMechanic* SelectionMechanic = MeshSelector->GetSelectionMechanic();
	if (!ensure(SelectionMechanic))
	{
		return;
	}
	
	GetToolManager()->BeginUndoTransaction(LOCTEXT("InfluenceCountSelectionChange", "Select by Influence Count"));
	SelectionMechanic->BeginChange();

	// create selection set
	FGroupTopologySelection Selection;

	// spin through all vertices and find the ones that have at least MinInfluences number of bones affecting them
	for (VertexIndex VertexID=0; VertexID<Weights.PreChangeWeights.Num(); ++VertexID)
	{
		const VertexWeights& SingleVertexWeights = Weights.PreChangeWeights[VertexID];
		if (SingleVertexWeights.Num() >= MinInfluences)
		{
			Selection.SelectedCornerIDs.Add(VertexID);
		}
	}
	
	// select vertices
	constexpr bool bBroadcast = true;
	SelectionMechanic->SetSelection(Selection, bBroadcast);
	SelectionMechanic->EndChangeAndEmitIfModified();
	GetToolManager()->EndUndoTransaction();
}

void USkinWeightsPaintTool::GetVerticesAffectedByBone(BoneIndex IndexOfBone, TSet<int32>& OutVertexIndices) const
{
	VertexIndex VertexID = 0;
	for (const VertexWeights& VertWeights : Weights.PreChangeWeights)
	{
		for (const FVertexBoneWeight& BoneWeight : VertWeights)
		{
			if (BoneWeight.BoneID != IndexOfBone)
			{
				continue;
			}

			if (BoneWeight.Weight < MinimumWeightThreshold)
			{
				continue;
			}
			
			OutVertexIndices.Add(VertexID);
		}
		
		++VertexID;
	}
}

void USkinWeightsPaintTool::GetInfluences(const TArray<int32>& VertexIndices, TArray<BoneIndex>& OutBoneIndices)
{
	for (const int32 SelectedVertex : VertexIndices)
	{
		for (const FVertexBoneWeight& VertexBoneData : Weights.CurrentWeights[SelectedVertex])
		{
			OutBoneIndices.AddUnique(VertexBoneData.BoneID);
		}
	}
	
	// sort hierarchically (bone indices are sorted root to leaf)
	OutBoneIndices.Sort([](BoneIndex A, BoneIndex B) {return A < B;});
}

float USkinWeightsPaintTool::GetAverageWeightOnBone(
	const BoneIndex InBoneIndex,
	const TArray<int32>& VertexIndices)
{
	float TotalWeight = 0.f;
	float NumVerticesInfluencedByBone = 0.f;
	
	for (const VertexIndex VertexID : VertexIndices)
	{
		if (!Weights.CurrentWeights.IsValidIndex(VertexID))
		{
			continue;
		}
		
		for (const FVertexBoneWeight& VertexBoneData : Weights.CurrentWeights[VertexID])
		{
			if (VertexBoneData.BoneID == InBoneIndex)
			{
				++NumVerticesInfluencedByBone;
				TotalWeight += VertexBoneData.Weight;
			}
		}
	}

	return NumVerticesInfluencedByBone > 0 ? TotalWeight / NumVerticesInfluencedByBone : TotalWeight;
}

FName USkinWeightsPaintTool::GetBoneNameFromIndex(BoneIndex InIndex) const
{
	const TArray<FName>& Names = Weights.Deformer.BoneNames;
	if (Names.IsValidIndex(InIndex))
	{
		return Names[InIndex];
	}

	return NAME_None;
}


BoneIndex USkinWeightsPaintTool::GetCurrentBoneIndex() const
{
	return GetBoneIndexFromName(CurrentBone);
}

void USkinWeightsPaintTool::SetDisplayVertexColors(bool bShowVertexColors)
{
	if (bShowVertexColors)
	{
		UMaterialInterface* VtxColorMaterial = GetToolManager()->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
		PreviewMesh->SetOverrideRenderMaterial(VtxColorMaterial);
		bVertexColorsNeedUpdated = true;
	}
	else
	{
		PreviewMesh->ClearOverrideRenderMaterial();
	}
}

void USkinWeightsPaintTool::OnPropertyModified(UObject* ModifiedObject, FProperty* ModifiedProperty)
{
	Super::OnPropertyModified(ModifiedObject, ModifiedProperty);

	if (ModifiedProperty)
	{
		if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, BrushStrength))
		{
			WeightToolProperties->GetBrushConfig().Strength = WeightToolProperties->BrushStrength;
		}
		else if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, BrushRadius))
		{
			// Honour the user's Sync toggle: lockstep across modes when on, per-mode save when off.
			if (WeightToolProperties->bSyncBrushRadiusAcrossModes)
			{
				WeightToolProperties->SetSharedRadius(WeightToolProperties->BrushRadius);
			}
			else
			{
				WeightToolProperties->GetBrushConfig().Radius = WeightToolProperties->BrushRadius;
			}
		}
		else if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, BrushFalloffAmount))
		{
			WeightToolProperties->GetBrushConfig().Falloff = WeightToolProperties->BrushFalloffAmount;
		}
		else if (ModifiedProperty->GetName() == GET_MEMBER_NAME_STRING_CHECKED(USkinWeightsPaintToolProperties, ColorMode))
		{
			bVertexColorsNeedUpdated = true;
		}

		// let the mesh transfer system react to properties being set
		TransferManager->OnPropertyModified(WeightToolProperties, ModifiedProperty);
	
		SetFocusInViewport();
	}
}

#undef LOCTEXT_NAMESPACE