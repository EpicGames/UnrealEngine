// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatform.h"

#include "SkinWeightsPaintToolPropertiesCustomVersion.generated.h"

/** 
 * Versioning for the skin weights paint tool properties (USkinWeightsPaintToolProperties).
 * Mind this differs fundamentally from the common custom object version logic,
 * it is and has to use the UInteractiveToolPropertySet::RestoreProperties system.
 * 
 * See USkinWeightsPaintToolProperties::RestoreProperties for the related upgrade path
 */
UENUM()
enum class ESkinWeightsPaintToolPropertiesCustomVersion : uint32
{
	// Before versioning was added
	NoVersion = 0x00,

	// Upgrade SkinWeightsPaintToolProperties to use FLinearColorRamp instead of an array of linear colors
	SkinWeightPaintToolUseLinearColorRamp = 0x01 << 0,

	// Clamp per-mode brush Strength values to [0,1] (Multiply slider semantics changed
	// from "factor 0..2" to "factor 0..1 with Ctrl mirroring across 1.0").
	SkinWeightPaintToolClampStrengthToUnit = 0x01 << 1,

	// Add new versions above 
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};
