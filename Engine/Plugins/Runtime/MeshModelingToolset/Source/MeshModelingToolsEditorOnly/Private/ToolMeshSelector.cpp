// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolMeshSelector.h"

#include "BaseBehaviors/DoubleClickBehavior.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Engine/World.h"
#include "InteractiveTool.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PreviewMesh.h"
#include "Selection/GroupTopologySelector.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Selections/GeometrySelection.h"

// thise method is deprecated - InViewportClient is no longer used 
void UToolMeshSelector::InitialSetup(UWorld* InWorld, UInteractiveTool* InParentTool, FEditorViewportClient* InViewportClient, TFunction<void()> OnSelectionChangedFunc)
{
	InitialSetup(InWorld, InParentTool, OnSelectionChangedFunc);
}

void UToolMeshSelector::InitialSetup(UWorld* InWorld, UInteractiveTool* InParentTool, TFunction<void()> OnSelectionChangedFunc)
{
	World = InWorld;

	// set up vertex selection mechanic
	PolygonSelectionMechanic = NewObject<UToolMeshSelectorSelectionMechanic>(this);
	PolygonSelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	PolygonSelectionMechanic->Setup(InParentTool);
	PolygonSelectionMechanic->SetIsEnabled(false, false);
	PolygonSelectionMechanic->OnSelectionChanged.AddLambda(OnSelectionChangedFunc);

	// set up style of vertex selection
	constexpr FLinearColor VertexSelectedPurple = FLinearColor(0.78f, 0.f, 0.78f);
	constexpr FLinearColor VertexSelectedYellow = FLinearColor(1.f, 1.f, 0.f);
	// adjust selection rendering for this context
	PolygonSelectionMechanic->HilightRenderer.PointColor = FLinearColor::Blue;
	PolygonSelectionMechanic->HilightRenderer.PointSize = 10.0f;
	// vertex highlighting once selected
	PolygonSelectionMechanic->SelectionRenderer.LineThickness = 1.0f;
	PolygonSelectionMechanic->SelectionRenderer.PointColor = VertexSelectedYellow;
	PolygonSelectionMechanic->SelectionRenderer.PointSize = 5.0f;
	PolygonSelectionMechanic->SelectionRenderer.DepthBias = 2.0f;
	// despite the name, this renders the vertices
	PolygonSelectionMechanic->PolyEdgesRenderer.PointColor = VertexSelectedPurple;
	PolygonSelectionMechanic->PolyEdgesRenderer.PointSize = 5.0f;
	PolygonSelectionMechanic->PolyEdgesRenderer.DepthBias = 2.0f;
	PolygonSelectionMechanic->PolyEdgesRenderer.LineThickness = 1.0f;
}

void UToolMeshSelector::SetMesh(
	UPreviewMesh* InPreviewMesh,
	const FTransform3d& InMeshTransform)
{
	// store the mesh this is operating on
	PreviewMesh = InPreviewMesh;

	if (!ensure(World))
	{
		return;
	}

	if (!PreviewMesh)
	{
		SetIsEnabled(false);
		return;
	}

	// reset selection topology and mesh spatial data
	static constexpr bool bAutoBuild = true;
	const FDynamicMesh3* DynamicMesh = PreviewMesh->GetMesh();
	SelectionTopology = MakeUnique<UE::Geometry::FTriangleGroupTopology>(DynamicMesh, bAutoBuild);
	PolygroupTopology = MakeUnique<UE::Geometry::FGroupTopology>(DynamicMesh, bAutoBuild);
	MeshSpatial = MakeUnique<FDynamicMeshAABBTree3>(DynamicMesh, bAutoBuild);

	// initialize the selection mechanic
	PolygonSelectionMechanic->Initialize(
		DynamicMesh,
		InMeshTransform,
		World,
		SelectionTopology.Get(),
		[this]() { return MeshSpatial.Get(); }
	);
	if (UToolMeshSelectorSelectionMechanic* SelectorMechanic = Cast<UToolMeshSelectorSelectionMechanic>(PolygonSelectionMechanic))
	{
		SelectorMechanic->SetPolygroupTopology(PolygroupTopology.Get());
		SelectorMechanic->ClearLastSingleClickedVertex();
	}

	// clear the selection (old selection is invalid on new topo)
	PolygonSelectionMechanic->ClearSelection();
	PolygonSelectionMechanic->ClearHighlight();

	// selection colors
	constexpr FLinearColor FaceSelectedOrange = FLinearColor(0.886f, 0.672f, 0.473f);
	// configure secondary render material for selected triangles
	// NOTE: the material returned by ToolSetupUtil::GetSelectionMaterial has a checkerboard pattern on back faces which makes it hard to use
	if (UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial")))
	{
		if (UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, World))
		{
			MatInstance->SetVectorParameterValue(TEXT("Color"), FaceSelectedOrange);
			PreviewMesh->SetSecondaryRenderMaterial(MatInstance);
		}
	}

	// secondary triangle buffer used to render face selection
	PreviewMesh->EnableSecondaryTriangleBuffers([this](const FDynamicMesh3* Mesh, int32 TriangleID)
		{
			if (!PolygonSelectionMechanic)
			{
				return false;
			}
			return PolygonSelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, SelectionTopology.Get(), TriangleID);
		});
	// notify preview mesh when triangle selection has been updated
	PolygonSelectionMechanic->OnSelectionChanged.AddWeakLambda(this, [this]()
		{
			PreviewMesh->FastNotifySecondaryTrianglesChanged();
		});
	PolygonSelectionMechanic->OnFaceSelectionPreviewChanged.AddWeakLambda(this, [this]()
		{
			PreviewMesh->FastNotifySecondaryTrianglesChanged();
		});
}

void UToolMeshSelector::Shutdown()
{
	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->Shutdown();
	}

	PolygonSelectionMechanic = nullptr;
}

void UToolMeshSelector::SetIsEnabled(bool bIsEnabled)
{
	if (!PolygonSelectionMechanic)
	{
		return;
	}

	// force off if there's no preview mesh
	bIsEnabled = PreviewMesh ? bIsEnabled : false;

	PolygonSelectionMechanic->SetIsEnabled(bIsEnabled, bIsEnabled);
}

void UToolMeshSelector::SetSelectionTool(EMeshSelectorTool InTool)
{
	if (!PolygonSelectionMechanic || !PolygonSelectionMechanic->Properties)
	{
		return;
	}

	switch (InTool)
	{
	case EMeshSelectorTool::Marquee:
		PolygonSelectionMechanic->Properties->SelectionDragTool = EMeshTopologySelectionMechanicDragTool::Marquee;
		break;

	case EMeshSelectorTool::Lasso:
		PolygonSelectionMechanic->Properties->SelectionDragTool = EMeshTopologySelectionMechanicDragTool::Lasso;
		break;

	case EMeshSelectorTool::Ray:
	default:
		PolygonSelectionMechanic->Properties->SelectionDragTool = EMeshTopologySelectionMechanicDragTool::None;
		break;
	}
}

EMeshSelectorTool UToolMeshSelector::GetSelectionTool() const
{
	if (!PolygonSelectionMechanic || !PolygonSelectionMechanic->Properties)
	{
		return EMeshSelectorTool::Ray;
	}
	switch (PolygonSelectionMechanic->Properties->SelectionDragTool)
	{
	case EMeshTopologySelectionMechanicDragTool::Marquee:
		return EMeshSelectorTool::Marquee;
	case EMeshTopologySelectionMechanicDragTool::Lasso:
		return EMeshSelectorTool::Lasso;
	default:
		return EMeshSelectorTool::Ray;
	}
}

void UToolMeshSelector::SetComponentSelectionMode(EComponentSelectionMode InMode)
{
	if (!(PolygonSelectionMechanic))
	{
		return;
	}

	PolygonSelectionMechanic->Properties->bSelectVertices = InMode == EComponentSelectionMode::Vertices;
	PolygonSelectionMechanic->Properties->bSelectEdges = InMode == EComponentSelectionMode::Edges;
	PolygonSelectionMechanic->Properties->bSelectFaces = InMode == EComponentSelectionMode::Faces;
	PolygonSelectionMechanic->SetShowSelectableCorners(InMode == EComponentSelectionMode::Vertices);
	PolygonSelectionMechanic->SetShowEdges(InMode == EComponentSelectionMode::Edges);
}

void UToolMeshSelector::SetTransform(const FTransform3d& InTargetTransform)
{
	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->SetTransform(InTargetTransform);
	}
}

void UToolMeshSelector::UpdateAfterMeshDeformation()
{
	MeshSpatial->Build();
	constexpr bool bTopologyDeformed = true;
	constexpr bool bTopologyModified = false;
	PolygonSelectionMechanic->GetTopologySelector()->Invalidate(bTopologyDeformed, bTopologyModified);
}

void UToolMeshSelector::Tick(float DeltaTime)
{
	if (PolygonSelectionMechanic)
	{
		PolygonSelectionMechanic->Tick(DeltaTime);
	}
}

void UToolMeshSelector::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (!PolygonSelectionMechanic)
	{
		return;
	}

	PolygonSelectionMechanic->DrawHUD(Canvas, RenderAPI);
}

void UToolMeshSelector::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (!(PolygonSelectionMechanic && PreviewMesh))
	{
		return;
	}

	PolygonSelectionMechanic->Render(RenderAPI);
}

const TArray<int32>& UToolMeshSelector::GetSelectedVertices()
{
	SelectedVerticesInternal.Empty();
	if (!(PolygonSelectionMechanic && PreviewMesh))
	{
		return SelectedVerticesInternal;
	}

	const FGroupTopologySelection& Selection = PolygonSelectionMechanic->GetActiveSelection();
	const FDynamicMesh3* DynamicMesh = PreviewMesh->GetMesh();

	// validate and add vertices to the output array
	auto AddVertices = [this](const TSet<int32>& VerticesToAdd)
		{
			for (const int32 VertexToAdd : VerticesToAdd)
			{
				SelectedVerticesInternal.Add(VertexToAdd);
			}
		};

	// add selected vertices
	AddVertices(Selection.SelectedCornerIDs);

	// add vertices on selected edges
	{
		TSet<int32> VerticesInSelectedEdges;
		for (const int32 SelectedEdgeIndex : Selection.SelectedEdgeIDs)
		{
			FDynamicMesh3::FEdge CurrentEdge = DynamicMesh->GetEdge(SelectedEdgeIndex);
			VerticesInSelectedEdges.Add(CurrentEdge.Vert.A);
			VerticesInSelectedEdges.Add(CurrentEdge.Vert.B);
		}

		AddVertices(VerticesInSelectedEdges);
	}

	// add vertices in selected faces
	{
		TSet<int32> VerticesInSelectedFaces;
		for (const int32 SelectedFaceIndex : Selection.SelectedGroupIDs)
		{
			UE::Geometry::FIndex3i TriangleVertices = DynamicMesh->GetTriangleRef(SelectedFaceIndex);
			VerticesInSelectedFaces.Add(TriangleVertices[0]);
			VerticesInSelectedFaces.Add(TriangleVertices[1]);
			VerticesInSelectedFaces.Add(TriangleVertices[2]);
		}

		AddVertices(VerticesInSelectedFaces);
	}

	return SelectedVerticesInternal;
}

bool UToolMeshSelector::IsAnyComponentSelected() const
{
	if (!PolygonSelectionMechanic)
	{
		return false;
	}

	return PolygonSelectionMechanic->HasSelection();
}

void UToolMeshSelector::GetSelectedTriangles(TArray<int32>& OutTriangleIndices) const
{
	OutTriangleIndices.Empty();
	if (!ensure(PolygonSelectionMechanic))
	{
		return;
	}

	const FGroupTopologySelection& Selection = PolygonSelectionMechanic->GetActiveSelection();
	const FDynamicMesh3* DynamicMesh = PreviewMesh->GetMesh();
	TSet<int32> TriangleSet;

	// add triangles connected to selected vertices
	for (const int32 VertexIndex : Selection.SelectedCornerIDs)
	{
		DynamicMesh->EnumerateVertexTriangles(VertexIndex, [&TriangleSet](int32 TriangleIndex)
			{
				TriangleSet.Add(TriangleIndex);
			});
	}

	// add triangles connected to selected edges
	for (const int32 EdgeIndex : Selection.SelectedEdgeIDs)
	{
		DynamicMesh->EnumerateEdgeTriangles(EdgeIndex, [&TriangleSet](int32 TriangleIndex)
			{
				TriangleSet.Add(TriangleIndex);
			});
	}

	// add selected triangles
	TriangleSet.Append(Selection.SelectedGroupIDs);

	OutTriangleIndices = TriangleSet.Array();
}

void UToolMeshSelector::GrowSelection() const
{
	if (!ensure(PolygonSelectionMechanic))
	{
		return;
	}

	PolygonSelectionMechanic->GrowSelection(/*bAsTriangleTopology*/ true);
}

void UToolMeshSelector::ShrinkSelection() const
{
	if (!ensure(PolygonSelectionMechanic))
	{
		return;
	}

	PolygonSelectionMechanic->ShrinkSelection(/*bAsTriangleTopology*/ true);
}

void UToolMeshSelector::InvertSelection() const
{
	if (!ensure(PolygonSelectionMechanic))
	{
		return;
	}

	PolygonSelectionMechanic->InvertSelection(/*bAsTriangleTopology*/ true);
}

void UToolMeshSelector::FloodSelection() const
{
	if (!ensure(PolygonSelectionMechanic))
	{
		return;
	}

	PolygonSelectionMechanic->FloodSelection();
}

void UToolMeshSelector::SelectBorder() const
{
	if (!ensure(PolygonSelectionMechanic))
	{
		return;
	}

	PolygonSelectionMechanic->ConvertSelectionToBorderVertices(/*bAsTriangleTopology*/ true);
}


void UToolMeshSelectorSelectionMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	Super::Setup(ParentToolIn);

	EdgeLoopDoubleClickBehavior = NewObject<ULocalDoubleClickInputBehavior>(this);
	EdgeLoopDoubleClickBehavior->Initialize();
	// Bumped to win the second-click capture race against the base mechanic's click-or-drag.
	EdgeLoopDoubleClickBehavior->SetDefaultPriority(
		BasePriority.MakeHigher(FInputCapturePriority::BINDING_COMPLEXITY_PRIORITY_STEP * 3));
	EdgeLoopDoubleClickBehavior->ModifierCheckFunc = [](const FInputDeviceState& Input)
	{
		return Input.bShiftKeyDown;
	};
	EdgeLoopDoubleClickBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& Click)
	{
		return OnEdgeLoopDoubleClickHit(Click);
	};
	EdgeLoopDoubleClickBehavior->OnClickedFunc = [this](const FInputDeviceRay& Click)
	{
		OnEdgeLoopDoubleClicked(Click);
	};
	ParentToolIn->AddInputBehavior(EdgeLoopDoubleClickBehavior, this);
}

void UToolMeshSelectorSelectionMechanic::Shutdown()
{
	if (UInteractiveTool* ParentToolPtr = GetParentTool())
	{
		ParentToolPtr->RemoveInputBehaviorsBySource(this);
	}
	EdgeLoopDoubleClickBehavior = nullptr;

	Super::Shutdown();
}

FInputRayHit UToolMeshSelectorSelectionMechanic::OnEdgeLoopDoubleClickHit(const FInputDeviceRay& ClickPos)
{
	if (!bIsEnabled)
	{
		return FInputRayHit();
	}
	if (!Properties || !Properties->bSelectVertices)
	{
		return FInputRayHit();
	}
	FHitResult OutHit;
	FGroupTopologySelection HitSelection;
	if (!TopologyHitTest(ClickPos.WorldRay, OutHit, HitSelection) || HitSelection.SelectedCornerIDs.IsEmpty())
	{
		return FInputRayHit();
	}
	return FInputRayHit(0.f);
}

void UToolMeshSelectorSelectionMechanic::OnEdgeLoopDoubleClicked(const FInputDeviceRay& ClickPos)
{
	using namespace UE::Geometry;

	if (!ensure(bIsEnabled))
	{
		return;
	}
	if (!Mesh)
	{
		return;
	}
	FHitResult OutHit;
	FGroupTopologySelection HitSelection;
	if (!TopologyHitTest(ClickPos.WorldRay, OutHit, HitSelection) || HitSelection.SelectedCornerIDs.IsEmpty())
	{
		return;
	}
	const int32 ClickedVid = *HitSelection.SelectedCornerIDs.CreateConstIterator();

	// Seed direction is the edge from the previous single-click-selected vertex to the
	// double-clicked vertex. PreviousSingleClickedVertexID rather than Last because the
	// first half of this shift+double-click was already dispatched as a single click and
	// overwrote LastSingleClickedVertexID with the double-clicked vertex itself. If the
	// previous tracked vert is missing, stale, equal to the clicked vert, or not adjacent,
	// no seed is produced and the gesture degenerates to simply adding the clicked vert.
	int32 SeedMeshEdgeID = INDEX_NONE;
	if (LoopWalkTopology
		&& PreviousSingleClickedVertexID != INDEX_NONE
		&& PreviousSingleClickedVertexID != ClickedVid
		&& PersistentSelection.SelectedCornerIDs.Contains(PreviousSingleClickedVertexID))
	{
		const int32 EdgeID = Mesh->FindEdge(PreviousSingleClickedVertexID, ClickedVid);
		if (EdgeID != IndexConstants::InvalidID)
		{
			SeedMeshEdgeID = EdgeID;
		}
	}

	TSet<int32> NewVerts = PersistentSelection.SelectedCornerIDs;
	NewVerts.Add(ClickedVid);

	if (SeedMeshEdgeID != INDEX_NONE)
	{
		const int32 SeedGroupEdgeID = LoopWalkTopology->FindGroupEdgeID(SeedMeshEdgeID);
		if (SeedGroupEdgeID != IndexConstants::InvalidID)
		{
			FGroupTopologySelection ExpandedEdges;
			ExpandedEdges.SelectedEdgeIDs.Add(SeedGroupEdgeID);
			FGroupTopologySelector LoopSelector(Mesh, LoopWalkTopology);
			LoopSelector.ExpandSelectionByEdgeLoops(ExpandedEdges);
			LoopSelector.ExpandSelectionByBoundaryLoops(ExpandedEdges);

			for (const int32 GroupEdgeID : ExpandedEdges.SelectedEdgeIDs)
			{
				for (int32 V : LoopWalkTopology->GetGroupEdgeVertices(GroupEdgeID))
				{
					NewVerts.Add(V);
				}
			}
		}
	}

	if (NewVerts.Num() == PersistentSelection.SelectedCornerIDs.Num())
	{
		return;
	}

	FGeometrySelection NewSelection;
	NewSelection.TopologyType = EGeometryTopologyType::Triangle;
	NewSelection.ElementType = EGeometryElementType::Vertex;
	for (int32 V : NewVerts)
	{
		NewSelection.Selection.Add(static_cast<uint64>(V));
	}

	BeginChange();
	constexpr bool bBroadcast = true;
	SetSelection_AsTriangleTopology(NewSelection, bBroadcast);
	EndChangeAndEmitIfModified();
}

void UToolMeshSelectorSelectionMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (!ensure(bIsEnabled))
	{
		return;
	}

	// Snapshot the corner under the cursor before Super mutates PersistentSelection. Only
	// a single, unambiguous corner hit qualifies; edge/face hits and empty-space clicks
	// push INDEX_NONE through the tracker.
	int32 ClickedCornerID = INDEX_NONE;
	if (Properties && Properties->bSelectVertices)
	{
		FHitResult OutHit;
		FGroupTopologySelection HitSelection;
		if (TopologyHitTest(ClickPos.WorldRay, OutHit, HitSelection, CameraState.bIsOrthographic)
			&& HitSelection.SelectedCornerIDs.Num() == 1)
		{
			ClickedCornerID = *HitSelection.SelectedCornerIDs.CreateConstIterator();
		}
	}

	Super::OnClicked(ClickPos);

	// The new "current" entry must actually be selected after Super (a ctrl-toggle that
	// removed it does not count).
	const int32 NewLast = (ClickedCornerID != INDEX_NONE
		&& PersistentSelection.SelectedCornerIDs.Contains(ClickedCornerID))
		? ClickedCornerID : INDEX_NONE;

	// Push current to previous so that a follow-up shift+double-click (whose first half
	// arrived as this very single click) can read the click that came before it.
	PreviousSingleClickedVertexID = LastSingleClickedVertexID;
	LastSingleClickedVertexID = NewLast;

	// A previously-tracked corner may have been removed from the selection by this click
	// (e.g. shift+click that cleared and reselected, ctrl-toggle, empty-space click).
	if (PreviousSingleClickedVertexID != INDEX_NONE
		&& !PersistentSelection.SelectedCornerIDs.Contains(PreviousSingleClickedVertexID))
	{
		PreviousSingleClickedVertexID = INDEX_NONE;
	}
}

void UToolMeshSelectorSelectionMechanic::SetPolygroupTopology(const UE::Geometry::FGroupTopology* InPolygroupTopology)
{
	LoopWalkTopology = InPolygroupTopology;
}

void UToolMeshSelectorSelectionMechanic::ClearLastSingleClickedVertex()
{
	LastSingleClickedVertexID = INDEX_NONE;
	PreviousSingleClickedVertexID = INDEX_NONE;
}