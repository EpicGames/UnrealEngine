// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolChange.h"
#include "AttributeEditorTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API


class IPrimitiveComponentBackedTarget;
class UAttributeEditorTool;


/**
 *
 */
UCLASS(MinimalAPI)
class UAttributeEditorToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};






UENUM()
enum class EAttributeEditorElementType : uint8
{
	Vertex = 0,
	VertexInstance = 1,
	Triangle = 2,
	Polygon = 3,
	Edge = 4,
	PolygonGroup = 5
};


UENUM()
enum class EAttributeEditorAttribType : uint8
{
	Int32 = 0,
	Boolean = 1,
	Float = 2,
	Vector2 = 3,
	Vector3 = 4,
	Vector4 = 5,
	String = 6,
	Unknown = 7
};


UCLASS(MinimalAPI)
class UAttributeEditorAttribProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(VisibleAnywhere, Category = AttributesInspector, meta=(NoResetToDefault))
	TArray<FString> VertexAttributes;

	UPROPERTY(VisibleAnywhere, Category = AttributesInspector, meta=(NoResetToDefault))
	TArray<FString> InstanceAttributes;

	UPROPERTY(VisibleAnywhere, Category = AttributesInspector, meta=(NoResetToDefault))
	TArray<FString> TriangleAttributes;

	UPROPERTY(VisibleAnywhere, Category = AttributesInspector, meta=(NoResetToDefault))
	TArray<FString> PolygonAttributes;

	UPROPERTY(VisibleAnywhere, Category = AttributesInspector, meta=(NoResetToDefault))
	TArray<FString> EdgeAttributes;

	UPROPERTY(VisibleAnywhere, Category = AttributesInspector, meta=(NoResetToDefault))
	TArray<FString> GroupAttributes;
};




UENUM()
enum class EAttributeEditorToolActions
{
	NoAction,

	ClearNormals,
	DiscardTangents,
	ClearAllUVs,
	ClearSelectedUVs,
	AddUVSet,
	DeleteSelectedUVSet,
	DuplicateSelectedUVSet,
	AddAttribute,
	AddWeightMapLayer,
	AddPolyGroupLayer,
	DeleteAttribute,
	ClearAttribute,
	CopyAttributeFromTo,

	EnableLightmapUVs,
	DisableLightmapUVs,
	ResetLightmapUVChannels
};



UCLASS(MinimalAPI)
class UAttributeEditorActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UAttributeEditorTool> ParentTool;

	void Initialize(UAttributeEditorTool* ParentToolIn) { ParentTool = ParentToolIn; }
	UE_API void PostAction(EAttributeEditorToolActions Action);

};


UCLASS(MinimalAPI)
class UAttributeEditorNormalsActions : public UAttributeEditorActionPropertySet
{
	GENERATED_BODY()
public:
	/** Remove any hard edges / split normals, setting all normals to a single averaged vertex normal */
	UFUNCTION(CallInEditor, Category = Normals, meta = (DisplayPriority = 1))
	void ResetHardNormals()
	{
		PostAction(EAttributeEditorToolActions::ClearNormals);
	}

	/** Clear Tangents from the mesh */
	UFUNCTION(CallInEditor, Category = Normals, meta = (DisplayPriority = 2))
	void DiscardTangents()
	{
		PostAction(EAttributeEditorToolActions::DiscardTangents);
	}
};



UCLASS(MinimalAPI)
class UAttributeEditorUVActions : public UAttributeEditorActionPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, Category = UVs, meta = (GetOptions = GetUVLayerNamesFunc, NoResetToDefault))
	FString UVLayer;

	UFUNCTION()
	UE_API TArray<FString> GetUVLayerNamesFunc();

	UPROPERTY()
	TArray<FString> UVLayerNamesList;

	/** Clear all UV layers, setting all UV values to (0,0) */
	UFUNCTION(CallInEditor, Category = UVs, meta = (DisplayPriority = 1))
	void ClearAll()
	{
		PostAction(EAttributeEditorToolActions::ClearAllUVs);
	}

	/** Add a new UV layer */
	UFUNCTION(CallInEditor, Category = UVs, meta = (DisplayPriority = 2))
	void AddNew()
	{
		PostAction(EAttributeEditorToolActions::AddUVSet);
	}

	/** Clear the selected UV layers, setting all UV values to (0,0) */
	UFUNCTION(CallInEditor, Category = UVs, meta = (DisplayPriority = 3))
	void DeleteSelected()
	{
		PostAction(EAttributeEditorToolActions::DeleteSelectedUVSet);
	}

	/** Duplicate the selected UV layers */
	UFUNCTION(CallInEditor, Category = UVs, meta = (DisplayPriority = 4))
	void DuplicateSelected()
	{
		PostAction(EAttributeEditorToolActions::DuplicateSelectedUVSet);
	}
};




UCLASS(MinimalAPI)
class UAttributeEditorLightmapUVActions : public UAttributeEditorActionPropertySet
{
	GENERATED_BODY()
public:
	/** Whether or not Lightmap UVs are enabled in the Static Mesh Build Settings. Use the Static Mesh Editor to change this value. */
	UPROPERTY(VisibleAnywhere, Category = LightmapUVs)
	bool bGenerateLightmapUVs;

	/** Source UV channel used to compute Lightmap UVs. Use the Static Mesh Editor to change this value. */
	UPROPERTY(VisibleAnywhere, Category = LightmapUVs, meta = (DisplayName = "Source Channel", NoResetToDefault))
	int32 SourceUVIndex;

	/** Lightmap UVs are stored in this UV Channel. Use the Static Mesh Editor to change this value. */
	UPROPERTY(VisibleAnywhere, Category = LightmapUVs, meta = (DisplayName = "Dest Channel", NoResetToDefault))
	int32 DestinationUVIndex;

	UFUNCTION(CallInEditor, Category = LightmapUVs, meta = (DisplayPriority = 1))
	void Enable()
	{
		PostAction(EAttributeEditorToolActions::EnableLightmapUVs);
	}

	UFUNCTION(CallInEditor, Category = LightmapUVs, meta = (DisplayPriority = 2))
	void Disable()
	{
		PostAction(EAttributeEditorToolActions::DisableLightmapUVs);
	}

	/** Reset Lightmap UV channels to Source Channel UV0 and Destination as UVMax+1 */
	UFUNCTION(CallInEditor, Category = LightmapUVs, meta = (DisplayPriority = 3))
	void Reset()
	{
		PostAction(EAttributeEditorToolActions::ResetLightmapUVChannels);
	}
};


UCLASS(MinimalAPI)
class UAttributeEditorNewAttributeActions : public UAttributeEditorActionPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = NewAttribute, meta = (DisplayName = "New Attribute Name") )
	FString NewName;

	//UPROPERTY(EditAnywhere, Category = NewAttribute)
	UPROPERTY()
	EAttributeEditorElementType ElementType;

	//UPROPERTY(EditAnywhere, Category = NewAttribute)
	UPROPERTY()
	EAttributeEditorAttribType DataType;

	//UFUNCTION(CallInEditor, Category = NewAttribute, meta = (DisplayPriority = 1))
	//void AddNew()
	//{
	//	PostAction(EAttributeEditorToolActions::AddAttribute);
	//}

	/** Add a new Per-Vertex Weight Map layer with the given Name */
	UFUNCTION(CallInEditor, Category = NewAttribute, meta = (DisplayPriority = 2))
	void AddWeightMapLayer()
	{
		PostAction(EAttributeEditorToolActions::AddWeightMapLayer);
	}

	/** Add a new PolyGroup layer with the given Name */
	UFUNCTION(CallInEditor, Category = NewAttribute, meta = (DisplayPriority = 3, DisplayName = "Add PolyGroup Layer"))
	void AddPolyGroupLayer()
	{
		PostAction(EAttributeEditorToolActions::AddPolyGroupLayer);
	}

};


UCLASS(MinimalAPI)
class UAttributeEditorModifyAttributeActions : public UAttributeEditorActionPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = ModifyAttribute, meta = (GetOptions = GetAttributeNamesFunc, NoResetToDefault))
	FString Attribute;

	UFUNCTION()
	UE_API TArray<FString> GetAttributeNamesFunc();

	UPROPERTY()
	TArray<FString> AttributeNamesList;

	/** Remove the selected Attribute Name from the mesh */
	UFUNCTION(CallInEditor, Category = ModifyAttribute, meta = (DisplayPriority = 1))
	void DeleteSelected()
	{
		PostAction(EAttributeEditorToolActions::DeleteAttribute);
	}

	//UFUNCTION(CallInEditor, Category = ModifyAttribute, meta = (DisplayPriority = 2))
	//void Clear()
	//{
	//	PostAction(EAttributeEditorToolActions::ClearAttribute);
	//}

};


UCLASS(MinimalAPI)
class UAttributeEditorCopyAttributeActions : public UAttributeEditorActionPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = CopyAttribute)
	TArray<FString> FromAttribute;

	UPROPERTY(EditAnywhere, Category = CopyAttribute)
	TArray<FString> ToAttribute;

	UFUNCTION(CallInEditor, Category = CopyAttribute, meta = (DisplayPriority = 1))
	void CopyFromTo()
	{
		PostAction(EAttributeEditorToolActions::CopyAttributeFromTo);
	}
};






/**
 * Mesh Attribute Editor Tool
 */
UCLASS(MinimalAPI)
class UAttributeEditorTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:

	UE_API UAttributeEditorTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return false; }
	virtual bool HasAccept() const override { return false; }
	virtual bool CanAccept() const override { return false; }

	UE_API virtual void RequestAction(EAttributeEditorToolActions ActionType);

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:

	UPROPERTY()
	TObjectPtr<UAttributeEditorNormalsActions> NormalsActions;

	UPROPERTY()
	TObjectPtr<UAttributeEditorUVActions> UVActions;

	UPROPERTY()
	TObjectPtr<UAttributeEditorLightmapUVActions> LightmapUVActions;

	UPROPERTY()
	TObjectPtr<UAttributeEditorAttribProperties> AttributeProps;

	UPROPERTY()
	TObjectPtr<UAttributeEditorNewAttributeActions> NewAttributeProps;

	UPROPERTY()
	TObjectPtr<UAttributeEditorModifyAttributeActions> ModifyAttributeProps;

	UPROPERTY()
	TObjectPtr<UAttributeEditorCopyAttributeActions> CopyAttributeProps;


protected:
	bool bTargetIsStaticMesh = true;

	bool bAttributeListsValid = false;
	bool bHaveAutoGeneratedLightmapUVSet = false;
	UE_API void InitializeAttributeLists();

	UE_API void UpdateAutoGeneratedLightmapUVChannel(UPrimitiveComponent* TargetComponent, int32 NewMaxUVChannels);

	EAttributeEditorToolActions PendingAction = EAttributeEditorToolActions::NoAction;
	UE_API void ClearNormals();
	UE_API void DiscardTangents();
	UE_API void ClearUVs();
	UE_API void DeleteSelectedUVSet();
	UE_API void DuplicateSelectedUVSet();
	UE_API void AddUVSet();
	UE_API void AddNewAttribute();
	UE_API void AddNewWeightMap();
	UE_API void AddNewGroupsLayer();
	UE_API void DeleteAttribute();
	UE_API void ClearAttribute();
	UE_API void SetLightmapUVsEnabled(bool bEnabled);
	UE_API void ResetLightmapUVsChannels();

	UE_API void AddNewAttribute(EAttributeEditorElementType ElemType, EAttributeEditorAttribType AttribType, FName AttributeName);

	friend class FAttributeEditor_AttributeListsChange;
	UE_API void EmitAttributesChange();
};


class FAttributeEditor_AttributeListsChange : public FToolCommandChange
{
public:
	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;
};

#undef UE_API
