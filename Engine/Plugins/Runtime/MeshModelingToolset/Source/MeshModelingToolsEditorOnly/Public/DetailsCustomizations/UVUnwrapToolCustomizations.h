// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class FPropertyRestriction;

// Customization for UUVEditorRecomputeUVsToolProperties
class FRecomputeUVsToolDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	void EvaluateLayoutTypeRestrictions(TSharedRef<IPropertyHandle> IslandGenerationModeHandle, TSharedRef<IPropertyHandle> UnwrapTypeHandle, TSharedRef<IPropertyHandle> LayoutTypeHandle);

	/** Property restriction applied to blend paint enum dropdown box */
	TSharedPtr<FPropertyRestriction> LayoutTypeEnumRestriction;
};


