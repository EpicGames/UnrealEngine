// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMeshModelingToolsEditor, Log, All);

class FMeshModelingToolsEditorOnlyModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	void OnPostEngineInit();

protected:

	TArray<FName> ClassesToUnregisterOnShutdown;
	TArray<FName> PropertiesToUnregisterOnShutdown;
};
