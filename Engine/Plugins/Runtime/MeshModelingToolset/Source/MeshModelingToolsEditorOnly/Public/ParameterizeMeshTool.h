// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshOpPreviewHelpers.h"
#include "Properties/MeshMaterialProperties.h"
#include "Properties/MeshUVChannelProperties.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Drawing/UVLayoutPreview.h"

#include "ParameterizeMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

// Forward declarations
class UDynamicMeshComponent;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UParameterizeMeshToolProperties;
class UParameterizeMeshToolUVAtlasProperties;
class UParameterizeMeshToolXAtlasProperties;
class UParameterizeMeshToolPatchBuilderProperties;
class UParameterizeMeshOperatorFactory;


UCLASS(MinimalAPI)
class UParameterizeMeshToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};


/**
 * UParameterizeMeshTool automatically decomposes the input mesh into charts, solves for UVs,
 * and then packs the resulting charts
 */
UCLASS(MinimalAPI)
class UParameterizeMeshTool : public USingleSelectionMeshEditingTool, public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolManageGeometrySelectionAPI -- this tool won't update external geometry selection or change selection-relevant mesh IDs
	virtual bool IsInputSelectionValidOnOutput() override
	{
		return true;
	}

protected:
	UPROPERTY()
	TObjectPtr<UMeshUVChannelProperties> UVChannelProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolUVAtlasProperties> UVAtlasProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolXAtlasProperties> XAtlasProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolPatchBuilderProperties> PatchBuilderProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygroupLayersProperties> PolygroupLayerProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UExistingMeshMaterialProperties> MaterialSettings = nullptr;

	UPROPERTY()
	bool bCreateUVLayoutViewOnSetup = true;

	UPROPERTY()
	TObjectPtr<UUVLayoutPreview> UVLayoutView = nullptr;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshOperatorFactory> Factory;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;

	UE_API void OnMethodTypeChanged();

	UE_API void OnPreviewMeshUpdated();
};

#undef UE_API
