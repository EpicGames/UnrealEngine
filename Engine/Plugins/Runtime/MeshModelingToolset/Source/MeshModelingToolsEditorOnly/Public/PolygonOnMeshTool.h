// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "InteractiveGizmo.h"
#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "MeshOpPreviewHelpers.h"
#include "CuttingOps/EmbedPolygonsOp.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "BaseTools/SingleClickTool.h"
#include "Mechanics/ConstructionPlaneMechanic.h"
#include "Mechanics/CollectSurfacePathMechanic.h"
#include "Polygon2.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "PolygonOnMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API


// predeclarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UTransformProxy;
class ULineSetComponent;




/**
 *
 */
UCLASS(MinimalAPI)
class UPolygonOnMeshToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};



UENUM()
enum class EPolygonType
{
	Circle,
	Square,
	Rectangle,
	RoundRect,
	Custom
};




/**
 * Standard properties of the polygon-on-mesh operations
 */
UCLASS(MinimalAPI)
class UPolygonOnMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** What operation to apply using the Polygon */
	UPROPERTY(EditAnywhere, Category = Operation)
	EEmbeddedPolygonOpMethod Operation = EEmbeddedPolygonOpMethod::CutThrough;

	/** Polygon Shape to use in this Operation */
	UPROPERTY(EditAnywhere, Category = Shape)
	EPolygonType Shape = EPolygonType::Circle;

	/** Use a volumetric boolean rather than curve projection; cuts through all layers and across edges */
	UPROPERTY(EditAnywhere, Category = Operation)
	bool bCutWithBoolean = true;

	/** Automatically attempt to fill any open boundaries left by CSG (e.g. due to numerical errors) */
	UPROPERTY(EditAnywhere, Category = Operation, meta = (EditCondition = 
		"bCutWithBoolean && (Operation == EEmbeddedPolygonOpMethod::CutThrough || Operation == EEmbeddedPolygonOpMethod::CutOutside || Operation == EEmbeddedPolygonOpMethod::InsertPolygon)",
		EditConditionHides))
	bool bTryToFixHoles = false;

	// TODO: re-add if/when extrude is added as a supported operation
	///** Amount to extrude, if extrude is enabled */
	//UPROPERTY(EditAnywhere, Category = Options)
	//float ExtrudeDistance;

	/** Scale of polygon to embed */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "0.01", UIMax = "10.0", ClampMin = "0.00001", ClampMax = "10000"))
	float PolygonScale = 1.0f;

	/** Width of Polygon */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "0.001", UIMax = "1000.0", ClampMin = "0.00001", ClampMax = "10000", EditCondition = "Shape != EPolygonType::Custom"))
	float Width = 100.0f;
		
	/** Height of Polygon */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "0.001", UIMax = "1000.0", ClampMin = "0.00001", ClampMax = "10000", EditCondition = "Shape == EPolygonType::Rectangle || Shape == EPolygonType::RoundRect"))
	float Height = 50.0f;

	/** Corner Ratio of RoundRect Polygon */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "0.01", ClampMin = "0.00001", ClampMax = "1.0", EditCondition = "Shape == EPolygonType::RoundRect"))
	float CornerRatio = 0.5f;

	/** Number of sides in Circle or RoundRect Corner */
	UPROPERTY(EditAnywhere, Category = Shape, meta = (UIMin = "3", UIMax = "20", ClampMin = "3", ClampMax = "10000", EditCondition = "Shape == EPolygonType::Circle || Shape == EPolygonType::RoundRect"))
	int32 Subdivisions = 12;

	/**
	 * Whether the tool will allow accepting a result if the operation fails, for instance due to inability to insert the
	 * polygon when not cutting with boolean, or due to unrepaired cracks in the result.
	 */
	UPROPERTY(EditAnywhere, Category = Operation, AdvancedDisplay)
	bool bCanAcceptFailedResult = false;

	/** 
	 * If an operation fails and we do not allow accepting the result, whether to show the intermediate failed result, or to
	 * show the original mesh.
	 */
	UPROPERTY(EditAnywhere, Category = Operation, AdvancedDisplay, meta = (EditCondition = "!bCanAcceptFailedResult"))
	bool bShowIntermediateResultOnFailure = false;
};




UENUM()
enum class EPolygonOnMeshToolActions
{
	NoAction,
	DrawPolygon
};


UCLASS(MinimalAPI)
class UPolygonOnMeshToolActionPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<UPolygonOnMeshTool> ParentTool;

	void Initialize(UPolygonOnMeshTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(EPolygonOnMeshToolActions Action);

	/** Extrude the current set of selected faces. Click in viewport to confirm extrude height. */
	UFUNCTION(CallInEditor, Category = Actions, meta = (DisplayName = "Draw Polygon", DisplayPriority = 1))
	void DrawPolygon()
	{
		PostAction(EPolygonOnMeshToolActions::DrawPolygon);
	}
};




/**
 * Simple Mesh Plane Cutting Tool
 */
UCLASS(MinimalAPI)
class UPolygonOnMeshTool : public USingleSelectionMeshEditingTool, public UE::Geometry::IDynamicMeshOperatorFactory, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:

	UE_API UPolygonOnMeshTool();

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;


	UE_API virtual void RequestAction(EPolygonOnMeshToolActions ActionType);



public:
	// support for hover and click, for drawing polygon. This should be in the UCollectSurfacePathMechanic,
	// but we don't support dynamically changing input behavior set yet

	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit);

	// IClickBehaviorTarget API
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget API
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}


protected:

	UPROPERTY()
	TObjectPtr<UPolygonOnMeshToolProperties> BasicProperties;

	UPROPERTY()
	TObjectPtr<UPolygonOnMeshToolActionPropertySet> ActionProperties;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnLineSet;

	TArray<int> EdgesOnFailure;
	TArray<int> EmbeddedEdges;
	bool bOperationSucceeded;

protected:
	UWorld* TargetWorld;

	FTransform3d WorldTransform;
	FViewCameraState CameraState;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalDynamicMesh;

	UPROPERTY()
	TObjectPtr<UConstructionPlaneMechanic> PlaneMechanic = nullptr;

	UPROPERTY()
	TObjectPtr<UCollectSurfacePathMechanic> DrawPolygonMechanic = nullptr;

	EPolygonOnMeshToolActions PendingAction = EPolygonOnMeshToolActions::NoAction;

	UE::Geometry::FFrame3d DrawPlaneWorld;

	UE::Geometry::FPolygon2d LastDrawnPolygon;
	UE::Geometry::FPolygon2d ActivePolygon;
	UE_API void UpdatePolygonType();

	UE_API void SetupPreview();
	UE_API void UpdateDrawPlane();

	UE_API void BeginDrawPolygon();
	UE_API void CompleteDrawPolygon();

	UE_API void UpdateVisualization();

	UE_API void GenerateAsset(const TArray<FDynamicMeshOpResult>& Results);
};

#undef UE_API
