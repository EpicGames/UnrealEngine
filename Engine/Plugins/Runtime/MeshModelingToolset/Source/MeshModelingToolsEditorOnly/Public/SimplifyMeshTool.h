// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/NoExportTypes.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "MeshOpPreviewHelpers.h"
#include "Properties/RemeshProperties.h"
#include "Properties/MeshWeightChannelProperties.h"
#include "CleaningOps/SimplifyMeshOp.h"		// required in header for enum types
#include "SimplifyMeshTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class UMeshStatisticsProperties;
class UMeshElementsVisualizer;
PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3);

/**
 *
 */
UCLASS(MinimalAPI)
class USimplifyMeshToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};



UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EApplyWeightMapToSimplifyMeasure : uint32
{
	None = 0 UMETA(Hidden),
	QuadricError = 1 << 0,
	GeometricTolerance = 1 << 1,
	EdgeLengths = 1 << 2,
	All = 0b111 UMETA(Hidden)
};
ENUM_CLASS_FLAGS(EApplyWeightMapToSimplifyMeasure);

/**
 * Standard properties of the Simplify operation
 */
UCLASS(MinimalAPI)
class USimplifyMeshToolProperties : public UMeshConstraintProperties
{
	GENERATED_BODY()
public:
	UE_API USimplifyMeshToolProperties();

	/** Simplification Scheme  */
	UPROPERTY(EditAnywhere, Category = Options)
	ESimplifyType SimplifierType;

	/** Simplification Target Type  */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SimplifierType != ESimplifyType::MinimalPlanar && SimplifierType != ESimplifyType::MinimalPolygroup && SimplifierType != ESimplifyType::ClusterBased"))
	ESimplifyTargetType TargetMode;

	/** Target percentage of original triangle count */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "100", ClampMin = "0.0", ClampMax = "100.0", 
				EditCondition = "SimplifierType != ESimplifyType::ClusterBased && SimplifierType != ESimplifyType::MinimalPolygroup && SimplifierType != ESimplifyType::MinimalPlanar && TargetMode == ESimplifyTargetType::Percentage"))
	int TargetPercentage;

	/** Target edge length */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "3.0", UIMax = "10.0", ClampMin = "0.001", ClampMax = "1000.0",
		EditCondition = "SimplifierType == ESimplifyType::ClusterBased || (TargetMode == ESimplifyTargetType::EdgeLength && SimplifierType != ESimplifyType::UEStandard && SimplifierType != ESimplifyType::MinimalPlanar && SimplifierType != ESimplifyType::MinimalPolygroup)"))
	float TargetEdgeLength;

	/** Square root of target quadric error */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.0", UIMax = "100.0", ClampMin = "0.", ClampMax = "10000.0",
		EditCondition = "TargetMode == ESimplifyTargetType::QuadricError && (SimplifierType != ESimplifyType::UEStandard && SimplifierType != ESimplifyType::MinimalPlanar && SimplifierType != ESimplifyType::MinimalPolygroup && SimplifierType != ESimplifyType::ClusterBased)"))
	float TargetQuadricErrorRoot = 0;

	/** Target triangle count */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "4", UIMax = "10000", ClampMin = "1", ClampMax = "9999999999",
				EditCondition = "TargetMode == ESimplifyTargetType::TriangleCount && SimplifierType != ESimplifyType::MinimalPlanar && SimplifierType != ESimplifyType::MinimalPolygroup && SimplifierType != ESimplifyType::ClusterBased"))
	int TargetTriangleCount;

	/** Target vertex count */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "4", UIMax = "10000", ClampMin = "1", ClampMax = "9999999999",
				EditCondition = "TargetMode == ESimplifyTargetType::VertexCount && SimplifierType != ESimplifyType::MinimalPlanar && SimplifierType != ESimplifyType::ClusterBased"))
	int TargetVertexCount;

	/** Angle threshold in degrees used for testing if two triangles should be considered coplanar, or two lines collinear */
	UPROPERTY()
	float MinimalAngleThreshold = 0.01f;

	//~ Note PolyEdgeAngleTolerance is very similar to MinimalAngleThreshold, but not redundant b/c the useful ranges are very different (MinimalAngleThreshold should generally be kept very small)
	/** Threshold angle change (in degrees) along a polygroup edge, above which a vertex must be added */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.001", ClampMin = "0.0", UIMax = "90.0", ClampMax = "180.0", EditCondition = "SimplifierType == ESimplifyType::MinimalPolygroup"))
	float PolyEdgeAngleTolerance = .1f;

	/** Threshold angle change (in degrees) between adjacent boundary edges, above which the common vertex must be preserved */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.001", ClampMin = "0.0", UIMax = "90.0", ClampMax = "180.0", EditCondition = "SimplifierType == ESimplifyType::ClusterBased"))
	float BoundaryEdgeAngleTolerance = 30.0f;

	/** Discard UVs and existing normals, allowing the simplifier to ignore any UV and normal seams. New per-vertex normals are computed. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bDiscardAttributes;

	/** If true, then simplification will consider geometric deviation with the input mesh  */
	UPROPERTY(EditAnywhere, Category = Options, meta = (EditCondition = "SimplifierType != ESimplifyType::MinimalPolygroup"))
	bool bGeometricConstraint;

	/** Geometric deviation tolerance used when bGeometricConstraint is enabled, to limit the geometric deviation between the simplified and original meshes */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0", ClampMax = "10000000.0",
				EditCondition = "bGeometricConstraint && SimplifierType != ESimplifyType::UEStandard && SimplifierType != ESimplifyType::MinimalPolygroup"))
	float GeometricTolerance;

	/** Optional regularization term to favor vertices remaining near their original locations. Using a small regularization weight can give cleaner topology in flat regions. */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.0", UIMax = "0.01", ClampMin = "0.0", ClampMax = "10.0",
		EditCondition = "SimplifierType != ESimplifyType::UEStandard && SimplifierType != ESimplifyType::MinimalPolygroup && SimplifierType != ESimplifyType::ClusterBased"))
	float RegularizeWeight = 0.000001f;

	UPROPERTY(EditAnywhere, Category = Options, meta = (Bitmask, BitmaskEnum = "/Script/MeshModelingToolsEditorOnly.EApplyWeightMapToSimplifyMeasure"))
	uint32 ApplyWeightMapTo = (int32)EApplyWeightMapToSimplifyMeasure::All;

	/** Display colors corresponding to the mesh's polygon groups */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowGroupColors = false;

	/** Enable projection back to input mesh */
	UPROPERTY(EditAnywhere, Category = Options, AdvancedDisplay)
	bool bReproject;

protected:
	
	virtual bool IsPreventNormalFlipsEnabled() const override
	{ 
		return SimplifierType == ESimplifyType::QEM ||
		       SimplifierType == ESimplifyType::MinimalExistingVertex || 
		       SimplifierType == ESimplifyType::Attribute || 
		       SimplifierType == ESimplifyType::MinimalPlanar;
	}

	virtual bool IsPreventTinyTrianglesEnabled() const override
	{
		return SimplifierType == ESimplifyType::QEM ||
			   SimplifierType == ESimplifyType::MinimalExistingVertex ||
			   SimplifierType == ESimplifyType::Attribute ||
			   SimplifierType == ESimplifyType::MinimalPlanar;
	}

};




/**
 * Simple Mesh Simplifying Tool
 */
UCLASS(MinimalAPI)
class USimplifyMeshTool : public USingleSelectionMeshEditingTool, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

private:
	UPROPERTY()
	TObjectPtr<USimplifyMeshToolProperties> SimplifyProperties;

	UPROPERTY()
	TObjectPtr<UMeshWeightChannelDensityProperties> WeightMapProperties;

	UPROPERTY()
	TObjectPtr<UMeshStatisticsProperties> MeshStatisticsProperties;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsDisplay;

	// cached mesh description version of the original mesh -- created (by conversion or copy) as needed
	TSharedPtr<FMeshDescription, ESPMode::ThreadSafe> OriginalMeshDescription;
	// Dynamic Mesh versions precomputed in Setup (rather than recomputed for every simplify op)
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial;

	UE_API void UpdateVisualization();
};

#undef UE_API
