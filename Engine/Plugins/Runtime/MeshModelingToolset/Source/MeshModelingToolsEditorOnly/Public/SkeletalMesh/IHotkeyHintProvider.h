// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"

#include "IHotkeyHintProvider.generated.h"


UINTERFACE(MinimalAPI)
class UHotkeyHintProvider :
	public UInterface
{
	GENERATED_BODY()
};


class IHotkeyHintProvider : public IInterface
{
	GENERATED_BODY()

public:
	struct FHotkeyHint
	{
		FText Label;
		FText HotkeyText;
	};

	virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const = 0;

	// If CommandList is provided, the hint is filtered by CanExecute and skipped if Command
	// isn't bound on the list. If null, the hint is always appended (tool actions, which are
	// inherently always-executable while the tool is active, can pass null).
	static bool TryAppendCommandHint(TArray<FHotkeyHint>& OutHints,
		const TSharedPtr<const FUICommandInfo>& Command,
		const TSharedPtr<const FUICommandList>& CommandList = nullptr)
	{
		if (!Command.IsValid())
		{
			return false;
		}
		if (CommandList.IsValid())
		{
			const FUIAction* Action = CommandList->GetActionForCommand(Command);
			if (!Action || !Action->CanExecute())
			{
				return false;
			}
		}
		const TSharedRef<const FInputChord> Chord = Command->GetActiveChord(EMultipleKeyBindingIndex::Primary);
		const FText ChordText = Chord->IsValidChord()
			? Chord->GetInputText()
			: NSLOCTEXT("HotkeyHintProvider", "UnboundChord", "Unbound");
		OutHints.Add({ Command->GetLabel(), ChordText });
		return true;
	}

	static bool TryAppendCommandHint(TArray<FHotkeyHint>& OutHints,
		const FName ContextName, const FName CommandName,
		const TSharedPtr<const FUICommandList>& CommandList = nullptr)
	{
		return TryAppendCommandHint(OutHints,
			FInputBindingManager::Get().FindCommandInContext(ContextName, CommandName),
			CommandList);
	}
};
