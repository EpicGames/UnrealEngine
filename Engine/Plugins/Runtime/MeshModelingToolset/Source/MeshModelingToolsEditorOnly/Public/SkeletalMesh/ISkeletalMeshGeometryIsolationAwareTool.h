// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "ISkeletalMeshGeometryIsolationAwareTool.generated.h"


UINTERFACE(MinimalAPI)
class USkeletalMeshGeometryIsolationAwareTool :
	public UInterface
{
	GENERATED_BODY()
};


class ISkeletalMeshGeometryIsolationAwareTool : public IInterface
{
	GENERATED_BODY()

public:
	virtual bool IsInputIsolationValidOnOutput() const = 0;
};
