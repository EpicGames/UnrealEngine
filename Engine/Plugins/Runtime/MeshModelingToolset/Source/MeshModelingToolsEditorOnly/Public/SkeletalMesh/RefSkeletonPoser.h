// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolChange.h"
#include "UObject/Object.h"
#include "ReferenceSkeleton.h"

#include "RefSkeletonPoser.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API


UCLASS(MinimalAPI)
class URefSkeletonPoser: public UObject
{
	GENERATED_BODY()

public:
	UE_API void SetRefSkeleton(const FReferenceSkeleton& InRefSkeleton);
	UE_API const FReferenceSkeleton& GetRefSkeleton() const;

	UE_API void ModifyBoneAdditiveTransform(int32 BoneIndex, TFunctionRef<void(FTransform&)> InModifyFunc);
	UE_API void ClearBoneAdditiveTransform(int32 BoneIndex);
	UE_API void ClearAllBoneAdditiveTransforms();

	UE_API TOptional<FTransform> GetBoneAdditiveTransform(int32 BoneIndex);

	UE_API const TArray<FTransform>& GetComponentSpaceTransforms();

	UE_API const FTransform& GetComponentSpaceTransform(const uint32 Index);

	UE_API const TArray<FTransform>& GetComponentSpaceTransformsRefPose();

	UE_API void BeginPoseChange();
	UE_API void EndPoseChange();
	UE_API bool IsRecordingPoseChange();

protected:
	void Invalidate(const uint32 Index = INDEX_NONE);
	
	FReferenceSkeleton RefSkeleton;
	TArray<FTransform> Transforms;
	TBitArray<> TransformCached;

	TMap<int32, FTransform> AdditiveTransforms;
		
	TArray<FTransform> RefTransforms;

	class FPoseChange : public FToolCommandChange
	{
	public:
		FPoseChange(URefSkeletonPoser* InPoser);
	
		TMap<int32, FTransform> OldAdditiveTransforms;
		TMap<int32, FTransform> NewAdditiveTransforms;
	
		virtual FString ToString() const override;

		virtual void Apply(UObject* Object) override;

		virtual void Revert(UObject* Object) override;

		void Close(URefSkeletonPoser* InPoser);
	};
	
	TUniquePtr<FPoseChange> ActivePoseChange;

	bool bPoseChanged = false;
};




#undef UE_API
