// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "SkeletalMeshNotifier.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Misc/Change.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class FTextFilterExpressionEvaluator;
class FUICommandList;
class USkeletonModifier;
class SInlineEditableTextBlock;

namespace ReferenceSkeletonTreeLocals
{
class SReferenceSkeletonTreeView;

class FSkeletonModifierChange : public FCommandChange
{
public:
	FSkeletonModifierChange(const USkeletonModifier* InModifier);

	virtual FString ToString() const override
	{
		return FString(TEXT("Edit Skeleton"));
	}

	virtual void Apply(UObject* Object) override;
	virtual void Revert(UObject* Object) override;

	void StoreSkeleton(const USkeletonModifier* InModifier);

private:
	FReferenceSkeleton PreChangeSkeleton;
	TArray<int32> PreBoneTracker;
	FReferenceSkeleton PostChangeSkeleton;
	TArray<int32> PostBoneTracker;
};

enum class EBoneRenameType : uint32
{
	Rename,
	SearchAndReplace,
	AddPrefix,
	AddSuffix
};
	
}

class FBoneElement : public TSharedFromThis<FBoneElement>
{
public:
	FBoneElement(const FName& InBoneName, TWeakObjectPtr<USkeletonModifier> InModifier);

	FName BoneName;

	bool bIsHidden = false;
	
	TSharedPtr<FBoneElement> UnFilteredParent;
	TSharedPtr<FBoneElement> Parent;
	TArray<TSharedPtr<FBoneElement>> Children;

	void RequestRename(const ReferenceSkeletonTreeLocals::EBoneRenameType InRenameType) const;
	
	DECLARE_DELEGATE_OneParam(FOnRenameRequested, const ReferenceSkeletonTreeLocals::EBoneRenameType);
	FOnRenameRequested OnRenameRequested;

	DECLARE_DELEGATE_TwoParams(FOnRenamed, const FName, const FName);
	FOnRenamed OnRenamed;

private:
	TWeakObjectPtr<USkeletonModifier> WeakModifier;
};

using FOnRefSkeletonTreeCanAcceptDrop = STableRow<TSharedPtr<FBoneElement>>::FOnCanAcceptDrop;
using FOnRefSkeletonTreeAcceptDrop = STableRow<TSharedPtr<FBoneElement>>::FOnAcceptDrop;
using FOnBoneRenamed = FBoneElement::FOnRenamed;

struct FRefSkeletonTreeDelegates
{	
	// Avoid deprecation warnings when the struct is constructed or copied with default methods
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRefSkeletonTreeDelegates() = default;
	FRefSkeletonTreeDelegates(const FRefSkeletonTreeDelegates&) = default;
	FRefSkeletonTreeDelegates(FRefSkeletonTreeDelegates&&) = default;
	FRefSkeletonTreeDelegates& operator=(const FRefSkeletonTreeDelegates&) = default;
	FRefSkeletonTreeDelegates& operator=(FRefSkeletonTreeDelegates&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FOnDragDetected					OnDragDetected;
	FOnRefSkeletonTreeCanAcceptDrop	OnCanAcceptDrop;
	FOnRefSkeletonTreeAcceptDrop	OnAcceptDrop;
	FOnBoneRenamed					OnBoneRenamed;	
	UE_DEPRECATED(5.8, "Instead use OnBoneElementNameCommitted which is explicit about the renamed element")
	FOnTextCommitted				OnBoneNameCommitted;

	DECLARE_DELEGATE_ThreeParams(FOnBoneElementNameCommitted, const FText& /** ElementName */, ETextCommit::Type /** TextCommitType */, TSharedRef<FBoneElement> /** BoneElement */)
	FOnBoneElementNameCommitted	OnBoneElementNameCommitted;

	// Tools can use these to extend the skeleton tree with extra commands or widgets
	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnGetRowWidget, FName)
	FOnGetRowWidget	OnGetRowWidget;
	
	DECLARE_DELEGATE_OneParam(FOnExtendCommandList, TSharedPtr<FUICommandList>)
	FOnExtendCommandList OnExtendCommandList;
	
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnExtendContextMenu, FMenuBuilder&)
	FOnExtendContextMenu OnExtendContextMenu;
};

class SBoneItem : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SBoneItem) {}
		SLATE_ARGUMENT(TWeakPtr<FBoneElement>, TreeElement)
		SLATE_ARGUMENT(TWeakObjectPtr<USkeletonModifier>, WeakModifier)
		SLATE_ARGUMENT(FRefSkeletonTreeDelegates, Delegates)
		SLATE_EVENT( FIsSelected, IsSelected )
		SLATE_ATTRIBUTE(bool, IsReadOnly)
	SLATE_END_ARGS()

	virtual ~SBoneItem() override;
	
	// Slate construction function
	void Construct(const FArguments& InArgs);

	void HandleOnRenameRequested(const ReferenceSkeletonTreeLocals::EBoneRenameType InType);
	
private:

	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage) const;
	void OnTextCommitted(const FText& Text, ETextCommit::Type TextCommitType);

	void OnEnterEditingMode();
	void OnExitEditingMode();

	static FSlateColor GetTextColor(FIsSelected InIsSelected);
	
	FText GetName() const;
	
	TWeakPtr<FBoneElement> WeakTreeElement;
	TWeakObjectPtr<USkeletonModifier> WeakModifier;
	TSharedPtr< SInlineEditableTextBlock > InlineWidget;
	
	bool bEditing = false;
	TOptional<ReferenceSkeletonTreeLocals::EBoneRenameType> OptRenameType;

	/** Delegate executed when the bone name was committed */
	FRefSkeletonTreeDelegates::FOnBoneElementNameCommitted OnBoneElementNameCommitted;
};

class FBoneItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FBoneItemDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FBoneItemDragDropOp> New(TArrayView<const TSharedPtr<FBoneElement>> InElements);

	// FDecoratedDragDropOp interface
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	// End FDecoratedDragDropOp interface
	
	TArray< TWeakPtr<FBoneElement> > Elements;
};

class SReferenceSkeletonRow : public SMultiColumnTableRow<TSharedPtr<FBoneElement>>
{
public:
	SLATE_BEGIN_ARGS(SReferenceSkeletonRow) {}
		SLATE_ARGUMENT(TWeakObjectPtr<USkeletonModifier>, WeakModifier)
		SLATE_ARGUMENT(TSharedPtr<FBoneElement>, TreeElement)
		SLATE_ARGUMENT(FRefSkeletonTreeDelegates, Delegates)
		SLATE_ARGUMENT(bool, IsReadOnly)
	SLATE_END_ARGS()

	// Slate construction function
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable);

	// SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	// End SMultiColumnTableRow interface

private:
	TWeakPtr<FBoneElement> WeakTreeElement;
	TWeakObjectPtr<USkeletonModifier> WeakModifier;
	FRefSkeletonTreeDelegates Delegates;
	bool bIsReadOnly = false;
};

class SReferenceSkeletonTree;

class FReferenceSkeletonWidgetNotifier: public ISkeletalMeshNotifier
{
public:
	FReferenceSkeletonWidgetNotifier(TSharedRef<SReferenceSkeletonTree> InTree);
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
	
private:
	TWeakPtr<SReferenceSkeletonTree> Tree;
};

class SReferenceSkeletonTree : public SCompoundWidget, public FEditorUndoClient
{
	using SReferenceSkeletonTreeView = ReferenceSkeletonTreeLocals::SReferenceSkeletonTreeView;
	
public:
	SLATE_BEGIN_ARGS(SReferenceSkeletonTree)
		: _bIsReadOnly(false)
		{}
		SLATE_ARGUMENT(TWeakObjectPtr<USkeletonModifier>, Modifier)
		SLATE_ARGUMENT(FRefSkeletonTreeDelegates, Delegates)
		SLATE_ARGUMENT_DEPRECATED(bool, bIsReadOnly, 5.8, "This argument did not provide functionality and is now deprecated")
	SLATE_END_ARGS()

	UE_API SReferenceSkeletonTree();
    UE_API virtual ~SReferenceSkeletonTree() override;

	// Slate construction function
	UE_API void Construct(const FArguments& InArgs);

	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	
	// Selection
	UE_API void GetSelectedBoneNames(TArray<FName>& OutSelectedBoneNames) const;
	TArray<TSharedPtr<FBoneElement>> GetSelectedItems() const;
	bool HasSelectedItems() const;
	UE_API void SelectItemFromNames(const TArray<FName>& InBoneNames, bool bFrameSelection = false);

	UE_API TSharedPtr<ISkeletalMeshNotifier> GetNotifier();

private:
	// SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	// END SWidget interface
	
	void BindCommands();

	void AddItemToSelection(const TSharedPtr<FBoneElement>& InItem);
	void RemoveItemFromSelection(const TSharedPtr<FBoneElement>& InItem);
	void ReplaceItemInSelection(const FText& OldName, const FText& NewName);

	// Create, Rename, Delete, Parent bones
	void HandleNewBone();
	bool CanAddNewBone() const;
	void HandleRenameBone(const ReferenceSkeletonTreeLocals::EBoneRenameType InRenameType) const;
	bool CanRenameBone() const;
	void HandleDeleteBone();
	bool CanDeleteBone() const;
	void HandleUnParentBone();
	bool CanUnParentBone() const;

	// Copy & Paste
	void HandleCopyBones() const;
	bool CanCopyBones() const;
	void HandlePasteBones(const bool bUsingGlobalTransforms);
	bool CanPasteBones() const;
	void HandleDuplicateBones();
	bool CanDuplicateBones() const;

	// Filter
	void OnFilterTextChanged(const FText& SearchText);
	
	// Callbacks
	void RefreshTreeView(bool IsInitialSetup=false);
	void HandleGetChildrenForTree(TSharedPtr<FBoneElement> InItem, TArray<TSharedPtr<FBoneElement>>& OutChildren);
	void OnSelectionChanged(TSharedPtr<FBoneElement> InItem, ESelectInfo::Type InSelectInfo);
	
	TSharedRef< SWidget > CreateAddNewMenu();
	EVisibility GetAddNewButtonVisibility() const;
	TSharedPtr< SWidget > CreateContextMenu();

	void OnItemDoubleClicked(TSharedPtr<FBoneElement> InItem);
	void OnSetExpansionRecursive(TSharedPtr<FBoneElement> InItem, bool bShouldBeExpanded);
	void SetExpansionRecursive(TSharedPtr<FBoneElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);

	// Delegates
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FBoneElement> TargetItem);
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FBoneElement> TargetItem);
	void OnBoneRenamed(const FName InOldName, const FName InNewName) const;
	
	void OnBoneElementNameCommitted(const FText& InText, ETextCommit::Type InCommitType, TSharedRef<FBoneElement> BoneElement);

	// The modifier that actually does modifications on the reference skeleton
	TWeakObjectPtr<USkeletonModifier> Modifier;
	
	TSharedPtr<FReferenceSkeletonWidgetNotifier> Notifier;

	// undo
	void BeginChange();
	void EndChange();
	void CancelChange();
	TUniquePtr<ReferenceSkeletonTreeLocals::FSkeletonModifierChange> ActiveChange;
	
	// Commands
	TSharedPtr<FUICommandList> CommandList;
	
	// Tree view
	TSharedPtr<SReferenceSkeletonTreeView> TreeView;
	TArray<TSharedPtr<FBoneElement>> RootElements;
	TArray<TSharedPtr<FBoneElement>> AllElements;

	// filter
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;


	bool bIsReadOnly = false;

	FRefSkeletonTreeDelegates Delegates;

	friend class FReferenceSkeletonWidgetNotifier;
};

#undef UE_API