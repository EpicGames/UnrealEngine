// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "UObject/WeakInterfacePtr.h"
#include "SkeletalMeshNotifier.h"

#include "HitProxies.h"
#include "ToolContextInterfaces.h"
#include "GenericPlatform/ICursor.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "SkeletalMeshEditingInterface.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class FSkeletalMeshToolNotifier;
class UTransformProxy;
class UInteractiveToolManager;
class IGizmoStateTarget;
class USkeletonModifier;
struct FInputDeviceRay;
class UToolTarget;
class ITransformGizmoSource;
class SWidget;
namespace UE::Geometry { class FMeshPlanarSymmetry; }

/**
 * USkeletalMeshEditingInterface
 */

UINTERFACE(MinimalAPI)
class USkeletalMeshEditingInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * ISkeletalMeshEditingInterface
 */

class ISkeletalMeshEditingInterface
{
	GENERATED_BODY()

public:
	UE_API TSharedPtr<ISkeletalMeshNotifier> GetNotifier();
	UE_API bool NeedsNotification() const;

	UE_API virtual bool CanModifySkeleton() const;
	UE_API virtual TSharedPtr<SWidget> GetCustomSkeletonTreeWidget(TSharedPtr<ISkeletalMeshEditorBinding> EditorBinding);
	

protected:
	virtual void HandleSkeletalMeshModified(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) = 0;
	
private:
	TSharedPtr<FSkeletalMeshToolNotifier> Notifier;
	
	friend class FSkeletalMeshToolNotifier;
};

/**
 * FSkeletalMeshToolNotifier
 */

class FSkeletalMeshToolNotifier: public ISkeletalMeshNotifier
{
public:
	FSkeletalMeshToolNotifier(TWeakInterfacePtr<ISkeletalMeshEditingInterface> InInterface);
	virtual void HandleNotification(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
	
protected:
	TWeakInterfacePtr<ISkeletalMeshEditingInterface> Interface;
};

/**
 * HBoneHitProxy
 */

struct HBoneHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( UE_API )

	int32 BoneIndex;
	FName BoneName;

	HBoneHitProxy(int32 InBoneIndex, FName InBoneName)
		: HHitProxy(HPP_Foreground)
		, BoneIndex(InBoneIndex)
		, BoneName(InBoneName)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }
};

/**
 * USkeletalMeshGizmoContextObjectBase
 */

UCLASS(MinimalAPI, Abstract)
class USkeletalMeshGizmoContextObjectBase : public UObject
{
	GENERATED_BODY()
	
public:
	virtual USkeletalMeshGizmoWrapperBase* GetNewWrapper(UInteractiveToolManager* InToolManager, UObject* Outer = nullptr,
		IGizmoStateTarget* InStateTarget = nullptr, const TScriptInterface<ITransformGizmoSource>& GizmoSource = nullptr)
	PURE_VIRTUAL(UGizmoContextObject::GetNewWrapper, return nullptr;);
};

/**
 * USkeletalMeshGizmoWrapperBase
 */

UCLASS(MinimalAPI, Abstract)
class USkeletalMeshGizmoWrapperBase : public UObject
{
	GENERATED_BODY()

protected:
	using FGetTransform = TUniqueFunction<FTransform(void)>;
	using FSetTransform = TUniqueFunction<void(const FTransform&)>;
	
public:
	virtual void Initialize(const FTransform& InTransform = FTransform::Identity, const EToolContextCoordinateSystem& InTransformMode = EToolContextCoordinateSystem::Local)
	PURE_VIRTUAL(USkeletalMeshGizmoWrapperBase::Initialize, return;)
	
	virtual void HandleBoneTransform(FGetTransform GetTransformFunc, FSetTransform SetTransformFunc)
	PURE_VIRTUAL(USkeletalMeshGizmoWrapperBase::HandleBoneTransform, return;)

	virtual void Clear() PURE_VIRTUAL(USkeletalMeshGizmoWrapperBase::Clear, return;)

	virtual bool CanInteract() const
	PURE_VIRTUAL(USkeletalMeshGizmoWrapperBase::CanInteract, return false;)
	
	virtual bool IsGizmoHit(const FInputDeviceRay& PressPos) const PURE_VIRTUAL(USkeletalMeshGizmoWrapperBase::IsGizmoHit, return false;)

	UPROPERTY()
	TWeakObjectPtr<USceneComponent> Component;
};

/**
 * USkeletalMeshEditorContextObjectBase
 */

UCLASS(MinimalAPI, Abstract)
class USkeletalMeshEditorContextObjectBase : public UObject
{
	GENERATED_BODY()

public:
	virtual EMeshLODIdentifier GetEditingLOD() PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::GetEditingLOD, return EMeshLODIdentifier::LOD0;)
	virtual void HideSkeleton() PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::HideSkeleton, return;)
	virtual void ShowSkeleton() PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::ShowSkeleton, return;)
	virtual void ToggleBoneManipulation(bool bEnable) PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::ToggleBoneManipulation, return;)
	virtual const TArray<FTransform>& GetComponentSpaceBoneTransforms(UToolTarget* InToolTarget) PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::GetComponentSpaceBoneTransforms, static TArray<FTransform> Dummy; return Dummy;)

	virtual FName GetEditingMorphTarget()  PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::GetEditingMorphTarget, return NAME_None;);
	virtual TMap<FName, float> GetMorphTargetWeights() PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::GetMorphTargetWeights, return {};);	
	virtual void NotifyMorphTargetEdited() PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::NotifyMorphTargetEdited, return;);

	// Base-mesh symmetry for the editing asset (ref pose, deltas excluded), or nullptr if unavailable.
	// Owned by the host; may be invalidated by edits that mutate base geometry.
	virtual const UE::Geometry::FMeshPlanarSymmetry* GetBaseMeshSymmetry() { return nullptr; }

	virtual const TArray<int32>& GetIsolatedTriangles() const { static const TArray<int32> Empty; return Empty; }
	
	virtual void BindTo(ISkeletalMeshEditingInterface* InEditingInterface) PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::BindTo, return;)
	virtual void UnbindFrom(ISkeletalMeshEditingInterface* InEditingInterface) PURE_VIRTUAL(USkeletalMeshEditorContextObjectBase::UnbindFrom, return;)
};

#undef UE_API
