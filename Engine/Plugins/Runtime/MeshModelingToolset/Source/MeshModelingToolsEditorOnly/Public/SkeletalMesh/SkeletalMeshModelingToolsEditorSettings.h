// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "SkeletalMeshModelingToolsEditorSettings.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

UCLASS(config = EditorPerProjectUserSettings)
class USkeletalMeshModelingToolsEditorSettings 
	: public UObject
{
	GENERATED_BODY()

public:
	/** Sets the local rotation axis length */
	UE_API void SetLocalRotationAxisLength(const float InLocalRotationAxisLength) { LocalRotationAxisLength = InLocalRotationAxisLength; }

	/** Gets the local rotation axis length */
	UE_API float GetLocalRotationAxisLength() const { return LocalRotationAxisLength; }

	/** Sets the local rotation axis thickness */
	UE_API void SetLocalRotationAxisThickness(const float InLocalRotationAxisThickness) { LocalRotationAxisThickness = InLocalRotationAxisThickness; }

	/** Gets the local rotation axis thickness */
	UE_API float GetLocalRotationAxisThickness() const { return LocalRotationAxisThickness; }

	/** Sets if all or only the selected local rotation axes are displayed */
	UE_API void SetShowAllLocalRotationAxes(const bool InShowAllLocalRotationAxes) { bShowAllLocalRotationAxes = InShowAllLocalRotationAxes; }

	/** Gets if all or only the selected local rotation axes are displayed */
	UE_API bool GetShowAllLocalRotationAxes() const { return bShowAllLocalRotationAxes; }

private:
	UPROPERTY(config)
	float LocalRotationAxisLength = 0.f;
	
	UPROPERTY(config)
	float LocalRotationAxisThickness = 0.f;

	UPROPERTY(config)
	bool bShowAllLocalRotationAxes = false;
};

#undef UE_API
