// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Changes/MeshChange.h"
#include "DynamicSubmesh3.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class UDynamicMeshComponent;

namespace UE::Geometry
{
	class FDynamicMesh3;
	class FMeshPlanarSymmetry;
}

namespace SkeletalMeshToolsHelper
{
	using namespace UE::Geometry;

	struct FVertInfo
	{
		int32 VertArrayIndex;
		int32 VertID;
	};


	// Sets tangent mode for the provided preview mesh based on if there is a valid tangent overlay.
	UE_API void SetupPreviewTangentMode(UDynamicMeshComponent* Component);

	// Unposes a mesh: for each SourceMesh vertex (or those in VertArray), reads the posed position
	// via GetPosedVertexFunc, removes skin transform, removes morph contributions in MorphTargetWeights
	// using deltas from SourceMesh, and emits the resulting ref-pose position via WriteFunc.
	UE_API void GetUnposedMesh(
		TFunctionRef<void(FVertInfo, const FVector&)> WriteFunc,
		TFunctionRef<FVector(int32)> GetPosedVertexFunc,
		const FDynamicMesh3& SourceMesh,
		const TArray<FMatrix>& BoneMatrices,
		FName SkinWeightProfile,
		const TMap<FName, float>& MorphTargetWeights,
		const TArray<int32>& VertArray = {}
		);

	
	UE_API void GetPosedMesh(
		TFunctionRef<void(FVertInfo, const FVector&)> WriteFunc,
		const FDynamicMesh3& SourceMesh,
		const TArray<FMatrix>& BoneMatrices,
		FName SkinWeightProfile,
		const TMap<FName, float>& MorphTargetWeights,
		const TArray<int32>& VertArray = {}
		);
	
	UE_API TArray<FMatrix> ComputeBoneMatrices(
		const TArray<FTransform>& ComponentSpaceTransformsRefPose,
		const TArray<FTransform>& ComponentSpaceTransforms
		);

	// Records before/after morph-target vertex deltas for undo/redo. Shared between the brush-stroke
	// commit path in the morph sculpt tool and the in-tool mirror action.
	class FMeshMorphTargetChange : public FMeshChange
	{
	public:
		FName MorphTargetName;
		TArray<int32> Vertices;
		TArray<FVector> OldDeltas;
		TArray<FVector> NewDeltas;

		FMeshMorphTargetChange() = default;

		UE_API virtual FString ToString() const override;
		UE_API virtual void ApplyChangeToMesh(UE::Geometry::FDynamicMesh3* Mesh, bool bRevert) const override;
	};

	// Mirror MorphTargetName's deltas on Mesh across Symmetry's plane in place. Positive side is the
	// source; negative side receives the reflected delta. On-plane vertices have their delta snapped
	// to the plane.
	//
	// Pass OutDeltaChange to capture the per-vertex before/after for an undo record. Pass nullptr
	// when the caller has its own change tracking (e.g. USkeletalMeshBackedDynamicMeshComponent's
	// FChangeScope + MorphTargetChangeTracker).
	UE_API void MirrorMorphTargetOnMesh(
		UE::Geometry::FDynamicMesh3& Mesh,
		FName MorphTargetName,
		const UE::Geometry::FMeshPlanarSymmetry& Symmetry,
		FMeshMorphTargetChange* OutDeltaChange = nullptr);

	struct FPoseChangeDetector
	{
		enum EState
		{
			PoseJustChanged,
			PoseChanged,
			PoseStoppedChanging
		};
		
		struct FPayload
		{
			EState CurrentState;
			const TArray<FTransform>& ComponentSpaceTransforms;
			const TMap<FName, float>& MorphTargetWeights;
			const TArray<FTransform>& PreviousComponentSpaceTransforms;
			const TMap<FName, float>& PreviousMorphTargetWeights;
		};

		DECLARE_MULTICAST_DELEGATE_OneParam(FNotifier, FPayload);	
		
		UE_API void CheckPose(const TArray<FTransform>& ComponentSpaceTransforms, const TMap<FName,float>& MorphTargetWeights);
		UE_API FNotifier& GetNotifier();
	
	protected:
		FNotifier Notifier;
	
		TArray<FTransform> PreviousComponentSpaceTransforms;
		TMap<FName, float> PreviousMorphTargetWeights;

		EState State = PoseStoppedChanging;
	};

}

#undef UE_API
