// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshBrushTool.h"
#include "GroupTopology.h"
#include "ISkeletalMeshGeometryIsolationAwareTool.h"
#include "IHotkeyHintProvider.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"

#include "SkeletalMeshNotifier.h"
#include "SkeletonModifier.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "Engine/World.h"
#include "Changes/ValueWatcher.h"
#include "Mechanics/RectangleMarqueeMechanic.h"
#include "Misc/EnumClassFlags.h"
#include "Selection/MeshTopologySelectionMechanic.h"
#include "Framework/Commands/Commands.h"

#include "SkeletonEditingTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class SReferenceSkeletonTree;
class USingleClickInputBehavior;
class UGizmoViewContext;
class UTransformGizmo;
class USkeletonEditingTool;
class USkeletonTransformProxy;
class USkeletalMeshGizmoContextObjectBase;
class USkeletalMeshGizmoWrapperBase;
class UPolygonSelectionMechanic;
class ISkeletonCommitter;

namespace SkeletonEditingTool
{

/**
 * A wrapper change class that stores a reference skeleton and the bones' indexes trackers to be used for undo/redo.
 */
class FRefSkeletonChange : public FToolCommandChange
{
public:
	UE_API FRefSkeletonChange(const USkeletonEditingTool* InTool);

	virtual FString ToString() const override
	{
		return FString(TEXT("Edit Skeleton"));
	}

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;

	UE_API void StoreSkeleton(const USkeletonEditingTool* InTool);

private:
	FReferenceSkeleton PreChangeSkeleton;
	TArray<int32> PreBoneTracker;
	FReferenceSkeleton PostChangeSkeleton;
	TArray<int32> PostBoneTracker;
};
	
}

class FSkeletonEditingToolCommands : public TCommands<FSkeletonEditingToolCommands>
{
public:
	FSkeletonEditingToolCommands();

	virtual void RegisterCommands() override;

	// Skeleton Editing commands
	TSharedPtr<FUICommandInfo> NewBone;
	TSharedPtr<FUICommandInfo> RemoveBone;
	TSharedPtr<FUICommandInfo> UnParentBone;
	TSharedPtr<FUICommandInfo> RenameBone;
	TSharedPtr<FUICommandInfo> SearchAndReplace;
	TSharedPtr<FUICommandInfo> AddPrefix;
	TSharedPtr<FUICommandInfo> AddSuffix;

	// Copy & Paste commands
	TSharedPtr<FUICommandInfo> CopyBones;
	TSharedPtr<FUICommandInfo> CopyBoneTransforms;
	TSharedPtr<FUICommandInfo> PasteBonesLocal;
	TSharedPtr<FUICommandInfo> PasteBonesGlobal;		
	TSharedPtr<FUICommandInfo> PasteBoneTransformsLocal;
	TSharedPtr<FUICommandInfo> PasteBoneTransformsGlobal;
	
	TSharedPtr<FUICommandInfo> DuplicateBones;

	TSharedPtr<FUICommandInfo> ImportBonesFromAssignedSkeleton;
	
	UE_DEPRECATED(5.8, "Instead use PasteBonesLocal")
	TSharedPtr<FUICommandInfo> PasteBones;
};	

/**
 * USkeletonEditingToolBuilder
 */

UCLASS(MinimalAPI)
class USkeletonEditingToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	// UInteractiveToolBuilder overrides
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	
protected:
	// UInteractiveToolWithToolTargetsBuilder overrides
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * EEditingOperation represents the current tool's operation 
 */

UENUM()
enum class EEditingOperation : uint8
{
	Select,		// Selecting bones in the viewport.
	Create,		// Creating bones in the viewport.
	Remove,		// Removing current selection.
	Transform,	// Transforming bones in the viewport or thru the details panel.
	Parent,		// Parenting bones in the viewport.
	Rename,		// Renaming bones thru the details panel.
	Mirror		// Mirroring bones thru the details panel.
};

/**
 * USkeletonEditingTool is a tool to edit a the ReferenceSkeleton of a SkeletalMesh (target)
 * Changed are actually commit to the SkeletalMesh and it's mesh description on Accept.
 */

UCLASS(MinimalAPI)
class USkeletonEditingTool :
	public USingleSelectionTool,
	public IClickDragBehaviorTarget,
	public ISkeletalMeshEditingInterface,
	public ISkeletalMeshGeometryIsolationAwareTool,
	public IInteractiveToolManageGeometrySelectionAPI,
	public IHotkeyHintProvider
{
	GENERATED_BODY()

public:
	static constexpr const TCHAR* ActionCommandsContextName = TEXT("SkeletalMeshModelingToolsSkeletonEditing");

	UE_API void Init(const FToolBuilderState& InSceneState);
	
	// UInteractiveTool overrides
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// ICLickDragBehaviorTarget overrides
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;
	
	UE_API virtual void OnClickPress(const FInputDeviceRay& InPressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& InDragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& InReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;
	
	UE_API virtual void OnTick(float DeltaTime) override;

	// IInteractiveToolCameraFocusAPI overrides
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	// Modifier functions
	UE_API void MirrorBones();
	UE_API void RenameBones();
	UE_API void MoveBones();
	UE_API void OrientBones();
	UE_API void RemoveBones();
	UE_API void UnParentBones();
	UE_API void SnapBoneToComponentSelection(const bool bCreate);
	
	UE_API const TArray<FName>& GetSelection() const;

	UE_API const FTransform& GetTransform(const FName InBoneName, const bool bWorld) const;
	UE_API void SetTransforms(const TArray<FName>& InBones, const TArray<FTransform>& InTransforms, const bool bWorld) const;

	// IModifierToggleBehaviorTarget overrides
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	// ISkeletalMeshEditingInterface overrides
	UE_API virtual bool CanModifySkeleton() const override;
	UE_API virtual TSharedPtr<SWidget> GetCustomSkeletonTreeWidget(TSharedPtr<ISkeletalMeshEditorBinding> EditorBinding) override;
	

	UE_API EEditingOperation GetOperation() const;
	UE_API void SetOperation(const EEditingOperation InOperation, const bool bUpdateGizmo = true);

	UE_API bool HasSelectedComponent() const;
	
	// ISkeletalMeshGeometryIsolationAwareTool
	UE_API virtual bool IsInputIsolationValidOnOutput() const override;
	
	// IInteractiveToolManageGeometrySelectionAPI
	UE_API virtual bool IsInputSelectionValidOnOutput() override;

	// IHotkeyHintProvider
	UE_API virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const override;
protected:

	// UInteractiveTool overrides
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	// ISkeletalMeshEditingInterface overrides
	UE_API virtual void HandleSkeletalMeshModified(const TArray<FName>& BoneNames, const ESkeletalMeshNotifyType InNotifyType) override;

	/** Build a transform source to override the mode tools functionality */
	UE_API virtual TScriptInterface<ITransformGizmoSource> BuildTransformSource();

	// Modifier functions
	UE_API void CreateNewBone();
	UE_API void ParentBones(const FName& InParentName);

	UE_API static TOptional<FName> GetBoneName(HHitProxy* InHitProxy);
	
public:
	
	UPROPERTY()
	TObjectPtr<USkeletonEditingProperties> Properties;

	UPROPERTY()
	TObjectPtr<UProjectionProperties> ProjectionProperties;
	
	UPROPERTY()
	TObjectPtr<UMirroringProperties> MirroringProperties;

	UPROPERTY()
	TObjectPtr<UOrientingProperties> OrientingProperties;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic;
	
protected:
	
	UPROPERTY()
	TObjectPtr<USkeletonModifier> Modifier = nullptr;
	
	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> LeftClickBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;

	UPROPERTY()
	TWeakObjectPtr<UGizmoViewContext> ViewContext = nullptr;

	UPROPERTY()
	EEditingOperation Operation = EEditingOperation::Select;

	// gizmo
	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshGizmoContextObjectBase> GizmoContext = nullptr;

	UPROPERTY()
	TObjectPtr<USkeletalMeshGizmoWrapperBase> GizmoWrapper = nullptr;

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext = nullptr;

	UE_API void UpdateGizmo() const;
	
	// ref skeleton transactions
	UE_API void BeginChange();
	UE_API void EndChange();
	UE_API void CancelChange();
	TUniquePtr<SkeletonEditingTool::FRefSkeletonChange> ActiveChange;

	friend class SkeletonEditingTool::FRefSkeletonChange;
	
private:
	/** Copies bone transforms to clipboard */
	void HandleCopyBoneTransforms() const;
	
	/** Returns true if bone transforms can be copied to the clipboard */
	bool CanCopyBoneTransforms() const;
	
	/** Pastes bone transforms from the clipboard to selected bones  */
	void HandlePasteBoneTransforms(const bool bUsingGlobalTransforms);
	
	/** Returns true if bone transforms can be pasted from the clipboard to selected bones */
	bool CanPasteBoneTransforms() const;

	/** Merges bones from the assigned USkeleton asset into the mesh skeleton currently being edited. */
	void HandleImportBonesFromAssignedSkeleton();

	/** Returns true if the target has an assigned USkeleton asset that can be merged in. */
	bool CanImportBonesFromAssignedSkeleton() const;

	UE_API TArray<int32> GetSelectedBoneIndexes() const;
	
	enum class EBoneSelectionMode : uint8
	{
		Single				= 0,
		Additive			= 1 << 0,
		Toggle				= 1 << 1
	};
	FRIEND_ENUM_CLASS_FLAGS(USkeletonEditingTool::EBoneSelectionMode)
	

	// flags used to identify behavior modifier keys/buttons
	static constexpr int AddToSelectionModifier = 1;
	static constexpr int ToggleSelectionModifier = 2;
	EBoneSelectionMode SelectionMode = EBoneSelectionMode::Single;

	TArray<FName> Selection;
	
	UE_API void SelectBone(const FName& InBoneName);
	UE_API FName GetCurrentBone() const;

	UE_API void NormalizeSelection();

	// setup
	UE_API void SetupModifier(ISkeletonCommitter* InCommitter);
	UE_API void SetupPreviewMesh();
	UE_API void SetupProperties();
	UE_API void SetupBehaviors();
	UE_API void SetupGizmo(USceneComponent* InComponent);
	UE_API void SetupWatchers();
	UE_API void SetupComponentsSelection();
	
	// actions
	UE_API void RegisterCreateAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterDeleteAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterSelectAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterParentAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterUnParentAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterCopyAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterPasteBonesLocalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterPasteBonesGlobalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterCopyBoneTransformsAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterPasteBoneTransformsLocalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterPasteBoneTransformsGlobalAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterDuplicateAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterSelectComponentsAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterSelectionFilterCyclingAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	UE_API void RegisterSnapAction(FInteractiveToolActionSet& InOutActionSet, const int32 InActionId);
	
	void PerformPasteBonesAction(const bool bUsingGlobalTransforms);
	void PerformPasteBoneTransformsAction(const bool bUsingGlobalTransforms);

	UE_API TArray<int32> GetSelectedComponents() const;

	UE_API FTransform ComputeTransformFromComponents(const TArray<int32>& InIDs) const;
	
	TValueWatcher<TArray<FName>> SelectionWatcher;
	TValueWatcher<EToolContextCoordinateSystem> CoordinateSystemWatcher; 
	
	int32 ParentIndex = INDEX_NONE;

	TFunction<void()> PendingClickFunction;
	TFunction<void()> PendingReleaseFunction;
	TFunction<void()> DeferredFocusFunction;

	TUniquePtr<UE::Geometry::FTriangleGroupTopology> Topology = nullptr;

	// Defer pending function on tick to allow other external notification to be handled. 
	bool bDeferUntilFocused = false;

	TWeakObjectPtr<USkeletalMesh> WeakMesh = nullptr;

	TUniquePtr<FSkeletalMeshNotifierBindScope> SkeletonTreeNotifierBindScope;
	
	/** The skeleton tree this tool displays */
	TSharedPtr<SReferenceSkeletonTree> SkeletonTree;
};

ENUM_CLASS_FLAGS(USkeletonEditingTool::EBoneSelectionMode);

/**
 * USkeletonEditingProperties
 */

UCLASS(MinimalAPI)
class USkeletonEditingProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UE_API void Initialize(USkeletonEditingTool* ParentToolIn);

	UPROPERTY(EditAnywhere, Category = "Details")
	FName Name;

	UPROPERTY()
	FTransform Transform;
	
	UPROPERTY(EditAnywhere, Category = "Details")
	bool bUpdateChildren = false;

	UPROPERTY()
	bool bEnableComponentSelection = false;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent) override;
#endif
	
	TWeakObjectPtr<USkeletonEditingTool> ParentTool = nullptr;
};

/**
 * EProjectionType
 */

UENUM()
enum class EProjectionType : uint8
{
	CameraPlane,	// The camera plane is used as the projection plane 
	OnMesh,			// The mesh surface is used for projection
	WithinMesh,		// The mesh volume is used for projection
};

/**
 * UProjectionProperties
 */

UCLASS(MinimalAPI)
class UProjectionProperties: public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API void Initialize(USkeletonEditingTool* ParentToolIn, TObjectPtr<UPreviewMesh> PreviewMesh);

	UE_API void UpdatePlane(const UGizmoViewContext& InViewContext, const FVector& InOrigin);
	UE_API bool GetProjectionPoint(const FInputDeviceRay& InRay, FVector& OutHitPoint) const;
	
	UPROPERTY(EditAnywhere, Category = "Project")
	EProjectionType ProjectionType = EProjectionType::WithinMesh;

	FViewCameraState CameraState;

	TWeakObjectPtr<USkeletonEditingTool> ParentTool = nullptr;
	
private:
	TWeakObjectPtr<UPreviewMesh> PreviewMesh = nullptr;
	
	UPROPERTY()
	FVector PlaneOrigin = FVector::ZeroVector;
	
	UPROPERTY()
	FVector PlaneNormal =  FVector::ZAxisVector;
};

/**
 * UMirroringProperties
 */

UCLASS(MinimalAPI)
class UMirroringProperties: public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API void Initialize(USkeletonEditingTool* ParentToolIn);

	UE_API void MirrorBones();

	UPROPERTY(EditAnywhere, Category = "Mirror")
	FMirrorOptions Options;

	TWeakObjectPtr<USkeletonEditingTool> ParentTool = nullptr;
};

/**
 * UOrientingProperties
 */

UCLASS(MinimalAPI)
class UOrientingProperties: public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UE_API void Initialize(USkeletonEditingTool* ParentToolIn);

	UE_API void OrientBones();

	UPROPERTY(EditAnywhere, Category = "Orient")
	bool bAutoOrient = false;
	
	UPROPERTY(EditAnywhere, Category = "Orient")
	FOrientOptions Options;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent) override;
#endif
	
	TWeakObjectPtr<USkeletonEditingTool> ParentTool = nullptr;
};

#undef UE_API
