// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseGizmos/TransformProxy.h"

#include "SkeletonTransformProxy.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class UObject;

/**
 * USkeletonTransformProxy is a derivation of UTransformProxy that manages several bones and update their transform individually.
 */

UCLASS(MinimalAPI, Transient)
class USkeletonTransformProxy : public UTransformProxy
{
	GENERATED_BODY()

public:

	UE_API void Initialize(const FTransform& InTransform, const EToolContextCoordinateSystem& InTransformMode);
	
	UE_API bool IsValid() const;

	// UTransformProxy overrides
	UE_API virtual FTransform GetTransform() const override;

	UPROPERTY()
	EToolContextCoordinateSystem TransformMode = EToolContextCoordinateSystem::Local;
	
protected:
	// UTransformProxy overrides
	UE_API virtual void UpdateSharedTransform() override;
	UE_API virtual void UpdateObjects() override;
};

#undef UE_API
