// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BoneContainer.h"
#include "InteractiveToolBuilder.h"
#include "ISkeletalMeshGeometryIsolationAwareTool.h"
#include "ModelingOperators.h"
#include "SkeletalMeshEditingInterface.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"

#include "SkinWeightsBindingTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class UMeshOpPreviewWithBackgroundCompute;
struct FDynamicMeshOpResult;
struct FMeshDescription;
class IMeshDescriptionCommitter;
struct FOccupancyGrid;

namespace UE::Geometry
{
	struct FOccupancyGrid3;
}

/**
 *
 */
UCLASS(MinimalAPI)
class USkinWeightsBindingToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// A mirror of UE::Geometry::ESkinBindingType
UENUM()
enum class ESkinWeightsBindType : uint8
{
	DirectDistance = 0 UMETA(DisplayName = "Direct Distance"),
	GeodesicVoxel = 1 UMETA(DisplayName = "Geodesic Voxel"),
};

UCLASS(MinimalAPI)
class USkinWeightsBindingToolProperties : 
	public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY()
	FName CurrentBone = NAME_None;

	/** Explicit confirmation is required before the tool commits new skin weights */
	UPROPERTY(EditAnywhere, Category = Tool, meta=(TransientToolProperty))
	bool ConfirmCurrentSettings = false;
	
	/** Binding type to use */
	UPROPERTY(EditAnywhere, Category = Binding)
	ESkinWeightsBindType BindingType = ESkinWeightsBindType::DirectDistance;

	/** Stiffness of binding. Lower values allow more distant bones to contribute more */
	UPROPERTY(EditAnywhere, Category = Binding, meta=(ClampMin="0.0", ClampMax="1.0", UIMin="0.0", UIMax="1.0"))
	float Stiffness = 0.2f;

	/** Maximum bones that will influence each vertex */
	UPROPERTY(EditAnywhere, Category = Binding, meta=(ClampMin="1", UIMin="1", UIMax="10"))
	int32 MaxInfluences = 5;

	/** The resolution of the voxel grid if doing geodesic voxel binding */
	UPROPERTY(EditAnywhere, Category = Binding, meta=(EditCondition = "BindingType == ESkinWeightsBindType::GeodesicVoxel", ClampMin="1", UIMin="8", UIMax="1024"))
	int32 VoxelResolution = 128;
	
	// UPROPERTY(EditAnywhere, Category = Debug)
	bool bDebugDraw = false;
};

/**
 *
 */
UCLASS(MinimalAPI)
class USkinWeightsBindingTool :
	public UMultiSelectionMeshEditingTool,
	public UE::Geometry::IDynamicMeshOperatorFactory,
	public ISkeletalMeshEditingInterface,
	public ISkeletalMeshGeometryIsolationAwareTool,
    public IInteractiveToolManageGeometrySelectionAPI
{
	GENERATED_BODY()

public:
	UE_API USkinWeightsBindingTool();
	UE_API virtual ~USkinWeightsBindingTool() override;

	UE_API void Init(const FToolBuilderState& InSceneState);
	
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	
	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	// ISkeletalMeshGeometryIsolationAwareTool
	UE_API virtual bool IsInputIsolationValidOnOutput() const override;
	
	// IInteractiveToolManageGeometrySelectionAPI
	UE_API virtual bool IsInputSelectionValidOnOutput() override;
	
	UPROPERTY()
	TObjectPtr<USkinWeightsBindingToolProperties> Properties = nullptr;
	
	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

protected:
	TSharedPtr<UE::Geometry::FOccupancyGrid3> Occupancy;
	
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	
	TUniquePtr<FMeshDescription> EditedMeshDescription;

	FReferenceSkeleton ReferenceSkeleton;
	
	UE_API void GenerateAsset(const FDynamicMeshOpResult& Result);

	static UE_API FVector4f WeightToColor(float Value);
	UE_API void UpdateVisualization(bool bInForce = false);

	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext = nullptr;

	// updates the skin weights using the operator's result
	UE_API bool UpdateSkinWeightsFromDynamicMesh(UE::Geometry::FDynamicMesh3& InResultMesh) const;
	
	// ISkeletalMeshEditingInterface
	UE_API virtual void HandleSkeletalMeshModified(const TArray<FName>& InBoneNames, const ESkeletalMeshNotifyType InNotifyType) override;

	// Pending update function */
	TFunction<void()> PendingUpdateFunction;
};

#undef UE_API
