// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshBrushTool.h"
#include "Curves/LinearColorRamp.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "MeshDescription.h"
#include "DynamicMesh/DynamicVerticesOctree3.h"
#include "BoneWeights.h"
#include "DynamicSubmesh3.h"
#include "GroupTopology.h"
#include "ISkeletalMeshGeometryIsolationAwareTool.h"
#include "IHotkeyHintProvider.h"
#include "SkeletalMeshAttributes.h"
#include "Misc/Optional.h"
#include "Containers/Map.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "SkeletalMesh/SkeletalMeshEditingInterface.h"
#include "Engine/SkeletalMesh.h"
#include "Selections/GeometrySelection.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"
#include "ToolMeshSelector.h"
#include "ToolTargetManager.h"
#include "Framework/Commands/Commands.h"


#include "SkinWeightsPaintTool.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API


class USkeletalMeshComponentReadOnlyToolTarget;
struct FMeshDescription;
class USkinWeightsPaintTool;
class UMeshElementsVisualizer;
class UPolygonSelectionMechanic;
class UPersonaEditorModeManagerContext;
class FEditorViewportClient;
class UVolumetricBrushStampIndicator;
class USkeletonModifier;
class USkinWeightsPaintTool;
class SButton;
class SReferenceSkeletonTree;
class FSkeletalMeshNotifierBindScope;
enum class ESkinWeightsPaintToolPropertiesCustomVersion : uint32;

namespace UE::Geometry 
{
	struct FGeometrySelection;
	template <typename BoneIndexType, typename BoneWeightType> class TBoneWeightsDataSource;
	template <typename BoneIndexType, typename BoneWeightType> class TSmoothBoneWeights;
}

using BoneIndex = int32;

// weight edit mode
UENUM()
enum class EWeightEditMode : uint8
{
	Brush,
	Mesh,
	Bones,
};

// weight transfers happen between a source and target
UENUM()
enum class EMeshTransferOption : uint8
{
	Source,
	Target,
};

// weight color mode
UENUM()
enum class EWeightColorMode : uint8
{
	Greyscale,
	Ramp,
	BoneColors,
	FullMaterial,
};

// brush falloff mode
UENUM()
enum class EWeightBrushFalloffMode : uint8
{
	Surface,
	Volume,
};

// operation type when editing weights
UENUM()
enum class EWeightEditOperation : uint8
{
	Add,
	Replace,
	Multiply,
	Relax,
	RelativeScale
};

// mirror direction mode
UENUM()
enum class EMirrorDirection : uint8
{
	PositiveToNegative,
	NegativeToPositive,
};

namespace SkinPaintTool
{
	struct FSkinToolWeights;

	struct FVertexBoneWeight
	{
		FVertexBoneWeight() : BoneID(INDEX_NONE), VertexInBoneSpace(FVector::ZeroVector), Weight(0.0f) {}
		FVertexBoneWeight(BoneIndex InBoneIndex, const FVector& InPosInRefPose, float InWeight) :
			BoneID(InBoneIndex), VertexInBoneSpace(InPosInRefPose), Weight(InWeight){}
		
		BoneIndex BoneID;
		FVector VertexInBoneSpace;
		float Weight;
	};

	using VertexWeights = TArray<FVertexBoneWeight, TFixedAllocator<UE::AnimationCore::MaxInlineBoneWeightCount>>;

	// data required to preview the skinning deformations as you paint
	struct FSkinToolDeformer
	{
		void Initialize(const FReferenceSkeleton& InRefSkeleton, const TArray<FTransform>& PoseComponentSpace, const FDynamicMesh3& InMesh);

		void SetAllVerticesToBeUpdated();

		void SetToRefPose(USkinWeightsPaintTool* Tool);

		void UpdateVertexDeformation(USkinWeightsPaintTool* Tool, const TArray<FTransform>& PoseComponentSpace);

		void SetVertexNeedsUpdated(int32 VertexIndex);
		
		// which vertices require updating (partially re-calculated skinning deformation while painting)
		TSet<int32> VerticesWithModifiedWeights;
		// position of all vertices in the reference pose
		TArray<FVector> RefPoseVertexPositions;
		// Store a ref to the Mesh so there is a way to iterate over valid verts
		const FDynamicMesh3* Mesh;
		// inverted, component space ref pose transform of each bone
		TArray<FTransform> InvCSRefPoseTransforms;
		// bones transforms used in last deformation update
		TArray<FTransform> PreviousPoseComponentSpace;
		// bones transforms stored for duration of async deformation update
		TArray<FTransform> RefPoseComponentSpace;
		// bone index to bone name
		TArray<FName> BoneNames;
		TMap<FName, BoneIndex> BoneNameToIndexMap;
		
		FReferenceSkeleton RefSkeleton;
		
	};

	// store a sparse set of modifications to a set of vertex weights on a SINGLE bone
	struct FSingleBoneWeightEdits
	{
		int32 BoneIndex;
		
		TMap<VertexIndex, float> OldWeights;
		TMap<VertexIndex, float> NewWeights;
		
		TArray<VertexIndex> VerticesAddedTo;
		TArray<VertexIndex> VerticesRemovedFrom;
	};

	// store a sparse set of modifications to a set of vertex weights for a SET of bones
	// with support for merging edits. these are used for transaction history undo/redo.
	struct FMultiBoneWeightEdits
	{
		void MergeSingleEdit(
			const int32 BoneIndex,
			const int32 VertexID,
			const float NewWeight,
			bool bPruneInfluence,
			const TArray<VertexWeights>& InPreChangeWeights);
		void MergeEdits(const FSingleBoneWeightEdits& BoneWeightEdits);

		// map of bone indices to weight edits made to that bone
		TMap<BoneIndex, FSingleBoneWeightEdits> PerBoneWeightEdits;
	};

	// Sparse first-touch gate over a fixed vertex window. Mask gives O(1) dedup;
	// Vertices gives O(touched) iteration. Clear() only wipes the bits that were
	// set, so clearing scales with what was touched, not with mesh size.
	struct FVertexChangeTracker
	{
		TBitArray<> Mask;
		TArray<int32> Vertices;

		void Init(int32 NumVertices)
		{
			Mask.Init(false, NumVertices);
			Vertices.Reset();
		}

		// Returns true on the first Mark this window; false if already marked.
		bool Mark(int32 VertexID)
		{
			if (Mask[VertexID]) { return false; }
			Mask[VertexID] = true;
			Vertices.Add(VertexID);
			return true;
		}

		void Clear()
		{
			for (const int32 V : Vertices) { Mask[V] = false; }
			Vertices.Reset();
		}

		bool IsEmpty() const { return Vertices.IsEmpty(); }
	};

	// Captures a per-vertex Before/After snapshot of weights touched in a stroke. Storing the
	// full VertexWeights avoids per-bone delta replay and order-dependent eviction at undo time.
	class FMeshSkinWeightsChange : public FToolCommandChange
	{
	public:
		FMeshSkinWeightsChange(const FName InSkinWeightProfile, int32 NumVertices)
			: FToolCommandChange()
			, SkinWeightProfile(InSkinWeightProfile)
		{
			PerTransactionVertexChangeTracker.Init(NumVertices);
		}

		virtual FString ToString() const override
		{
			return FString(TEXT("Edit Skin Weights"));
		}

		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;

		// Called per stamp during a stroke to accumulate touched vertices/bones into the change.
		void StoreMultipleWeightEdits(const FMultiBoneWeightEdits& WeightEdits);

		// Captures Before/After snapshots from the tool's weight buffers. Must run before
		// SyncWeightBuffers (which overwrites PreChangeWeights).
		void Close(USkinWeightsPaintTool* Tool);

	private:
		static void RestoreVertexFromSnapshot(USkinWeightsPaintTool* Tool, int32 VertexID, const VertexWeights& Snapshot);

		FName SkinWeightProfile = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;

		// First-touch-this-transaction gate. Mark() dedups touches; Vertices drives Close's
		// snapshot capture and Apply/Revert's replay.
		FVertexChangeTracker PerTransactionVertexChangeTracker;

		// Parallel with PerTransactionVertexChangeTracker.Vertices, populated by Close.
		TArray<VertexWeights> BeforeStates;
		TArray<VertexWeights> AfterStates;

		TSet<BoneIndex> ChangedBones;
	};

	// intermediate storage of the weight maps for duration of tool
	struct FSkinToolWeights
	{
		// copy the initial weight values from the skeletal mesh
		void InitializeSkinWeights(
			USkinWeightsPaintTool* InTool,
			const FDynamicMesh3& Mesh);

		bool IsInitialized() const;
		
		// applies an edit to a single vertex weight on a single bone, then normalizes the remaining weights while
		// keeping the edited weight intact (ie, adapts OTHER influences to achieve normalization)
		void CreateWeightEditForVertex(
			const int32 BoneIndex,
			const int32 VertexId,
			float NewWeightValue,
			FMultiBoneWeightEdits& WeightEdits);

		void ApplyCurrentWeightsToMesh(FDynamicMesh3& Mesh, TFunction<int32(int32)> WeightsIndexToMeshIndex = {});
		
		static float GetWeightOfBoneOnVertex(
			const int32 BoneIndex,
			const int32 VertexID,
			const TArray<VertexWeights>& InVertexWeights);

		static void FillWeightEdit(
			const int32 BoneIndex,
			const int32 VertexID,
			const float NewWeight,
			const TArray<VertexWeights>& InVertexWeights);

		void RemoveInfluenceFromVertex(
			const VertexIndex VertexID,
			const BoneIndex BoneID,
			TArray<VertexWeights>& InOutVertexWeights);

		// some weight editing operations are RELATIVE to existing weights before the change started (Multiply, Add etc)
		// these "existing weights" are stored in the PreChangeWeights buffer
		// PreChange and Current buffers must be synchronized after a transaction
		void SyncWeightBuffers();

		float SetCurrentFalloffAndGetMaxFalloffThisStroke(int32 VertexID, float CurrentStrength);

		void ApplyEditsToCurrentWeights(const FMultiBoneWeightEdits& Edits, FVertexChangeTracker& InOutVertexChangeTracker);

		void UpdateIsBoneWeighted(BoneIndex BoneToUpdate);

		BoneIndex GetParentBoneToWeightTo(BoneIndex ChildBone);
		
		// double-buffer of the entire weight matrix (stored sparsely for fast deformation)
		// "Pre" is state of weights at stroke start
		// "Current" is state of weights during stroke
		// When stroke is over, PreChangeWeights are synchronized with CurrentWeights
		TArray<VertexWeights> PreChangeWeights;
		TArray<VertexWeights> CurrentWeights;

		// record the current maximum amount of falloff applied to each vertex during the current stroke
		// values range from 0-1, this allows brushes to sweep over the same vertex, and apply only the maximum amount
		// of modification (add/replace/relax etc) that was encountered for the duration of the stroke.
		TArray<float> MaxFalloffPerVertexThisStroke;

		// Per-stamp scratch used by ApplyEditsToCurrentWeights. Persistent allocation; reset
		// in place between calls. PerWeightEditVertexChangeTracker gates the first touch of a
		// vertex's slot during a stamp and drives the patch-back loop.
		using BoneWeight = TPair<BoneIndex, float>;
		using UnlimitedVertexWeights = TArray<BoneWeight, TInlineAllocator<16>>;
		TArray<UnlimitedVertexWeights> PerWeightEditScratchPad;
		FVertexChangeTracker PerWeightEditVertexChangeTracker;

		// record which bones have any weight assigned to them
		TArray<bool> IsBoneWeighted;

		// update deformation when vertex weights are modified
		FSkinToolDeformer Deformer;

		// which skin profile is currently edited
		FName Profile = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;

		TWeakObjectPtr<USkinWeightsPaintTool> Tool;
	};

	struct FSkinMirrorData
	{
		// lazily updates the mirror data tables for the current skeleton/mesh/mirror plane
		void EnsureMirrorDataIsUpdated(
			const TArray<FName>& BoneNames,
			const TMap<FName, BoneIndex>& BoneNameToIndexMap,
			const FReferenceSkeleton& RefSkeleton,
			const TArray<FVector>& RefPoseVertices,
			EAxis::Type InMirrorAxis,
			EMirrorDirection InMirrorDirection);

		// get a map of Target > Source bone ids across the current mirror plane
		const TMap<int32, int32>& GetBoneMap() const { return BoneMap; };
		// get the map of Target > Source vertex ids across the current mirror plane
		const TMap<int32, int32>& GetVertexMap() const;
		// return true if the point lies on the TARGET side of the mirror plane
		bool IsPointOnTargetMirrorSide(const FVector& InPoint) const;
		// forces mirror tables to be re-generated (do this after any mesh change operation)
		void SetNeedsReinitialized() {bIsInitialized = false;};
		
	private:
		
		bool bIsInitialized = false;
		TEnumAsByte<EAxis::Type> Axis;
		EMirrorDirection Direction; 
		TMap<int32, int32> BoneMap;
		TMap<int32, int32> VertexMap; // <Target, Source>
	};

	
}

class FSkinWeightsPaintToolCommands : public TCommands<FSkinWeightsPaintToolCommands>
{
public:
	FSkinWeightsPaintToolCommands();

	virtual void RegisterCommands() override;
	
	TSharedPtr<FUICommandInfo> IsolateSelectedBones;
	TSharedPtr<FUICommandInfo> ToggleSelectedBonesLockState;
	
	TSharedPtr<FUICommandInfo> LockSelectedBones;
	TSharedPtr<FUICommandInfo> UnlockSelectedBones;

	TSharedPtr<FUICommandInfo> LockAllBones;
	TSharedPtr<FUICommandInfo> UnlockAllBones;

	TSharedPtr<FUICommandInfo> BypassLocks;
};	


UCLASS(MinimalAPI)
class USkinWeightsPaintToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// for saving/restoring the brush settings separately for each brush mode (Add, Replace, etc...)
// Note: Radius and FalloffMode are stored per-mode for serialization compatibility, but at
// runtime they behave as shared values — every writer propagates to all four per-mode configs
// (see USkinWeightsPaintToolProperties::SetSharedRadius / SetSharedFalloffMode) and
// RestoreProperties forces all four entries into sync on load.
USTRUCT()
struct FSkinWeightBrushConfig
{
	GENERATED_BODY()
	
	UPROPERTY()
	float Strength = 1.f;

	UPROPERTY()
	float Radius = 20.0f;
	
	UPROPERTY()
	float Falloff = 1.0f;

	UPROPERTY()
	EWeightBrushFalloffMode FalloffMode = EWeightBrushFalloffMode::Surface;
};

USTRUCT()
struct FSkinWeightRelaxBrushAdvancedConfig
{
	GENERATED_BODY()

	// When true, Relax stamps read CurrentWeights so wiggling the brush keeps smoothing.
	UPROPERTY()
	bool bAccumulate = true;
};

struct FDirectEditWeightState
{
	EWeightEditOperation EditMode;
	float StartValue = 0.f;
	float CurrentValue = 0.f;
	bool bInTransaction = false;

	UE_API void Reset();
	UE_API float GetModeDefaultValue();
	UE_API float GetModeMinValue();
	UE_API float GetModeMaxValue();
};

// Container for properties displayed in Details panel while using USkinWeightsPaintTool
UCLASS(MinimalAPI, config = EditorSettings)
class USkinWeightsPaintToolProperties : public UBrushBaseProperties
{
	GENERATED_BODY()

	UE_API USkinWeightsPaintToolProperties();
	
public:
	//~ Begin UInteractiveToolPropertySet interface
	virtual void RestoreProperties(UInteractiveTool* RestoreToTool, const FString& CacheIdentifier = TEXT("")) override;
	//~ End UInteractiveToolPropertySet interface

	// brush vs selection modes
	UPROPERTY(Config)
	EWeightEditMode EditingMode;

	// custom brush modes and falloff types
	UPROPERTY(Config)
	EWeightEditOperation BrushMode;
	EWeightEditOperation PriorBrushMode; // when toggling with modifier key

	// are we selecting vertices, edges or faces
	UPROPERTY(Config)
	EComponentSelectionMode ComponentSelectionMode = EComponentSelectionMode::Faces;

	// weight color properties
	UPROPERTY(EditAnywhere, Config, Category = MeshDisplay)
	EWeightColorMode ColorMode;

	UE_DEPRECATED(5.8, "Use LinearColorRamp instead")
	UPROPERTY(Config, meta = (DeprecatedProperty, DeprecationMessage = "Use LinearColorRamp instead."))
	TArray<FLinearColor> ColorRamp;
	
	UPROPERTY(EditAnywhere, Config, Category = MeshDisplay, Meta = (EditCondition="ColorMode==EWeightColorMode::Ramp", EditConditionHides))
	FLinearColorRamp LinearColorRamp;

	/** Show polygroup boundary edges in the viewport. */
	UPROPERTY(EditAnywhere, Config, Category = MeshDisplay)
	bool bShowGroupBoundaries = true;

	/** Color of polygroup boundary edges. */
	UPROPERTY(EditAnywhere, Config, Category = MeshDisplay)
	FColor GroupBoundaryColor = FColor(220, 100, 220);

	/** Thickness multiplier for polygroup boundary edges. Drives the visualizer's ThicknessScale since the visualizer renders only polygroup edges. */
	UPROPERTY(EditAnywhere, Config, Category = MeshDisplay)
	float GroupBoundaryThickness = 1.5f;

	// weight editing arguments
	UPROPERTY(Config)
	TEnumAsByte<EAxis::Type> MirrorAxis = EAxis::X;
	UPROPERTY(Config)
	EMirrorDirection MirrorDirection = EMirrorDirection::PositiveToNegative;
	UPROPERTY(Config)
	float PruneValue = 0.01f;
	UPROPERTY(Config)
	int32 ClampValue = 8;
	UPROPERTY(Config)
	int32 ClampSelectValue = 8;
	UPROPERTY(Config)
	float AddStrength = 1.0;
	UPROPERTY(Config)
	float ReplaceValue = 1.0;
	UPROPERTY(Config)
	float RelaxStrength = 0.5;
	UPROPERTY(Config)
	float AverageStrength = 1.0;
	// the state of the direct weight editing tools (mode buttons + slider)
	FDirectEditWeightState DirectEditState;

	// save/restore user specified settings for each tool mode
	UE_API FSkinWeightBrushConfig& GetBrushConfig();
	TMap<EWeightEditOperation, FSkinWeightBrushConfig*> BrushConfigs;
	UPROPERTY(Config)
	FSkinWeightBrushConfig BrushConfigAdd;
	UPROPERTY(Config)
	FSkinWeightBrushConfig BrushConfigReplace;
	UPROPERTY(Config)
	FSkinWeightBrushConfig BrushConfigMultiply;
	UPROPERTY(Config)
	FSkinWeightBrushConfig BrushConfigRelax;

	UPROPERTY(Config)
	FSkinWeightRelaxBrushAdvancedConfig RelaxBrushAdvancedConfig;

	// Persisted form of USkinWeightsPaintTool::LockedBoneIndices. Stored by name so a
	// renamed/removed bone between launches drops the lock instead of mis-locking.
	UPROPERTY()
	TArray<FName> LockedBoneNames;

	// Cache key for the per-mesh RestoreProperties/SaveProperties round-trip; populated
	// at Setup from the skeletal mesh asset path. Transient.
	FString PropertySetCacheIdentifier;

	// Radius and FalloffMode behave as shared brush state across all brush modes (matching the
	// sculpt tool's pattern). Storage is still per-mode for config serialization compatibility;
	// these setters propagate the value to every BrushConfig entry so the modes stay in lockstep.
	UE_API void SetSharedRadius(float InRadius);
	UE_API void SetSharedFalloffMode(EWeightBrushFalloffMode InFalloffMode);

	UPROPERTY(config)
	bool bSyncBrushRadiusAcrossModes = true;

	UPROPERTY(EditAnywhere, Category = SkinWeightLayer, meta = (DisplayName = "Active Profile", GetOptions = GetTargetSkinWeightProfilesFunc))
	FName SkinWeightProfileOption = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;
	
	// new profile properties
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowNewProfileName = false;
	UPROPERTY(EditAnywhere, Category = SkinWeightLayer, meta = (TransientToolProperty, DisplayName = "New Profile Name",
		EditCondition = bShowNewProfileName, HideEditConditionToggle, NoResetToDefault))
	FName NewSkinWeightProfile = "Profile";

	UE_API FName GetDesiredSkinWeightProfileForTool() const;
	
	// pointer back to paint tool
	TObjectPtr<USkinWeightsPaintTool> WeightTool;

	UE_DEPRECATED(5.8, "Instead use SetComponentSelectionMode")
	UE_API void SetComponentMode(EComponentSelectionMode InComponentMode);
	
	/** 
	 * Sets the component selection mode 
	 * 
	 * @param InComponentMode		The component selection mode to set
	 * @param bInConvertSelection	When set to true, converts the current selection to the new mode
	 */
	UE_API void SetComponentSelectionMode(const EComponentSelectionMode InComponentMode, const bool bInConvertSelection);
	
	UE_API void SetFalloffMode(EWeightBrushFalloffMode InFalloffMode);
	UE_API void SetColorMode(EWeightColorMode InColorMode);
	UE_API void SetBrushMode(EWeightEditOperation InBrushMode);

	// transfer
	UPROPERTY(EditAnywhere, Transient, Category = "WeightTransfer")
	TWeakObjectPtr<USkeletalMesh> SourceSkeletalMesh;
	UPROPERTY(EditAnywhere, Transient, Category = "WeightTransfer")
	EMeshTransferOption MeshSelectMode = EMeshTransferOption::Target;
	UPROPERTY(EditAnywhere, Category = "WeightTransfer", meta = (GetOptions = GetSourceLODsFunc))
	FName SourceLOD = "LOD0";
	UPROPERTY(EditAnywhere, Category = "WeightTransfer", meta = (DisplayName = "Source Profile", GetOptions = GetSourceSkinWeightProfilesFunc))
	FName SourceSkinWeightProfile = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;
	UPROPERTY(EditAnywhere, Transient, Category = "WeightTransfer")
	FTransform SourcePreviewOffset = FTransform::Identity;
	
private:
	UFUNCTION()
	UE_API TArray<FName> GetTargetSkinWeightProfilesFunc() const;
	UFUNCTION()
	UE_API TArray<FName> GetSourceLODsFunc() const;
	UFUNCTION()
	UE_API TArray<FName> GetSourceSkinWeightProfilesFunc() const;

	/** 
	 * Defines a fully custom version stored in the UInteractiveToolPropertySet system.
	 * This allows to bump property versions much like common UObject allow bumping post load.
	 */
	UPROPERTY(Config)
	ESkinWeightsPaintToolPropertiesCustomVersion CustomVersion;
};

// this class wraps the all the components to enable selection on a single mesh in the skin weights tool
// this allows us to make selections on multiple different meshes
// NOTE: at some point we may want to do component selections on multiple meshes in any/all viewports
// at which time this class should be centralized and renamed to UMeshSelector or something like that.
// But there will need to be some sort of centralized facility to manage that and make sure it interacts nicely with other tools.
UCLASS(MinimalAPI, Deprecated)
class UDEPRECATED_WeightToolMeshSelector : public UToolMeshSelector
{
	GENERATED_BODY()
};

// this class wraps a source skeletal mesh used to transfer skin weights to the tool target mesh
UCLASS(MinimalAPI)
class UWeightToolTransferManager : public UObject
{
	GENERATED_BODY()

public:

	// this must be called from within the parent tool's Setup() so that the selection mechanics are registered for capturing input
	UE_API void InitialSetup(USkinWeightsPaintTool* InWeightTool, FEditorViewportClient* InViewportClient);
	
	// called when the tool is shutdown
	UE_API void Shutdown();

	// render the selection mechanism
	UE_API void Render(IToolsContextRenderAPI* RenderAPI);
	UE_API void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	// update the mesh we are transferring from
	UE_API void SetSourceMesh(USkeletalMesh* InSkeletalMesh = nullptr);

	// run the weight transfer
	UE_API void TransferWeights();

	// returns true if everything is setup and ready to transfer
	UE_API bool CanTransferWeights() const;
	
	// gets the tool target for the source mesh
	UToolTarget* GetTarget() const { return SourceTarget; }

	// get the preview mesh for the source mesh
	UPreviewMesh* GetPreviewMesh() const { return SourcePreviewMesh; };
	
	// get the mesh selector for the source mesh
	UToolMeshSelector* GetMeshSelector() const { return MeshSelector; };
	
	// called when tool settings are modified
	UE_API void OnPropertyModified(const USkinWeightsPaintToolProperties* WeightToolProperties, const FProperty* ModifiedProperty);

private:
	
	// actually run the weight transfer to copy weights from the source to the target
	UE_API void TransferWeightsFromOtherMeshOrSubset();

	// actually run the weight transfer to copy weights from the source to the target
	UE_API void TransferWeightsFromSameMeshAndLOD();

	UE_API void ApplyTranferredWeightsAsTransaction(
		const UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* InTransferredSkinWeights,
		const TArray<int32>& InVertexSubset,
		const FDynamicMesh3& InTargetMesh);
	
	UPROPERTY()
	TObjectPtr<UPreviewMesh> SourcePreviewMesh = nullptr;
	UPROPERTY()
	TObjectPtr<USkeletalMesh> SourceSkeletalMesh = nullptr;
	UPROPERTY()
	TObjectPtr<UToolTarget> SourceTarget = nullptr;
	UPROPERTY()
	TObjectPtr<UToolMeshSelector> MeshSelector;
	TWeakObjectPtr<USkinWeightsPaintTool> WeightTool;
};

// this class wraps all the data needed to isolate a selection of a mesh while editing skin weights
UCLASS(MinimalAPI)
class UWeightToolSelectionIsolator : public UObject
{
	GENERATED_BODY()

public:

	// call during tool Setup()
	UE_API void InitialSetup(USkinWeightsPaintTool* InTool);

	// call every tick to apply deferred changes to mesh
	UE_API void UpdateIsolatedSelection();

	// returns true if further isolation can be performed
	UE_API bool CanIsolate() const;

	// returns true if any triangles are currently isolated
	UE_API bool IsMeshIsolated() const;
	
	// isolate the current selection
	UE_API void IsolateSelectionAsTransaction();

	// unisolate the current selection
	UE_API void UnIsolateSelectionAsTransaction();

	// isolate the array of triangles
	UE_API void SetTrianglesToIsolate(const TArray<int32>& TrianglesToIsolate);
	
	// restores the whole mesh
	UE_API void RestoreFullMesh();
	
	// apply weights directly
	UE_API void ApplyWeightsToFullMesh() const;
	
	// get the current triangles that are isolated
	const TArray<int32>& GetIsolatedTriangles() { return CurrentlyIsolatedTriangles; };

	// get the partial submesh mapping (valid only when IsMeshIsolated() is true)
	const UE::Geometry::FDynamicSubmesh3& GetPartialSubmesh() const { return PartialSubMesh; }

	// get the weight tool this isolator is attached to
	USkinWeightsPaintTool& GetWeightTool() const { return *WeightTool; }


	// convert to/from partial-isolated and full mesh vertex indices
	UE_API int32 PartialToFullMeshVertexIndex(int32 PartialMeshVertexIndex) const;
	UE_API int32 FullToPartialMeshVertexIndex(int32 FullMeshVertexIndex) const;

	// returns the isolated partial mesh (if PartialMeshDescription is not null, this will return an empty DynamicMesh3)
	UE_API const FDynamicMesh3& GetPartialMesh() const;

private:

	UE_API void CreatePartialMesh();
	
	UPROPERTY()
	TObjectPtr<USkinWeightsPaintTool> WeightTool;
		
	// when selection is isolated, we hide the full mesh and show a submesh
	// when isolated selection is unhidden, we remap all changes from the submesh back to the full mesh
	TArray<int32> CurrentlyIsolatedTriangles;

	/** Flag to indicate the isolated mesh needs to be updated, handled on the next OnTick */
	bool bIsolatedMeshNeedsUpdated = false;

	// isolate selection sub-meshes
	UE::Geometry::FDynamicSubmesh3 PartialSubMesh;
};

class FIsolateSelectionChange : public FToolCommandChange
{
public:
	TArray<int32> IsolatedTrianglesBefore;
	TArray<int32> IsolatedTrianglesAfter;

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;
};



// An interactive tool for painting and editing skin weights.
UCLASS(MinimalAPI)
class USkinWeightsPaintTool : 
	public UDynamicMeshBrushTool, 
	public ISkeletalMeshEditingInterface, 
	public ISkeletalMeshGeometryIsolationAwareTool,
	public IInteractiveToolManageGeometrySelectionAPI,
	public IHotkeyHintProvider
{
	GENERATED_BODY()

	// Undo/redo writes vertex snapshots straight into Weights.
	friend class SkinPaintTool::FMeshSkinWeightsChange;

public:
	static constexpr const TCHAR* ActionCommandsContextName = TEXT("SkeletalMeshModelingToolsSkinWeightsPaintTool");

	// Derived tools can optionally override these if they are invoked in editors that don't offer skeletal mesh editor context object
	UE_API virtual EMeshLODIdentifier GetEditingLOD();
	UE_API virtual const TArray<FTransform>& GetComponentSpaceBoneTransforms();
	UE_API virtual void ToggleBoneManipulation(bool bEnable);

	// UBaseBrushTool overrides
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }
	UE_API virtual bool HitTest(const FRay& Ray, FHitResult& OutHit) override;
	UE_API virtual void OnBeginDrag(const FRay& Ray) override;
	UE_API virtual void OnUpdateDrag(const FRay& Ray) override;
	UE_API virtual void OnEndDrag(const FRay& Ray) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual double EstimateMaximumTargetDimension() override;
	UE_API virtual void SetupBrushStampIndicator() override;
	UE_API virtual void UpdateBrushStampIndicator() override;
	UE_API virtual void ShutdownBrushStampIndicator() override;

	UE_API void Init(const FToolBuilderState& InSceneState);
	
	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API FName GetActiveSkinWeightProfileName() const;
	
	// IInteractiveToolCameraFocusAPI
	virtual bool SupportsWorldSpaceFocusBox() override { return true; }
	UE_API virtual FBox GetWorldSpaceFocusBox() override;

	// IClickDragBehaviorTarget implementation
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;
	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& InPressPos) override;

	// Switches active skin profile during undo/redo replay.
	UE_API void ExternalUpdateSkinWeightLayer(const FName InSkinWeightProfile);

	// weight editing operations (selection based)
	UE_API void MirrorWeights(EAxis::Type Axis, EMirrorDirection Direction);
	UE_API void PruneWeights(const float Threshold, const TArray<BoneIndex>& BonesToPrune);
	UE_API void AverageWeights(const float Strength);
	UE_API void NormalizeWeights();
	UE_API void HammerWeights();
	UE_API void ClampInfluences(const int32 MaxInfluences);

	// HELPER functions for modifying weights
	//
	// given a map of BoneIndex > Weight values for a single vertex, modify the map by removing the smallest weights to fit in Max Influences - NumLockedBones
	static UE_API void TruncateWeightMap(TMap<BoneIndex, float>& InOutWeights, int32 NumLockedBones = 0);
	// given a map of BoneIndex > Weight values for a single vertex, modify the weights to sum to ExpectedSum
	static UE_API void NormalizeWeightMap(TMap<BoneIndex, float>& InOutWeights, float ExpectedSum = 1.0f);
	// sum all the weights on all bones for a given list of vertices (results are not normalized!)
	static UE_API void AccumulateWeights(
		const TArray<SkinPaintTool::VertexWeights>& AllWeights,
		const TArray<VertexIndex>& VerticesToAccumulate,
		const TSet<BoneIndex>& ExcludedBones,
		TMap<BoneIndex, float>& OutAccumulatedWeights);

	// query information about how weights are locked on a specific vertex
	static UE_API void GetLockedWeight(
		const TArray<SkinPaintTool::VertexWeights>& AllWeights,
		VertexIndex InVertexID,
		const TSet<BoneIndex>& LockedBones,
		float& OutTotalLockedWeight,
		int32& OutNumLockedBones);
	
	// copy paste
	UE_API void CopyWeights();
	UE_API void PasteWeights();
	static UE_API const FString CopyPasteWeightsIdentifier;
	
	// method to set weights directly (numeric input, for example)
	UE_API void EditWeightsOnVertices(
		BoneIndex Bone,
		const float Value,
		const int32 Iterations,
		EWeightEditOperation EditOperation,
		const TArray<VertexIndex>& VerticesToEdit,
		const bool bShouldTransact);

	// toggle brush / selection mode
	UE_API void ToggleEditingMode();
	// update the state of the mesh selectors
	UE_API void UpdateSelectorState() const;
	
	// get access to the mesh selector for the main mesh
	UE_API UToolMeshSelector* GetMainMeshSelector();
	// get access to the currently active mesh selector (may be on the transfer source mesh)
	UE_API UToolMeshSelector* GetActiveMeshSelector();
	// does the main mesh have an active selection ("active" meaning the selection is currently being rendered in the view and is editable)
	UE_API bool HasActiveSelectionOnMainMesh();
	// select all vertices affected by the currently selected bone(s)
	UE_API void SelectAffected() const;
	// select all vertices affected by at least MinInfluences number of bones
	UE_API void SelectByInfluenceCount(const int32 MinInfluences) const;

	// get the average weight value of each influence on the given vertices
	UE_API void GetInfluences(const TArray<int32>& VertexIndices, TArray<BoneIndex>& OutBoneIndices);
	// get the average weight value of a single bone on the given vertices
	UE_API float GetAverageWeightOnBone(const BoneIndex InBoneIndex, const TArray<int32>& VertexIndices);
	// convert an index to a name
	UE_API FName GetBoneNameFromIndex(BoneIndex InIndex) const;
	// get the currently selected bone
	UE_API BoneIndex GetCurrentBoneIndex() const;
	// get a list of vertices affected by the given bone
	UE_API void GetVerticesAffectedByBone(BoneIndex IndexOfBone, TSet<int32>& OutVertexIndices) const;

	// toggle the display of weights on the preview mesh (if false, uses the normal skeletal mesh material)
	UE_API void SetDisplayVertexColors(bool bShowVertexColors=true);
	// set focus back to viewport so that hotkeys are immediately detected while hovering
	UE_API void SetFocusInViewport() const;

	// get the target manager (cached from Setup)
	UToolTargetManager* GetTargetManager() const { return TargetManager.Get(); };

	// Set the target manager 
	void SetTargetManager(UToolTargetManager* InTargetManager) { TargetManager = InTargetManager; };

	// allows outside systems to access the weight data
	SkinPaintTool::FSkinToolWeights& GetWeights() { return Weights; };

	// get access to the weight tranfer system
	UWeightToolTransferManager* GetWeightTransferManager() const { return TransferManager; };

	// get the viewport this tool is operating in
	UE_API virtual FEditorViewportClient* GetViewportClient() const;
	
	// get access to the selection isolation system
	UWeightToolSelectionIsolator* GetSelectionIsolator() const { return SelectionIsolator; };

	// get access to the mesh-element visualizer (used for polygroup-boundary edge rendering)
	UMeshElementsVisualizer* GetMeshElementsDisplay() const { return MeshElementsDisplay; };

	// get the tool properties
	UE_API USkinWeightsPaintToolProperties* GetWeightToolProperties() const;

	// get access to the mesh being edited
	FDynamicMesh3& GetMesh() {return EditedMesh;};

	// Returns default isolation triangles from the editor context (empty if none).
	// Stable for the tool's lifetime — the mode saves/restores isolation around tool activation.
	UE_API const TArray<int32>& GetDefaultIsolationTriangles() const;

	// get the set of bones whose weights are actually locked, taking into account bBypassAllLocks
	UE_API const TSet<int32>& GetFinalLockedBoneIndices();

	// get the set of bones whose weights are locked by user, does not consider bBypassAllLocks
	UE_API const TSet<int32>& GetRawLockedBoneIndices();

	// whether weight edits should respect locks or not
	UE_API bool IsBypassingAllLocks() const;;
	
	// Certain operations do not support bone locking yet, show a dialog to check if user still want to proceed
	UE_API bool ShouldProceedWithAllBonesUnlocked();
	
	// HOW TO EDIT WEIGHTS WITH UNDO/REDO:
	//
	// "Interactive" Edits:
	// For multiple weight editing operations that need to be grouped into a single transaction, like dragging a slider or
	// dragging a brush, you must call:
	//  1. BeginChange()
	//  2. ApplyWeightEditsWithoutTransaction() (this may be called multiple times)
	//  2. EndChange()
	// All the edits are stored into the "ActiveChange" and applied as a single transaction in EndChange().
	// Deformations and vertex colors will be updated throughout the duration of the change.
	UE_API void BeginChange();
	UE_API void EndChange(const FText& TransactionLabel);
	UE_API void ApplyWeightEditsWithoutTransaction(const SkinPaintTool::FMultiBoneWeightEdits& WeightEdits);
	// "One-off" Edits:
	// For all one-and-done edits, you can call ApplyWeightEditsAsTransaction().
	// It will Begin/End the change and create a transaction for it.
	UE_API void ApplyWeightEditsAsTransaction(const SkinPaintTool::FMultiBoneWeightEdits& WeightEdits, const FText& TransactionLabel);

	// call this whenever the target mesh is modified
	UE_API void UpdateCurrentlyEditedMesh(const FDynamicMesh3& InDynamicMesh);

	// Update the brush indicators visibility state
	UE_API void UpdateBrushIndicators();

	// called whenever the selection is modified
	DECLARE_MULTICAST_DELEGATE(FOnSelectionChanged);
	FOnSelectionChanged OnSelectionChanged;

	// called whenever the weights are modified
	DECLARE_MULTICAST_DELEGATE(FOnWeightsChanged);
	FOnWeightsChanged OnWeightsChanged;

	/** Returns the skeleton tree displayed when using this tool */
	const TSharedPtr<SReferenceSkeletonTree>& GetToolSkeletonTree() const { return ToolSkeletonTree; }

	// ISkeletalMeshGeometryIsolationAwareTool
	UE_API virtual bool IsInputIsolationValidOnOutput() const override;
	
	// IInteractiveToolManageGeometrySelectionAPI
	UE_API virtual bool IsInputSelectionValidOnOutput() override;

	// IHotkeyHintProvider
	UE_API virtual void GetHotkeyHints(TArray<FHotkeyHint>& OutHints) const override;
protected:
	// UInteractiveTool overrides
	UE_API virtual void RegisterActions(FInteractiveToolActionSet& ActionSet) override;

	UE_API virtual void ApplyStamp(const FBrushStampData& Stamp);
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnTick(float DeltaTime) override;

	/** Called when linear color ramp curves changed */
	void OnLinearColorRampChanged(TArray<FRichCurve*> ChangedCurves);

	// stamp
	UE_API float CalculateBrushFalloff(float Distance) const;
	UE_API void CalculateVertexROI(
		const FBrushStampData& InStamp,
		TArray<VertexIndex>& OutVertexIDs,
		TArray<float>& OutVertexFalloffs);
	UE_API float CalculateBrushStrengthToUse(EWeightEditOperation EditMode) const;
	bool bInvertStroke = false;
	FBrushStampData StartStamp;
	FBrushStampData LastStamp;
	bool bStampPending;
	int32 TriangleUnderStamp;
	FVector StampLocalPos;

	// Last accepted stamp's world-space position/radius; consulted by the Relax-accumulate
	// dwell throttle in ApplyStamp.
	TOptional<FVector3d> LastStampWorldPosition;
	double LastStampWorldRadius = 0.0;
	
	// generating bone weight edits to be stored in a transaction
	// does not actually change the weight buffers
	UE_API void CreateWeightEditsForVertices(
		EWeightEditOperation EditOperation,
		const BoneIndex Bone,
		const TArray<int32>& VerticesToEdit,
		const TArray<float>& VertexFalloffs,
		const float InValue,
		SkinPaintTool::FMultiBoneWeightEdits& InOutWeightEdits);
	// Single smoothing pass. To iterate, pair each call with ApplyWeightEditsWithoutTransaction
	// so CurrentWeights advances before the next call. bUsePreChangeWeight=false reads
	// CurrentWeights for cross-call compounding; true reads the at-stroke-start snapshot.
	UE_API void CreateWeightEditsToRelaxVertices(
		TArray<int32> VerticesToEdit,
		TArray<float> VertexFalloffs,
		const float Strength,
		const bool bUsePreChangeWeight,
		SkinPaintTool::FMultiBoneWeightEdits& InOutWeightEdits);

	// used to accelerate mesh queries
	using DynamicVerticesOctree = UE::Geometry::TDynamicVerticesOctree3<FDynamicMesh3>;
	TUniquePtr<DynamicVerticesOctree> VerticesOctree;
	using DynamicTrianglesOctree = UE::Geometry::FDynamicMeshOctree3;
	TUniquePtr<DynamicTrianglesOctree> TrianglesOctree;
	TFuture<void> TriangleOctreeFuture;
	TArray<int32> TrianglesToReinsert;
	UE_API void InitializeOctrees();

	// tool properties
	UPROPERTY()
	TObjectPtr<USkinWeightsPaintToolProperties> WeightToolProperties;
	UE_API virtual void OnPropertyModified(UObject* ModifiedObject, FProperty* ModifiedProperty) override;
	
	// the currently edited mesh descriptions
	FDynamicMesh3 EditedMesh;
	
	FName ActiveSkinWeightProfileName = FSkeletalMeshAttributesShared::DefaultSkinWeightProfileName;

	// storage of vertex weights per bone 
	SkinPaintTool::FSkinToolWeights Weights;

	// cached mirror data
	SkinPaintTool::FSkinMirrorData MirrorData;

	// storage for weight edits in the current transaction
	TUniquePtr<SkinPaintTool::FMeshSkinWeightsChange> ActiveChange;

	// Smooth weights data source and operator
	TUniquePtr<UE::Geometry::TBoneWeightsDataSource<int32, float>> SmoothWeightsDataSource;
	TUniquePtr<UE::Geometry::TSmoothBoneWeights<int32, float>> SmoothWeightsOp;
	UE_API void InitializeSmoothWeightsOperator(const FDynamicMesh3& RestPoseMesh);

	// vertex colors updated when switching current bone or initializing whole mesh
	UE_API void UpdateVertexColorForAllVertices();
	bool bVertexColorsNeedUpdated = false;
	// Per-tick gate for vertex-color refresh. OnTick consumes and Clear()s.
	UE_API void UpdateVertexColorForSubsetOfVertices();
	SkinPaintTool::FVertexChangeTracker PerColorUpdateVertexChangeTracker;
	UE_API FVector4f GetColorOfVertex(VertexIndex InVertexIndex, BoneIndex InBoneIndex) const;

	// which bone are we currently painting?
	UE_API void UpdateCurrentBone(const FName &BoneName);
	UE_API BoneIndex GetBoneIndexFromName(const FName BoneName) const;
	UE_API bool HasSelectedBone() const;
	FName CurrentBone = NAME_None;
	TOptional<FName> PendingCurrentBone;
	TArray<FName> SelectedBoneNames;
	TArray<BoneIndex> SelectedBoneIndices;

	// ISkeletalMeshEditingInterface
	UE_API virtual void HandleSkeletalMeshModified(const TArray<FName>& InBoneNames, const ESkeletalMeshNotifyType InNotifyType) override;
	UE_API virtual TSharedPtr<SWidget> GetCustomSkeletonTreeWidget(TSharedPtr<ISkeletalMeshEditorBinding> EditorBinding) override;
	
	// used to spawn a tool skeleton tree widget
	UPROPERTY()
	TObjectPtr<USkeletonModifier> SkeletonReader = nullptr;

	// skin weight locking
	UE_API virtual TSharedRef<SWidget> OnGetLockWidget(FName BoneName);
	UE_API void SetLockAllBones(bool bLock);
	UE_API void SetLockSelectedBones(bool bLock);
	UE_API void IsolateSelectedBones();
	UE_API void ToggleSelectedBonesLockState();
	UE_API void ToggleBypassLocks();
	UE_API void SetLockBone(BoneIndex InBoneIndex, bool bLock);

	TMap<BoneIndex, TSharedPtr<SButton>> LockButtons;
	TSet<BoneIndex> LockedBoneIndices;
	TSet<BoneIndex> UnlockedBoneIndices;
	bool bBypassLocks = false;

	
	// the selection system for the main mesh
	UPROPERTY()
	TObjectPtr<UToolMeshSelector> MeshSelector;

	// skin weight layer
	UE_API void OnActiveSkinWeightProfileChanged();
	UE_API void OnNewSkinWeightProfileChanged();
	UE_API bool IsProfileValid(const FName InProfileName);

	// global properties stored on initialization
	UPROPERTY()
	TWeakObjectPtr<USkeletalMeshEditorContextObjectBase> EditorContext = nullptr;
	UPROPERTY()
	TWeakObjectPtr<UPersonaEditorModeManagerContext> PersonaModeManagerContext = nullptr;
	UPROPERTY()
	TWeakObjectPtr<UToolTargetManager> TargetManager = nullptr;

	// manages transferring skin weights from a separate mesh
	UPROPERTY()
	TObjectPtr<UWeightToolTransferManager> TransferManager = nullptr;

	// manages isolating a selection of the mesh
	UPROPERTY()
	TObjectPtr<UWeightToolSelectionIsolator> SelectionIsolator = nullptr;

	// fast retained-mode wireframe for polygroup boundary edges and other mesh elements
	UPROPERTY()
	TObjectPtr<class UMeshElementsVisualizer> MeshElementsDisplay = nullptr;

	/** True while the config changed but wasn't saved yet */
	bool bConfigDirty = false;

	/** The skeleton tree displayed when using this tool */
	TSharedPtr<SReferenceSkeletonTree> ToolSkeletonTree;

	// editor state to restore when exiting the paint tool
	FString PreviewProfileToRestore;

	TArray<FTransform> DefaultComponentSpaceBoneTransforms;

	TUniquePtr<FSkeletalMeshNotifierBindScope> SkeletonTreeNotifierBindScope;
	
	friend SkinPaintTool::FSkinToolDeformer;
	
	// Need to reisolate the mesh if making structural changes to the edited mesh
	// submesh needs to be recreated
	struct FScopedSuspendIsolation
	{
		FScopedSuspendIsolation(USkinWeightsPaintTool* InTool);
		~FScopedSuspendIsolation();
		
		USkinWeightsPaintTool* Tool;
		TArray<int32> SavedIsolatedTriangles;
	};
	
	FName GetValidNewProfileName(const FName& InProposedName) const;
};

#undef UE_API
