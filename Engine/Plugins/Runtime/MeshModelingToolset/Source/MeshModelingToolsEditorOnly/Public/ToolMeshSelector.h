// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "InputState.h"
#include "MeshDescription.h"
#include "GroupTopology.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"

#include "Math/MathFwd.h"

#include "ToolMeshSelector.generated.h"

#define UE_API MESHMODELINGTOOLSEDITORONLY_API

class FCanvas;
class FEditorViewportClient;
class IToolsContextRenderAPI;
class UInteractiveTool;
class ULocalDoubleClickInputBehavior;
class UPolygonSelectionMechanic;
class UPreviewMesh;
class UWorld;

// component selection mode
UENUM()
enum class EComponentSelectionMode : uint8
{
	Vertices,
	Edges,
	Faces
};

UENUM()
enum class EMeshSelectorTool : uint8
{
	Ray,
	Marquee,
	Lasso
};

using VertexIndex = int32;

// this class wraps the all the components to enable selection on a single mesh in the skin weights tool
// this allows us to make selections on multiple different meshes
// NOTE: at some point we may want to do component selections on multiple meshes in any/all viewports
// at which time this class should be centralized and renamed to UMeshSelector or something like that.
// But there will need to be some sort of centralized facility to manage that and make sure it interacts nicely with other tools.
UCLASS(MinimalAPI)
class UToolMeshSelector : public UObject
{
	GENERATED_BODY()

public:

	// must be called during the Setup of the parent tool
	UE_DEPRECATED(5.7, "Use InitialSetup that does not need use a Viewport Client")
	UE_API void InitialSetup(UWorld* InWorld, UInteractiveTool* InParentTool, FEditorViewportClient* InViewportClient, TFunction<void()> OnSelectionChangedFunc);
	UE_API void InitialSetup(UWorld* InWorld, UInteractiveTool* InParentTool, TFunction<void()> OnSelectionChangedFunc);

	// must be called AFTER InitialSetup, and any time the mesh is changed
	// passing in a null preview mesh will disable the selector
	UE_API void SetMesh(
		UPreviewMesh* InMesh,
		const FTransform3d& InMeshTransform);

	UE_API void UpdateAfterMeshDeformation();

	UE_API void Shutdown();

	UE_API void SetIsEnabled(bool bIsEnabled);
	UE_API void SetComponentSelectionMode(EComponentSelectionMode InMode);
	UE_API void SetTransform(const FTransform3d& InTargetTransform);

	UE_API void SetSelectionTool(EMeshSelectorTool InTool);
	UE_API EMeshSelectorTool GetSelectionTool() const;

	// viewport 
	UE_API void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);
	UE_API void Render(IToolsContextRenderAPI* RenderAPI);

	// get a list of currently selected vertices (converting edges and faces to vertices)
	UE_API const TArray<int32>& GetSelectedVertices();
	UE_API bool IsAnyComponentSelected() const;
	UE_API void GetSelectedTriangles(TArray<int32>& OutTriangleIndices) const;

	// edit selection
	UE_API void GrowSelection() const;
	UE_API void ShrinkSelection() const;
	UE_API void InvertSelection() const;
	UE_API void FloodSelection() const;
	UE_API void SelectBorder() const;

	UE_API void Tick(float DeltaTime);

	// get access to the selection mechanic
	UPolygonSelectionMechanic* GetSelectionMechanic() { return PolygonSelectionMechanic; };

private:

	UPROPERTY()
	TObjectPtr<UInteractiveTool> ParentTool;
	UPROPERTY()
	TObjectPtr<UWorld> World;
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;
	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> PolygonSelectionMechanic;

	TUniquePtr<UE::Geometry::FDynamicMeshAABBTree3> MeshSpatial = nullptr;
	TUniquePtr<UE::Geometry::FTriangleGroupTopology> SelectionTopology = nullptr;
	// Optional polygroup topology pushed to UToolMeshSelectorSelectionMechanic to enable
	// the edge-loop gesture (the mechanic's primary FTriangleGroupTopology can't drive it).
	TUniquePtr<UE::Geometry::FGroupTopology> PolygroupTopology = nullptr;

	TArray<VertexIndex> SelectedVerticesInternal;
};

UCLASS(MinimalAPI)
class UToolMeshSelectorSelectionMechanic : public UPolygonSelectionMechanic
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup(UInteractiveTool* ParentToolIn) override;
	UE_API virtual void Shutdown() override;

	// Provide a polygroup topology for the shift+double-click edge-loop gesture. Pointer
	// is borrowed; the caller (UToolMeshSelector) owns the lifetime. Pass nullptr to disable.
	UE_API void SetPolygroupTopology(const UE::Geometry::FGroupTopology* InPolygroupTopology);

	// Drop the tracked seed used by the shift+double-click edge-loop gesture. Must be called
	// when the underlying mesh/topology changes so a stale vertex ID cannot be used as a seed.
	UE_API void ClearLastSingleClickedVertex();

	// Overridden to track recently single-click-selected corners, which seed the
	// shift+double-click edge-loop gesture deterministically.
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

private:
	FInputRayHit OnEdgeLoopDoubleClickHit(const FInputDeviceRay& ClickPos);
	void OnEdgeLoopDoubleClicked(const FInputDeviceRay& ClickPos);

	UPROPERTY()
	TObjectPtr<ULocalDoubleClickInputBehavior> EdgeLoopDoubleClickBehavior;

	const UE::Geometry::FGroupTopology* LoopWalkTopology = nullptr;

	// Two-deep history of single-click-selected corners. The double-click gesture uses
	// PreviousSingleClickedVertexID because the first half of a shift+double-click is
	// dispatched as a single click (see UDoubleClickInputBehavior state machine) and so
	// overwrites LastSingleClickedVertexID with the double-clicked vertex itself.
	// PreviousSingleClickedVertexID retains the click before that, which is the user's
	// actual seed.
	int32 LastSingleClickedVertexID = INDEX_NONE;
	int32 PreviousSingleClickedVertexID = INDEX_NONE;
};

#undef UE_API
