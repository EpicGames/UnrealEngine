// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelingComponents : ModuleRules
{
	public ModelingComponents(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"PhysicsCore",
				"InteractiveToolsFramework",
				"MeshDescription",
				"StaticMeshDescription",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"DynamicMesh",
				"MeshConversion",
				"ModelingOperators",
				"DeveloperSettings",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Chaos",
				"Engine",
				"RenderCore",
				"RHI",
				"ImageWriteQueue",
				"SkeletalMeshDescription", // FSkeletalMeshAttributes::DefaultSkinWeightProfileName
				"SlateCore",
				"ImageCore",
				"PlanarCut",
				"GeometryCollectionEngine",
				"MeshConversionEngineTypes",
				"GeometryCache"
				// ... add private dependencies that you statically link with here ...
			}
		);
		
		if (Target.bCompileAgainstEditor) // #if WITH_EDITOR
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"Slate"
				});
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}