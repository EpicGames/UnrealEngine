// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetUtils/MeshDescriptionUtil.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"




void UE::MeshDescription::InitializeAutoGeneratedAttributes(FMeshDescription& Mesh, const FMeshBuildSettings* BuildSettings)
{
	check(BuildSettings);

	bool bHasValidNormals = false;
	bool bHasValidTangents = false;
	FStaticMeshConstAttributes Attributes(Mesh);
	TArrayView<const FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<const FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
	for (const FVertexInstanceID VertexInstanceID : Mesh.VertexInstances().GetElementIDs())
	{
		bHasValidNormals |= (!VertexInstanceNormals[VertexInstanceID].IsNearlyZero() && !VertexInstanceNormals[VertexInstanceID].ContainsNaN());
		bHasValidTangents |= (!VertexInstanceTangents[VertexInstanceID].IsNearlyZero() && !VertexInstanceTangents[VertexInstanceID].ContainsNaN());
		if (bHasValidNormals && bHasValidTangents)
		{
			break;
		}
	}

	// rebuild normals/tangents if there are any invalid ones or the build settings request it
	if (!bHasValidNormals || !bHasValidTangents || BuildSettings->bRecomputeNormals || BuildSettings->bRecomputeTangents)
	{
		bool bRecomputeNormals = !bHasValidNormals || BuildSettings->bRecomputeNormals;
		bool bRecomputeTangents = !bHasValidTangents || BuildSettings->bRecomputeTangents;

		if (!Attributes.GetTriangleNormals().IsValid() || !Attributes.GetTriangleTangents().IsValid())
		{
			// If these attributes don't exist, create them and compute their values for each triangle
			FStaticMeshOperations::ComputeTriangleTangentsAndNormals(Mesh);
		}

		EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
		ComputeNTBsOptions |= bRecomputeNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
		ComputeNTBsOptions |= bRecomputeTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
		ComputeNTBsOptions |= BuildSettings->bUseMikkTSpace ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;
		ComputeNTBsOptions |= BuildSettings->bComputeWeightedNormals ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;
		ComputeNTBsOptions |= BuildSettings->bRemoveDegenerates ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;

		FStaticMeshOperations::ComputeTangentsAndNormals(Mesh, ComputeNTBsOptions);
	}
}





void UE::MeshDescription::InitializeAutoGeneratedAttributes(FMeshDescription& Mesh, UStaticMesh* StaticMesh, int32 SourceLOD)
{
#if WITH_EDITOR
	if (ensureMsgf(SourceLOD < StaticMesh->GetNumSourceModels(), TEXT("InitializeMeshDescription requested LOD index greater than num available in UStaticMesh")))
	{
		const FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(SourceLOD);
		InitializeAutoGeneratedAttributes(Mesh, &SourceModel.BuildSettings);
	}
#else
	ensureMsgf(false, TEXT("InitializeMeshDescription requires Editor-only SourceModel field of UStaticMesh"));
#endif
}



void UE::MeshDescription::InitializeAutoGeneratedAttributes(FMeshDescription& Mesh, UActorComponent* StaticMeshComponent, int32 SourceLOD)
{
#if WITH_EDITOR
	if (Cast<UStaticMeshComponent>(StaticMeshComponent) != nullptr)
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(StaticMeshComponent)->GetStaticMesh();
		if (ensure(StaticMesh != nullptr))
		{
			InitializeAutoGeneratedAttributes(Mesh, StaticMesh, SourceLOD);
		}
	}
#else
	//ensureMsgf(false, TEXT("InitializeMeshDescription requires Editor-only SourceModel field of UStaticMesh"));
#endif
}



void UE::MeshDescription::ConfigureBuildSettings(UStaticMesh* StaticMesh, int32 SourceLOD, FStaticMeshBuildSettingChange NewSettings)
{
#if WITH_EDITOR
	if (ensureMsgf(SourceLOD < StaticMesh->GetNumSourceModels(), TEXT("ConfigureBuildSettings requested LOD index greater than num available in UStaticMesh")))
	{
		FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(SourceLOD);
		
		if (NewSettings.AutoGeneratedNormals != EBuildSettingBoolChange::NoChange)
		{
			SourceModel.BuildSettings.bRecomputeNormals = 
				(NewSettings.AutoGeneratedNormals == EBuildSettingBoolChange::Disable) ? false : true;
		}
		if (NewSettings.AutoGeneratedTangents != EBuildSettingBoolChange::NoChange)
		{
			SourceModel.BuildSettings.bRecomputeTangents = 
				(NewSettings.AutoGeneratedTangents == EBuildSettingBoolChange::Disable) ? false : true;
		}
		if (NewSettings.UseMikkTSpaceTangents != EBuildSettingBoolChange::NoChange)
		{
			SourceModel.BuildSettings.bUseMikkTSpace = 
				(NewSettings.UseMikkTSpaceTangents == EBuildSettingBoolChange::Disable) ? false : true;
		}
	}
#else
	// just ignore?
#endif
}
