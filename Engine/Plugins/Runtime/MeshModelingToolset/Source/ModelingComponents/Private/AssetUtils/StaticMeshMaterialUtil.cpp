// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetUtils/StaticMeshMaterialUtil.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "MeshDescription.h"
#include "StaticMeshAttributes.h"

using namespace UE::AssetUtils;


bool UE::AssetUtils::GetStaticMeshLODAssetMaterials(
	UStaticMesh* StaticMeshAsset,
	int32 LODIndex,
	FStaticMeshLODMaterialSetInfo& MaterialInfoOut)
{
	if (!StaticMeshAsset)
	{
		return false;
	}

#if WITH_EDITOR

	if (StaticMeshAsset->IsSourceModelValid(LODIndex) == false)
	{
		return false;
	}

	// Need to access the mesh here because # Sections == # PolygonGroups and this info doesn't seem to be anywhere else.
	// Otherwise do not use the mesh in this function (would be nice to avoid this call if possible)
	const FMeshDescription* SourceMesh = StaticMeshAsset->GetMeshDescription(LODIndex);
	int32 NumSections = SourceMesh->PolygonGroups().Num();


	TArray<FStaticMaterial> StaticMaterials = StaticMeshAsset->GetStaticMaterials();
	MaterialInfoOut.MaterialSlots.Reset();
	for (FStaticMaterial Mat : StaticMaterials)
	{
		MaterialInfoOut.MaterialSlots.Add( FStaticMeshMaterialSlot{ Mat.MaterialInterface, Mat.MaterialSlotName } );
	}

	// This is complicated. A UStaticMesh has N MaterialSlots and each LOD has M Sections.
	// Each Section can have any MaterialSlot assigned to it, ie it is not necessarily 1-1 or in-order.
	// The SectionInfoMap is a TMap that will contain the SectionIndex-to-SlotIndex mapping
	// *if* the mapping is not (SectionIndex == SlotIndex), or has ever been edited.
	// So if the SectionIndex is not found in the SectionInfoMap, then it should be used as the SlotIndex directly.

	const FMeshSectionInfoMap& SectionInfoMap = StaticMeshAsset->GetSectionInfoMap();
	MaterialInfoOut.LODIndex = LODIndex;
	MaterialInfoOut.NumSections = NumSections;

	MaterialInfoOut.SectionSlotIndexes.SetNum(MaterialInfoOut.NumSections);
	MaterialInfoOut.SectionMaterials.SetNum(MaterialInfoOut.NumSections);
	for (int32 SectionIndex = 0; SectionIndex < MaterialInfoOut.NumSections; ++SectionIndex)
	{
		MaterialInfoOut.SectionSlotIndexes[SectionIndex] = -1;
		MaterialInfoOut.SectionMaterials[SectionIndex] = nullptr;

		if (SectionInfoMap.IsValidSection(LODIndex, SectionIndex) == false)
		{
			// did not find this section 
			if ( StaticMaterials.IsValidIndex(SectionIndex) )
			{
				MaterialInfoOut.SectionSlotIndexes[SectionIndex] = SectionIndex;
				MaterialInfoOut.SectionMaterials[SectionIndex] = MaterialInfoOut.MaterialSlots[SectionIndex].Material;
			}
			else
			{
				ensure(false);		// material list is broken? use default material.
			}
		}
		else
		{
			FMeshSectionInfo SectionInfo = SectionInfoMap.Get(LODIndex, SectionIndex);
			if ( StaticMaterials.IsValidIndex(SectionInfo.MaterialIndex) )
			{
				MaterialInfoOut.SectionSlotIndexes[SectionIndex] = SectionInfo.MaterialIndex;
				MaterialInfoOut.SectionMaterials[SectionIndex] = MaterialInfoOut.MaterialSlots[SectionInfo.MaterialIndex].Material;
			}
			else
			{
				ensure(false);		// this is *not* supposed to be able to happen! SectionMap is broken...
			}
		}
	}

	return true;

#else
	// TODO: how would we handle this for runtime static mesh?
	return false;
#endif

}



bool UE::AssetUtils::GetStaticMeshLODMaterialListBySection(
	UStaticMesh* StaticMeshAsset,
	int32 LODIndex,
	TArray<UMaterialInterface*>& MaterialListOut,
	TArray<int32>& MaterialIndexOut,
	TArray<FName>& MaterialSlotNameOut)
{
#if WITH_EDITOR
	// need valid MeshDescription in Editor path
	if (StaticMeshAsset->IsMeshDescriptionValid(LODIndex) == false)
	{
		return false;
	}
#endif

	FStaticMeshLODMaterialSetInfo MaterialSetInfo;
	if (GetStaticMeshLODAssetMaterials(StaticMeshAsset, LODIndex, MaterialSetInfo) == false)
	{
		return false;
	}

#if WITH_EDITOR

	const FMeshDescription* SourceMesh = StaticMeshAsset->GetMeshDescription(LODIndex);

	// # Sections == # PolygonGroups
	int32 NumPolygonGroups = SourceMesh->PolygonGroups().Num();

	MaterialListOut.Reset();
	MaterialIndexOut.Reset();
	for (int32 k = 0; k < NumPolygonGroups; ++k)
	{
		int32 UseSlotIndex = -1;
		if (k < MaterialSetInfo.SectionSlotIndexes.Num())
		{
			UseSlotIndex = MaterialSetInfo.SectionSlotIndexes[k];
		}
		else if (MaterialSetInfo.SectionSlotIndexes.Num() > 0)
		{
			UseSlotIndex = 0;
		}

		if (UseSlotIndex >= 0)
		{
			MaterialIndexOut.Add(UseSlotIndex);
			MaterialListOut.Add(MaterialSetInfo.MaterialSlots[UseSlotIndex].Material);
			MaterialSlotNameOut.Add(MaterialSetInfo.MaterialSlots[UseSlotIndex].SlotName);
		}
		else
		{
			MaterialIndexOut.Add(-1);
			MaterialListOut.Add(nullptr);
			MaterialSlotNameOut.Add(NAME_None);
		}
	}

	return true;

#else
	// TODO: how would we handle this for runtime static mesh?
	return false;
#endif


}



FName UE::AssetUtils::GenerateNewMaterialSlotName(
	const TArray<FStaticMaterial>& ExistingMaterials,
	UMaterialInterface* SlotMaterial,
	int32 NewSlotIndex)
{
	FString MaterialName = (SlotMaterial) ? SlotMaterial->GetName() : TEXT("Material");
	FName BaseName(MaterialName);

	bool bFound = false;
	for (const FStaticMaterial& Mat : ExistingMaterials)
	{
		if (Mat.MaterialSlotName == BaseName || Mat.ImportedMaterialSlotName == BaseName)
		{
			bFound = true;
			break;
		}
	}
	if (bFound == false && SlotMaterial != nullptr)
	{
		return BaseName;
	}

	bFound = true;
	while (bFound)
	{
		bFound = false;

		BaseName = FName(FString::Printf(TEXT("%s_%d"), *MaterialName, NewSlotIndex++));
		for (const FStaticMaterial& Mat : ExistingMaterials)
		{
			if (Mat.MaterialSlotName == BaseName || Mat.ImportedMaterialSlotName == BaseName)
			{
				bFound = true;
				break;
			}
		}
	}

	return BaseName;
}