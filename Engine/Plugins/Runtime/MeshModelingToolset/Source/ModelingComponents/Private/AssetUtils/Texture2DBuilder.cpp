// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetUtils/Texture2DBuilder.h"
#include "TextureResource.h"

using namespace UE::Geometry;



bool FTexture2DBuilder::Initialize(ETextureType BuildTypeIn, FImageDimensions DimensionsIn)
{
	// create new texture
	EPixelFormat UsePixelFormat = (BuildTypeIn == ETextureType::EmissiveHDR) ? PF_FloatRGBA : PF_B8G8R8A8;
	UTexture2D* TransientTexture = UTexture2D::CreateTransient((int32)DimensionsIn.GetWidth(), (int32)DimensionsIn.GetHeight(), UsePixelFormat);
	return InitializeInternal(BuildTypeIn, DimensionsIn, TransientTexture);
}


bool FTexture2DBuilder::InitializeAndReplaceExistingTexture(UTexture2D* ExistingTexture, ETextureType BuildTypeIn, FImageDimensions DimensionsIn)
{
	// adapted from UTexture2D::CreateTransient

	// create new texture
	EPixelFormat InFormat = (BuildTypeIn == ETextureType::EmissiveHDR) ? PF_FloatRGBA : PF_B8G8R8A8;

	int32 InSizeX = (int32)DimensionsIn.GetWidth(), InSizeY = (int32)DimensionsIn.GetHeight();

	if (ensureMsgf(InSizeX > 0 && InSizeY > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0, TEXT("Invalid size and/or pixel format for new texture")))
	{
		UTexture2D* NewTexture = NewObject<UTexture2D>(
			ExistingTexture->GetOuter(),
			ExistingTexture->GetFName(),
			ExistingTexture->GetFlags()
			);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = InSizeX;
		NewTexture->GetPlatformData()->SizeY = InSizeY;
		NewTexture->GetPlatformData()->PixelFormat = InFormat;

		// Allocate first mipmap.
		int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
		int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;
		FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		Mip->BulkData.Realloc(NumBlocksX * NumBlocksY * GPixelFormats[InFormat].BlockBytes);
		Mip->BulkData.Unlock();

		return InitializeInternal(BuildTypeIn, DimensionsIn, NewTexture);
	}
	
	return false;
}


bool FTexture2DBuilder::InitializeAndReplaceExistingTexture(UTexture2D* ExistingTexture, UTexture2D* SourceTexture)
{
	if (!ExistingTexture || !SourceTexture)
	{
		return false;
	}

	// create new texture
	const EPixelFormat InFormat = SourceTexture->GetPixelFormat();

	int32 InSizeX = SourceTexture->GetSizeX();
	int32 InSizeY = SourceTexture->GetSizeY();

	if (ensureMsgf(InSizeX > 0 && InSizeY > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0, TEXT("Invalid size and/or pixel format for new texture")))
	{
		UTexture2D* NewTexture = NewObject<UTexture2D>(
			ExistingTexture->GetOuter(),
			ExistingTexture->GetFName(),
			ExistingTexture->GetFlags()
			);

		NewTexture->SetPlatformData(new FTexturePlatformData());
		NewTexture->GetPlatformData()->SizeX = InSizeX;
		NewTexture->GetPlatformData()->SizeY = InSizeY;
		NewTexture->GetPlatformData()->PixelFormat = InFormat;

		// Allocate first mipmap.
		const int32 NumBlocksX = InSizeX / GPixelFormats[InFormat].BlockSizeX;
		const int32 NumBlocksY = InSizeY / GPixelFormats[InFormat].BlockSizeY;
		FTexture2DMipMap* Mip = new FTexture2DMipMap(InSizeX, InSizeY);
		NewTexture->GetPlatformData()->Mips.Add(Mip);
		Mip->BulkData.Lock(LOCK_READ_WRITE);
		Mip->BulkData.Realloc(NumBlocksX * NumBlocksY * GPixelFormats[InFormat].BlockBytes);
		Mip->BulkData.Unlock();

		// Initialize texture settings from SourceTexture.
		NewTexture->SRGB = SourceTexture->SRGB;
		NewTexture->CompressionSettings = SourceTexture->CompressionSettings;
		NewTexture->LODGroup = SourceTexture->LODGroup;
#if WITH_EDITOR
		NewTexture->MipGenSettings = SourceTexture->MipGenSettings;
#endif
		NewTexture->UpdateResource();

		BuildType = ETextureType::Color; // This will only be used to determine clear color.
		Dimensions = FImageDimensions(InSizeX, InSizeY);
		RawTexture2D = NewTexture;
		CurrentPixelFormat = InFormat;

		// lock
		if (LockForEditing() == false)
		{
			return false;
		}

		if (IsEditable())
		{
			Clear();
		}
		return true;
	}
	
	return false;
}


bool FTexture2DBuilder::InitializeInternal(ETextureType BuildTypeIn, FImageDimensions DimensionsIn, UTexture2D* CreatedTextureIn)
{
	check(DimensionsIn.IsSquare());
	BuildType = BuildTypeIn;
	Dimensions = DimensionsIn;

	RawTexture2D = CreatedTextureIn;
	if (RawTexture2D == nullptr)
	{
		return false;
	}
	CurrentPixelFormat = CreatedTextureIn->GetPixelFormat();

	if (BuildType == ETextureType::ColorLinear 
		|| BuildType == ETextureType::Roughness 
		|| BuildType == ETextureType::Metallic 
		|| BuildType == ETextureType::Specular
		|| BuildType == ETextureType::AmbientOcclusion)
	{
		RawTexture2D->SRGB = false;
		RawTexture2D->UpdateResource();
	}
	else if (BuildType == ETextureType::EmissiveHDR)
	{
		RawTexture2D->SRGB = false;
		RawTexture2D->CompressionSettings = TC_HDR;
		RawTexture2D->UpdateResource();
	}
	else if (BuildType == ETextureType::NormalMap)
	{
		RawTexture2D->CompressionSettings = TC_Normalmap;
		RawTexture2D->SRGB = false;
		RawTexture2D->LODGroup = TEXTUREGROUP_WorldNormalMap;
		//RawTexture2D->bFlipGreenChannel = true;
		RawTexture2D->UpdateResource();
	}

	// lock
	if (LockForEditing() == false)
	{
		return false;
	}

	if (IsEditable())
	{
		Clear();
	}

	return true;
}


bool FTexture2DBuilder::Initialize(UTexture2D* ExistingTexture, ETextureType BuildTypeIn, bool bLockForEditing)
{
	if (!ensure(ExistingTexture != nullptr)) return false;
	const FTexturePlatformData* TexPlatformData = ExistingTexture->GetPlatformData();
	if (!ensure(TexPlatformData != nullptr)) return false;
	if (!ensure(TexPlatformData->Mips.Num() > 0)) return false;

	EPixelFormat UsePixelFormat = (BuildTypeIn == ETextureType::EmissiveHDR) ? PF_FloatRGBA : PF_B8G8R8A8;
	if (!ensure(TexPlatformData->PixelFormat == UsePixelFormat))
	{
		return false;
	}
	CurrentPixelFormat = UsePixelFormat;

	int32 Width = TexPlatformData->Mips[0].SizeX;
	int32 Height = TexPlatformData->Mips[0].SizeY;
	Dimensions = FImageDimensions(Width, Height);
	BuildType = BuildTypeIn;
	RawTexture2D = ExistingTexture;

	// lock for editing
	if (bLockForEditing && LockForEditing() == false)
	{
		return false;
	}

	return true;
}



bool FTexture2DBuilder::LockForEditing()
{
	if (!ensure(RawTexture2D != nullptr && CurrentMipData == nullptr && CurrentMipDataFloat16 == nullptr))
	{
		return false;
	}

	if (IsByteTexture())
	{
		if (RawTexture2D && CurrentMipData == nullptr)
		{
			CurrentMipData = reinterpret_cast<FColor*>(RawTexture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
			ensure(CurrentMipData);
		}
	}
	else
	{
		if (RawTexture2D && CurrentMipDataFloat16 == nullptr)
		{
			CurrentMipDataFloat16 = reinterpret_cast<FFloat16Color*>(RawTexture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE));
			ensure(CurrentMipDataFloat16);
		}
	}

	return IsEditable();
}


void FTexture2DBuilder::Commit(bool bUpdateSourceData)
{
	if (ensure(RawTexture2D != nullptr && IsEditable()))
	{
		if (bUpdateSourceData)
		{
			UpdateSourceData();
		}

		RawTexture2D->GetPlatformData()->Mips[0].BulkData.Unlock();
		RawTexture2D->UpdateResource();

		CurrentMipData = nullptr;
		CurrentMipDataFloat16 = nullptr;
	}
}

void FTexture2DBuilder::UpdateSourceData()
{
	// source data only exists in Editor
#if WITH_EDITOR
	check(RawTexture2D);

	bool bIsEditable = IsEditable();

	if (IsByteTexture())
	{
		const FColor* SourceMipData = (bIsEditable) ? CurrentMipData :
			reinterpret_cast<const FColor*>(RawTexture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_ONLY));
		RawTexture2D->Source.Init2DWithMipChain(Dimensions.GetWidth(), Dimensions.GetHeight(), TSF_BGRA8);
		uint8* DestData = RawTexture2D->Source.LockMip(0);
		FMemory::Memcpy(DestData, SourceMipData, Dimensions.GetWidth() * Dimensions.GetHeight() * 4 * sizeof(uint8));
	}
	else
	{
		const FFloat16Color* SourceMipData = (bIsEditable) ? CurrentMipDataFloat16 :
			reinterpret_cast<const FFloat16Color*>(RawTexture2D->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_ONLY));
		RawTexture2D->Source.Init2DWithMipChain(Dimensions.GetWidth(), Dimensions.GetHeight(), TSF_RGBA16F);
		uint8* DestData = RawTexture2D->Source.LockMip(0);
		FMemory::Memcpy((void*)DestData, (void*)SourceMipData, Dimensions.GetWidth() * Dimensions.GetHeight() * sizeof(FFloat16Color));
	}

	RawTexture2D->Source.UnlockMip(0);
	if (bIsEditable == false)
	{
		RawTexture2D->GetPlatformData()->Mips[0].BulkData.Unlock();
	}
#endif
}

void FTexture2DBuilder::Cancel()
{
	bool bIsEditable = IsEditable();
	if (bIsEditable)
	{
		RawTexture2D->GetPlatformData()->Mips[0].BulkData.Unlock();
		CurrentMipData = nullptr;
		CurrentMipDataFloat16 = nullptr;
	}
}


void FTexture2DBuilder::Clear()
{
	if (IsByteTexture())
	{
		Clear(GetClearColor());
	}
	else
	{
		Clear(GetClearColorFloat16());
	}
}


void FTexture2DBuilder::Clear(const FColor& ClearColor)
{
	if (ensure(IsEditable() && IsByteTexture()))
	{
		int64 Num = Dimensions.Num();
		for (int64 k = 0; k < Num; ++k)
		{
			CurrentMipData[k] = ClearColor;
		}
	}
}

/**
 * Clear all texels in the current Mip to the given ClearColor
 */
void FTexture2DBuilder::Clear(const FFloat16Color& ClearColor)
{
	if (ensure(IsEditable() && IsFloat16Texture()))
	{
		int64 Num = Dimensions.Num();
		for (int64 k = 0; k < Num; ++k)
		{
			CurrentMipDataFloat16[k] = ClearColor;
		}
	}
}


bool FTexture2DBuilder::Copy(const TImageBuilder<FVector3f>& SourceImage, const bool bConvertToSRGB)
{
	if (ensure(SourceImage.GetDimensions() == Dimensions) == false)
	{
		return false;
	}
	if (IsFloat16Texture() && bConvertToSRGB)
	{
		ensure(false);		// not currently supported
	}

	int64 Num = Dimensions.Num();
	for (int64 i = 0; i < Num; ++i)
	{
		FVector3f Pixel = SourceImage.GetPixel(i);
		if (IsByteTexture())
		{
			FColor Texel = ToLinearColor(Pixel).ToFColor(bConvertToSRGB);
			SetTexel(i, Texel);
		}
		else
		{
			FFloat16Color Texel(ToLinearColor(Pixel));
			SetTexel(i, Texel);
		}
	}
	return true;
}


bool FTexture2DBuilder::Copy(const TImageBuilder<FVector4f>& SourceImage, const bool bConvertToSRGB)
{
	if (ensure(SourceImage.GetDimensions() == Dimensions) == false)
	{
		return false;
	}
	if (IsFloat16Texture() && bConvertToSRGB)
	{
		ensure(false);		// not currently supported
	}

	int64 Num = Dimensions.Num();
	for (int64 i = 0; i < Num; ++i)
	{
		FVector4f Pixel = SourceImage.GetPixel(i);
		if (IsByteTexture())
		{
			FColor Texel = ToLinearColor(Pixel).ToFColor(bConvertToSRGB);
			SetTexel(i, Texel);
		}
		else
		{
			FFloat16Color Texel(ToLinearColor(Pixel));
			SetTexel(i, Texel);
		}
	}
	return true;
}


bool FTexture2DBuilder::CopyImageToSourceData(const TImageBuilder<FVector4f>& SourceImage, const ETextureSourceFormat SourceDataFormat, const bool bConvertToSRGB)
{
	// source data only exists in Editor
#if WITH_EDITOR
	if(!ensure(RawTexture2D))
	{
		return false;
	}

	if (ensure(SourceImage.GetDimensions() == Dimensions) == false)
	{
		return false;
	}

	// Currently only support these source formats.
	if (!ensure(SourceDataFormat == TSF_BGRA8 || SourceDataFormat == TSF_RGBA16F))
	{
		return false;
	}

	if (SourceDataFormat == TSF_RGBA16F && bConvertToSRGB)
	{
		ensure(false);		// not currently supported
	}

	RawTexture2D->Source.Init2DWithMipChain(Dimensions.GetWidth(), Dimensions.GetHeight(), SourceDataFormat);

	const int64 Num = Dimensions.Num();
	if (SourceDataFormat == TSF_BGRA8)
	{
		FColor* DestData = reinterpret_cast<FColor*>(RawTexture2D->Source.LockMip(0));
		for (int64 i = 0; i < Num; ++i)
		{
			FVector4f Pixel = SourceImage.GetPixel(i);
			const FColor Texel = ToLinearColor(Pixel).ToFColor(bConvertToSRGB);
			DestData[i] = Texel;
		}
	}
	else if (SourceDataFormat == TSF_RGBA16F)
	{
		FFloat16Color* DestData = reinterpret_cast<FFloat16Color*>(RawTexture2D->Source.LockMip(0));
		for (int64 i = 0; i < Num; ++i)
		{
			FVector4f Pixel = SourceImage.GetPixel(i);
			const FFloat16Color Texel(ToLinearColor(Pixel));
			DestData[i] = Texel;
		}
	}

	RawTexture2D->Source.UnlockMip(0);
#endif
	return false;
}



bool FTexture2DBuilder::CopyTo(TImageBuilder<FVector4f>& DestImage) const
{
	if (ensure(DestImage.GetDimensions() == Dimensions) == false)
	{
		return false;
	}
	int64 Num = Dimensions.Num();
	for (int64 i = 0; i < Num; ++i)
	{
		if (IsByteTexture())
		{
			FColor ByteColor = GetTexel(i);
			FLinearColor FloatColor(ByteColor);
			DestImage.SetPixel(i, ToVector4<float>(FloatColor));
		}
		else
		{
			FFloat16Color Float16Color = GetTexelFloat16(i);
			FLinearColor FloatColor = Float16Color.GetFloats();
			DestImage.SetPixel(i, ToVector4<float>(FloatColor));
		}
	}
	return true;
}


bool FTexture2DBuilder::CopyPlatformDataToSourceData(UTexture2D* Texture, ETextureType TextureType)
{
	FTexture2DBuilder Builder;
	bool bOK = Builder.Initialize(Texture, TextureType, false);
	if (bOK)
	{
		Builder.UpdateSourceData();
	}
	return bOK;
}



const FColor& FTexture2DBuilder::GetClearColor() const
{
	static const FColor DefaultColor = FColor::Black;
	static const FColor DefaultRoughness(128, 128, 128);
	static const FColor DefaultSpecular(100, 100, 100);
	static const FColor DefaultMetallic(16, 16, 16);
	static const FColor DefaultNormalColor(128, 128, 255);
	static const FColor DefaultAOColor = FColor::White;

	switch (BuildType)
	{
		default:
		case ETextureType::Color:
		case ETextureType::ColorLinear:
		case ETextureType::EmissiveHDR:
			return DefaultColor;
		case ETextureType::Roughness:
			return DefaultRoughness;
		case ETextureType::Metallic:
			return DefaultMetallic;
		case ETextureType::Specular:
			return DefaultSpecular;
		case ETextureType::NormalMap:
			return DefaultNormalColor;
		case ETextureType::AmbientOcclusion:
			return DefaultAOColor;
	}
}


/**
 * @return the default color for the current texture build type
 */
FFloat16Color FTexture2DBuilder::GetClearColorFloat16() const
{
	static const FLinearColor DefaultColor(0.0f, 0.0f, 0.0f);
	static const FLinearColor DefaultRoughness(0.5f, 0.5f, 0.5f);
	static const FLinearColor DefaultSpecular(0.4f, 0.4f, 0.4f);
	static const FLinearColor DefaultMetallic(0.05f, 0.05f, 0.05f);
	static const FLinearColor DefaultNormalColor(0.5f, 0.5f, 0.5f);
	static const FLinearColor DefaultAOColor(1.0f, 1.0f, 1.0f);
	switch (BuildType)
	{
		default:
		case ETextureType::Color:
		case ETextureType::ColorLinear:
		case ETextureType::EmissiveHDR:
			return FFloat16Color(DefaultColor);
		case ETextureType::Roughness:
			return FFloat16Color(DefaultRoughness);
		case ETextureType::Metallic:
			return FFloat16Color(DefaultMetallic);
		case ETextureType::Specular:
			return FFloat16Color(DefaultSpecular);
		case ETextureType::NormalMap:
			return FFloat16Color(DefaultNormalColor);
		case ETextureType::AmbientOcclusion:
			return FFloat16Color(DefaultAOColor);
	}
}