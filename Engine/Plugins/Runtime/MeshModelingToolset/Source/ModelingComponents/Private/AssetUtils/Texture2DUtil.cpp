// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetUtils/Texture2DUtil.h"
#include "EngineModule.h"
#include "ImageUtils.h"
#include "RendererInterface.h"
#include "RenderUtils.h"
#include "TextureResource.h"
#include "TextureCompiler.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

using namespace UE::Geometry;

static bool ReadTexture_PlatformData(
	UTexture2D* TextureMap,
	TImageBuilder<FVector4f>& DestImage)
{
	// Read from PlatformData
	// UBlueprintMaterialTextureNodesBPLibrary::Texture2D_SampleUV_EditorOnly() shows how to read from PlatformData
	// without converting formats if it's already uncompressed. And can read PF_FloatRGBA. Would make sense to do
	// that when possible, and perhaps is possible to convert to PF_FloatRGBA instead of using TC_VectorDisplacementmap?
	// 
	// Note that the current code cannot run on a background thread, UpdateResource() will call FlushRenderingCommands()
	// which will check() if it's on the Game Thread

	if (TextureMap->GetPlatformData() == nullptr)
	{
		return false;
	}

	const int32 Width = TextureMap->GetPlatformData()->Mips[0].SizeX;
	const int32 Height = TextureMap->GetPlatformData()->Mips[0].SizeY;
	const FImageDimensions Dimensions = FImageDimensions(Width, Height);
	DestImage.SetDimensions(Dimensions);
	const int64 Num = Dimensions.Num();

	// convert built platform texture data to uncompressed RGBA8 format
	const TextureCompressionSettings InitialCompressionSettings = TextureMap->CompressionSettings;

	// the Platform BulkData for most texture formats is compressed and we cannot process it on
	// the CPU. However TC_VectorDisplacementmap is uncompressed RGBA8.
	// in the Editor, if we have source data, we can temporarily change the texture type and rebuild the PlatformData.
	// However at Runtime we cannot, and so we can only error-out if we do not have a readable compression type.
#if WITH_EDITOR
	const TextureMipGenSettings InitialMipGenSettings = TextureMap->MipGenSettings;
	bool bNeedToRevertTextureChanges = false;
	if (InitialMipGenSettings != TextureMipGenSettings::TMGS_NoMipmaps || InitialCompressionSettings != TextureCompressionSettings::TC_VectorDisplacementmap)
	{
		if (!TextureMap->Source.IsValid())
		{
			return false;
		}

		TextureMap->CompressionSettings = TextureCompressionSettings::TC_VectorDisplacementmap;
		TextureMap->MipGenSettings = TextureMipGenSettings::TMGS_NoMipmaps;
		TextureMap->UpdateResource();
		bNeedToRevertTextureChanges = true;
	}
#else
	if (InitialCompressionSettings != TC_VectorDisplacementmap)
	{
		return false;
	}
#endif

	// lock texture and read as FColor (RGBA8)
	const FColor* FormattedImageData = reinterpret_cast<const FColor*>(TextureMap->GetPlatformData()->Mips[0].BulkData.LockReadOnly());

	// maybe could be done more quickly by row?
	for (int64 i = 0; i < Num; ++i)
	{
		FColor ByteColor = FormattedImageData[i];
		FLinearColor FloatColor = (TextureMap->SRGB) ?
				FLinearColor::FromSRGBColor(ByteColor) :
				ByteColor.ReinterpretAsLinear();
		DestImage.SetPixel(i, ToVector4<float>(FloatColor));
	}

	TextureMap->GetPlatformData()->Mips[0].BulkData.Unlock();

#if WITH_EDITOR
	// restore built platform texture data to initial state, if we modified it
	if (bNeedToRevertTextureChanges)
	{
		TextureMap->CompressionSettings = InitialCompressionSettings;
		TextureMap->MipGenSettings = InitialMipGenSettings;
		TextureMap->UpdateResource();
	}
#endif

	return true;
}


#if WITH_EDITOR
static bool ReadTexture_SourceData(
	UTexture2D* TextureMap,
	TImageBuilder<FVector4f>& DestImage)
{
	if ( ! TextureMap->Source.IsValid() )
	{
		return false;
	}

	// TODO #jira UE-176086 Enable this code when it is less risky to do so, and after it has been tested. This code
	// also fixes an issue where not all texture source formats are supported
	if constexpr (false)
	{
		const FTextureSource& TextureSource = TextureMap->Source;
		const int32 Width = TextureSource.GetSizeX();
		const int32 Height = TextureSource.GetSizeY();

		FImage Image;
		verify( FImageUtils::GetTexture2DSourceImage(TextureMap, Image) );

		FImage LinearImage;
		Image.Linearize(LinearImage);

		if (ensure(Width == LinearImage.SizeX) && ensure(Height == LinearImage.SizeY))
		{
			const FImageDimensions Dimensions = FImageDimensions(Width, Height);
			DestImage.SetDimensions(Dimensions);

			TArrayView64<FLinearColor> ImageData = Image.AsRGBA32F();
			for (int64 LinearIndex = 0; LinearIndex < Dimensions.Num(); ++LinearIndex)
			{
				DestImage.SetPixel(Dimensions.GetCoords(LinearIndex), ToVector4<float>(ImageData[LinearIndex]));
			}

			return true;
		}

		return false;
	}

	FTextureSource& TextureSource = TextureMap->Source;

	const int32 Width = TextureSource.GetSizeX();
	const int32 Height = TextureSource.GetSizeY();
	const FImageDimensions Dimensions = FImageDimensions(Width, Height);
	DestImage.SetDimensions(Dimensions);
	const int64 Num = Dimensions.Num(); //< number of pixels

	TArray64<uint8> SourceData;
	verify( TextureMap->Source.GetMipData(SourceData, 0, 0, 0) );
	const ETextureSourceFormat SourceFormat = TextureSource.GetFormat();
	const int32 BytesPerPixel = TextureSource.GetBytesPerPixel();
	const uint8* SourceDataPtr = SourceData.GetData();
		
	// code below is derived from UBlueprintMaterialTextureNodesBPLibrary::Texture2D_SampleUV_EditorOnly()
	if (SourceFormat == TSF_BGRA8)
	{
		check(BytesPerPixel == sizeof(FColor));
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			FColor PixelColor = *((FColor*)PixelPtr);

			FLinearColor FloatColor = (TextureMap->SRGB) ?
				FLinearColor::FromSRGBColor(PixelColor) :
				PixelColor.ReinterpretAsLinear();

			DestImage.SetPixel(i, ToVector4<float>(FloatColor));
		}
		return true;
	}
	else if (SourceFormat == TSF_BGRE8)
	{
		check(BytesPerPixel == sizeof(FColor));
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			FColor PixelColor = *((FColor*)PixelPtr);

			FLinearColor FloatColor = PixelColor.FromRGBE();

			DestImage.SetPixel(i, ToVector4<float>(FloatColor));
		}
		return true;
	}
	// code below is also derived from FImage::CopyTo (CopyImage) - invoked during BuildTexture on import.
	else if (SourceFormat == TSF_RGBA16)
	{
		check(BytesPerPixel == sizeof(uint16) * 4);
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* SourcePtr = SourceDataPtr + (i * BytesPerPixel);
			const uint16* PixelPtr = (const uint16*)SourcePtr;
			DestImage.SetPixel(i, FVector4f(
				PixelPtr[0] / 65535.0f,
				PixelPtr[1] / 65535.0f,
				PixelPtr[2] / 65535.0f,
				PixelPtr[3] / 65535.0f
				));
		}
		return true;
	}
	else if (SourceFormat == TSF_RGBA16F)
	{
		check(BytesPerPixel == sizeof(FFloat16Color));
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			FLinearColor LinearColor = ((const FFloat16Color*)PixelPtr)->GetFloats();
			DestImage.SetPixel(i, ToVector4<float>(LinearColor) );
		}
		return true;
	}
	else if (SourceFormat == TSF_RGBA32F)
	{
		check(BytesPerPixel == sizeof(FVector4f));
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			DestImage.SetPixel(i, *reinterpret_cast<const FVector4f*>(PixelPtr));
		}
		return true;
	}
	else if (SourceFormat == TSF_R16F)
	{
		check(BytesPerPixel == sizeof(FFloat16));
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			const float Value = reinterpret_cast<const FFloat16*>(PixelPtr)->GetFloat();
			DestImage.SetPixel(i, FVector4f(Value, 0.0f, 0.0f, 1.f));
		}
		return true;
	}
	else if (SourceFormat == TSF_R32F)
	{
		check(BytesPerPixel == sizeof(float));
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			const float Value = *reinterpret_cast<const float*>(PixelPtr);
			DestImage.SetPixel(i, FVector4f(Value, 0.0f, 0.0f, 1.f));
		}
		return true;
	}
	else if (SourceFormat == TSF_G16)
	{
		check(BytesPerPixel == 2);
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			const uint16 ByteColor = *((const uint16*)PixelPtr);
			const float FloatColor = float(ByteColor) / 65535.0f;
			DestImage.SetPixel(i, FVector4f(FloatColor, FloatColor, FloatColor, 1.0));
		}
		return true;
	}
	else if (SourceFormat == TSF_G8)
	{
		check(BytesPerPixel == 1);
		for (int64 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			const uint8 PixelColor = *PixelPtr;
			const float PixelColorf = float(PixelColor) / 255.0f;
			FLinearColor FloatColor = (TextureMap->SRGB) ?
				FLinearColor::FromSRGBColor(FColor(PixelColor, PixelColor, PixelColor, 255)) :
				FLinearColor(PixelColorf, PixelColorf, PixelColorf, 1.0);
			DestImage.SetPixel(i, ToVector4<float>(FloatColor));
		}
		return true;
	}

	ensureAlwaysMsgf(false, TEXT("Texture source format not handled."));

	return false;
}
#endif 


bool UE::AssetUtils::ReadTexture(
	UTexture2D* TextureMap,
	TImageBuilder<FVector4f>& DestImageOut,
	const bool bPreferPlatformData)
{
	if (ensure(TextureMap) == false) return false;

#if WITH_EDITOR
	FTextureCompilingManager::Get().FinishCompilation({ TextureMap });

	const bool bPlatformDataHasMips = TextureMap->GetPlatformData()->Mips.Num() != 0;

	if (TextureMap->Source.IsValid() && (bPreferPlatformData == false || bPlatformDataHasMips == false))
	{
		return ReadTexture_SourceData(TextureMap, DestImageOut);
	}
#endif

	return ReadTexture_PlatformData(TextureMap, DestImageOut);
}



bool UE::AssetUtils::ConvertToSingleChannel(UTexture2D* TextureMap)
{
	if (ensure(TextureMap) == false) return false;

#if WITH_EDITOR
	bool bHasVTData = TextureMap->GetPlatformData()->VTData != nullptr;
	bool bHasMips = TextureMap->GetPlatformData()->Mips.Num() != 0;
	ensure(bHasVTData != bHasMips);		// should be one or the other

	if (ensure(TextureMap->Source.IsValid()) && bHasVTData == false)
	{
		FTextureSource& TextureSource = TextureMap->Source;
		int32 Width = TextureSource.GetSizeX();
		int32 Height = TextureSource.GetSizeY();
		int64 Num = Width * Height;
		ETextureSourceFormat SourceFormat = TextureSource.GetFormat();
		if (SourceFormat == TSF_G8)
		{
			return true;		// already single channel
		}
		if (SourceFormat != TSF_BGRA8 && SourceFormat != TSF_BGRE8)
		{
			ensureMsgf(false, TEXT("ConvertToSingleChannel currently only supports RGBA8 textures"));
			return false;
		}

		TArray64<uint8> NewSourceData;
		NewSourceData.SetNum(Width * Height);

		TArray64<uint8> SourceData;
		verify( TextureMap->Source.GetMipData(SourceData, 0, 0, 0) );
		int32 BytesPerPixel = TextureSource.GetBytesPerPixel();
		check(BytesPerPixel == sizeof(FColor));
		const uint8* SourceDataPtr = SourceData.GetData();
		for (int32 i = 0; i < Num; ++i)
		{
			const uint8* PixelPtr = SourceDataPtr + (i * BytesPerPixel);
			FColor PixelColor = *((FColor*)PixelPtr);
			NewSourceData[i] = PixelColor.R;
		}

		TextureSource.Init(Width, Height, 1, 1, TSF_G8, &NewSourceData[0]);

		TextureMap->UpdateResource();

		return true;
	}
	return false;
#else

	ensureMsgf(false, TEXT("ConvertToSingleChannel currently requires editor-only SourceData"));
	return false;
#endif

}


bool UE::AssetUtils::ForceVirtualTexturePrefetch(FImageDimensions ScreenSpaceDimensions, bool bWaitForPrefetchToComplete)
{
	// Prefetch all virtual textures so that we have content available
	if (UseVirtualTexturing(GMaxRHIShaderPlatform))
	{
		const FVector2D ScreenSpaceSize(ScreenSpaceDimensions.GetWidth(), ScreenSpaceDimensions.GetHeight());

		UE::RenderCommandPipe::FSyncScope SyncScope;

		ENQUEUE_RENDER_COMMAND(AssetUtils_ForceVirtualTexturePrefetch)(
			[ScreenSpaceSize](FRHICommandListImmediate& RHICmdList)
		{
			GetRendererModule().RequestVirtualTextureTiles(ScreenSpaceSize, -1);
			GetRendererModule().LoadPendingVirtualTextureTiles(RHICmdList, GMaxRHIFeatureLevel);
		});

		if (bWaitForPrefetchToComplete)
		{
			FlushRenderingCommands();
		}

		return true;
	}

	return false;
}



bool UE::AssetUtils::SaveDebugImage(
	const TArray<FColor>& Pixels,
	FImageDimensions Dimensions,
	FString DebugSubfolder,
	FString FilenameBase,
	int32 UseFileCounter)
{
	static int32 CaptureIndex = 0;
#if WITH_EDITOR
	// Save capture result to a file to ease debugging
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetUtils_SaveDebugImage);

	FString DirectoryPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir());
	if (DebugSubfolder.Len() > 0)
	{
		DirectoryPath = FPaths::Combine(DirectoryPath, DebugSubfolder);
	}
	int32 FileCounter = (UseFileCounter > 0) ? UseFileCounter : CaptureIndex++;
	FString Filename = FString::Printf(TEXT("%s-%04d.bmp"), *FilenameBase, FileCounter);
	FString FilePath = FPaths::Combine(DirectoryPath, Filename);

	return FFileHelper::CreateBitmap(*FilePath, Dimensions.GetWidth(), Dimensions.GetHeight(), Pixels.GetData());
#else
	return false;
#endif
}



bool UE::AssetUtils::SaveDebugImage(
	const TArray<FLinearColor>& Pixels,
	FImageDimensions Dimensions,
	bool bConvertToSRGB,
	FString DebugSubfolder,
	FString FilenameBase,
	int32 UseFileCounter )
{
	static int32 CaptureIndex = 0;
#if WITH_EDITOR
	// Save capture result to a file to ease debugging
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetUtils_SaveDebugImage);

	FString DirectoryPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir()); 
	if (DebugSubfolder.Len() > 0)
	{
		DirectoryPath = FPaths::Combine(DirectoryPath, DebugSubfolder);
	}
	int32 FileCounter = (UseFileCounter > 0) ? UseFileCounter : CaptureIndex++;
	FString Filename = FString::Printf(TEXT("%s-%04d.bmp"), *FilenameBase, FileCounter);
	FString FilePath = FPaths::Combine(DirectoryPath, Filename);

	TArray<FColor> ConvertedColor;
	ConvertedColor.Reserve(Pixels.Num());
	for (const FLinearColor& LinearColor : Pixels)
	{
		ConvertedColor.Add(LinearColor.ToFColor(bConvertToSRGB));
	}

	return FFileHelper::CreateBitmap(*FilePath, Dimensions.GetWidth(), Dimensions.GetHeight(), ConvertedColor.GetData());

#else
	return false;
#endif
}



bool UE::AssetUtils::SaveDebugImage(
	const FImageAdapter& Image,
	bool bConvertToSRGB,
	FString DebugSubfolder,
	FString FilenameBase,
	int32 UseFileCounter)
{
	static int32 CaptureIndex = 0;
#if WITH_EDITOR
	// Save capture result to a file to ease debugging
	TRACE_CPUPROFILER_EVENT_SCOPE(AssetUtils_SaveDebugImage);

	FString DirectoryPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectIntermediateDir());
	if (DebugSubfolder.Len() > 0)
	{
		DirectoryPath = FPaths::Combine(DirectoryPath, DebugSubfolder);
	}
	int32 FileCounter = (UseFileCounter > 0) ? UseFileCounter : CaptureIndex++;
	FString Filename = FString::Printf(TEXT("%s-%04d.bmp"), *FilenameBase, FileCounter);
	FString FilePath = FPaths::Combine(DirectoryPath, Filename);

	FImageDimensions Dimensions = Image.GetDimensions();
	int64 N = Dimensions.Num();
	TArray<FColor> ConvertedColor;
	ConvertedColor.Reserve(N);
	for ( int64 i = 0; i < N; ++i )
	{
		FLinearColor LinearColor = ToLinearColor(Image.GetPixel(i));
		ConvertedColor.Add(LinearColor.ToFColor(bConvertToSRGB));
	}

	return FFileHelper::CreateBitmap(*FilePath, Dimensions.GetWidth(), Dimensions.GetHeight(), ConvertedColor.GetData());
#else
	return false;
#endif
}