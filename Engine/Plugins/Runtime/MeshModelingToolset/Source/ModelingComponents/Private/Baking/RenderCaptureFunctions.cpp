// Copyright Epic Games, Inc. All Rights Reserved.

#include "Baking/RenderCaptureFunctions.h"
#include "Scene/SceneCapturePhotoSet.h"
#include "Sampling/MeshMapBaker.h"
#include "Sampling/RenderCaptureMapEvaluator.h"
#include "AssetUtils/Texture2DBuilder.h"

#include "Engine/Texture2D.h"
#include "GameFramework/Actor.h"
#include "Algo/NoneOf.h"
#include "Algo/AnyOf.h"
#include "Misc/ScopedSlowTask.h"


using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "RenderCaptureFunctions"



FRenderCaptureOcclusionHandler::FRenderCaptureOcclusionHandler(FImageDimensions Dimensions)
{
	SampleStats.SetDimensions(Dimensions);
	SampleStats.Clear({});
}

void FRenderCaptureOcclusionHandler::RegisterSample(const FVector2i& ImageCoords, bool bSampleValid)
{
	checkSlow(SampleStats.GetDimensions().IsValidCoords(ImageCoords));
	if (bSampleValid)
	{
		SampleStats.GetPixel(ImageCoords).NumValid += 1;
	}
	else
	{
		SampleStats.GetPixel(ImageCoords).NumInvalid += 1;
	}
}

void FRenderCaptureOcclusionHandler::PushInfillRequired(bool bInfillRequired)
{
	InfillRequired.Add(bInfillRequired);
}

void FRenderCaptureOcclusionHandler::ComputeAndApplyInfill(TArray<TUniquePtr<TImageBuilder<FVector4f>>>& Images)
{
	check(Images.Num() == InfillRequired.Num());

	if (Images.IsEmpty() || Algo::NoneOf(InfillRequired))
	{
		return;
	}

	ComputeInfill();

	for (int ImageIndex = 0; ImageIndex < Images.Num(); ImageIndex++)
	{
		if (InfillRequired[ImageIndex])
		{
			ApplyInfill(*Images[ImageIndex]);
		}
	}
}

void FRenderCaptureOcclusionHandler::ComputeInfill()
{
	// Find pixels that need infill
	TArray<FVector2i> MissingPixels;
	FCriticalSection MissingPixelsLock;
	ParallelFor(SampleStats.GetDimensions().GetHeight(), [this, &MissingPixels, &MissingPixelsLock](int32 Y)
	{
		for (int32 X = 0; X < SampleStats.GetDimensions().GetWidth(); X++)
		{
			const FSampleStats& Stats = SampleStats.GetPixel(X, Y);
			// TODO experiment with other classifications
			if (Stats.NumInvalid > 0 && Stats.NumValid == 0)
			{
				MissingPixelsLock.Lock();
				MissingPixels.Add(FVector2i(X, Y));
				MissingPixelsLock.Unlock();
			}
		}
	});
	
	auto DummyNormalizeStatsFunc = [](FSampleStats SumValue, int32 Count)
	{
		// The return value must be different from MissingValue below so that ComputeInfill works correctly
		return FSampleStats{TNumericLimits<uint16>::Max(), TNumericLimits<uint16>::Max()};
	};

	// This must be the same as the value of exterior pixels, otherwise infill will spread the exterior values into the texture
	FSampleStats MissingValue{0, 0};
	Infill.ComputeInfill(SampleStats, MissingPixels, MissingValue, DummyNormalizeStatsFunc);
}

void FRenderCaptureOcclusionHandler::ApplyInfill(TImageBuilder<FVector4f>& Image) const
{
	auto NormalizeFunc = [](FVector4f SumValue, int32 Count)
	{
		float InvSum = (Count == 0) ? 1.0f : (1.0f / Count);
		return FVector4f(SumValue.X * InvSum, SumValue.Y * InvSum, SumValue.Z * InvSum, 1.0f);
	};

	Infill.ApplyInfill<FVector4f>(Image, NormalizeFunc);
}

bool FRenderCaptureOcclusionHandler::FSampleStats::operator==(const FSampleStats& Other) const
{
	return (NumValid == Other.NumValid) && (NumInvalid == Other.NumInvalid);
}

bool FRenderCaptureOcclusionHandler::FSampleStats::operator!=(const FSampleStats& Other) const
{
	return !(*this == Other);
}

FRenderCaptureOcclusionHandler::FSampleStats& FRenderCaptureOcclusionHandler::FSampleStats::operator+=(const FSampleStats& Other)
{
	NumValid += Other.NumValid;
	NumInvalid += Other.NumInvalid;
	return *this;
}

FRenderCaptureOcclusionHandler::FSampleStats FRenderCaptureOcclusionHandler::FSampleStats::Zero()
{
	return FSampleStats{0, 0};
}








FSceneCapturePhotoSetSampler::FSceneCapturePhotoSetSampler(
		FSceneCapturePhotoSet* SceneCapture,
		float ValidSampleDepthThreshold,
		const FDynamicMesh3* Mesh,
		const FDynamicMeshAABBTree3* Spatial,
		const FMeshTangentsd* Tangents) :
	FMeshBakerDynamicMeshSampler(Mesh, Spatial, Tangents),
	SceneCapture(SceneCapture),
	ValidSampleDepthThreshold(ValidSampleDepthThreshold)
{
	check(SceneCapture != nullptr);
	check(Mesh != nullptr);
	check(Spatial != nullptr);
	check(Tangents != nullptr);

	bool bHasDepth = SceneCapture->GetCaptureTypeStatus(ERenderCaptureType::DeviceDepth) == FSceneCapturePhotoSet::ECaptureTypeStatus::Computed;
	if (bHasDepth)
	{
		ensure(ValidSampleDepthThreshold > 0); // We only need the depth capture if this threshold is positive
	}

	// Compute an offset shift surface positions toward the render camera position so we don't 	immediately self intersect
	// The Max expression ensures the code works when the base mesh is 2D along one coordinate direction
	const double RayOffsetHackDist((double)(100.0 * FMathf::ZeroTolerance * FMath::Max(Mesh->GetBounds().MinDim(), 0.01)));

	VisibilityFunction = [this, RayOffsetHackDist](const FVector3d& SurfPos, const FVector3d& ViewPos)
	{
		FVector3d RayDir = ViewPos - SurfPos;
		double Dist = Normalize(RayDir);
		FVector3d RayOrigin = SurfPos + RayOffsetHackDist * RayDir;
		int32 HitTID = DetailSpatial->FindNearestHitTriangle(FRay3d(RayOrigin, RayDir), IMeshSpatial::FQueryOptions(Dist));
		return (HitTID == IndexConstants::InvalidID);
	};
}

bool FSceneCapturePhotoSetSampler::SupportsCustomCorrespondence() const
{
	return true;
}

void* FSceneCapturePhotoSetSampler::ComputeCustomCorrespondence(const FMeshUVSampleInfo& SampleInfo, FMeshMapEvaluator::FCorrespondenceSample& Sample) const
{
	
	// Perform a ray-cast to determine which photo/coordinate, if any, should be sampled
	int PhotoIndex;
	FVector2d PhotoCoords;
	SceneCapture->ComputeSampleLocation(
		Sample.BaseSample.SurfacePoint,
		Sample.BaseNormal,
		ValidSampleDepthThreshold,
		VisibilityFunction,
		PhotoIndex,
		PhotoCoords);

	// Store the photo coordinates and index in the correspondence sample
	Sample.DetailMesh = SceneCapture;
	Sample.DetailTriID = PhotoIndex;
	Sample.DetailBaryCoords.X = PhotoCoords.X;
	Sample.DetailBaryCoords.Y = PhotoCoords.Y;

	// This will be set to Sample.DetailMesh but we can already do that internally so it's kindof redundant
	return SceneCapture;
}

bool FSceneCapturePhotoSetSampler::IsValidCorrespondence(const FMeshMapEvaluator::FCorrespondenceSample& Sample) const
{
	return Sample.DetailTriID != IndexConstants::InvalidID;
}



bool FSceneCaptureConfig::operator==(const FSceneCaptureConfig& Other) const
{
	return Flags == Other.Flags
		&& RenderCaptureImageSize == Other.RenderCaptureImageSize
		&& bAntiAliasing == Other.bAntiAliasing
		&& FieldOfViewDegrees == Other.FieldOfViewDegrees
		&& NearPlaneDist == Other.NearPlaneDist;
}

bool FSceneCaptureConfig::operator!=(const FSceneCaptureConfig& Other) const
{
	return !this->operator==(Other);
}






void UE::Geometry::ConfigureSceneCapture(
	FSceneCapturePhotoSet& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& Options,
	bool bAllowCancel)
{
	ForEachCaptureType([&SceneCapture, &Options](ERenderCaptureType CaptureType)
	{
		const bool bCaptureTypeEnabled = Options.Flags[CaptureType];
		SceneCapture.SetCaptureTypeEnabled(CaptureType, bCaptureTypeEnabled);

		FRenderCaptureConfig Config;
		Config.bAntiAliasing = (CaptureType == ERenderCaptureType::DeviceDepth ? false : Options.bAntiAliasing);
		SceneCapture.SetCaptureConfig(CaptureType, Config);
	});

	UWorld* World = Actors.IsEmpty() ? nullptr : Actors[0]->GetWorld();

	SceneCapture.SetCaptureSceneActors(World, Actors);

	const TArray<FSpatialPhotoParams> SpatialParams = ComputeStandardExteriorSpatialPhotoParameters(
		World,
		Actors,
		TArray<UActorComponent*>(),
		FImageDimensions(Options.RenderCaptureImageSize, Options.RenderCaptureImageSize),
		Options.FieldOfViewDegrees,
		Options.NearPlaneDist,
		true, true, true, true, true);

	SceneCapture.SetSpatialPhotoParams(SpatialParams);

	SceneCapture.SetAllowCancel(bAllowCancel);
}



UE::Geometry::FRenderCaptureTypeFlags UE::Geometry::UpdateSceneCapture(
	FSceneCapturePhotoSet& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& DesiredConfig,
	bool bAllowCancel)
{
	const FSceneCapturePhotoSet::FStatus PreConfigStatus = SceneCapture.GetSceneCaptureStatus();
	ConfigureSceneCapture(SceneCapture, Actors, DesiredConfig, true);
	const FSceneCapturePhotoSet::FStatus PreComputeStatus = SceneCapture.GetSceneCaptureStatus();
	SceneCapture.Compute();
	const FSceneCapturePhotoSet::FStatus PostComputeStatus = SceneCapture.GetSceneCaptureStatus();

	const FSceneCaptureConfig AchievedConfig = GetSceneCaptureConfig(SceneCapture);

	// DesiredConfig and AchievedConfig mismatch iff the SceneCapture was cancelled. Also, the mismatch should only be in the Flags member
	ensure(SceneCapture.Cancelled() == (AchievedConfig != DesiredConfig));
	ensure(AchievedConfig.RenderCaptureImageSize == DesiredConfig.RenderCaptureImageSize);
	ensure(AchievedConfig.FieldOfViewDegrees == DesiredConfig.FieldOfViewDegrees);
	ensure(AchievedConfig.bAntiAliasing == DesiredConfig.bAntiAliasing);
	ensure(AchievedConfig.NearPlaneDist == DesiredConfig.NearPlaneDist);

	// Deduce which captures were changed in this scene capture update
	FRenderCaptureTypeFlags UpdatedCaptures;
	ForEachCaptureType([&UpdatedCaptures, &PreConfigStatus, &PreComputeStatus, &PostComputeStatus] (ERenderCaptureType CaptureType)
	{
		using EStatus = FSceneCapturePhotoSet::ECaptureTypeStatus;

		// This boolean is true in two scenarios:
		// 1. Computed -> Disabled occurs when the CaptureType was disabled by the DesiredConfig
		// 2. Computed -> Pending occurs when the SceneCapture was cancelled while/before the CaptureType was computed
		bool bCleared =  PreConfigStatus[CaptureType]  == EStatus::Computed && PostComputeStatus[CaptureType] != EStatus::Computed;

		// This boolean is true in two scenarios, in both cases the PreComputeStatus of the given CaptureType is Pending
		// 1. DesiredConfig enabled a new CaptureType
		// 2. DesiredConfig requires an already computed CaptureType to be recomputed
		bool bComputed = PreComputeStatus[CaptureType] == EStatus::Pending  && PostComputeStatus[CaptureType] == EStatus::Computed;

		UpdatedCaptures[CaptureType] = bCleared || bComputed;
	});

	return UpdatedCaptures;
}




PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<FSceneCapturePhotoSet> UE::Geometry::CapturePhotoSet(
	const TArray<TObjectPtr<AActor>>& Actors,
	const FRenderCaptureOptions& Options,
	FRenderCaptureUpdate& Update,
	bool bAllowCancel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CapturePhotoSet);

	TUniquePtr<FSceneCapturePhotoSet> SceneCapture = MakeUnique<FSceneCapturePhotoSet>();

	Update = UpdatePhotoSets(SceneCapture, Actors, Options, bAllowCancel);

	return SceneCapture;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FRenderCaptureUpdate UE::Geometry::UpdatePhotoSets(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FRenderCaptureOptions& Options,
	bool bAllowCancel)
{
	FSceneCaptureConfig Config;
	Config.RenderCaptureImageSize = Options.RenderCaptureImageSize;
	Config.bAntiAliasing          = Options.bAntiAliasing;
	Config.FieldOfViewDegrees     = Options.FieldOfViewDegrees;
	Config.NearPlaneDist          = Options.NearPlaneDist;
	Config.Flags.bBaseColor       = Options.bBakeBaseColor;
	Config.Flags.bRoughness       = Options.bBakeRoughness;
	Config.Flags.bMetallic        = Options.bBakeMetallic;
	Config.Flags.bSpecular        = Options.bBakeSpecular;
	Config.Flags.bEmissive        = Options.bBakeEmissive;
	Config.Flags.bWorldNormal     = Options.bBakeNormalMap;
	Config.Flags.bOpacity         = Options.bBakeOpacity;
	Config.Flags.bSubsurfaceColor = Options.bBakeSubsurfaceColor;
	Config.Flags.bDeviceDepth     = Options.bBakeDeviceDepth;
	Config.Flags.bCombinedMRS     = Options.bUsePackedMRS;

	FRenderCaptureTypeFlags UpdatedCaptures = UpdateSceneCapture(SceneCapture, Actors, Config, bAllowCancel);

	FRenderCaptureUpdate Updated;
	Updated.bUpdatedBaseColor   = UpdatedCaptures.bBaseColor;
	Updated.bUpdatedRoughness   = UpdatedCaptures.bRoughness;
	Updated.bUpdatedSpecular    = UpdatedCaptures.bSpecular;
	Updated.bUpdatedMetallic    = UpdatedCaptures.bMetallic;
	Updated.bUpdatedPackedMRS   = UpdatedCaptures.bCombinedMRS;
	Updated.bUpdatedNormalMap   = UpdatedCaptures.bWorldNormal;
	Updated.bUpdatedEmissive    = UpdatedCaptures.bEmissive;
	Updated.bUpdatedOpacity     = UpdatedCaptures.bOpacity;
	Updated.bUpdatedDeviceDepth = UpdatedCaptures.bDeviceDepth;
	Updated.bUpdatedSubsurfaceColor = UpdatedCaptures.bSubsurfaceColor;

	return Updated;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Return the SceneCapture configuration, the Flags in the returned struct are set if the corresponding capture type is computed
FSceneCaptureConfig UE::Geometry::GetSceneCaptureConfig(
	const FSceneCapturePhotoSet& SceneCapture,
	const FSceneCapturePhotoSet::ECaptureTypeStatus QueryStatus)
{
	FSceneCaptureConfig Result;

	ForEachCaptureType([&SceneCapture, &QueryStatus, &Result](ERenderCaptureType CaptureType)
	{
		Result.Flags[CaptureType] = (SceneCapture.GetCaptureTypeStatus(CaptureType) == QueryStatus);
	});

	const TArray<FSpatialPhotoParams> SpatialParams = SceneCapture.GetSpatialPhotoParams();

	if (!SpatialParams.IsEmpty())
	{
		ensure(SpatialParams[0].Dimensions.IsSquare());
		Result.RenderCaptureImageSize = SpatialParams[0].Dimensions.GetWidth();
		Result.FieldOfViewDegrees = SpatialParams[0].HorzFOVDegrees;
		Result.NearPlaneDist = SpatialParams[0].NearPlaneDist;

		const bool Values[] = {
			SceneCapture.GetCaptureConfig(ERenderCaptureType::BaseColor).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::WorldNormal).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::Metallic).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::Roughness).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::Specular).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::CombinedMRS).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::Emissive).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::Opacity).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::SubsurfaceColor).bAntiAliasing,
			SceneCapture.GetCaptureConfig(ERenderCaptureType::DeviceDepth).bAntiAliasing,
		};
		Result.bAntiAliasing = Algo::AnyOf(Values);
	}

	return Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FRenderCaptureOptions UE::Geometry::GetComputedPhotoSetOptions(const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture)
{
	FSceneCaptureConfig Config = GetSceneCaptureConfig(SceneCapture);

	FRenderCaptureOptions Options;

	Options.RenderCaptureImageSize = Config.RenderCaptureImageSize;
	Options.bAntiAliasing          = Config.bAntiAliasing;
	Options.FieldOfViewDegrees     = Config.FieldOfViewDegrees;
	Options.NearPlaneDist          = Config.NearPlaneDist;
	Options.bBakeBaseColor         = Config.Flags.bBaseColor;
	Options.bBakeRoughness         = Config.Flags.bRoughness;
	Options.bBakeMetallic          = Config.Flags.bMetallic;
	Options.bBakeSpecular          = Config.Flags.bSpecular;
	Options.bBakeEmissive          = Config.Flags.bEmissive;
	Options.bBakeNormalMap         = Config.Flags.bWorldNormal;
	Options.bBakeOpacity           = Config.Flags.bOpacity;
	Options.bBakeSubsurfaceColor   = Config.Flags.bSubsurfaceColor;
	Options.bBakeDeviceDepth       = Config.Flags.bDeviceDepth;
	Options.bUsePackedMRS          = Config.Flags.bCombinedMRS;

	return Options;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS



template <ERenderCaptureType CaptureType>
TSharedPtr<FRenderCaptureMapEvaluator<FVector4f>>
MakeColorEvaluator(
	const FSceneCapturePhotoSet::FSceneSample& DefaultSample,
	const FSceneCapturePhotoSet* SceneCapture)
{
	// Bake will crash if we haven't computed the photo sets corresponding to the CaptureType 
	check(SceneCapture->GetCaptureTypeStatus(CaptureType) == FSceneCapturePhotoSet::ECaptureTypeStatus::Computed);

	TSharedPtr<FRenderCaptureMapEvaluator<FVector4f>> Evaluator = MakeShared<FRenderCaptureMapEvaluator<FVector4f>>();

	switch (CaptureType) {
	case ERenderCaptureType::BaseColor:
		Evaluator->Channel = ERenderCaptureChannel::BaseColor;
		break;
	case ERenderCaptureType::Roughness:
		Evaluator->Channel = ERenderCaptureChannel::Roughness;
		break;
	case ERenderCaptureType::Metallic:
		Evaluator->Channel = ERenderCaptureChannel::Metallic;
		break;
	case ERenderCaptureType::Specular:
		Evaluator->Channel = ERenderCaptureChannel::Specular;
		break;
	case ERenderCaptureType::Emissive:
		Evaluator->Channel = ERenderCaptureChannel::Emissive;
		break;
	case ERenderCaptureType::WorldNormal:
		Evaluator->Channel = ERenderCaptureChannel::WorldNormal;
		break;
	case ERenderCaptureType::CombinedMRS:
		Evaluator->Channel = ERenderCaptureChannel::CombinedMRS;
		break;
	case ERenderCaptureType::Opacity:
		Evaluator->Channel = ERenderCaptureChannel::Opacity;
		break;
	case ERenderCaptureType::SubsurfaceColor:
		Evaluator->Channel = ERenderCaptureChannel::SubsurfaceColor;
		break;
	case ERenderCaptureType::DeviceDepth:
		Evaluator->Channel = ERenderCaptureChannel::DeviceDepth;
		break;
	}

	Evaluator->DefaultResult = DefaultSample.GetValue4f(CaptureType);

	Evaluator->EvaluateSampleCallback = [DefaultSample, SceneCapture](const FMeshMapEvaluator::FCorrespondenceSample& Sample)
	{
		const int PhotoIndex = Sample.DetailTriID;
		const FVector2d PhotoCoords(Sample.DetailBaryCoords.X, Sample.DetailBaryCoords.Y);
		const FVector4f SampleColor = SceneCapture->ComputeSampleNearest<CaptureType>(PhotoIndex, PhotoCoords, DefaultSample);
		return SampleColor;
	};

	Evaluator->EvaluateColorCallback = [](const int DataIdx, float*& In)
	{
		const FVector4f Out(In[0], In[1], In[2], In[3]);
		In += 4;
		return Out;
	};

	return Evaluator;
}


TUniquePtr<FMeshMapBaker> UE::Geometry::MakeRenderCaptureBaker(
	FDynamicMesh3* BaseMesh,
	TSharedPtr<UE::Geometry::FMeshTangentsd, ESPMode::ThreadSafe> BaseMeshTangents,
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> BaseMeshUVCharts,
	FSceneCapturePhotoSet* SceneCapture,
	FSceneCapturePhotoSetSampler* Sampler,
	FRenderCaptureTypeFlags PendingBake,
	int32 TargetUVLayer,
	EBakeTextureResolution TextureImageSize,
	EBakeTextureSamplesPerPixel SamplesPerPixel,
	FRenderCaptureOcclusionHandler* OcclusionHandler)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MakeRenderCaptureBaker);

	check(BaseMesh != nullptr);
	check(BaseMeshTangents.IsValid());
	check(BaseMeshUVCharts.IsValid());
	check(SceneCapture != nullptr);
	check(Sampler != nullptr);
	check(OcclusionHandler != nullptr);

	auto RegisterSampleStats = [OcclusionHandler](bool bSampleValid, const FMeshMapEvaluator::FCorrespondenceSample& Sample, const FVector2d& UVPosition, const FVector2i& ImageCoords)
	{
		OcclusionHandler->RegisterSample(ImageCoords, bSampleValid);
	};

	auto ComputeAndApplyInfill = [OcclusionHandler](TArray<TUniquePtr<TImageBuilder<FVector4f>>>& BakeResults)
	{
		OcclusionHandler->ComputeAndApplyInfill(BakeResults);
	};

	TUniquePtr<FMeshMapBaker> Result = MakeUnique<FMeshMapBaker>();
	Result->SetTargetMesh(BaseMesh);
	Result->SetTargetMeshTangents(BaseMeshTangents);
	Result->SetTargetMeshUVCharts(BaseMeshUVCharts.Get());
	Result->SetDimensions(FImageDimensions(static_cast<int32>(TextureImageSize), static_cast<int32>(TextureImageSize)));
	Result->SetSamplesPerPixel(static_cast<int32>(SamplesPerPixel));
	Result->SetFilter(FMeshMapBaker::EBakeFilterType::BSpline);
	Result->SetTargetMeshUVLayer(TargetUVLayer);
	Result->InteriorSampleCallback = RegisterSampleStats;
	Result->PostWriteToImageCallback = ComputeAndApplyInfill;
	Result->SetDetailSampler(Sampler);
	Result->SetCorrespondenceStrategy(FMeshMapBaker::ECorrespondenceStrategy::Custom);

	// Pixels in the output textures which don't map onto the mesh have a light grey color (except the normal map which
	// will show a color corresponding to a unit z tangent space normal)
	const FVector4f InvalidColor(.42f, .42f, .42f, 1);
	const FVector3f DefaultNormal = FVector3f::UnitZ();

	FSceneCapturePhotoSet::FSceneSample DefaultColorSample;
	DefaultColorSample.BaseColor = FVector3f(InvalidColor.X, InvalidColor.Y, InvalidColor.Z);
	DefaultColorSample.Roughness = InvalidColor.X;
	DefaultColorSample.Specular = InvalidColor.X;
	DefaultColorSample.Metallic = InvalidColor.X;
	DefaultColorSample.Emissive = FVector3f(InvalidColor.X, InvalidColor.Y, InvalidColor.Z);
	DefaultColorSample.Opacity = InvalidColor.X;
	DefaultColorSample.SubsurfaceColor = FVector3f(InvalidColor.X, InvalidColor.Y, InvalidColor.Z);
	DefaultColorSample.WorldNormal = FVector4f((DefaultNormal + FVector3f::One()) * .5f, InvalidColor.W);
	DefaultColorSample.DeviceDepth = 0; // We use 0 here since this corresponds to the infinite far plane value. Also, we don't preview the depth texture

	auto AddColorEvaluator = [&Result, OcclusionHandler] (const TSharedPtr<FRenderCaptureMapEvaluator<FVector4f>>& Evaluator)
	{
		Result->AddEvaluator(Evaluator);
		OcclusionHandler->PushInfillRequired(true);
	};

	if (PendingBake.bDeviceDepth)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::DeviceDepth>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bBaseColor)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::BaseColor>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bCombinedMRS)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::CombinedMRS>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bRoughness)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::Roughness>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bMetallic)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::Metallic>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bSpecular)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::Specular>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bEmissive)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::Emissive>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bOpacity)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::Opacity>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bSubsurfaceColor)
	{
		AddColorEvaluator(MakeColorEvaluator<ERenderCaptureType::SubsurfaceColor>(DefaultColorSample, SceneCapture));
	}
	if (PendingBake.bWorldNormal)
	{
		// Bake will crash if we haven't computed the WorldNormal photo sets
		check(SceneCapture->GetCaptureTypeStatus(ERenderCaptureType::WorldNormal) == FSceneCapturePhotoSet::ECaptureTypeStatus::Computed);

		TSharedPtr<FRenderCaptureMapEvaluator<FVector3f>> Evaluator = MakeShared<FRenderCaptureMapEvaluator<FVector3f>>();

		Evaluator->Channel = ERenderCaptureChannel::WorldNormal;

		Evaluator->DefaultResult = DefaultNormal;

		Evaluator->EvaluateSampleCallback = [SceneCapture, BaseMeshTangents, DefaultColorSample](const FMeshMapEvaluator::FCorrespondenceSample& Sample)
		{
			const int32 TriangleID = Sample.BaseSample.TriangleIndex;
			const FVector3d BaryCoords = Sample.BaseSample.BaryCoords;
			const int PhotoIndex = Sample.DetailTriID;
			const FVector2d PhotoCoords(Sample.DetailBaryCoords.X, Sample.DetailBaryCoords.Y);

			const FVector4f NormalColor = SceneCapture->ComputeSampleNearest<ERenderCaptureType::WorldNormal>(PhotoIndex, PhotoCoords, DefaultColorSample);

			// Map from color components [0,1] to normal components [-1,1]
			const FVector3f WorldSpaceNormal(
				(NormalColor.X - 0.5f) * 2.0f,
				(NormalColor.Y - 0.5f) * 2.0f,
				(NormalColor.Z - 0.5f) * 2.0f);

			// Get tangents on base mesh
			FVector3d BaseTangentX, BaseTangentY;
			BaseMeshTangents->GetInterpolatedTriangleTangent(TriangleID, BaryCoords, BaseTangentX, BaseTangentY);

			// Compute normal in tangent space
			const FVector3f TangentSpaceNormal(
					(float)WorldSpaceNormal.Dot(FVector3f(BaseTangentX)),
					(float)WorldSpaceNormal.Dot(FVector3f(BaseTangentY)),
					(float)WorldSpaceNormal.Dot(FVector3f(Sample.BaseNormal)));

			return TangentSpaceNormal;
		};

		Evaluator->EvaluateColorCallback = [](const int DataIdx, float*& In)
		{
			// Map normal space [-1,1] to color space [0,1]
			const FVector3f Normal(In[0], In[1], In[2]);
			const FVector3f Color = (Normal + FVector3f::One()) * 0.5f;
			const FVector4f Out(Color.X, Color.Y, Color.Z, 1.0f);
			In += 3;
			return Out;
		};

		Result->AddEvaluator(Evaluator);

		// Note: No infill on normal map for now, doesn't make sense to do after mapping to tangent space!
		//  (should we build baked normal map in world space, and then resample to tangent space??)
		OcclusionHandler->PushInfillRequired(false);
	}

	return Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<FMeshMapBaker> UE::Geometry::MakeRenderCaptureBaker(
	FDynamicMesh3* BaseMesh,
	TSharedPtr<UE::Geometry::FMeshTangentsd, ESPMode::ThreadSafe> BaseMeshTangents,
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> BaseMeshUVCharts,
	FSceneCapturePhotoSet* SceneCapture,
	FSceneCapturePhotoSetSampler* Sampler,
	FRenderCaptureOptions Options,
	int32 TargetUVLayer,
	EBakeTextureResolution TextureImageSize,
	EBakeTextureSamplesPerPixel SamplesPerPixel,
	FRenderCaptureOcclusionHandler* OcclusionHandler)
{
	FRenderCaptureTypeFlags Flags;

	Flags.bBaseColor       = Options.bBakeBaseColor;
	Flags.bRoughness       = Options.bBakeRoughness;
	Flags.bMetallic        = Options.bBakeMetallic;
	Flags.bSpecular        = Options.bBakeSpecular;
	Flags.bEmissive        = Options.bBakeEmissive;
	Flags.bWorldNormal     = Options.bBakeNormalMap;
	Flags.bOpacity         = Options.bBakeOpacity;
	Flags.bSubsurfaceColor = Options.bBakeSubsurfaceColor;
	Flags.bDeviceDepth     = Options.bBakeDeviceDepth;
	Flags.bCombinedMRS     = Options.bUsePackedMRS;

	return MakeRenderCaptureBaker(
		BaseMesh,
		BaseMeshTangents,
		BaseMeshUVCharts,
		SceneCapture,
		Sampler,
		Flags,
		TargetUVLayer,
		TextureImageSize,
		SamplesPerPixel,
		OcclusionHandler);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


void UE::Geometry::GetTexturesFromRenderCaptureBaker(FMeshMapBaker& Baker, FRenderCaptureTextures& TexturesOut)
{
	// We do this to defer work I guess, it was like this in the original ApproximateActors implementation :DeferredPopulateSourceData 
	constexpr bool bPopulateSourceData = false;

	const int32 NumEval = Baker.NumEvaluators();
	for (int32 EvalIdx = 0; EvalIdx < NumEval; ++EvalIdx)
	{
		FMeshMapEvaluator* BaseEval = Baker.GetEvaluator(EvalIdx);
		check(BaseEval->DataLayout().Num() == 1);
		switch (BaseEval->DataLayout()[0])
		{
		case FMeshMapEvaluator::EComponents::Float4:

			{
				FRenderCaptureMapEvaluator<FVector4f>* Eval = static_cast<FRenderCaptureMapEvaluator<FVector4f>*>(BaseEval);
				TUniquePtr<TImageBuilder<FVector4f>> ImageBuilder = MoveTemp(Baker.GetBakeResults(EvalIdx)[0]);
				
				if (ensure(ImageBuilder.IsValid()) == false) return;

				switch (Eval->Channel)
				{
				case ERenderCaptureChannel::BaseColor:

					TexturesOut.BaseColorMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::Color,
						true,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::SubsurfaceColor:

					// The SubsurfaceColor GBuffer channel is gamma encoded so:
					// 1. Dont convert the data in ImageBuilder to sRGB, it is already gamma encoded
					// 2. Pass the ETextureType::Color enum so the built UTexture2D will have SRGB checked
					TexturesOut.SubsurfaceColorMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::Color,
						false,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::Opacity:

					TexturesOut.OpacityMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::ColorLinear,
						false,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::Roughness:

					TexturesOut.RoughnessMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::Roughness,
						false,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::Metallic:

					TexturesOut.MetallicMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::Metallic,
						false,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::Specular:

					TexturesOut.SpecularMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::Specular,
						false,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::Emissive:

					TexturesOut.EmissiveMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::EmissiveHDR,
						false,
						bPopulateSourceData);
					TexturesOut.EmissiveMap->CompressionSettings = TC_HDR_Compressed;
					break;

				case ERenderCaptureChannel::CombinedMRS:

					TexturesOut.PackedMRSMap = FTexture2DBuilder::BuildTextureFromImage(
						*ImageBuilder,
						FTexture2DBuilder::ETextureType::ColorLinear,
						false,
						bPopulateSourceData);
					break;

				case ERenderCaptureChannel::DeviceDepth:

					// Add a null pointer for this evaluator, the depth capture is used internally and not presented to the user
					// TODO Implemented this, it could be useful for applications outside BakeRC
					break;

				case ERenderCaptureChannel::WorldNormal:

					// This should be handled in the Float3 branch
					ensure(false);
					return;

				default:
					ensure(false);
					return;
				}

			} break; // Float4

		case FMeshMapEvaluator::EComponents::Float3:

			{
				FRenderCaptureMapEvaluator<FVector3f>* Eval = static_cast<FRenderCaptureMapEvaluator<FVector3f>*>(BaseEval);
				TUniquePtr<TImageBuilder<FVector4f>> ImageBuilder = MoveTemp(Baker.GetBakeResults(EvalIdx)[0]);
				
				if (ensure(ImageBuilder.IsValid()) == false) return;
				if (ensure(Eval->Channel == ERenderCaptureChannel::WorldNormal) == false) return;

				TexturesOut.NormalMap = FTexture2DBuilder::BuildTextureFromImage(
					*ImageBuilder,
					FTexture2DBuilder::ETextureType::NormalMap,
					false,
					bPopulateSourceData);

			} break; // Float3
			
		default:
			ensure(false);
			return;
		}
	}
}



////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Everything that follows is deprecated 
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UE::Geometry::ConfigureSceneCapture(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& Options,
	bool bAllowCancel)
{
	return ConfigureSceneCapture(*SceneCapture, Actors, Options, bAllowCancel);
}

UE::Geometry::FRenderCaptureTypeFlags UE::Geometry::UpdateSceneCapture(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& DesiredConfig,
	bool bAllowCancel)
{
	return UpdateSceneCapture(*SceneCapture, Actors, DesiredConfig, bAllowCancel);
}

FSceneCaptureConfig UE::Geometry::GetSceneCaptureConfig(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const FSceneCapturePhotoSet::ECaptureTypeStatus QueryStatus)
{
	return GetSceneCaptureConfig(*SceneCapture, QueryStatus);
}

void UE::Geometry::GetTexturesFromRenderCaptureBaker(const TUniquePtr<FMeshMapBaker>& Baker, FRenderCaptureTextures& TexturesOut)
{
	GetTexturesFromRenderCaptureBaker(*Baker, TexturesOut);
}

#undef LOCTEXT_NAMESPACE
