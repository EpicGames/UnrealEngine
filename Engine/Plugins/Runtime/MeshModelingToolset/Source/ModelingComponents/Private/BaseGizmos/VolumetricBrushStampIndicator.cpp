// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseGizmos/VolumetricBrushStampIndicator.h"

#include "Components/PrimitiveComponent.h"
#include "Generators/SphereGenerator.h"
#include "InteractiveGizmoManager.h"
#include "InteractiveTool.h"
#include "PreviewMesh.h"
#include "ToolDataVisualizer.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumetricBrushStampIndicator)

UInteractiveGizmo* UVolumetricBrushStampIndicatorBuilder::BuildGizmo(const FToolBuilderState& SceneState) const
{
	UVolumetricBrushStampIndicator* NewGizmo = NewObject<UVolumetricBrushStampIndicator>(SceneState.GizmoManager);
	return NewGizmo;
}

void UVolumetricBrushStampIndicator::MakeBrushIndicatorMesh(UInteractiveTool* ParentTool, UWorld* World)
{
	if (ensure(ParentTool))
	{
		SphereMesh = NewObject<UPreviewMesh>(ParentTool);
		if (SphereMesh)
		{
			SphereMesh->CreateInWorld(World, FTransform::Identity);
			UE::Geometry::FSphereGenerator SphereGen;
			SphereGen.NumPhi = SphereGen.NumTheta = 32;
			SphereGen.Generate();
			FDynamicMesh3 Mesh(&SphereGen);
			SphereMesh->UpdatePreview(&Mesh);

			UMaterialInstanceDynamic* BrushIndicatorMaterial = ToolSetupUtil::GetDefaultBrushVolumeMaterial(ParentTool->GetToolManager());
			if (BrushIndicatorMaterial)
			{
				SphereMesh->SetMaterial(BrushIndicatorMaterial);
			}

			// make sure raytracing is disabled on the brush indicator
			if (UDynamicMeshComponent* DynamicComponent = Cast<UDynamicMeshComponent>(SphereMesh->GetRootComponent()))
			{
				DynamicComponent->SetEnableRaytracing(false);
				SphereMesh->SetShadowsEnabled(false);
			}
			AttachedComponent = SphereMesh->GetRootComponent();

			// Todo : set a "fully transparent" material for wireframe			
		}
	}
}

UPreviewMesh* UVolumetricBrushStampIndicator::GetPreviewMesh() const
{
	if (SphereMesh)
	{
		return SphereMesh;
	}

	return nullptr;
}

void UVolumetricBrushStampIndicator::Shutdown()
{
	if (SphereMesh)
	{
		SphereMesh->Disconnect();
	}
	SphereMesh = nullptr;
}


void UVolumetricBrushStampIndicator::SetVolumetricBrushVisible(bool bInVisible)
{
	if (SphereMesh)
	{
		SphereMesh->SetVisible(bInVisible);
	}
}

bool UVolumetricBrushStampIndicator::IsVolumetricBrushVisible()
{
	if (SphereMesh)
	{
		return SphereMesh->IsVisible();
	};
	return false;
}
