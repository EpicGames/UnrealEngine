// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/BaseCreateFromSelectedTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "BaseGizmos/TransformGizmoUtil.h"

#include "Components/StaticMeshComponent.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshToMeshDescription.h"

#include "ModelingObjectsCreationAPI.h"
#include "Physics/ComponentCollisionUtil.h"
#include "Selection/ToolSelectionUtil.h"
#include "ShapeApproximation/SimpleShapeSet3.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseCreateFromSelectedTool)

#define LOCTEXT_NAMESPACE "UBaseCreateFromSelectedTool"

using namespace UE::Geometry;

/*
 * ToolBuilder
 */

bool UBaseCreateFromSelectedToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	int32 ComponentCount = SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements());
	return ComponentCount >= MinComponentsSupported() && (!MaxComponentsSupported().IsSet() || ComponentCount <= MaxComponentsSupported().GetValue());
}

UMultiSelectionMeshEditingTool* UBaseCreateFromSelectedToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	// Subclasses must override this method.
	check(false);
	return nullptr;
}


/*
 * Tool
 */

void UBaseCreateFromSelectedTool::Setup()
{
	UInteractiveTool::Setup();

	// hide input StaticMeshComponents
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::HideSourceObject(Targets[ComponentIdx]);
	}

	// initialize our properties

	SetupProperties();
	TransformProperties = NewObject<UTransformInputsToolProperties>(this);
	TransformProperties->RestoreProperties(this);
	AddToolPropertySource(TransformProperties);
	
	OutputTypeProperties = NewObject<UCreateMeshObjectTypeProperties>(this);
	OutputTypeProperties->InitializeDefaultWithAuto();
	OutputTypeProperties->OutputType = UCreateMeshObjectTypeProperties::AutoIdentifier;
	OutputTypeProperties->RestoreProperties(this, TEXT("OutputTypeFromInputTool"));
	OutputTypeProperties->WatchProperty(OutputTypeProperties->OutputType, [this](FString) { OutputTypeProperties->UpdatePropertyVisibility(); });
	AddToolPropertySource(OutputTypeProperties);

	HandleSourcesProperties = NewObject<UBaseCreateFromSelectedHandleSourceProperties>(this);
	HandleSourcesProperties->RestoreProperties(this);
	AddToolPropertySource(HandleSourcesProperties);

	if (SupportsCollisionTransfer())
	{
		CollisionProperties = NewObject<UBaseCreateFromSelectedCollisionProperties>(this);
		CollisionProperties->RestoreProperties(this);
		AddToolPropertySource(CollisionProperties);
	}

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this);
	Preview->Setup(GetTargetWorld(), this);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, nullptr);

	int32 SourcesTriCount = 0;
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		SourcesTriCount += UE::ToolTarget::GetTriangleCount(Targets[ComponentIdx]);
	}
	Preview->SetMaxActiveBackgroundTasksFromMeshSizeHeuristic(SourcesTriCount);

	SetPreviewCallbacks();
	Preview->OnMeshUpdated.AddLambda(
		[this](const UMeshOpPreviewWithBackgroundCompute* UpdatedPreview)
		{
			UpdateAcceptWarnings(UpdatedPreview->HaveEmptyResult() ? EAcceptWarning::EmptyForbidden : EAcceptWarning::NoWarning);
		}
	);

	SetTransformGizmos();

	ConvertInputsAndSetPreviewMaterials(true);

	// output name fields
	HandleSourcesProperties->OutputNewName = PrefixWithSourceNameIfSingleSelection(GetCreatedAssetName());
	HandleSourcesProperties->WatchProperty(HandleSourcesProperties->OutputWriteTo, [&](EBaseCreateFromSelectedTargetType NewType)
	{
		SetToolPropertySourceEnabled(OutputTypeProperties, (NewType == EBaseCreateFromSelectedTargetType::NewObject));
		if (NewType == EBaseCreateFromSelectedTargetType::NewObject)
		{
			HandleSourcesProperties->OutputExistingName = TEXT("");
			UpdateGizmoVisibility();
		}
		else
		{
			int32 Index = (HandleSourcesProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::FirstInputObject) ? 0 : Targets.Num() - 1;
			HandleSourcesProperties->OutputExistingName = UE::Modeling::GetComponentAssetBaseName(UE::ToolTarget::GetTargetComponent(Targets[Index]), false);

			// Reset the hidden gizmo to its initial position
			const FTransform ComponentTransform = (FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[Index]);
			TransformGizmos[Index]->SetNewGizmoTransform(ComponentTransform, true);
			UpdateGizmoVisibility();
		}
	});

	Preview->InvalidateResult();
}


int32 UBaseCreateFromSelectedTool::GetHiddenGizmoIndex() const
{
	if (HandleSourcesProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::NewObject)
	{
		return -1;
	}
	else
	{
		return (HandleSourcesProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::FirstInputObject) ? 0 : Targets.Num() - 1;
	}
}


void UBaseCreateFromSelectedTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);
}



void UBaseCreateFromSelectedTool::UpdateGizmoVisibility()
{
	for (int32 GizmoIndex = 0; GizmoIndex < TransformGizmos.Num(); GizmoIndex++)
	{
		UCombinedTransformGizmo* Gizmo = TransformGizmos[GizmoIndex];
		Gizmo->SetVisibility(TransformProperties->bShowTransformGizmo && GizmoIndex != GetHiddenGizmoIndex());
	}
}



void UBaseCreateFromSelectedTool::SetTransformGizmos()
{
	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();

	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UTransformProxy* Proxy = TransformProxies.Add_GetRef(NewObject<UTransformProxy>(this));
		UCombinedTransformGizmo* Gizmo = TransformGizmos.Add_GetRef(UE::TransformGizmoUtil::Create3AxisTransformGizmo(GizmoManager, this));
		Proxy->SetTransform((FTransform) UE::ToolTarget::GetLocalToWorldTransform(Targets[ComponentIdx]));
		Gizmo->SetActiveTarget(Proxy, GetToolManager());
		Proxy->OnTransformChanged.AddUObject(this, &UBaseCreateFromSelectedTool::TransformChanged);
	}
	UpdateGizmoVisibility();
}


void UBaseCreateFromSelectedTool::TransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	Preview->InvalidateResult();
}


FText UBaseCreateFromSelectedTool::GetActionName() const
{
	return LOCTEXT("BaseCreateFromSelectedTool", "Generated Mesh");
}


TArray<UMaterialInterface*> UBaseCreateFromSelectedTool::GetOutputMaterials() const
{
	return Preview->StandardMaterials;
}


void UBaseCreateFromSelectedTool::GenerateAsset(const FDynamicMeshOpResult& OpResult)
{
	if (OpResult.Mesh.Get() == nullptr) return;

	FTransform3d NewTransform;
	if (Targets.Num() == 1) // in the single-selection case, shove the result back into the original component space
	{
		FTransform3d FromSourceComponentSpace = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
		MeshTransforms::ApplyTransform(*OpResult.Mesh, OpResult.Transform, true);
		MeshTransforms::ApplyTransformInverse(*OpResult.Mesh, FromSourceComponentSpace, true);
		NewTransform = UE::ToolTarget::GetLocalToWorldTransform(Targets[0]);
	}
	else // in the multi-selection case, center the pivot for the combined result
	{
		FVector3d Center = OpResult.Mesh->GetBounds().Center();
		double Rescale = OpResult.Transform.GetScale().X;
		FTransform3d LocalTransform(-Center * Rescale);
		LocalTransform.SetScale3D(FVector3d(Rescale, Rescale, Rescale));
		MeshTransforms::ApplyTransform(*OpResult.Mesh, LocalTransform, true);
		NewTransform = OpResult.Transform;
		NewTransform.SetScale3D(FVector3d::One());
		NewTransform.SetTranslation(NewTransform.GetTranslation() + NewTransform.TransformVector(Center * Rescale));
	}

	// max len explicitly enforced here, would ideally notify user
	FString UseBaseName = HandleSourcesProperties->OutputNewName.Left(250);
	if (UseBaseName.IsEmpty())
	{
		UseBaseName = PrefixWithSourceNameIfSingleSelection(GetCreatedAssetName());
	}

	FCreateMeshObjectParams NewMeshObjectParams;
	NewMeshObjectParams.TargetWorld = GetTargetWorld();
	NewMeshObjectParams.Transform = (FTransform)NewTransform;
	NewMeshObjectParams.BaseName = UseBaseName;
	NewMeshObjectParams.Materials = GetOutputMaterials();
	NewMeshObjectParams.SetMesh(OpResult.Mesh.Get());
	if (OutputTypeProperties->OutputType == UCreateMeshObjectTypeProperties::AutoIdentifier)
	{
		UE::ToolTarget::ConfigureCreateMeshObjectParams(Targets[0], NewMeshObjectParams);
	}
	else
	{
		OutputTypeProperties->ConfigureCreateMeshObjectParams(NewMeshObjectParams);
	}

	bool bOutputSupportsCustomCollision = NewMeshObjectParams.TypeHint != ECreateObjectTypeHint::Volume;
	if (SupportsCollisionTransfer() && CollisionProperties->bTransferCollision && bOutputSupportsCustomCollision)
	{
		for (int32 TargetIdx = 0; TargetIdx < Targets.Num(); ++TargetIdx)
		{
			if (!KeepCollisionFrom(TargetIdx))
			{
				continue;
			}
			UPrimitiveComponent* SourceComponent = UE::ToolTarget::GetTargetComponent(Targets[TargetIdx]);
			if (UE::Geometry::ComponentTypeSupportsCollision(SourceComponent, UE::Geometry::EComponentCollisionSupportLevel::ReadOnly))
			{
				NewMeshObjectParams.bEnableCollision = true;
				FComponentCollisionSettings CollisionSettings = UE::Geometry::GetCollisionSettings(SourceComponent);
				// flag will be set to match the last selection that had collision
				NewMeshObjectParams.CollisionMode = (ECollisionTraceFlag)CollisionSettings.CollisionTypeFlag;

				FSimpleShapeSet3d ShapeSet;
				FTransform Transform = FTransform::Identity;
				if (Targets.Num() > 1)
				{
					// Note: NewTransform in the multi-target case is constructed to have uniform scale, so should have a valid inverse
					Transform = TransformProxies[TargetIdx]->GetTransform() * NewTransform.Inverse();
				}
				if (UE::Geometry::GetCollisionShapes(SourceComponent, ShapeSet))
				{
					if (!NewMeshObjectParams.CollisionShapeSet.IsSet())
					{
						NewMeshObjectParams.CollisionShapeSet.Emplace();
					}
					NewMeshObjectParams.CollisionShapeSet->Append(ShapeSet, Transform);
				}
			}
		}
	}

	FCreateMeshObjectResult Result = UE::Modeling::CreateMeshObject(GetToolManager(), MoveTemp(NewMeshObjectParams));
	if (Result.IsOK() && Result.NewActor != nullptr)
	{
		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), Result.NewActor);
	}
}



void UBaseCreateFromSelectedTool::UpdateAsset(const FDynamicMeshOpResult& Result, UToolTarget* UpdateTarget)
{
	check(Result.Mesh.Get() != nullptr);

	FTransform3d TargetToWorld = UE::ToolTarget::GetLocalToWorldTransform(UpdateTarget);

	FTransform3d ResultTransform = Result.Transform;
	MeshTransforms::ApplyTransform(*Result.Mesh, ResultTransform, true);
	MeshTransforms::ApplyTransformInverse(*Result.Mesh, TargetToWorld, true);

	UE::ToolTarget::CommitDynamicMeshUpdate(UpdateTarget, *Result.Mesh, true);

	FComponentMaterialSet MaterialSet;
	MaterialSet.Materials = GetOutputMaterials();
	// apply updated materials to both asset and component, to ensure that they appear in the result
	// TODO: consider adding a method to the material provider interface that more-directly handles this use case
	Cast<IMaterialProvider>(UpdateTarget)->CommitMaterialSetUpdate(MaterialSet, true);
	Cast<IMaterialProvider>(UpdateTarget)->CommitMaterialSetUpdate(MaterialSet, false);
}


FString UBaseCreateFromSelectedTool::PrefixWithSourceNameIfSingleSelection(const FString& AssetName) const
{
	if (Targets.Num() == 1)
	{
		FString CurName = UE::Modeling::GetComponentAssetBaseName(UE::ToolTarget::GetTargetComponent(Targets[0]));
		return FString::Printf(TEXT("%s_%s"), *CurName, *AssetName);
	}
	else
	{
		return AssetName;
	}
}


void UBaseCreateFromSelectedTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTransformInputsToolProperties, bShowTransformGizmo)))
	{
		UpdateGizmoVisibility();
	}
	else if (Property && 
		(  PropertySet == HandleSourcesProperties
		))
	{
		// nothing
	}
	else
	{
		Preview->InvalidateResult();
	}
}


void UBaseCreateFromSelectedTool::OnShutdown(EToolShutdownType ShutdownType)
{
	SaveProperties();
	HandleSourcesProperties->SaveProperties(this);
	if (SupportsCollisionTransfer())
	{
		CollisionProperties->SaveProperties(this);
	}
	OutputTypeProperties->SaveProperties(this, TEXT("OutputTypeFromInputTool"));
	TransformProperties->SaveProperties(this);

	FDynamicMeshOpResult Result = Preview->Shutdown();
	// Restore (unhide) the source meshes
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		UE::ToolTarget::ShowSourceObject(Targets[ComponentIdx]);
	}

	if (ShutdownType == EToolShutdownType::Accept)
	{
		// Test if we need to pre-update the collision shapes, if we are updating an asset and have multiple targets
		// Note: For static meshes, this must be done as a separate transaction due to a long-standing bug where undo/redo 
		// will crash if an asset updates both its collision and geometry in the same transaction
		// TODO: If/when that static mesh bug is fixed, this collision updating code can be separated out into the UpdateAsset() method
		constexpr bool bWorkaroundForCrashIfConvexAndMeshModifiedInSameTransaction = false;
		bool bAlreadyOpenedMainTransaction = false;
		if (SupportsCollisionTransfer() && HandleSourcesProperties->OutputWriteTo != EBaseCreateFromSelectedTargetType::NewObject && Targets.Num() > 1)
		{
			int32 KeepTargetIndex = (HandleSourcesProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::FirstInputObject) ? 0 : (Targets.Num() - 1);
			UPrimitiveComponent* TargetComponent = UE::ToolTarget::GetTargetComponent(Targets[KeepTargetIndex]);

			if (CollisionProperties->bTransferCollision && UE::Geometry::ComponentTypeSupportsCollision(TargetComponent, UE::Geometry::EComponentCollisionSupportLevel::ReadWrite))
			{
				bool bHasAddedShapes = false;
				FSimpleShapeSet3d AccumulateShapes;
				UE::Geometry::GetCollisionShapes(TargetComponent, AccumulateShapes);

				FTransform3d TargetToWorld = UE::ToolTarget::GetLocalToWorldTransform(Targets[KeepTargetIndex]);
				FTransformSequence3d InvKeepTargetTransform;
				InvKeepTargetTransform.AppendInverse(TargetToWorld);

				for (int32 TargetIdx = 0; TargetIdx < Targets.Num(); ++TargetIdx)
				{
					if (TargetIdx == KeepTargetIndex || !KeepCollisionFrom(TargetIdx))
					{
						continue;
					}
					UPrimitiveComponent* SourceComponent = UE::ToolTarget::GetTargetComponent(Targets[TargetIdx]);
					FSimpleShapeSet3d ShapeSet;
					if (UE::Geometry::ComponentTypeSupportsCollision(SourceComponent, UE::Geometry::EComponentCollisionSupportLevel::ReadOnly) &&
						UE::Geometry::GetCollisionShapes(SourceComponent, ShapeSet))
					{
						bHasAddedShapes = true;
						// Create a best-effort transform sequence to take the collision shapes from their proxy transform space to world, and then back to the local space of the target
						// Note because transforms cannot be fully applied to some collision shape types (e.g. spheres), this will still give weird results in non-uniform scale cases (but this is somewhat unavoidable)
						FTransformSequence3d TransformSeq;
						TransformSeq.Append(TransformProxies[TargetIdx]->GetTransform());
						TransformSeq.Append(InvKeepTargetTransform);
						AccumulateShapes.Append(ShapeSet, TransformSeq);
					}
				}

				if (bHasAddedShapes)
				{
					bool bWorkaroundNotNeeded = !bWorkaroundForCrashIfConvexAndMeshModifiedInSameTransaction | !Cast<UStaticMeshComponent>(TargetComponent); // slightly odd conditional to workaround compiler warning
					if (bWorkaroundNotNeeded)
					{
						// If we're not operating on a static mesh, we can open the main transaction here so the collision transaction becomes part of it ...
						// for static mesh, we need the collision update to be a separate transaction to avoid crashing on undo/redo :(
						bAlreadyOpenedMainTransaction = true;
						GetToolManager()->BeginUndoTransaction(GetActionName());
					}
					GetToolManager()->BeginUndoTransaction(FText::Format(LOCTEXT("SplitTransactionCollisionUpdatePart", "Update Collision ({0})"), GetActionName()));
					TargetComponent->Modify();
					FComponentCollisionSettings Settings = GetCollisionSettings(TargetComponent);
					UE::Geometry::SetSimpleCollision(TargetComponent, &AccumulateShapes, Settings);
					GetToolManager()->EndUndoTransaction();
				}
			}
		}
		if (!bAlreadyOpenedMainTransaction)
		{
			GetToolManager()->BeginUndoTransaction(GetActionName());
		}

		// Generate the result
		AActor* KeepActor = nullptr;
		if (HandleSourcesProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::NewObject)
		{
			GenerateAsset(Result);
		}
		else
		{
			int32 TargetIndex = (HandleSourcesProperties->OutputWriteTo == EBaseCreateFromSelectedTargetType::FirstInputObject) ? 0 : (Targets.Num() - 1);
			KeepActor = UE::ToolTarget::GetTargetActor(Targets[TargetIndex]);

			UpdateAsset(Result, Targets[TargetIndex]);
		}

		TArray<AActor*> Actors;
		for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
		{
			AActor* Actor = UE::ToolTarget::GetTargetActor(Targets[ComponentIdx]);
			Actors.Add(Actor);
		}
		HandleSourcesProperties->ApplyMethod(Actors, GetToolManager(), KeepActor);

		if (KeepActor != nullptr)
		{
			// select the actor we kept
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), KeepActor);
		}

		GetToolManager()->EndUndoTransaction();
	}

	UInteractiveGizmoManager* GizmoManager = GetToolManager()->GetPairedGizmoManager();
	GizmoManager->DestroyAllGizmosByOwner(this);
}


bool UBaseCreateFromSelectedTool::CanAccept() const
{
	return Super::CanAccept() && Preview->HaveValidNonEmptyResult();
}




#undef LOCTEXT_NAMESPACE

