// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/BaseMeshProcessingTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "Components/DynamicMeshComponent.h"
#include "Async/Async.h"

#include "DynamicMesh/MeshNormals.h"
#include "MeshBoundaryLoops.h"
#include "DynamicMesh/MeshTransforms.h"
#include "WeightMapUtil.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "ModelingToolTargetUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseMeshProcessingTool)

#define LOCTEXT_NAMESPACE "UBaseMeshProcessingTool"

using namespace UE::Geometry;

/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UBaseMeshProcessingToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool UBaseMeshProcessingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

USingleTargetWithSelectionTool* UBaseMeshProcessingToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return  MakeNewToolInstance(SceneState.ToolManager);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}




/*
 * Tool
 */

void UBaseMeshProcessingTool::Setup()
{
	UInteractiveTool::Setup();

	// hide input StaticMeshComponent
	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	TargetComponent->SetOwnerVisibility(false);

	// initialize our properties
	ToolPropertyObjects.Add(this);

	// populate the BaseMesh with a conversion of the input mesh.
	InitialMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);

	if (RequiresScaleNormalization())
	{
		// compute area of the input mesh and compute normalization scaling factor
		FVector2d VolArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(InitialMesh);
		double UnitScalingMeasure = FMathd::Max(0.01, FMathd::Sqrt(VolArea.Y / 6.0));  // 6.0 is a bit arbitrary here...surface area of unit box

		// translate to origin and then apply inverse of scale
		FAxisAlignedBox3d Bounds = InitialMesh.GetBounds();
		SrcTranslate = Bounds.Center();
		MeshTransforms::Translate(InitialMesh, -SrcTranslate);
		SrcScale = UnitScalingMeasure;
		MeshTransforms::Scale(InitialMesh, (1.0 / SrcScale) * FVector3d::One(), FVector3d::Zero());

		// apply that transform to target transform so that visible mesh stays in the same spot
		OverrideTransform = TargetComponent->GetWorldTransform();
		FVector TranslateDelta = OverrideTransform.TransformVector((FVector)SrcTranslate);
		FVector CurScale = OverrideTransform.GetScale3D();
		OverrideTransform.AddToTranslation(TranslateDelta);
		CurScale.X *= (float)SrcScale;
		CurScale.Y *= (float)SrcScale;
		CurScale.Z *= (float)SrcScale;
		OverrideTransform.SetScale3D(CurScale);

		bIsScaleNormalizationApplied = true;
	}
	else
	{
		SrcTranslate = FVector3d::Zero();
		SrcScale = 1.0;
		OverrideTransform = TargetComponent->GetWorldTransform();
		bIsScaleNormalizationApplied = false;
	}

	// pending startup computations
	TArray<TFuture<void>> PendingComputes;

	// calculate base mesh vertex normals if necessary normals
	if (RequiresInitialVtxNormals())
	{
		TFuture<void> NormalsCompute = Async(EAsyncExecution::ThreadPool, [&]() {
			InitialVtxNormals = MakeShared<FMeshNormals>(&InitialMesh);
			InitialVtxNormals->ComputeVertexNormals();
		});
		PendingComputes.Add(MoveTemp(NormalsCompute));
	}

	// calculate base mesh boundary loops if necessary
	if (RequiresInitialBoundaryLoops())
	{
		TFuture<void> LoopsCompute = Async(EAsyncExecution::ThreadPool, [&]() {
			InitialBoundaryLoops = MakeShared<FMeshBoundaryLoops>(&InitialMesh);
		});
		PendingComputes.Add(MoveTemp(LoopsCompute));
	}

	// Construct the preview object and set the material on it.
	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>(this, "Preview");
	Preview->Setup(GetTargetWorld(), this); // Adds the actual functional tool in the Preview object
	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->SetMaxActiveBackgroundTasksFromMeshSizeHeuristic(InitialMesh.TriangleCount());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target);

	FComponentMaterialSet MaterialSet;
	Cast<IMaterialProvider>(Target)->GetMaterialSet(MaterialSet);
	Preview->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);
	Preview->SetWorkingMaterialDelay(0.75);
	Preview->PreviewMesh->SetTransform(OverrideTransform);
	Preview->PreviewMesh->UpdatePreview(&InitialMesh);

	// show the preview mesh
	Preview->SetVisibility(true);

	InitializeProperties();
	UpdateOptionalPropertyVisibility();

	for (TFuture<void>& Future : PendingComputes)
	{
		Future.Wait();
	}

	// start the compute
	InvalidateResult();

	GetToolManager()->DisplayMessage( GetToolMessageString(), EToolMessageLevel::UserNotification);
}



FText UBaseMeshProcessingTool::GetToolMessageString() const
{
	return FText::GetEmpty();
}

FText UBaseMeshProcessingTool::GetAcceptTransactionName() const
{
	return LOCTEXT("BaseMeshProcessingToolTransactionName", "Update Mesh");
}



void UBaseMeshProcessingTool::SavePropertySets()
{
	for (FOptionalPropertySet& PropStruct : OptionalProperties)
	{
		if (PropStruct.PropertySet.IsValid())
		{
			PropStruct.PropertySet->SaveProperties(this);
		}
	}

	if (WeightMapPropertySet.IsValid())
	{
		WeightMapPropertySet->SaveProperties(this);
	}

}



void UBaseMeshProcessingTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept && AreAllTargetsValid() == false)
	{
		UE_LOGF(LogTemp, Error, "Tool Target has become Invalid (possibly it has been Force Deleted). Aborting Tool.");
		ShutdownType = EToolShutdownType::Cancel;
	}

	OnShutdown(ShutdownType);

	SavePropertySets();

	// Restore (unhide) the source meshes
	Cast<IPrimitiveComponentBackedTarget>(Target)->SetOwnerVisibility(true);

	if (Preview != nullptr)
	{
		FDynamicMeshOpResult Result = Preview->Shutdown();

		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(GetAcceptTransactionName());

			FDynamicMesh3* DynamicMeshResult = Result.Mesh.Get();
			check(DynamicMeshResult != nullptr);

			// un-apply scale normalization if it was applied
			if (bIsScaleNormalizationApplied)
			{
				MeshTransforms::Scale(*DynamicMeshResult, FVector3d(SrcScale, SrcScale, SrcScale), FVector3d::Zero());
				MeshTransforms::Translate(*DynamicMeshResult, SrcTranslate);
			}

			bool bTopologyChanged = HasMeshTopologyChanged();
			UE::ToolTarget::CommitDynamicMeshUpdate(Target, *DynamicMeshResult, bTopologyChanged);

			GetToolManager()->EndUndoTransaction();
		}
	}
}

void UBaseMeshProcessingTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	UpdateResult();
}

void UBaseMeshProcessingTool::OnTick(float DeltaTime)
{
	Preview->Tick(DeltaTime);
}

void UBaseMeshProcessingTool::InvalidateResult()
{
	Preview->InvalidateResult();
	bResultValid = false;
}

void UBaseMeshProcessingTool::UpdateResult()
{
	if (bResultValid) 
	{
		return;
	}

	bResultValid = Preview->HaveValidResult();
}

bool UBaseMeshProcessingTool::HasAccept() const
{
	return true;
}

bool UBaseMeshProcessingTool::CanAccept() const
{
	return Super::CanAccept() && bResultValid && Preview->HaveValidNonEmptyResult();
}




void UBaseMeshProcessingTool::AddOptionalPropertySet(
	UInteractiveToolPropertySet* PropSet, 
	TUniqueFunction<bool()> VisibilityFunc, 
	TUniqueFunction<void()> OnModifiedFunc,
	bool bChangeInvalidatesResult)
{
	AddToolPropertySource(PropSet);
	PropSet->RestoreProperties(this);
	SetToolPropertySourceEnabled(PropSet, false);

	FOptionalPropertySet PropSetStruct;
	PropSetStruct.PropertySet = PropSet;
	PropSetStruct.IsVisible = MoveTemp(VisibilityFunc);
	PropSetStruct.OnModifiedFunc = MoveTemp(OnModifiedFunc);
	PropSetStruct.bInvalidateOnModify = bChangeInvalidatesResult;
	int32 Index = OptionalProperties.Num();
	OptionalProperties.Add(MoveTemp(PropSetStruct));

	PropSet->GetOnModified().AddLambda([Index, this](UObject*, FProperty*) { OnOptionalPropSetModified(Index); } );
}


void UBaseMeshProcessingTool::OnOptionalPropSetModified(int32 Index)
{
	const FOptionalPropertySet& PropSetStruct = OptionalProperties[Index];
	PropSetStruct.OnModifiedFunc();
	if (PropSetStruct.bInvalidateOnModify)
	{
		InvalidateResult();
	}
}


void UBaseMeshProcessingTool::UpdateOptionalPropertyVisibility()
{
	for (FOptionalPropertySet& PropStruct : OptionalProperties)
	{
		if (PropStruct.PropertySet.IsValid())
		{
			bool bVisible = PropStruct.IsVisible();
			SetToolPropertySourceEnabled(PropStruct.PropertySet.Get(), bVisible);
		}
	}

	if (WeightMapPropertySet.IsValid())
	{
		bool bVisible = WeightMapPropertySetVisibleFunc();
		SetToolPropertySourceEnabled(WeightMapPropertySet.Get(), bVisible);
		
	}
}




TSharedPtr<FMeshNormals>& UBaseMeshProcessingTool::GetInitialVtxNormals()
{
	checkf(InitialVtxNormals.IsValid(), TEXT("Initial Vertex Normals have not been computed - must return true from RequiresInitialVtxNormals()") );
	return InitialVtxNormals;
}

TSharedPtr<FMeshBoundaryLoops>& UBaseMeshProcessingTool::GetInitialBoundaryLoops()
{
	checkf(InitialBoundaryLoops.IsValid(), TEXT("Initial Boundary Loops have not been computed - must return true from RequiresInitialBoundaryLoops()"));
	return InitialBoundaryLoops;
}



void UBaseMeshProcessingTool::SetupWeightMapPropertySet(UWeightMapSetProperties* Properties)
{
	AddToolPropertySource(Properties);
	Properties->RestoreProperties(this);
	WeightMapPropertySet = Properties;

	// initialize property list
	Properties->InitializeFromMesh(UE::ToolTarget::GetMeshDescription(Target));

	Properties->WatchProperty(Properties->WeightMap,
		[&](FName) { OnSelectedWeightMapChanged(true); });
	Properties->WatchProperty(Properties->bInvertWeightMap,
		[&](bool) { OnSelectedWeightMapChanged(true); });

	OnSelectedWeightMapChanged(false);
}


void UBaseMeshProcessingTool::OnSelectedWeightMapChanged(bool bInvalidate)
{
	TSharedPtr<FIndexedWeightMap1f> NewWeightMap = MakeShared<FIndexedWeightMap1f>();

	// this will return all-ones weight map if None is selected
	bool bFound = UE::WeightMaps::GetVertexWeightMap(UE::ToolTarget::GetMeshDescription(Target), WeightMapPropertySet->WeightMap, *NewWeightMap, 1.0f);
	if (bFound && WeightMapPropertySet->bInvertWeightMap)
	{
		NewWeightMap->InvertWeightMap();
	}

	// This will automatically reset the weightmap to None if the new value doesn't exist. This could easily occur during a preset load, for instance, and we 
	// don't want to pollute the value with garbage from a preset value that doesn't apply to the active target.
	if (!bFound)
	{
		WeightMapPropertySet->SetSelectedFromWeightMapIndex(-1);
	}

	ActiveWeightMap = NewWeightMap;

	if (bInvalidate)
	{
		InvalidateResult();
	}
}


bool UBaseMeshProcessingTool::HasActiveWeightMap() const
{
	return WeightMapPropertySet.IsValid() && WeightMapPropertySet->HasSelectedWeightMap();
}

TSharedPtr<FIndexedWeightMap1f>& UBaseMeshProcessingTool::GetActiveWeightMap()
{
	checkf(ActiveWeightMap.IsValid(), TEXT("Weight Map has not been initialized - must call SetupWeightMapPropertySet() in property set"));
	return ActiveWeightMap;
}



#undef LOCTEXT_NAMESPACE

