// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/BaseVoxelTool.h"
#include "InteractiveToolManager.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "TargetInterfaces/MeshDescriptionProvider.h"

#include "MeshDescriptionToDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseVoxelTool)


#define LOCTEXT_NAMESPACE "UBaseVoxelTool"

using namespace UE::Geometry;

void UBaseVoxelTool::SetupProperties()
{
	Super::SetupProperties();
	VoxProperties = NewObject<UVoxelProperties>(this);
	VoxProperties->RestoreProperties(this);
	AddToolPropertySource(VoxProperties);
}


void UBaseVoxelTool::SaveProperties()
{
	Super::SaveProperties();
	VoxProperties->SaveProperties(this);
}

bool UBaseVoxelTool::HasOpenBoundariesInMeshInputs()
{
	for (int32 Idx = 0; Idx < OriginalDynamicMeshes.Num(); ++Idx)
	{
		if (!OriginalDynamicMeshes[Idx]->IsClosed() && OriginalDynamicMeshes[Idx]->TriangleCount() > 0)
		{
			return true;
		}
	}
	return false;
}


TArray<UMaterialInterface*> UBaseVoxelTool::GetOutputMaterials() const
{
	TArray<UMaterialInterface*> Materials;
	Materials.Add(LoadObject<UMaterial>(nullptr, TEXT("MATERIAL")));
	return Materials;
}


void UBaseVoxelTool::ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh)
{
	OriginalDynamicMeshes.SetNum(Targets.Num());

	for (int ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		OriginalDynamicMeshes[ComponentIdx] = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		*OriginalDynamicMeshes[ComponentIdx] = UE::ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx]);
	}

	//if (bSetPreviewMesh)
	//{
	//	// TODO: create a low quality preview result for initial display?
	//}

	Preview->ConfigureMaterials(
		ToolSetupUtil::GetDefaultSculptMaterial(GetToolManager()),
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);
}


#undef LOCTEXT_NAMESPACE

