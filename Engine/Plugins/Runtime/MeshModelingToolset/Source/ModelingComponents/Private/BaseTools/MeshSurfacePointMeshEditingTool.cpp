// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"

#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshSurfacePointMeshEditingTool)


/*
 * ToolBuilder
 */

const FToolTargetTypeRequirements& UMeshSurfacePointMeshEditingToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

UMeshSurfacePointTool* UMeshSurfacePointMeshEditingToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UMeshSurfacePointTool>(SceneState.ToolManager);
}



