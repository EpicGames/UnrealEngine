// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/MultiSelectionMeshEditingTool.h"

#include "Engine/World.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "ToolTargetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiSelectionMeshEditingTool)

/*
 * ToolBuilder
 */
const FToolTargetTypeRequirements& UMultiSelectionMeshEditingToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool UMultiSelectionMeshEditingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) > 0;
}

UInteractiveTool* UMultiSelectionMeshEditingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMultiSelectionMeshEditingTool* NewTool = CreateNewTool(SceneState);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}

void UMultiSelectionMeshEditingToolBuilder::InitializeNewTool(UMultiSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	const TArray<TObjectPtr<UToolTarget>> Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, GetTargetRequirements());
	NewTool->SetTargets(Targets);
	NewTool->SetWorld(SceneState.World);
}


/**
 * Tool
 */

void UMultiSelectionMeshEditingTool::Shutdown(EToolShutdownType ShutdownType)
{
	OnShutdown(ShutdownType);
	TargetWorld = nullptr;
}

void UMultiSelectionMeshEditingTool::OnShutdown(EToolShutdownType ShutdownType)
{
}


void UMultiSelectionMeshEditingTool::SetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* UMultiSelectionMeshEditingTool::GetTargetWorld()
{
	return TargetWorld.Get();
}


bool UMultiSelectionMeshEditingTool::GetMapToSharedSourceData(TArray<int32>& MapToFirstOccurrences)
{
	bool bSharesSources = false;
	MapToFirstOccurrences.SetNumUninitialized(Targets.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		MapToFirstOccurrences[ComponentIdx] = -1;
	}
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		if (MapToFirstOccurrences[ComponentIdx] >= 0) // already mapped
		{
			continue;
		}

		MapToFirstOccurrences[ComponentIdx] = ComponentIdx;

		IAssetBackedTarget* Target = Cast<IAssetBackedTarget>(Targets[ComponentIdx]);
		if (!Target)
		{
			continue;
		}
		for (int32 VsIdx = ComponentIdx + 1; VsIdx < Targets.Num(); VsIdx++)
		{
			IAssetBackedTarget* OtherTarget = Cast<IAssetBackedTarget>(Targets[VsIdx]);
			if (OtherTarget && OtherTarget->GetSourceData() == Target->GetSourceData())
			{
				bSharesSources = true;
				MapToFirstOccurrences[VsIdx] = ComponentIdx;
			}
		}
	}
	return bSharesSources;
}




