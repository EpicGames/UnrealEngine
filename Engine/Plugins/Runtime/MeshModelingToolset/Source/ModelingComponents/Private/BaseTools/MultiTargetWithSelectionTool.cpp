// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/MultiTargetWithSelectionTool.h"

#include "ModelingToolTargetUtil.h"
#include "Engine/World.h"
#include "UDynamicMesh.h"

#include "Drawing/PreviewGeometryActor.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/GeometrySelectionVisualization.h"
#include "Selections/GeometrySelection.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiTargetWithSelectionTool)


/*
 * ToolBuilder
 */
const FToolTargetTypeRequirements& UMultiTargetWithSelectionToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UMeshDescriptionCommitter::StaticClass(),
		UMeshDescriptionProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool UMultiTargetWithSelectionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (RequiresInputSelection() && UE::Geometry::HaveAvailableGeometrySelection(SceneState) == false )
	{
		return false;
	}

	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) > 0;
}

UInteractiveTool* UMultiTargetWithSelectionToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	UMultiTargetWithSelectionTool* NewTool = CreateNewTool(SceneState);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}

void UMultiTargetWithSelectionToolBuilder::InitializeNewTool(UMultiTargetWithSelectionTool* NewTool, const FToolBuilderState& SceneState) const
{

	const TArray<TObjectPtr<UToolTarget>> Targets = SceneState.TargetManager->BuildAllSelectedTargetable(SceneState, GetTargetRequirements());
	const int NumTargets = Targets.Num();
	
	NewTool->InitializeGeometrySelectionArrays(NumTargets);
	NewTool->SetTargets(Targets);
	NewTool->SetTargetWorld(SceneState.World);

	TArray<UE::Geometry::FGeometrySelection> Selections;
	Selections.SetNum(NumTargets);
	for (int TargetIndex = 0; TargetIndex < NumTargets; TargetIndex++)
	{
		bool bHaveSelection = UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, Targets[TargetIndex], Selections[TargetIndex]);

		if (bHaveSelection)
		{
			NewTool->SetGeometrySelection(MoveTemp(Selections[TargetIndex]), TargetIndex);
		}
	}

}

void UMultiTargetWithSelectionTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (GeometrySelectionViz)
	{
		UE::Geometry::UpdateGeometrySelectionVisualization(GeometrySelectionViz, GeometrySelectionVizProperties);
	}
}


void UMultiTargetWithSelectionTool::Shutdown(EToolShutdownType ShutdownType)
{
	OnShutdown(ShutdownType);
	TargetWorld = nullptr;

	Super::Shutdown(ShutdownType);
}

void UMultiTargetWithSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (GeometrySelectionViz)
	{
		GeometrySelectionViz->Disconnect();
	}

	if (GeometrySelectionVizProperties)
	{
		GeometrySelectionVizProperties->SaveProperties(this);
	}
}

void UMultiTargetWithSelectionTool::SetTargetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* UMultiTargetWithSelectionTool::GetTargetWorld()
{
	return TargetWorld.Get();
}

void UMultiTargetWithSelectionTool::InitializeGeometrySelectionArrays(const int NumTargets)
{
	GeometrySelectionArray.SetNum(NumTargets);
	GeometrySelectionBoolArray.SetNum(NumTargets);
	for (int BoolArrIndex = 0; BoolArrIndex < NumTargets; BoolArrIndex++)
	{
		GeometrySelectionBoolArray[BoolArrIndex] = false;
	}
}



void UMultiTargetWithSelectionTool::SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn, const int TargetIndex)
{
	GeometrySelectionArray[TargetIndex] = SelectionIn;
	GeometrySelectionBoolArray[TargetIndex] = true;
}

void UMultiTargetWithSelectionTool::SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn, const int TargetIndex)
{
	GeometrySelectionArray[TargetIndex] = MoveTemp(SelectionIn);
	GeometrySelectionBoolArray[TargetIndex] = true;
}

bool UMultiTargetWithSelectionTool::HasGeometrySelection(const int TargetIndex) const
{
	return GeometrySelectionBoolArray[TargetIndex];
}

const UE::Geometry::FGeometrySelection& UMultiTargetWithSelectionTool::GetGeometrySelection(const int TargetIndex) const
{
	return GeometrySelectionArray[TargetIndex];
}

bool UMultiTargetWithSelectionTool::HasAnyGeometrySelection() const
{
	bool bHasGeometrySelectedAcrossAllTargets = false;
	for (const bool bGeoSelectedPerTarget : GeometrySelectionBoolArray)
	{
		bHasGeometrySelectedAcrossAllTargets = bHasGeometrySelectedAcrossAllTargets || bGeoSelectedPerTarget;
	}
	return bHasGeometrySelectedAcrossAllTargets;
}
