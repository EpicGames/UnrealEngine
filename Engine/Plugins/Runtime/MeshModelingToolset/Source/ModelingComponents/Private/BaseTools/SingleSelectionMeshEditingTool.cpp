// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/SingleSelectionMeshEditingTool.h"

#include "Engine/World.h"

#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "Selection/StoredMeshSelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleSelectionMeshEditingTool)

/*
 * ToolBuilder
 */
const FToolTargetTypeRequirements& USingleSelectionMeshEditingToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool USingleSelectionMeshEditingToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

UInteractiveTool* USingleSelectionMeshEditingToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	USingleSelectionMeshEditingTool* NewTool = CreateNewTool(SceneState);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}

void USingleSelectionMeshEditingToolBuilder::InitializeNewTool(USingleSelectionMeshEditingTool* NewTool, const FToolBuilderState& SceneState) const
{
	UToolTarget* Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	check(Target);
	NewTool->SetTarget(Target);
	NewTool->SetWorld(SceneState.World);
}


void USingleSelectionMeshEditingTool::Shutdown(EToolShutdownType ShutdownType)
{
	OnShutdown(ShutdownType);
	TargetWorld = nullptr;
}

void USingleSelectionMeshEditingTool::OnShutdown(EToolShutdownType ShutdownType)
{
}


void USingleSelectionMeshEditingTool::SetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* USingleSelectionMeshEditingTool::GetTargetWorld()
{
	return TargetWorld.Get();
}





