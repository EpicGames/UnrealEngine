// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseTools/SingleTargetWithSelectionTool.h"

#include "ModelingToolTargetUtil.h"
#include "Engine/World.h"
#include "UDynamicMesh.h"

#include "Drawing/PreviewGeometryActor.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "ToolTargetManager.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/GeometrySelectionVisualization.h"
#include "Selections/GeometrySelection.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SingleTargetWithSelectionTool)


/*
 * ToolBuilder
 */
const FToolTargetTypeRequirements& USingleTargetWithSelectionToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshCommitter::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass()
		});
	return TypeRequirements;
}

bool USingleTargetWithSelectionToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	if (RequiresInputSelection() && UE::Geometry::HaveAvailableGeometrySelection(SceneState) == false )
	{
		return false;
	}

	return SceneState.TargetManager->CountSelectedAndTargetable(SceneState, GetTargetRequirements()) == 1;
}

UInteractiveTool* USingleTargetWithSelectionToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	USingleTargetWithSelectionTool* NewTool = CreateNewTool(SceneState);
	InitializeNewTool(NewTool, SceneState);
	return NewTool;
}

void USingleTargetWithSelectionToolBuilder::InitializeNewTool(USingleTargetWithSelectionTool* NewTool, const FToolBuilderState& SceneState) const
{
	UToolTarget* Target = SceneState.TargetManager->BuildFirstSelectedTargetable(SceneState, GetTargetRequirements());
	check(Target);
	NewTool->SetTarget(Target);
	NewTool->SetTargetWorld(SceneState.World);

	UE::Geometry::FGeometrySelection Selection;
	bool bHaveSelection = UE::Geometry::GetCurrentGeometrySelectionForTarget(SceneState, Target, Selection);
	if (bHaveSelection)
	{
		NewTool->SetGeometrySelection(MoveTemp(Selection));
	}

}

void USingleTargetWithSelectionTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	if (GeometrySelectionViz)
	{
		UE::Geometry::UpdateGeometrySelectionVisualization(GeometrySelectionViz, GeometrySelectionVizProperties);
	}
}


void USingleTargetWithSelectionTool::Shutdown(EToolShutdownType ShutdownType)
{
	OnShutdown(ShutdownType);
	TargetWorld = nullptr;

	Super::Shutdown(ShutdownType);
}

void USingleTargetWithSelectionTool::OnShutdown(EToolShutdownType ShutdownType)
{
	if (GeometrySelectionViz)
	{
		GeometrySelectionViz->Disconnect();
	}

	if (GeometrySelectionVizProperties)
	{
		GeometrySelectionVizProperties->SaveProperties(this);
	}
}

void USingleTargetWithSelectionTool::SetTargetWorld(UWorld* World)
{
	TargetWorld = World;
}

UWorld* USingleTargetWithSelectionTool::GetTargetWorld()
{
	return TargetWorld.Get();
}


void USingleTargetWithSelectionTool::SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn)
{
	GeometrySelection = SelectionIn;
	bGeometrySelectionInitialized = true;
}

void USingleTargetWithSelectionTool::SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn)
{
	GeometrySelection = MoveTemp(SelectionIn);
	bGeometrySelectionInitialized = true;
}

bool USingleTargetWithSelectionTool::HasGeometrySelection() const
{
	return bGeometrySelectionInitialized;
}

const UE::Geometry::FGeometrySelection& USingleTargetWithSelectionTool::GetGeometrySelection() const
{
	return GeometrySelection;
}