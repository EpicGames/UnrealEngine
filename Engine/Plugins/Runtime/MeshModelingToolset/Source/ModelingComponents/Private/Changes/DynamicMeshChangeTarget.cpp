// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/DynamicMeshChangeTarget.h"
#include "DynamicMesh/DynamicMesh3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshChangeTarget)

using namespace UE::Geometry;

void UDynamicMeshReplacementChangeTarget::ApplyChange(const FMeshReplacementChange* Change, bool bRevert)
{
	Mesh = Change->GetMesh(bRevert);
	OnMeshChanged.Broadcast();
}

TUniquePtr<FMeshReplacementChange> UDynamicMeshReplacementChangeTarget::ReplaceMesh(const TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>& UpdateMesh)
{
	TUniquePtr<FMeshReplacementChange> Change = MakeUnique<FMeshReplacementChange>(Mesh, UpdateMesh);
	Mesh = UpdateMesh;
	return Change;
}
