// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/MeshPolygroupChange.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;

void FDynamicMeshGroupEdit::ApplyToMesh(FDynamicMesh3* Mesh, bool bRevert)
{
	int32 NumTris = Triangles.Num();
	if (GroupLayerIndex < 0)
	{
		if (ensure(Mesh->HasTriangleGroups()) == false) return;

		for (int32 k = 0; k < NumTris; ++k)
		{
			int32 tid = Triangles[k];
			if (Mesh->IsTriangle(tid))
			{
				int32 SetGroup = (bRevert) ? OldGroups[k] : NewGroups[k];
				Mesh->SetTriangleGroup(tid, SetGroup);
			}
		}
	}
	else
	{
		if ( ensure(Mesh->HasAttributes()) == false || 
			 ensure(GroupLayerIndex <= Mesh->Attributes()->NumPolygroupLayers()) == false ) return;

		FDynamicMeshPolygroupAttribute* PolygroupLayer = Mesh->Attributes()->GetPolygroupLayer(GroupLayerIndex);
		if (ensure(PolygroupLayer))
		{
			for (int32 k = 0; k < NumTris; ++k)
			{
				int32 tid = Triangles[k];
				if (Mesh->IsTriangle(tid))
				{
					int32 SetGroup = (bRevert) ? OldGroups[k] : NewGroups[k];
					PolygroupLayer->SetValue(tid, SetGroup);
				}
			}
		}
	}
}




FDynamicMeshGroupEditBuilder::FDynamicMeshGroupEditBuilder(FDynamicMesh3* Mesh)
{
	PolygroupSet = MakePimpl<UE::Geometry::FPolygroupSet>(Mesh);
	Edit = MakeUnique<FDynamicMeshGroupEdit>();
}

FDynamicMeshGroupEditBuilder::FDynamicMeshGroupEditBuilder(FDynamicMesh3* Mesh, int32 GroupLayer)
{
	PolygroupSet = MakePimpl<UE::Geometry::FPolygroupSet>(Mesh, GroupLayer);

	Edit = MakeUnique<FDynamicMeshGroupEdit>();
	if (ensure(PolygroupSet->GetPolygroupIndex() != -1))
	{
		Edit->GroupLayerIndex = PolygroupSet->GetPolygroupIndex();
	}
}

FDynamicMeshGroupEditBuilder::FDynamicMeshGroupEditBuilder(UE::Geometry::FPolygroupSet* PolygroupSetIn)
{
	PolygroupSet = MakePimpl<UE::Geometry::FPolygroupSet>(*PolygroupSetIn);

	Edit = MakeUnique<FDynamicMeshGroupEdit>();
	if (PolygroupSet->GetPolygroupIndex() != -1)
	{
		Edit->GroupLayerIndex = PolygroupSet->GetPolygroupIndex();
	}
}



void FDynamicMeshGroupEditBuilder::SaveTriangle(int32 TriangleID)
{
	int32 NewGroupID = PolygroupSet->GetGroup(TriangleID);
	const int32* NewIndex = SavedIndexMap.Find(TriangleID);
	if (NewIndex == nullptr)
	{
		int32 Index = Edit->Triangles.Num();
		SavedIndexMap.Add(TriangleID, Index);
		Edit->Triangles.Add(TriangleID);
		Edit->OldGroups.Add(NewGroupID);
		Edit->NewGroups.Add(NewGroupID);
	}
	else
	{
		Edit->NewGroups[*NewIndex] = PolygroupSet->GetGroup(TriangleID);
	}
}


void FDynamicMeshGroupEditBuilder::SaveTriangle(int32 TriangleID, int32 OldGroup, int32 NewGroup)
{
	const int32* NewIndex = SavedIndexMap.Find(TriangleID);
	if (NewIndex == nullptr)
	{
		int32 Index = Edit->Triangles.Num();
		SavedIndexMap.Add(TriangleID, Index);
		Edit->Triangles.Add(TriangleID);
		Edit->OldGroups.Add(OldGroup);
		Edit->NewGroups.Add(NewGroup);
	}
	else
	{
		Edit->NewGroups[*NewIndex] = NewGroup;
	}
}



FMeshPolygroupChange::FMeshPolygroupChange(TUniquePtr<FDynamicMeshGroupEdit>&& GroupEditIn)
{
	GroupEdit = MoveTemp(GroupEditIn);
}

void FMeshPolygroupChange::ApplyChangeToMesh(FDynamicMesh3* Mesh, bool bRevert) const
{
	GroupEdit->ApplyToMesh(Mesh, bRevert);
}
