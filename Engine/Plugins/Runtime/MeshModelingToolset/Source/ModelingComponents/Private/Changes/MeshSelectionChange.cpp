// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/MeshSelectionChange.h"


void FMeshSelectionChange::Apply(UObject* Object)
{
	UMeshSelectionSet* Selection = CastChecked<UMeshSelectionSet>(Object);

	if (bAdded)
	{
		Selection->AddIndices(ElementType, Indices);
	}
	else
	{
		Selection->RemoveIndices(ElementType, Indices);
	}
}

void FMeshSelectionChange::Revert(UObject* Object)
{
	UMeshSelectionSet* Selection = CastChecked<UMeshSelectionSet>(Object);

	if (bAdded)
	{
		Selection->RemoveIndices(ElementType, Indices);
	}
	else
	{
		Selection->AddIndices(ElementType, Indices);
	}
}


FString FMeshSelectionChange::ToString() const
{
	return FString(TEXT("Mesh Selection Change"));
}




FMeshSelectionChangeBuilder::FMeshSelectionChangeBuilder(EMeshSelectionElementType ElementType, bool bAdding)
{
	Change = MakeUnique<FMeshSelectionChange>();
	Change->ElementType = ElementType;
	Change->bAdded = bAdding;
}

void FMeshSelectionChangeBuilder::Add(int32 ElementID)
{
	Change->Indices.Add(ElementID);
}



void FMeshSelectionChangeBuilder::Add(const TArray<int32>& Elements)
{
	Change->Indices.Append(Elements);
}

void FMeshSelectionChangeBuilder::Add(const TSet<int32>& Elements)
{
	for (int32 Index : Elements)
	{
		Change->Indices.Add(Index);
	}
}

