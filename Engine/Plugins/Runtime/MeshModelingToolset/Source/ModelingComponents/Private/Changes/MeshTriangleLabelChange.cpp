// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/MeshTriangleLabelChange.h"
#include "DynamicMesh/DynamicMesh3.h"

using namespace UE::Geometry;

void FDynamicMeshTriangleLabelEdit::ApplyToMesh(FDynamicMesh3* Mesh, bool bRevert)
{
	if (!ensure(Mesh && LayerName != NAME_None))
	{
		return;
	}
	
	FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
	if (!ensure(Attributes))
	{
		return;
	}
	
	FDynamicMeshTriangleLabelAttribute* LabelAttribute = Attributes->FindTriangleLabelAttribute(LayerName);
	if (ensure(LabelAttribute))
	{
		const int32 NumTris = Triangles.Num();
		for (int32 k = 0; k < NumTris; ++k)
		{
			const int32 tid = Triangles[k];
			if (Mesh->IsTriangle(tid) && IsTriangleIndexValid(k, bRevert))
			{
				const FName Label = (bRevert) ? OldLabels[k] : NewLabels[k];
				LabelAttribute->SetValue(tid, Label);
			}
		}
	}
}

bool FDynamicMeshTriangleLabelEdit::IsTriangleIndexValid(const int32 TriangleIndex, const bool bRevert) const
{
	return bRevert ? OldLabels.IsValidIndex(TriangleIndex) : NewLabels.IsValidIndex(TriangleIndex);
}

FDynamicMeshTriangleLabelEditBuilder::FDynamicMeshTriangleLabelEditBuilder(FTriangleLabelAdapter* InAdapter)
	: Adapter(MakePimpl<FTriangleLabelAdapter>(*InAdapter))
	, Edit(MakeUnique<FDynamicMeshTriangleLabelEdit>())
{
	Edit->LayerName = InAdapter->GetLayerName();
}

void FDynamicMeshTriangleLabelEditBuilder::SaveTriangle(int32 TriangleID)
{
	FName NewGroupLabel = Adapter->GetValue(TriangleID);
	
	const int32* NewIndex = SavedIndexMap.Find(TriangleID);
	if (NewIndex == nullptr)
	{
		int32 Index = Edit->Triangles.Num();
		SavedIndexMap.Add(TriangleID, Index);
		Edit->Triangles.Add(TriangleID);
		Edit->OldLabels.Add(NewGroupLabel);
		Edit->NewLabels.Add(NewGroupLabel);
	}
	else
	{
		Edit->NewLabels[*NewIndex] = NewGroupLabel;
	}
}


void FDynamicMeshTriangleLabelEditBuilder::SaveTriangle(int32 TriangleID, FName OldLabel, FName NewLabel)
{
	const int32* NewIndex = SavedIndexMap.Find(TriangleID);
	if (NewIndex == nullptr)
	{
		int32 Index = Edit->Triangles.Num();
		SavedIndexMap.Add(TriangleID, Index);
		Edit->Triangles.Add(TriangleID);
		Edit->OldLabels.Add(OldLabel);
		Edit->NewLabels.Add(NewLabel);
	}
	else
	{
		Edit->NewLabels[*NewIndex] = NewLabel;
	}
}

FMeshTriangleLabelChange::FMeshTriangleLabelChange(TUniquePtr<FDynamicMeshTriangleLabelEdit>&& LabelEditIn)
	: LabelEdit(MoveTemp(LabelEditIn))
{}

void FMeshTriangleLabelChange::ApplyChangeToMesh(FDynamicMesh3* Mesh, bool bRevert) const
{
	LabelEdit->ApplyToMesh(Mesh, bRevert);
}
