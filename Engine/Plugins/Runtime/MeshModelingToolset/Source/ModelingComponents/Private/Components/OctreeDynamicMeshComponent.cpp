// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Components/OctreeDynamicMeshComponent.h"
#include "RenderingThread.h"
#include "RenderResource.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "VertexFactory.h"
#include "MaterialShared.h"
#include "Engine/CollisionProfile.h"
#include "Materials/Material.h"
#include "LocalVertexFactory.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMeshBuilder.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "StaticMeshResources.h"
#include "StaticMeshAttributes.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshDescriptionToDynamicMesh.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"


// default proxy for this component
#include "OctreeDynamicMeshSceneProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OctreeDynamicMeshComponent)

using namespace UE::Geometry;



UOctreeDynamicMeshComponent::UOctreeDynamicMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	MeshObject = CreateDefaultSubobject<UDynamicMesh>(TEXT("DynamicMesh"));
	MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UOctreeDynamicMeshComponent::OnMeshObjectChanged);
	PreMeshChangeHandle = MeshObject->OnPreMeshChanged().AddUObject(this, &UOctreeDynamicMeshComponent::OnPreMeshObjectChanged);
	

	Octree = MakeUnique<FDynamicMeshOctree3>();
	Octree->Initialize(GetMesh());
	OctreeCut = MakeUnique<FDynamicMeshOctree3::FTreeCutSet>();
}



void UOctreeDynamicMeshComponent::SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh)
{
	MeshObject->SetMesh(MoveTemp(MoveMesh));

	FAxisAlignedBox3d MeshBounds = GetMesh()->GetBounds(true);
	Octree = MakeUnique<FDynamicMeshOctree3>();
	Octree->RootDimension = MeshBounds.MaxDim() * 0.25;

	Octree->Initialize(GetMesh());

	FDynamicMeshOctree3::FStatistics Stats;
	Octree->ComputeStatistics(Stats);
	//UE_LOGF(LogTemp, Warning, "OctreeStats %ls", *Stats.ToString());

	OctreeCut = MakeUnique<FDynamicMeshOctree3::FTreeCutSet>();
	CutCellSetMap.Reset();

	NotifyMeshUpdated();
}



void UOctreeDynamicMeshComponent::ApplyTransform(const FTransform3d& Transform, bool bInvert)
{
	if (bInvert)
	{
		MeshTransforms::ApplyTransformInverse(*GetMesh(), Transform, true);
	}
	else
	{
		MeshTransforms::ApplyTransform(*GetMesh(), Transform, true);
	}

	if (GetCurrentSceneProxy() != nullptr)
	{
		Octree->ModifiedBounds = FAxisAlignedBox3d(
			-TNumericLimits<float>::Max() * FVector3d::One(),
			TNumericLimits<float>::Max() * FVector3d::One());
		NotifyMeshUpdated();
	}
	else
	{
		FAxisAlignedBox3d MeshBounds = GetMesh()->GetBounds(true);
		Octree = MakeUnique<FDynamicMeshOctree3>();
		Octree->RootDimension = MeshBounds.MaxDim() * 0.25;
		Octree->Initialize(GetMesh());
		OctreeCut = MakeUnique<FDynamicMeshOctree3::FTreeCutSet>();
		CutCellSetMap.Reset();
	}

}



void UOctreeDynamicMeshComponent::NotifyMeshUpdated()
{
	if (GetCurrentSceneProxy() != nullptr)
	{
		FAxisAlignedBox3d DirtyBox = Octree->ModifiedBounds;
		Octree->ResetModifiedBounds();

		// update existing cells
		int TotalTriangleCount = 0;
		int SpillTriangleCount = 0;
		TArray<int32> SetsToUpdate;

		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateExisting);
			FCriticalSection SetsLock;
			int NumCutCells = CutCellSetMap.Num();
			ParallelFor(NumCutCells, [&](int i) 
			{
				const FCutCellIndexSet& CutCellSet = CutCellSetMap[i];

				if (Octree->TestCellIntersection(CutCellSet.CellRef, DirtyBox) == false)
				{
					return;
				}

				TArray<int32>& TriangleSet = TriangleDecomposition.GetIndexSetArray(CutCellSet.DecompSetID);
				TriangleSet.Reset();
				Octree->CollectTriangles(CutCellSet.CellRef, [&TriangleSet](int TriangleID) {
					TriangleSet.Add(TriangleID);
				});

				SetsLock.Lock();
				TotalTriangleCount += TriangleSet.Num();
				SetsToUpdate.Add(CutCellSet.DecompSetID);
				SetsLock.Unlock();
			}, false);
		}

		// update cut set to find new cells
		TArray<FDynamicMeshOctree3::FCellReference> NewCutCells;
		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateCutSet);
			Octree->UpdateLevelCutSet(*OctreeCut, NewCutCells);
		}

		// add new ones
		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_CreateNew);
			for (const FDynamicMeshOctree3::FCellReference& CellRef : NewCutCells)
			{

				int32 IndexSetID = TriangleDecomposition.CreateNewIndexSet();
				TArray<int32>& TriangleSet = TriangleDecomposition.GetIndexSetArray(IndexSetID);
				Octree->CollectTriangles(CellRef, [&TriangleSet](int TriangleID) {
					TriangleSet.Add(TriangleID);
				});
				TotalTriangleCount += TriangleSet.Num();
				FCutCellIndexSet SetMapEntry = { CellRef, IndexSetID };
				CutCellSetMap.Add(SetMapEntry);
				SetsToUpdate.Add(IndexSetID);
			}
		}

		// rebuild spill set (always for now...should track bounds? and do separately for each root cell?)
		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateSpill);
			TArray<int32>& SpillTriangleSet = TriangleDecomposition.GetIndexSetArray(SpillDecompSetID);
			SpillTriangleSet.Reset();
			Octree->CollectRootTriangles(*OctreeCut,
				[&SpillTriangleSet](int TriangleID) {
				SpillTriangleSet.Add(TriangleID);
			});
			Octree->CollectSpillTriangles(
				[&SpillTriangleSet](int TriangleID) {
				SpillTriangleSet.Add(TriangleID);
			});
			TotalTriangleCount += SpillTriangleSet.Num();
			SpillTriangleCount += SpillTriangleSet.Num();
			SetsToUpdate.Add(SpillDecompSetID);
		}


		//UE_LOGF(LogTemp, Warning, "Updating %d of %d decomposition sets, %d tris total, spillcount %d", SetsToUpdate.Num(), CutCellSetMap.Num(), TotalTriangleCount, SpillTriangleCount);
		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateFromDecomp);
			GetCurrentSceneProxy()->UpdateFromDecomposition(TriangleDecomposition, SetsToUpdate);
		}

	}
}




static void InitializeOctreeCutSet(const FDynamicMesh3& Mesh, const FDynamicMeshOctree3& Octree, TUniquePtr<FDynamicMeshOctree3::FTreeCutSet>& CutSet)
{
	int TriangleCount = Mesh.TriangleCount();
	if (TriangleCount < 50000)
	{
		*CutSet = Octree.BuildLevelCutSet(1);
		return;
	}

	FDynamicMeshOctree3::FStatistics Stats;
	Octree.ComputeStatistics(Stats);
	int MaxLevel = Stats.Levels;

	int CutLevel = 0;
	while ( CutLevel < MaxLevel-1 && Stats.LevelBoxCounts[CutLevel] < 200 && Stats.LevelBoxCounts[CutLevel+1] < 300)
	{
		CutLevel++;
	}
	*CutSet = Octree.BuildLevelCutSet(CutLevel);
	//UE_LOGF(LogTemp, Warning, "InitializeOctreeCutSet - level %d cells %d", CutLevel, CutSet->CutCells.Num());
}




FPrimitiveSceneProxy* UOctreeDynamicMeshComponent::CreateSceneProxy()
{
	check(GetCurrentSceneProxy() == nullptr);

	FOctreeDynamicMeshSceneProxy* NewProxy = nullptr;
	if (GetMesh()->TriangleCount() > 0)
	{
		NewProxy = new FOctreeDynamicMeshSceneProxy(this);

		if (TriangleColorFunc != nullptr)
		{
			NewProxy->MeshRenderBufferSetConverter.bUsePerTriangleColor = true;
			NewProxy->MeshRenderBufferSetConverter.PerTriangleColorFunc = [this](const FDynamicMesh3* MeshIn, int TriangleID) { return GetTriangleColor(TriangleID); };
		}

		OctreeCut = MakeUnique<FDynamicMeshOctree3::FTreeCutSet>();
		CutCellSetMap.Reset();
		InitializeOctreeCutSet(*GetMesh(), *Octree, OctreeCut);

		TriangleDecomposition = FArrayIndexSetsDecomposition();

		SpillDecompSetID = TriangleDecomposition.CreateNewIndexSet();

		for (auto CellRef : OctreeCut->CutCells)
		{
			int32 IndexSetID = TriangleDecomposition.CreateNewIndexSet();
			TArray<int32>& TriangleSet = TriangleDecomposition.GetIndexSetArray(IndexSetID);
			Octree->CollectTriangles(CellRef, [&TriangleSet](int TriangleID) {
				TriangleSet.Add(TriangleID);
			});

			FCutCellIndexSet SetMapEntry = { CellRef, IndexSetID };
			CutCellSetMap.Add(SetMapEntry);
		}

		// collect spill triangles
		{
			TArray<int32>& SpillTriangleSet = TriangleDecomposition.GetIndexSetArray(SpillDecompSetID);
			Octree->CollectRootTriangles(*OctreeCut, 
				[&SpillTriangleSet](int TriangleID) {
					SpillTriangleSet.Add(TriangleID);
			});
			Octree->CollectSpillTriangles(
				[&SpillTriangleSet](int TriangleID) {
				SpillTriangleSet.Add(TriangleID);
			});
		}

		NewProxy->InitializeFromDecomposition(TriangleDecomposition);
	}
	return NewProxy;
}



void UOctreeDynamicMeshComponent::NotifyMaterialSetUpdated()
{
	if (GetCurrentSceneProxy() != nullptr)
	{
		GetCurrentSceneProxy()->UpdatedReferencedMaterials();
	}
}




FColor UOctreeDynamicMeshComponent::GetTriangleColor(int TriangleID)
{
	if (TriangleColorFunc != nullptr)
	{
		return TriangleColorFunc(GetMesh(), TriangleID);
	}

	return (TriangleID % 2 == 0) ? FColor::Red : FColor::White;
}



FBoxSphereBounds UOctreeDynamicMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// Bounds are tighter if the box is generated from transformed vertices.
	FBox BoundingBox(ForceInit);
	for ( FVector3d Vertex : GetMesh()->VerticesItr() ) 
	{
		BoundingBox += LocalToWorld.TransformPosition((FVector)Vertex);
	}
	return FBoxSphereBounds(BoundingBox);
}



void UOctreeDynamicMeshComponent::ApplyChange(const FMeshVertexChange* Change, bool bRevert)
{
	// This function is not currently used in the codebase, and needs to be restructured
	// to work properly with passing the change down to the UDynamicMesh
	check(false);

/*
	Octree->ResetModifiedBounds();

	const FDynamicMesh3* Mesh = GetMesh();

	TSet<int> TrianglesToUpdate;
	auto NotifyTriVerticesAffected = [&](int32 vid) 
	{
		for (int tid : Mesh->VtxTrianglesItr(vid))
		{
			if (TrianglesToUpdate.Contains(tid) == false)
			{
				Octree->NotifyPendingModification(tid);
				TrianglesToUpdate.Add(tid);
			}
		}
	};

	int NV = Change->Vertices.Num();
	const TArray<FVector3d>& Positions = (bRevert) ? Change->OldPositions : Change->NewPositions;
	for (int k = 0; k < NV; ++k)
	{
		int vid = Change->Vertices[k];
		NotifyTriVerticesAffected(vid);
		Mesh->SetVertex(vid, Positions[k]);
	}

	if (Change->bHaveOverlayNormals && Mesh->HasAttributes() && Mesh->Attributes()->PrimaryNormals())
	{
		FDynamicMeshNormalOverlay* Overlay = Mesh->Attributes()->PrimaryNormals();
		int32 NumNormals = Change->Normals.Num();
		const TArray<FVector3f>& UseNormals = (bRevert) ? Change->OldNormals : Change->NewNormals;
		for (int32 k = 0; k < NumNormals; ++k)
		{
			int32 elemid = Change->Normals[k];
			if (Overlay->IsElement(elemid))
			{
				Overlay->SetElement(elemid, UseNormals[k]);
				int32 ParentVID = Overlay->GetParentVertex(elemid);
				NotifyTriVerticesAffected(ParentVID);
			}
		}
	}

	Octree->ReinsertTriangles(TrianglesToUpdate);

	//NotifyMeshUpdated();

	OnMeshChanged.Broadcast();
*/
}




void UOctreeDynamicMeshComponent::ApplyChange(const FMeshChange* Change, bool bRevert)
{
	MeshObject->ApplyChange(Change, bRevert);

	//NotifyMeshUpdated();

	OnMeshChanged.Broadcast();
}


void UOctreeDynamicMeshComponent::ApplyChange(const FMeshReplacementChange* Change, bool bRevert)
{
	// full clear and reset
	MeshObject->ApplyChange(Change, bRevert);

	Octree = MakeUnique<FDynamicMeshOctree3>();
	Octree->Initialize(GetMesh());
	
	OctreeCut = MakeUnique<FDynamicMeshOctree3::FTreeCutSet>();
	CutCellSetMap.Reset();

	// need to force proxy recreation because we changed the Octree pointer...
	MarkRenderStateDirty();

	OnMeshChanged.Broadcast();
}


void UOctreeDynamicMeshComponent::OnPreMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo)
{
	TArray<int> RemoveTriangles;
	if (ChangeInfo.Type == EDynamicMeshChangeType::MeshChange && ensure(ChangeInfo.MeshChange != nullptr))
	{
		ChangeInfo.MeshChange->DynamicMeshChange->GetSavedTriangleList(RemoveTriangles, !ChangeInfo.bIsRevertChange);
	}
	if (RemoveTriangles.Num() > 0)
	{
		Octree->ResetModifiedBounds();
		Octree->RemoveTriangles(RemoveTriangles);
	}
}


void UOctreeDynamicMeshComponent::OnMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo)
{
	if (ChangeInfo.Type == EDynamicMeshChangeType::MeshChange && ensure(ChangeInfo.MeshChange != nullptr))
	{
		TArray<int> AddTriangles;
		ChangeInfo.MeshChange->DynamicMeshChange->GetSavedTriangleList(AddTriangles, ChangeInfo.bIsRevertChange);
		Octree->InsertTriangles(AddTriangles);
	}

	NotifyMeshUpdated();
	OnMeshChanged.Broadcast();
}


void UOctreeDynamicMeshComponent::SetDynamicMesh(UDynamicMesh* NewMesh)
{
	if (ensure(NewMesh) == false)
	{
		return;
	}

	if (ensure(MeshObject))
	{
		MeshObject->OnMeshChanged().Remove(MeshObjectChangedHandle);
		MeshObject->OnPreMeshChanged().Remove(PreMeshChangeHandle);
	}

	MeshObject = NewMesh;
	MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UOctreeDynamicMeshComponent::OnMeshObjectChanged);
	PreMeshChangeHandle = MeshObject->OnPreMeshChanged().AddUObject(this, &UOctreeDynamicMeshComponent::OnPreMeshObjectChanged);

	Octree = MakeUnique<FDynamicMeshOctree3>();
	Octree->Initialize(GetMesh());
	OctreeCut = MakeUnique<FDynamicMeshOctree3::FTreeCutSet>();
	CutCellSetMap.Reset();

	NotifyMeshUpdated();
	OnMeshChanged.Broadcast();
}
