// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "Components/OctreeDynamicMeshComponent.h"
#include "Components/BaseDynamicMeshSceneProxy.h"
#include "Util/IndexSetDecompositions.h"
#include "SceneInterface.h"


DECLARE_STATS_GROUP(TEXT("SculptToolOctree"), STATGROUP_SculptToolOctree, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_UpdateExisting"), STAT_SculptToolOctree_UpdateExisting, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_UpdateCutSet"), STAT_SculptToolOctree_UpdateCutSet, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_CreateNew"), STAT_SculptToolOctree_CreateNew, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_UpdateSpill"), STAT_SculptToolOctree_UpdateSpill, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_UpdateFromDecomp"), STAT_SculptToolOctree_UpdateFromDecomp, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_UpdateDecompDestroy"), STAT_SculptToolOctree_UpdateDecompDestroy, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_UpdateDecompCreate"), STAT_SculptToolOctree_UpdateDecompCreate, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_InitializeBufferFromOverlay"), STAT_SculptToolOctree_InitializeBufferFromOverlay, STATGROUP_SculptToolOctree);
DECLARE_CYCLE_STAT(TEXT("SculptToolOctree_BufferUpload"), STAT_SculptToolOctree_BufferUpload, STATGROUP_SculptToolOctree);


/**
 * Scene Proxy for a mesh buffer.
 * 
 * Based on FProceduralMeshSceneProxy but simplified in various ways.
 * 
 * Supports wireframe-on-shaded rendering.
 * 
 */
class FOctreeDynamicMeshSceneProxy final : public FBaseDynamicMeshSceneProxy
{
	using FIndex2i = UE::Geometry::FIndex2i;
	using FIndex3i = UE::Geometry::FIndex3i;
private:
	FMaterialRelevance MaterialRelevance;

	// note: FBaseDynamicMeshSceneProxy owns and will destroy these
	TMap<int32, FMeshRenderBufferSet*> RenderBufferSets;

public:
	/** Component that created this proxy (is there a way to look this up?) */
	UOctreeDynamicMeshComponent* ParentComponent;


	FOctreeDynamicMeshSceneProxy(UOctreeDynamicMeshComponent* Component)
		: FBaseDynamicMeshSceneProxy(Component)
		, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
	{
		// This is an assumption we are currently making. We do not necessarily require this
		// but if this check is hit then possibly an assumption is wrong
		check(IsInGameThread());

		ParentComponent = Component;
	}



	virtual void GetActiveRenderBufferSets(TArray<FMeshRenderBufferSet*>& Buffers) const override
	{
		for (auto MapPair : RenderBufferSets)
		{
			Buffers.Add(MapPair.Value);
		}
	}



	TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> MakeTangentsFunc()
	{
		return [](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
		{
			UE::Geometry::VectorUtil::MakePerpVectors(Normal, TangentX, TangentY);
		};
	}



	void InitializeSingleBuffer()
	{
		check(RenderBufferSets.Num() == 0);

		FDynamicMesh3* Mesh = ParentComponent->GetMesh();

		FMeshRenderBufferSet* RenderBuffers = AllocateNewRenderBufferSet();
		RenderBuffers->Material = GetMaterial(0);

		// find suitable overlays
		FDynamicMeshUVOverlay* UVOverlay = nullptr;
		FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		if (Mesh->HasAttributes())
		{
			UVOverlay = Mesh->Attributes()->PrimaryUV();
			NormalOverlay = Mesh->Attributes()->PrimaryNormals();
			ColorOverlay = Mesh->Attributes()->PrimaryColors();
		}

		const bool bTrackTriangles = false;
		const bool bParallel = true;
		InitializeBuffersFromOverlays(RenderBuffers, Mesh,
			Mesh->TriangleCount(), Mesh->TriangleIndicesItr(),
			UVOverlay, NormalOverlay, ColorOverlay, MakeTangentsFunc(), 
			bTrackTriangles, bParallel);

		ENQUEUE_RENDER_COMMAND(FOctreeDynamicMeshSceneProxyInitializeSingle)(
			[this, RenderBuffers](FRHICommandListImmediate& RHICmdList)
		{
			RenderBuffers->Upload();
			RenderBufferSets.Add(0, RenderBuffers);
		});
	}



	void InitializeFromDecomposition(const UE::Geometry::FArrayIndexSetsDecomposition& Decomposition)
	{
		check(RenderBufferSets.Num() == 0);

		FDynamicMesh3* Mesh = ParentComponent->GetMesh();

		// find suitable overlays
		FDynamicMeshUVOverlay* UVOverlay = nullptr;
		FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		if (Mesh->HasAttributes())
		{
			UVOverlay = Mesh->Attributes()->PrimaryUV();
			NormalOverlay = Mesh->Attributes()->PrimaryNormals();
			ColorOverlay = Mesh->Attributes()->PrimaryColors();
		}
		auto TangentsFunc = MakeTangentsFunc();

		const TArray<int32>& SetIDs = Decomposition.GetIndexSetIDs();
		for (int32 SetID : SetIDs)
		{
			const TArray<int32>& Tris = Decomposition.GetIndexSetArray(SetID);
			
			FMeshRenderBufferSet* RenderBuffers = AllocateNewRenderBufferSet();
			RenderBuffers->Material = GetMaterial(0);

			const bool bTrackTriangles = false;
			const bool bParallel = true;

			InitializeBuffersFromOverlays(RenderBuffers, Mesh,
				Tris.Num(), Tris,
				UVOverlay, NormalOverlay, ColorOverlay, TangentsFunc, 
				bTrackTriangles, bParallel);

			ENQUEUE_RENDER_COMMAND(FOctreeDynamicMeshSceneProxyInitializeFromDecomposition)(
				[this, SetID, RenderBuffers](FRHICommandListImmediate& RHICmdList)
			{
				RenderBuffers->Upload();
				RenderBufferSets.Add(SetID, RenderBuffers);
			});
		}
	}



	void UpdateFromDecomposition(const UE::Geometry::FArrayIndexSetsDecomposition& Decomposition, const TArray<int32>& SetsToUpdate )
	{
		// CAN WE REUSE EXISTING BUFFER SETS??
		//   - could have timestamp for each decomposition set array...if tris don't change we only have to update vertices
		//   - can re-use allocated memory if new data is smaller

		SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateFromDecomp);

		// remove sets to update
		ENQUEUE_RENDER_COMMAND(FOctreeDynamicMeshSceneProxyUpdatePreClean)(
			[this, SetsToUpdate](FRHICommandListImmediate& RHICmdList)
		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateDecompDestroy);
			for (int32 SetID : SetsToUpdate)
			{
				if (RenderBufferSets.Contains(SetID))
				{
					FMeshRenderBufferSet* BufferSet = RenderBufferSets.FindAndRemoveChecked(SetID);
					ReleaseRenderBufferSet(BufferSet);
				}
			}
		});

		FDynamicMesh3* Mesh = ParentComponent->GetMesh();

		// find suitable overlays
		FDynamicMeshUVOverlay* UVOverlay = nullptr;
		FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		if (Mesh->HasAttributes())
		{
			UVOverlay = Mesh->Attributes()->PrimaryUV();
			NormalOverlay = Mesh->Attributes()->PrimaryNormals();
			ColorOverlay = Mesh->Attributes()->PrimaryColors();
		}
		auto TangentsFunc = MakeTangentsFunc();

		{
			SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_UpdateDecompCreate);
			int NumSets = SetsToUpdate.Num();
			ParallelFor(NumSets, [&](int k)
			{
				int32 SetID = SetsToUpdate[k];
				const TArray<int32>& Tris = Decomposition.GetIndexSetArray(SetID);

				FMeshRenderBufferSet* RenderBuffers = AllocateNewRenderBufferSet();
				RenderBuffers->Material = GetMaterial(0);

				InitializeBuffersFromOverlays(RenderBuffers, Mesh,
					Tris.Num(), Tris,
					UVOverlay, NormalOverlay, ColorOverlay, TangentsFunc);

				ENQUEUE_RENDER_COMMAND(FOctreeDynamicMeshSceneProxyUpdateAddOne)(
					[this, SetID, RenderBuffers](FRHICommandListImmediate& RHICmdList)
				{
					SCOPE_CYCLE_COUNTER(STAT_SculptToolOctree_BufferUpload);
					RenderBuffers->Upload();
					RenderBufferSets.Add(SetID, RenderBuffers);
				});
			}, false);


		}

		//UE_LOGF(LogTemp, Warning, "Have %d renderbuffers", RenderBufferSets.Num());
	}



public:




	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;

		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

		return Result;
	}

	virtual void UpdatedReferencedMaterials() override
	{
		FBaseDynamicMeshSceneProxy::UpdatedReferencedMaterials();

		// The material relevance may need updating.
		MaterialRelevance = ParentComponent->GetMaterialRelevance(GetScene().GetShaderPlatform());
	}

	virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override
	{
		FPrimitiveSceneProxy::GetLightRelevance(LightSceneProxy, bDynamic, bRelevant, bLightMapped, bShadowMapped);
	}


	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }

	uint32 GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }



	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}
};







