// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"

#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshConversionOptions.h" //FConversionToMeshDescriptionOptions
#include "MeshDescriptionToDynamicMesh.h"

FDynamicMesh3 UE::Geometry::GetDynamicMeshViaMeshDescription(
	IMeshDescriptionProvider& MeshDescriptionProvider, bool bRequestTangents)
{
	FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = bRequestTangents;
	return GetDynamicMeshViaMeshDescription(MeshDescriptionProvider, GetMeshParams);
}

FDynamicMesh3 UE::Geometry::GetDynamicMeshViaMeshDescription(
	IMeshDescriptionProvider& MeshDescriptionProvider,
	const FGetMeshParameters& InGetMeshParams)
{
	FDynamicMesh3 DynamicMesh;
	FMeshDescriptionToDynamicMesh Converter;
	Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
	Converter.bUseCompactedPolygonGroupIDValues = true;
	Converter.SetPolygonGroupToMaterialIndexMap(MeshDescriptionProvider.GetPolygonGroupToMaterialIndexMap());
	if (InGetMeshParams.bWantMeshTangents)
	{
		FMeshDescription MeshDescriptionCopy = MeshDescriptionProvider.GetMeshDescriptionCopy(InGetMeshParams);
		Converter.Convert(&MeshDescriptionCopy, DynamicMesh, InGetMeshParams.bWantMeshTangents);
	}
	else
	{
		Converter.Convert(MeshDescriptionProvider.GetMeshDescription(InGetMeshParams), DynamicMesh);
	}
	return DynamicMesh;
}

void UE::Geometry::CommitDynamicMeshViaMeshDescription(
	FMeshDescription&& CurrentMeshDescription,
	IMeshDescriptionCommitter& MeshDescriptionCommitter, 
	const FDynamicMesh3& Mesh, const IDynamicMeshCommitter::FDynamicMeshCommitInfo& CommitInfo)
{
	FConversionToMeshDescriptionOptions ConversionOptions;
	ConversionOptions.bSetPolyGroups = CommitInfo.bPolygroupsChanged;
	ConversionOptions.bUpdatePositions = CommitInfo.bPositionsChanged;
	ConversionOptions.bUpdateNormals = CommitInfo.bNormalsChanged;
	ConversionOptions.bUpdateTangents = CommitInfo.bTangentsChanged;
	ConversionOptions.bUpdateUVs = CommitInfo.bUVsChanged;
	ConversionOptions.bUpdateVtxColors = CommitInfo.bVertexColorsChanged;
	ConversionOptions.bTransformVtxColorsSRGBToLinear = CommitInfo.bTransformVertexColorsSRGBToLinear;

	FDynamicMeshToMeshDescription Converter(ConversionOptions);
	Converter.SetMaterialIDMapFromInverseMap(MeshDescriptionCommitter.GetPolygonGroupToMaterialIndexMap());
	if (!CommitInfo.bTopologyChanged)
	{
		Converter.UpdateUsingConversionOptions(&Mesh, CurrentMeshDescription);
	}
	else
	{
		// Do a full conversion.
		Converter.Convert(&Mesh, CurrentMeshDescription);
	}

	MeshDescriptionCommitter.CommitMeshDescription(MoveTemp(CurrentMeshDescription));
}