// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/GeometryCacheToDynamicMesh.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "GeometryCache.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "ToDynamicMesh.h"

namespace UE::Conversion
{

// internal helpers
namespace Private
{
	template<typename T, typename CustomInterpF>
	void InterpolateArrayHelperCustomFn(int32 Start, TArrayView<T> UpdateSource, TConstArrayView<T> Toward, CustomInterpF InterpF)
	{
		for (int32 Idx = 0; Idx < Toward.Num(); ++Idx)
		{
			InterpF(UpdateSource[Start + Idx], Toward[Idx]);
		}
	}

	template<typename T>
	void InterpolateArrayHelper(int32 Start, TArrayView<T> UpdateSource, TConstArrayView<T> Toward, float SourceWt, float TowardWt)
	{
		for (int32 Idx = 0; Idx < Toward.Num(); ++Idx)
		{
			UpdateSource[Start + Idx] = UpdateSource[Start + Idx] * SourceWt + Toward[Idx] * TowardWt;
		}
	}

	inline FVector4f ToLinearVec4(const FColor& Color)
	{
		FLinearColor Linear = Color.ReinterpretAsLinear();
		return FVector4f(Linear.R, Linear.G, Linear.B, Linear.A);
	}
}

// Struct for the ToDynamicMesh converter, holding combined flattened buffers of the geometry cache tracks vertex and triangle data
// w/ the interface that TToDynamicMesh expects
struct FGeometryCacheMeshBufferWrapper
{
	// Typedefs expected by TToDynamicMesh
	typedef int32 TriIDType;
	typedef int32 VertIDType;
	typedef int32 WedgeIDType;
	typedef int32 UVIDType;
	typedef int32 NormalIDType;
	typedef int32 ColorIDType;

	// Per vertex data
	TArray<FVector3f> Positions;
	TArray<FPackedNormal> Normals, Tangents;
	TArray<FVector2f> UVs;
	TArray<FColor> Colors;

	// Per triangle data
	TArray<UE::Geometry::FIndex3i> Triangles;
	TArray<int32> MaterialIndices;
	TArray<int32> TriSourceIndices;

	// Note these are currently just identity maps, but the ToDynamicMesh converter template expects them
	TArray<int32> VertIDs, TriIDs;
	TArray<int32> EmptyArray;

	FGeometryCacheMeshBufferWrapper(TConstArrayView<FGeometryCacheMeshData> MeshDataTracks)
	{
		int32 NumV = 0, NumT = 0;
		bool bHasUVs = false;
		bool bHasColors = false;
		bool bHasNormals = false;
		bool bHasTangents = false;
		for (const FGeometryCacheMeshData& Track : MeshDataTracks)
		{
			NumT += Track.Indices.Num() / 3;
			NumV += Track.Positions.Num();

			bHasUVs = bHasUVs || Track.VertexInfo.bHasUV0;
			bHasColors = bHasColors || Track.VertexInfo.bHasColor0;
			bHasNormals = bHasNormals || Track.VertexInfo.bHasTangentZ;
			bHasTangents = bHasTangents || Track.VertexInfo.bHasTangentX;
		}
		Positions.SetNumZeroed(NumV);
		if (bHasNormals)
		{
			Normals.Init(FPackedNormal(FVector3f::ZAxisVector), NumV);
		}
		if (bHasTangents)
		{
			Tangents.Init(FPackedNormal(FVector3f::XAxisVector), NumV);
		}
		if (bHasColors)
		{
			Colors.Init(FColor::White, NumV);
		}
		if (bHasUVs)
		{
			UVs.SetNumZeroed(NumV);
		}

		VertIDs.Reserve(NumV);
		Triangles.Reserve(NumT);
		TriIDs.Reserve(NumT);
		MaterialIndices.SetNumZeroed(NumT);
		TriSourceIndices.SetNumZeroed(NumT);
		
		for (int32 TrackIdx = 0, BaseV = 0, BaseT = 0; TrackIdx < MeshDataTracks.Num(); TrackIdx++)
		{
			const FGeometryCacheMeshData& Track = MeshDataTracks[TrackIdx];


			for (int32 TrackVertIdx = 0; TrackVertIdx < Track.Positions.Num(); ++TrackVertIdx)
			{
				VertIDs.Add(BaseV + TrackVertIdx);
				Positions[BaseV + TrackVertIdx] = Track.Positions[TrackVertIdx];
			}
			if (Track.VertexInfo.bHasUV0)
			{
				for (int32 TrackVertIdx = 0; TrackVertIdx < Track.TextureCoordinates.Num(); ++TrackVertIdx)
				{
					UVs[BaseV + TrackVertIdx] = Track.TextureCoordinates[TrackVertIdx];
				}
			}
			if (Track.VertexInfo.bHasColor0)
			{
				for (int32 TrackVertIdx = 0; TrackVertIdx < Track.Colors.Num(); ++TrackVertIdx)
				{
					Colors[BaseV + TrackVertIdx] = Track.Colors[TrackVertIdx];
				}
			}
			if (Track.VertexInfo.bHasTangentZ)
			{
				for (int32 TrackVertIdx = 0; TrackVertIdx < Track.TangentsZ.Num(); ++TrackVertIdx)
				{
					Normals[BaseV + TrackVertIdx] = Track.TangentsZ[TrackVertIdx];
				}
			}
			if (Track.VertexInfo.bHasTangentX)
			{
				for (int32 TrackVertIdx = 0; TrackVertIdx < Track.TangentsX.Num(); ++TrackVertIdx)
				{
					Tangents[BaseV + TrackVertIdx] = Track.TangentsX[TrackVertIdx];
				}
			}

			
			const int32 TrackNumT = Track.Indices.Num() / 3;
			for (int32 TrackTriWedgeIdx = 0; TrackTriWedgeIdx + 2 < Track.Indices.Num(); TrackTriWedgeIdx += 3)
			{
				TriIDs.Add((int32)Triangles.Emplace((int32)Track.Indices[TrackTriWedgeIdx], (int32)Track.Indices[TrackTriWedgeIdx + 1], (int32)Track.Indices[TrackTriWedgeIdx + 2]));
			}
			for (int32 TriIdx = BaseT, TriEndIdx = BaseT + TrackNumT; TriIdx < TriEndIdx; ++TriIdx)
			{
				TriSourceIndices[TriIdx] = TrackIdx;
			}
			for (const FGeometryCacheMeshBatchInfo& BatchInfo : Track.BatchesInfo)
			{
				for (int32 TrackTriIdx = BatchInfo.StartIndex, EndIdx = BatchInfo.StartIndex + BatchInfo.NumTriangles; TrackTriIdx < EndIdx; ++TrackTriIdx)
				{
					MaterialIndices[BaseT + TrackTriIdx] = BatchInfo.MaterialIndex;
				}
			}

			BaseT += TrackNumT;
			BaseV += Track.Positions.Num();
		}
	}

	int32 NumTris() const
	{
		return TriIDs.Num();
	}
	int32 NumVerts() const
	{
		return VertIDs.Num();
	}

	int32 NumUVLayers() const
	{
		return UVs.IsEmpty() ? 0 : 1;
	}

	const TArray<VertIDType>& GetVertIDs() const
	{
		return VertIDs;
	}

	FVector3d GetPosition(VertIDType VtxID) const
	{
		return static_cast<FVector3d>(Positions[VtxID]);
	}

	const TArray<TriIDType>& GetTriIDs() const
	{
		return TriIDs;
	}

	bool GetTri(TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
	{
		UE::Geometry::FIndex3i Tri = Triangles[(int32)TriID];

		VID0 = Tri.A;
		VID1 = Tri.B;
		VID2 = Tri.C;

		return true;
	}

	int32 GetMaterialIndex(TriIDType TriID) const
	{
		return MaterialIndices[TriID];
	}

	int32 GetTrackIndex(TriIDType TriID) const
	{
		return TriSourceIndices[TriID];
	}

	bool HasNormals() const { return !Normals.IsEmpty(); }
	bool HasTangents() const { return !Tangents.IsEmpty(); }
	bool HasBiTangents() const { return HasNormals() && HasTangents(); }
	bool HasColors() const { return !Colors.IsEmpty(); }

	inline FVector2f GetVertexUV(int32 VID) const
	{
		return UVs[VID];
	}
	inline FVector3f GetVertexNormal(int32 VID) const
	{
		return Normals[VID].ToFVector3f();
	}
	inline FVector3f GetVertexTangent(int32 VID) const
	{
		return Tangents[VID].ToFVector3f();
	}
	inline FVector3f GetVertexBiTangent(int32 VID) const
	{
		const FVector3f TangentX = Tangents[VID].ToFVector3f();
		const FVector4f Normal = Normals[VID].ToFVector4f();
		const float OrientationSign = Normal.W;
		const FVector3f TangentY = (static_cast<FVector3f>(Normal).Cross(TangentX)).GetSafeNormal() * OrientationSign;
		return TangentY;
	}
	inline FVector4f GetVertexColor(int32 VID) const
	{
		return Private::ToLinearVec4(Colors[VID]);
	}

	//
	// Wedge methods just return per-vertex attributes
	// 
	
	void GetWedgeIDs(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const { GetTri(TriID, WID0, WID1, WID2); }
	FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const { return GetVertexUV(WID); }
	FVector3f GetWedgeNormal(WedgeIDType WID) const	{ return GetVertexNormal(WID); }
	FVector3f GetWedgeTangent(WedgeIDType WID) const { return GetVertexTangent(WID); }
	FVector3f GetWedgeBiTangent(WedgeIDType WID) const { return GetVertexBiTangent(WID); }
	FVector4f GetWedgeColor(WedgeIDType WID) const { return GetVertexColor(WID); }

	//
	// No skin weights or bones
	//

	int32 NumSkinWeightAttributes() const { return 0; }
	UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType VertexID) const { checkNoEntry(); return UE::AnimationCore::FBoneWeights(); }
	FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const { checkNoEntry(); return FName(); };
	int32 GetNumBones() const { return 0; }
	FName GetBoneName(int32 BoneIdx) const { checkNoEntry(); return FName(); }
	int32 GetBoneParentIndex(int32 BoneIdx) const { checkNoEntry();	return INDEX_NONE; }
	FTransform GetBonePose(int32 BoneIdx) const { checkNoEntry(); return FTransform::Identity; }
	FVector4f GetBoneColor(int32 BoneIdx) const { checkNoEntry(); return FVector4f::One(); }

	//
	// Shared attribute accessors return per-vertex data
	//

	const TArray<int32>& GetUVIDs(int32 LayerID) const { return NumUVLayers() > 0 ? VertIDs : EmptyArray; }
	FVector2f GetUV(int32 LayerID, UVIDType UVID) const { return GetVertexUV(UVID); }
	bool GetUVTri(int32 LayerID, const TriIDType& TID, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const { GetTri(TID, ID0, ID1, ID2);	return true; }

	const TArray<int32>& GetNormalIDs() const { return HasNormals() ? VertIDs : EmptyArray; }
	FVector3f GetNormal(NormalIDType ID) const { return GetVertexNormal(ID); }
	bool GetNormalTri(const TriIDType& TID, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { GetTri(TID, ID0, ID1, ID2); return true; }

	const TArray<int32>& GetTangentIDs() const { return HasTangents() ? VertIDs : EmptyArray; }
	FVector3f GetTangent(NormalIDType ID) const { return GetVertexTangent(ID); }
	bool GetTangentTri(const TriIDType& TID, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { GetTri(TID, ID0, ID1, ID2); return true; }

	const TArray<int32>& GetBiTangentIDs() const { return HasBiTangents() ? VertIDs : EmptyArray; }
	FVector3f GetBiTangent(NormalIDType ID) const { return GetVertexBiTangent(ID); }
	bool GetBiTangentTri(const TriIDType& TID, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { GetTri(TID, ID0, ID1, ID2); return true; }

	const TArray<int32>& GetColorIDs() const { return HasColors() ? VertIDs : EmptyArray; }
	FVector4f GetColor(ColorIDType ID) const { return GetVertexColor(ID); }
	bool GetColorTri(const TriIDType& TID, ColorIDType& ID0, ColorIDType& ID1, ColorIDType& ID2) const { GetTri(TID, ID0, ID1, ID2); return true; }

	// No weight maps

	int32 NumWeightMapLayers() const { return 0; }
	float GetVertexWeight(int32 WeightMapIndex, int32 SrcVertID) const { return 0; }
	FName GetWeightMapName(int32 WeightMapIndex) const { return FName(); }
};




bool GeometryCacheToDynamicMesh(const UGeometryCache& GeometryCache, Geometry::FDynamicMesh3& MeshOut, const FGeometryCacheToDynamicMeshOptions& Options)
{
	static const IConsoleVariable* InterpFramesCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("GeometryCache.InterpolateFrames"));
	bool bInterpolateFrames = InterpFramesCVar ? InterpFramesCVar->GetBool() : true;
	bool bUseInterpolate = bInterpolateFrames && Options.bAllowInterpolation;

	MeshOut.Clear();

	TArray<FGeometryCacheMeshData> MeshData;
	TArray<int32> SourceTrackIdx;
	// Helper struct to keep track of what frames we need if we're interpolating
	struct FFrameInfo
	{
		int32 FrameIndex = INDEX_NONE, NextFrameIndex = INDEX_NONE;
		float InterpFactor = 0;
	};
	TArray<FFrameInfo> FrameInfos;

	float UseTime = Options.Time;
	// Note: The Track->GetMeshDataAtTime method does not account for looping, so apply the looping beforehand if applicable
	if (Options.bLooping)
	{
		UseTime = GeometyCacheHelpers::WrapAnimationTime(UseTime, GeometryCache.CalculateDuration());
	}

	bool bAnyNonZeroInterp = false;

	for (int32 TrackIdx = 0; TrackIdx < GeometryCache.Tracks.Num(); TrackIdx++)
	{
		FGeometryCacheMeshData TrackMeshData;
		UGeometryCacheTrack* Track = GeometryCache.Tracks[TrackIdx];
		FFrameInfo FrameInfo;
		// GetMeshDataAtTime does not interpolate for us, so if we want to interpolate we need to fetch the relevant sample indices manually
		if (bUseInterpolate)
		{
			// The interface for getting the frames to interpolate is only on UGeometryCacheTrackStreamable, not on the base UGeometryCacheTrack 
			if (UGeometryCacheTrackStreamable* StreamableTrack = Cast<UGeometryCacheTrackStreamable>(Track))
			{
				StreamableTrack->FindSampleIndexesFromTime(UseTime, Options.bLooping, Options.bReversed, FrameInfo.FrameIndex, FrameInfo.NextFrameIndex, FrameInfo.InterpFactor);
			}
		}
		if (FrameInfo.FrameIndex != INDEX_NONE ? Track->GetMeshDataAtSampleIndex(FrameInfo.FrameIndex, TrackMeshData) : Track->GetMeshDataAtTime(UseTime, TrackMeshData))
		{
			if (FrameInfo.InterpFactor != 0)
			{
				bAnyNonZeroInterp = true;
			}
			MeshData.Add(MoveTemp(TrackMeshData));
			SourceTrackIdx.Add(TrackIdx);
			FrameInfos.Add(FrameInfo);
		}
	}

	// Do initial conversion to buffers
	FGeometryCacheMeshBufferWrapper Wrapper(MeshData);

	// If we might need to interpolate, look through tracks for compatible frames to interpolate
	if (bAnyNonZeroInterp)
	{
		for (int32 SourceIdx = 0, BaseV = 0; SourceIdx < SourceTrackIdx.Num(); BaseV += MeshData[SourceIdx++].Positions.Num())
		{
			int32 TrackIdx = SourceTrackIdx[SourceIdx];
			FFrameInfo FrameInfo = FrameInfos[SourceIdx];

			// if FrameInfo indicates we don't need interpolation, skip to next track
			if (FrameInfo.InterpFactor == 0)
			{
				continue;
			}

			float CurWt = 1.f - FrameInfo.InterpFactor;

			bool bTopoCompat = false;
			UGeometryCacheTrack* Track = GeometryCache.Tracks[TrackIdx];

			FGeometryCacheMeshData NextData;
			if (!Track->GetMeshDataAtSampleIndex(FrameInfo.NextFrameIndex, NextData))
			{
				continue;
			}

			// Only interpolate if vertices are 1:1
			// Note the rendering code uses an abstracted IsTopologyCompatible method instead, but we can't access it from here
			// and in practice it is implemented by checking the vertex counts, so we just do that.
			if (NextData.Positions.Num() != MeshData[SourceIdx].Positions.Num())
			{
				continue;
			}

			Private::InterpolateArrayHelper<FVector3f>(BaseV, Wrapper.Positions, NextData.Positions, CurWt, FrameInfo.InterpFactor);
			if (NextData.VertexInfo.bHasUV0)
			{
				Private::InterpolateArrayHelper<FVector2f>(BaseV, Wrapper.UVs, NextData.TextureCoordinates, CurWt, FrameInfo.InterpFactor);
			}

			// Interpolation matching what is implemented in the geometry cache scene proxy
			const VectorRegister4Float WeightA = VectorSetFloat1(CurWt);
			const VectorRegister4Float WeightB = VectorSetFloat1(FrameInfo.InterpFactor);
			const VectorRegister4Float Half = VectorSetFloat1(0.5f);
			const uint32 SignMask = 0x80808080u;
			auto InterpTangentFn = [WeightA, WeightB, Half, SignMask](FPackedNormal& A, const FPackedNormal& B) -> void
			{
				uint32 TangentXA = A.Vector.Packed ^ SignMask;
				uint32 TangentXB = B.Vector.Packed ^ SignMask;
				VectorRegister4Float InterpolatedTangentX = VectorMultiplyAdd(VectorLoadByte4(&TangentXA), WeightA,
					VectorMultiplyAdd(VectorLoadByte4(&TangentXB), WeightB, Half));	// +0.5f so truncation becomes round to nearest.
				uint32 PackedInterpolatedTangentX;
				VectorStoreByte4(InterpolatedTangentX, &PackedInterpolatedTangentX);
				A.Vector.Packed = PackedInterpolatedTangentX ^ SignMask;	// Convert back to signed
			};

			if (NextData.VertexInfo.bHasTangentX)
			{
				Private::InterpolateArrayHelperCustomFn<FPackedNormal>(BaseV, Wrapper.Tangents, NextData.TangentsX, InterpTangentFn);
			}
			if (NextData.VertexInfo.bHasTangentZ)
			{
				Private::InterpolateArrayHelperCustomFn<FPackedNormal>(BaseV, Wrapper.Normals, NextData.TangentsZ, InterpTangentFn);
			}

			auto InterpColorFn = [WeightA, WeightB, Half](FColor& A, const FColor& B) -> void
			{
				VectorRegister4Float InterpolatedColor = VectorMultiplyAdd(VectorLoadByte4(&A), WeightA,
					VectorMultiplyAdd(VectorLoadByte4(&B), WeightB, Half));	// +0.5f so truncation becomes round to nearest.
				VectorStoreByte4(InterpolatedColor, &A);
			};
			if (NextData.VertexInfo.bHasColor0)
			{
				Private::InterpolateArrayHelperCustomFn<FColor>(BaseV, Wrapper.Colors, NextData.Colors, InterpColorFn);
			}
		}
	}

	// Run the standard ToDynamicMesh converter
	auto TriToGroupID = [&Wrapper](const int32& SrcTriID)->int32 { return Wrapper.GetTrackIndex(SrcTriID) + 1; };
	auto TriToMaterialID = [&Wrapper](const int32& SrcTriID)->int32 { return Wrapper.GetMaterialIndex(SrcTriID); };
	UE::Geometry::TToDynamicMesh<FGeometryCacheMeshBufferWrapper> Converter;
	Converter.Convert(MeshOut, Wrapper, TriToGroupID, TriToMaterialID, Options.bWantTangents);

	return true;
}

} // end namespace UE::Conversion

