// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/SceneComponentToDynamicMesh.h"

#include "Engine/StaticMesh.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "UObject/Package.h"

#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "StaticMeshComponentLODInfo.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "ConversionUtils/SkinnedMeshToDynamicMesh.h"
#include "ConversionUtils/SplineComponentDeformDynamicMesh.h"
#include "ConversionUtils/GeometryCacheToDynamicMesh.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "GeometryCacheComponent.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "MeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Physics/ComponentCollisionUtil.h"
#include "PlanarCut.h"
#include "SkeletalMeshOperations.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshLODResourcesToDynamicMesh.h"
#include "StaticMeshOperations.h"
#include "Internationalization/Text.h"

#define LOCTEXT_NAMESPACE "ModelingComponents_SceneComponentToDynamicMesh"

namespace UE 
{
namespace Conversion
{

bool CanConvertSceneComponentToDynamicMesh(USceneComponent* Component)
{
	if (!Component)
	{
		return false;
	}
	else if (const USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(Component))
	{
#if WITH_EDITOR
		const USkinnedAsset* SkinnedAsset = (!SkinnedMeshComponent->IsUnreachable() && SkinnedMeshComponent->IsValidLowLevel()) ? SkinnedMeshComponent->GetSkinnedAsset() : nullptr;
		return SkinnedAsset && !SkinnedAsset->GetOutermost()->bIsCookedForEditor;
#else
		return true;
#endif
	}
	else if (Cast<USplineMeshComponent>(Component))
	{
		return true;
	}
	else if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
#if WITH_EDITOR
		const UStaticMesh* StaticMesh = (!StaticMeshComponent->IsUnreachable() && StaticMeshComponent->IsValidLowLevel()) ? StaticMeshComponent->GetStaticMesh().Get() : nullptr;
		return StaticMesh && !StaticMesh->GetOutermost()->bIsCookedForEditor;
#else
		return true;
#endif
	}
	else if (Cast<UDynamicMeshComponent>(Component))
	{
		return true;
	}
	else if (Cast<UBrushComponent>(Component))
	{
		return true;
	}
	else if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(Component))
	{
#if WITH_EDITOR
		const UGeometryCollection* GeometryCollectionAsset = (!GeometryCollectionComponent->IsUnreachable() && GeometryCollectionComponent->IsValidLowLevel()) ? GeometryCollectionComponent->GetRestCollection() : nullptr;
		return GeometryCollectionAsset && !GeometryCollectionAsset->GetOutermost()->bIsCookedForEditor;
#else
		return true;
#endif
	}
	return false;
}

// Conversion helpers
namespace Private::ConversionHelper
{
	// Static mesh conversion functions (from geometry script MeshAssetFunctions.cpp)
	// TODO: these static mesh conversion helpers should be pulled out to their own StaticMeshToDynamicMesh converter method


	// helper for the material ID remapping used for source LODs
	// note: returns empty array if no remapping needed (or if not WITH_EDITOR)
	TArray<int32> MapSectionToMaterialID(const UStaticMesh* Mesh, int32 SourceLOD, bool bHighResLOD)
	{
#if WITH_EDITOR
		check(Mesh);
		TMap<int32, int32> SectionToMaterial;
		const int32 NumMaterials = Mesh->GetStaticMaterials().Num();
		int32 NumSectionIndex = 0;
		if (bHighResLOD)
		{
			// custom path for HiResSource, where the section info map isn't available so we use mesh description slot names
			// (note that in practice this info seems to be incorrect for some meshes; prefer the section info map where available)
			const FMeshDescription* MeshDescription = Mesh->GetHiResMeshDescription();
			if (!MeshDescription)
			{
				// fall back to empty array (treated as identity map)
				return TArray<int32>();
			}
			const FStaticMeshConstAttributes MeshDescriptionAttributes(*MeshDescription);
			TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
			int32 SectionIndex = 0;
			for (FPolygonGroupID PolygonGroupID : MeshDescription->PolygonGroups().GetElementIDs())
			{
				int32 MaterialIndex = PolygonGroupID >= 0 && PolygonGroupID < MaterialSlotNames.GetNumElements() ? Mesh->GetStaticMaterials().IndexOfByPredicate(
					[&MaterialSlotName = MaterialSlotNames[PolygonGroupID]](const FStaticMaterial& StaticMaterial) { return StaticMaterial.MaterialSlotName == MaterialSlotName; }
					) : INDEX_NONE;
				if (MaterialIndex != INDEX_NONE)
				{
					SectionToMaterial.Add(SectionIndex, MaterialIndex);
				}
				++SectionIndex;
			}
			NumSectionIndex = SectionIndex;
		}
		else
		{
			int32 UseLOD = SourceLOD;
			const FMeshSectionInfoMap& SectionMap = Mesh->GetSectionInfoMap();
			int32 LODSectionNum = SectionMap.GetSectionNumber(UseLOD);
			TArray<int32> Result;
			for (int32 SectionIndex = 0; SectionIndex < LODSectionNum; ++SectionIndex)
			{
				if (SectionMap.IsValidSection(UseLOD, SectionIndex))
				{
					int32 MaterialIndex = SectionMap.Get(UseLOD, SectionIndex).MaterialIndex;
					SectionToMaterial.Add(SectionIndex, MaterialIndex);
				}
			}
			NumSectionIndex = LODSectionNum;
		}

		TArray<int32> Result;
		Result.SetNumUninitialized(NumSectionIndex);
		// Fill in identity mapping first to cover any unmapped indices
		for (int32 Idx = 0; Idx < Result.Num(); ++Idx)
		{
			Result[Idx] = Idx;
		}
		for (TPair<int32, int32> SectionMaterial : SectionToMaterial)
		{
			Result[SectionMaterial.Key] = FMath::Clamp(SectionMaterial.Value, 0, NumMaterials - 1);
		}
		return Result;
#else
		return TArray<int32>();
#endif
	}

	static bool CopyMeshFromStaticMesh_SourceData(
		UStaticMesh* FromStaticMeshAsset,
		FStaticMeshConversionOptions AssetOptions,
		EMeshLODType LODType,
		int32 LODIndex,
		FDynamicMesh3& OutMesh,
		FText& OutErrorMessage
	)
	{
		using namespace ::UE::Geometry;

		bool bSuccess = false;
		OutMesh.Clear();

		if (!FromStaticMeshAsset)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMeshSource_NullMesh", "Static Mesh is null");
			return false;
		}

		if (LODType != EMeshLODType::MaxAvailable && LODType != EMeshLODType::SourceModel && LODType != EMeshLODType::HiResSourceModel)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMesh_LODNotAvailable", "Requested LOD Type is not available");
			return false;
		}

#if WITH_EDITOR
		if (LODType == EMeshLODType::HiResSourceModel && FromStaticMeshAsset->IsHiResMeshDescriptionValid() == false)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMesh_HiResLODNotAvailable", "HiResSourceModel LOD Type is not available");
			return false;
		}

		const FMeshDescription* SourceMesh = nullptr;
		const FMeshBuildSettings* BuildSettings = nullptr;

		TArray<int32> PolygonGroupToMaterialMap = GetPolygonGroupToMaterialIndexMap(FromStaticMeshAsset, LODType, LODIndex);

		if ((LODType == EMeshLODType::HiResSourceModel) ||
			(LODType == EMeshLODType::MaxAvailable && FromStaticMeshAsset->IsHiResMeshDescriptionValid()))
		{
			SourceMesh = FromStaticMeshAsset->GetHiResMeshDescription();
			const FStaticMeshSourceModel& SourceModel = FromStaticMeshAsset->GetHiResSourceModel();
			BuildSettings = &SourceModel.BuildSettings;
		}
		else
		{
			int32 UseLODIndex = FMath::Clamp(LODIndex, 0, FromStaticMeshAsset->GetNumSourceModels() - 1);
			SourceMesh = FromStaticMeshAsset->GetMeshDescription(UseLODIndex);
			const FStaticMeshSourceModel& SourceModel = FromStaticMeshAsset->GetSourceModel(UseLODIndex);
			BuildSettings = &SourceModel.BuildSettings;
		}

		if (SourceMesh == nullptr)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMesh_SourceLODIsNull", "Requested SourceModel LOD is null, only RenderData Mesh is available");
			return false;
		}

		bool bHasDirtyBuildSettings = BuildSettings->bRecomputeNormals
			|| (BuildSettings->bRecomputeTangents && AssetOptions.bRequestTangents);
		bool bNeedsBuildScale = AssetOptions.bUseBuildScale && BuildSettings && !BuildSettings->BuildScale3D.Equals(FVector::OneVector);
		bool bNeedsOtherBuildSettings = AssetOptions.bApplyBuildSettings && bHasDirtyBuildSettings;

		FMeshDescription LocalSourceMeshCopy;
		if (bNeedsBuildScale || bNeedsOtherBuildSettings)
		{
			LocalSourceMeshCopy = *SourceMesh;

			FStaticMeshAttributes Attributes(LocalSourceMeshCopy);

			if (bNeedsBuildScale)
			{
				FTransform BuildScaleTransform = FTransform::Identity;
				BuildScaleTransform.SetScale3D(BuildSettings->BuildScale3D);
				FStaticMeshOperations::ApplyTransform(LocalSourceMeshCopy, BuildScaleTransform, true /*use correct normal transforms*/);
			}

			if (bNeedsOtherBuildSettings)
			{
				if (!Attributes.GetTriangleNormals().IsValid() || !Attributes.GetTriangleTangents().IsValid())
				{
					// If these attributes don't exist, create them and compute their values for each triangle
					FStaticMeshOperations::ComputeTriangleTangentsAndNormals(LocalSourceMeshCopy);
				}

				EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
				ComputeNTBsOptions |= BuildSettings->bRecomputeNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
				if (AssetOptions.bRequestTangents)
				{
					ComputeNTBsOptions |= BuildSettings->bRecomputeTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
					ComputeNTBsOptions |= BuildSettings->bUseMikkTSpace ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;
				}
				ComputeNTBsOptions |= BuildSettings->bComputeWeightedNormals ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;
				if (AssetOptions.bIgnoreRemoveDegenerates == false)
				{
					ComputeNTBsOptions |= BuildSettings->bRemoveDegenerates ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;
				}

				FStaticMeshOperations::ComputeTangentsAndNormals(LocalSourceMeshCopy, ComputeNTBsOptions);
			}

			SourceMesh = &LocalSourceMeshCopy;
		}

		FMeshDescriptionToDynamicMesh Converter;
		Converter.bVIDsFromNonManifoldMeshDescriptionAttr = AssetOptions.bIncludeNonManifoldSrcInfo;
		Converter.bUseCompactedPolygonGroupIDValues = true;
		if (!AssetOptions.bUseSectionMaterialIndices)
		{
			Converter.SetPolygonGroupToMaterialIndexMap(PolygonGroupToMaterialMap);
		}
		Converter.Convert(SourceMesh, OutMesh, AssetOptions.bRequestTangents);

		bSuccess = true;
#else
		OutErrorMessage = LOCTEXT("CopyMeshFromAsset_EditorOnly", "Source Models are not available at Runtime");
#endif

		return bSuccess;
	}



	static bool CopyMeshFromStaticMesh_RenderData(
		UStaticMesh* FromStaticMeshAsset,
		UStaticMeshComponent* StaticMeshComponent,
		FStaticMeshConversionOptions AssetOptions,
		EMeshLODType LODType,
		int32 LODIndex,
		bool bRequestInstanceVertexColors,
		FDynamicMesh3& OutMesh,
		FText& OutErrorMessage
	)
	{
		using namespace ::UE::Geometry;

		OutMesh.Clear();

		if (LODType != EMeshLODType::MaxAvailable && LODType != EMeshLODType::RenderData)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMeshRender_LODNotAvailable", "Requested LOD Type is not available");
			return false;
		}

#if !WITH_EDITOR
		if (FromStaticMeshAsset->bAllowCPUAccess == false)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMesh_CPUAccess", "StaticMesh bAllowCPUAccess must be set to true to read mesh data at Runtime");
			return false;
		}
#endif

		int32 UseLODIndex = FMath::Clamp(LODIndex, 0, FromStaticMeshAsset->GetNumLODs() - 1);

		const FStaticMeshLODResources* LODResources = nullptr;
		if (FStaticMeshRenderData* RenderData = FromStaticMeshAsset->GetRenderData())
		{
			LODResources = &RenderData->LODResources[UseLODIndex];
		}
		if (LODResources == nullptr)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMesh_NoLODResources", "LOD Data is not available");
			return false;
		}

		FStaticMeshLODResourcesToDynamicMesh::ConversionOptions ConvertOptions;
#if WITH_EDITOR
		const bool bIsSourceModelValid = FromStaticMeshAsset->IsSourceModelValid(UseLODIndex);
		if (AssetOptions.bUseBuildScale && bIsSourceModelValid)
		{
			// respect BuildScale build setting
			const FMeshBuildSettings& LODBuildSettings = FromStaticMeshAsset->GetSourceModel(UseLODIndex).BuildSettings;
			ConvertOptions.BuildScale = (FVector3d)LODBuildSettings.BuildScale3D;
		}
		// In case of cooked editor, Source model won't be valid, so it will follow the same rules as the runtime path.
		else if (!AssetOptions.bUseBuildScale && !bIsSourceModelValid)
#else
		if (!AssetOptions.bUseBuildScale)
#endif
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMesh_BuildScaleAlreadyBaked", "Requested mesh without BuildScale, but BuildScale is already baked into the RenderData.");
			return false;
		}

		FStaticMeshLODResourcesToDynamicMesh Converter;
		if (bRequestInstanceVertexColors && StaticMeshComponent && StaticMeshComponent->LODData.IsValidIndex(UseLODIndex))
		{
			FStaticMeshComponentLODInfo* InstanceMeshLODInfo = &StaticMeshComponent->LODData[UseLODIndex];
			const bool bValidInstanceData = InstanceMeshLODInfo
				&& InstanceMeshLODInfo->OverrideVertexColors
				&& InstanceMeshLODInfo->OverrideVertexColors->GetAllowCPUAccess()
				&& InstanceMeshLODInfo->OverrideVertexColors->GetNumVertices() == LODResources->GetNumVertices();
			Converter.Convert(LODResources, ConvertOptions, OutMesh, bValidInstanceData,
				[InstanceMeshLODInfo](int32 LODVID)
				{
					return InstanceMeshLODInfo->OverrideVertexColors->VertexColor(LODVID);
				});
		}
		else
		{
			Converter.Convert(LODResources, ConvertOptions, OutMesh);
		}

		return true;
	}

	static bool CopyMeshFromStaticMesh(
		UStaticMesh* FromStaticMeshAsset,
		UStaticMeshComponent* StaticMeshComponent,
		FStaticMeshConversionOptions AssetOptions,
		EMeshLODType LODType,
		int32 LODIndex,
		bool bUseClosestLOD,
		bool bRequestInstanceVertexColors,
		FDynamicMesh3& OutMesh,
		FText& OutErrorMessage
	)
	{
		if (!FromStaticMeshAsset)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromStaticMeshRender_NullMesh", "Static Mesh is null");
			return false;
		}

		if (bUseClosestLOD)
		{
			// attempt to detect if an unavailable LOD was requested, and if so re-map to an available one
			if (LODType == EMeshLODType::MaxAvailable || LODType == EMeshLODType::HiResSourceModel)
			{
				LODIndex = 0;
			}
#if WITH_EDITOR
			if (LODType == EMeshLODType::MaxAvailable)
			{
				LODType = EMeshLODType::HiResSourceModel;
			}
			if (LODType == EMeshLODType::HiResSourceModel && !FromStaticMeshAsset->IsHiResMeshDescriptionValid())
			{
				LODType = EMeshLODType::SourceModel;
			}
			if (LODType == EMeshLODType::SourceModel)
			{
				LODIndex = FMath::Clamp(LODIndex, 0, FromStaticMeshAsset->GetNumSourceModels() - 1);
				if (!FromStaticMeshAsset->GetSourceModel(LODIndex).IsSourceModelInitialized())
				{
					LODType = EMeshLODType::RenderData;
				}
			}
			if (LODType == EMeshLODType::RenderData)
			{
				LODIndex = FMath::Clamp(LODIndex, 0, FromStaticMeshAsset->GetNumLODs() - 1);
			}
#else
			LODType = EMeshLODType::RenderData;
			LODIndex = FMath::Clamp(LODIndex, 0, FromStaticMeshAsset->GetNumLODs() - 1);
#endif
		}

		if (LODType == EMeshLODType::RenderData)
		{
			return CopyMeshFromStaticMesh_RenderData(FromStaticMeshAsset, StaticMeshComponent, AssetOptions, LODType, LODIndex, bRequestInstanceVertexColors, OutMesh, OutErrorMessage);
		}
		else
		{
			return CopyMeshFromStaticMesh_SourceData(FromStaticMeshAsset, AssetOptions, LODType, LODIndex, OutMesh, OutErrorMessage);
		}
	}
	
	
	static bool CopyMeshFromSkinnedAsset(
		USkinnedAsset* FromSkinnedAsset,
		USkinnedMeshComponent* SkinnedMeshComponent,
		EMeshLODType LODType,
		int32 LODIndex,
		bool bUseClosestLOD,
		bool bWantTangents,
		FDynamicMesh3& OutMesh,
		FText& OutErrorMessage
	)
	{
		if (!FromSkinnedAsset)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromSkinnedAsset_NullMesh", "Skinned mesh is null");
			return false;
		}

		USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(FromSkinnedAsset);

		// If using non-skeletal mesh variations of skinned meshes, just go straight to render data.
		if (!SkeletalMesh)
		{
			LODType = EMeshLODType::RenderData;
		}

		if (bUseClosestLOD)
		{
			// attempt to detect if an unavailable LOD was requested, and if so re-map to an available one
			if (LODType == EMeshLODType::MaxAvailable || LODType == EMeshLODType::HiResSourceModel)
			{
				LODIndex = 0;
			}
#if WITH_EDITOR
			if (LODType == EMeshLODType::MaxAvailable || LODType == EMeshLODType::HiResSourceModel)
			{
				LODType = EMeshLODType::SourceModel;
			}
			if (LODType == EMeshLODType::SourceModel)
			{
				LODIndex = FMath::Clamp(LODIndex, 0, SkeletalMesh->GetNumSourceModels() - 1);
				if (!SkeletalMesh->GetSourceModel(LODIndex).HasMeshDescription())
				{
					LODType = EMeshLODType::RenderData;
				}
			}
			if (LODType == EMeshLODType::RenderData)
			{
				LODIndex = FMath::Clamp(LODIndex, 0, FromSkinnedAsset->GetLODNum() - 1);
			}
#else
			LODType = EMeshLODType::RenderData;
			LODIndex = FMath::Clamp(LODIndex, 0, FromSkinnedAsset->GetLODNum() - 1);
#endif
		}

		if (LODType == EMeshLODType::RenderData)
		{
			return SkinnedMeshComponentToDynamicMesh(*SkinnedMeshComponent, OutMesh, LODIndex, bWantTangents);
		}
		else
		{
#if WITH_EDITOR
			const FMeshDescription* SourceMesh = nullptr;

			// Check first if we have bulk data available and non-empty.
			if (SkeletalMesh->HasMeshDescription(LODIndex))
			{
				SourceMesh = SkeletalMesh->GetMeshDescription(LODIndex); 
			}
			if (SourceMesh == nullptr)
			{
				OutErrorMessage = LOCTEXT("CopyMeshFromSkinnedAsset_LODNotAvailable", "Requested LOD source mesh is not available");
				return false;
			}

			TMap<FName, float> MorphTargetWeights;

			for (const TPair<const UMorphTarget*, int32>& MorphTarget: SkinnedMeshComponent->ActiveMorphTargets)
			{
				const FName MorphName = MorphTarget.Key->GetFName();
				const float MorphWeight = SkinnedMeshComponent->MorphTargetWeights[MorphTarget.Value];

				MorphTargetWeights.Add(MorphName, MorphWeight);
			}
			
			const TArray<FTransform>& ComponentSpaceTransforms = SkinnedMeshComponent->GetComponentSpaceTransforms();
			FMeshDescription DeformedMesh;
			if (!FSkeletalMeshOperations::GetPosedMesh(*SourceMesh, DeformedMesh, ComponentSpaceTransforms, NAME_None, MorphTargetWeights))
			{
				OutErrorMessage = LOCTEXT("CopyMeshFromSkinnedAsset_CannotPose", "Unable to pose the source mesh");
				return false;
			}
			
			FDynamicMesh3 NewMesh;
			FMeshDescriptionToDynamicMesh Converter;

			// Leave this on, since the set morph target node uses this. 
			Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
			
			Converter.Convert(&DeformedMesh, OutMesh, bWantTangents);
		
			return true;
#else
			OutErrorMessage = LOCTEXT("CopyMeshFromSkinnedAsset_EditorOnly", "Source Models are not available at Runtime");
			return false;
#endif
		}
	}
	
}

TArray<int32> GetPolygonGroupToMaterialIndexMap(const UStaticMesh* StaticMesh, EMeshLODType LODType, int32 LODIndex)
{
#if WITH_EDITOR
	if (LODType == EMeshLODType::RenderData)
	{
		// don't need to remap material indices for render LODs
		return TArray<int32>();
	}
	// map the 'max available' lod type
	if (LODType == EMeshLODType::MaxAvailable)
	{
		LODType = StaticMesh->IsHiResMeshDescriptionValid() ? EMeshLODType::HiResSourceModel : EMeshLODType::SourceModel;
		LODIndex = 0;
	}
	return Private::ConversionHelper::MapSectionToMaterialID(StaticMesh, LODIndex, LODType == EMeshLODType::HiResSourceModel);
#else
	return TArray<int32>();
#endif
}

bool StaticMeshToDynamicMesh(UStaticMesh* InMesh, Geometry::FDynamicMesh3& OutMesh, FText& OutErrorMessage,
	const FStaticMeshConversionOptions& ConversionOptions, EMeshLODType LODType, int32 LODIndex, bool bUseClosestLOD)
{
	constexpr UStaticMeshComponent* StaticMeshComponent = nullptr; // ok to leave this null when converting from asset
	constexpr bool bRequestInstanceVertexColors = false; // cannot request instance colors from the asset
	return Private::ConversionHelper::CopyMeshFromStaticMesh(
		InMesh, StaticMeshComponent, ConversionOptions, LODType, LODIndex, bUseClosestLOD, bRequestInstanceVertexColors, OutMesh, OutErrorMessage);
}

bool SceneComponentToDynamicMesh(USceneComponent* Component, const FToMeshOptions& Options, bool bTransformToWorld, 
	Geometry::FDynamicMesh3& OutMesh, FTransform& OutLocalToWorld, FText& OutErrorMessage,
	TArray<UMaterialInterface*>* OutComponentMaterials, TArray<UMaterialInterface*>* OutAssetMaterials)
{
	using namespace ::UE::Geometry;

	bool bSuccess = false;
	OutMesh.Clear();

	if (!Component)
	{
		OutErrorMessage = LOCTEXT("CopyMeshFromComponent_NullComponent", "Scene Component is null");
		return false;
	}
	OutLocalToWorld = Component->GetComponentTransform();

	auto GetPrimitiveComponentMaterials = [](UPrimitiveComponent* PrimComp, TArray<UMaterialInterface*>& Materials)
	{
		int32 NumMaterials = PrimComp->GetNumMaterials();
		Materials.SetNum(NumMaterials);
		for (int32 k = 0; k < NumMaterials; ++k)
		{
			Materials[k] = PrimComp->GetMaterial(k);
		}
	};

	// if Component Materials were requested, try to get them generically off the primitive component
	// Note: Currently all supported types happen to be primitive components as well; will need to update if this changes
	if (OutComponentMaterials)
	{
		OutComponentMaterials->Empty();
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			GetPrimitiveComponentMaterials(PrimComp, *OutComponentMaterials);
		}
	}


	if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(Component))
	{
		const int32 NumLODs = SkinnedMeshComponent->GetNumLODs();
		int32 RequestedLOD = Options.LODType == EMeshLODType::MaxAvailable ? 0 : Options.LODIndex;
		if (Options.bUseClosestLOD)
		{
			RequestedLOD = FMath::Clamp(RequestedLOD, 0, NumLODs - 1);
		}
		if (RequestedLOD < 0 || RequestedLOD > NumLODs - 1)
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingSkinnedMeshComponentLOD", "SkinnedMeshComponent requested LOD does not exist");
		}
		else
		{
			if (USkinnedAsset* SkinnedAsset = SkinnedMeshComponent->GetSkinnedAsset())
			{
				bSuccess = Private::ConversionHelper::CopyMeshFromSkinnedAsset(SkinnedAsset, SkinnedMeshComponent, Options.LODType, Options.LODIndex, Options.bUseClosestLOD, Options.bWantTangents, OutMesh, OutErrorMessage);
				if (bSuccess)
				{
					OutMesh.DiscardTriangleGroups();

					if (OutAssetMaterials)
					{
						const TArray<FSkeletalMaterial>& Materials = SkinnedAsset->GetMaterials();
						OutAssetMaterials->SetNum(Materials.Num());
						for (int32 k = 0; k < Materials.Num(); ++k)
						{
							(*OutAssetMaterials)[k] = Materials[k].MaterialInterface;
						}
					}
				}
			}
			else
			{
				OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingSkinnedAsset", "SkinnedMeshComponent has a null SkinnedAsset");
			}
		}
	}
	else if (USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(Component))
	{
		UStaticMesh* StaticMesh = SplineMeshComponent->GetStaticMesh();
		if (StaticMesh)
		{
			FStaticMeshConversionOptions AssetOptions;
			AssetOptions.bApplyBuildSettings = (Options.bWantNormals || Options.bWantTangents);
			AssetOptions.bRequestTangents = Options.bWantTangents;
			bSuccess = Private::ConversionHelper::CopyMeshFromStaticMesh(
				StaticMesh, SplineMeshComponent, AssetOptions, Options.LODType, Options.LODIndex, Options.bUseClosestLOD, Options.bWantInstanceColors, OutMesh, OutErrorMessage);

			// deform the dynamic mesh and its tangent space with the spline
			if (bSuccess)
			{
				const bool bUpdateTangentSpace = Options.bWantTangents;
				SplineDeformDynamicMesh(*SplineMeshComponent, OutMesh, bUpdateTangentSpace);

				if (OutAssetMaterials)
				{
					int32 NumMaterials = StaticMesh->GetStaticMaterials().Num();
					OutAssetMaterials->SetNum(NumMaterials);
					for (int32 k = 0; k < NumMaterials; ++k)
					{
						(*OutAssetMaterials)[k] = StaticMesh->GetMaterial(k);
					}
				}
			}
		}
		else
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromSplineMeshComponent_MissingStaticMesh", "SplineMeshComponent has a null StaticMesh");
		}
	}
	else if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh)
		{
			FStaticMeshConversionOptions AssetOptions;
			AssetOptions.bApplyBuildSettings = (Options.bWantNormals || Options.bWantTangents);
			AssetOptions.bRequestTangents = Options.bWantTangents;
			bool bRequestInstanceVertexColors = Options.bWantInstanceColors;
			bSuccess = Private::ConversionHelper::CopyMeshFromStaticMesh(
				StaticMesh, StaticMeshComponent, AssetOptions, Options.LODType, Options.LODIndex, Options.bUseClosestLOD, bRequestInstanceVertexColors, OutMesh, OutErrorMessage);

			// if we have an ISMC, append instances
			if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
			{
				FDynamicMesh3 InstancedMesh = MoveTemp(OutMesh);
				OutMesh.Clear();

				FDynamicMesh3 AccumMesh;
				AccumMesh.EnableMatchingAttributes(InstancedMesh);
				FDynamicMeshEditor Editor(&AccumMesh);
				FMeshIndexMappings Mappings;

				int32 NumInstances = ISMComponent->GetInstanceCount();
				for (int32 InstanceIdx = 0; InstanceIdx < NumInstances; ++InstanceIdx)
				{
					if (ISMComponent->IsValidInstance(InstanceIdx))
					{
						FTransform InstanceTransform;
						ISMComponent->GetInstanceTransform(InstanceIdx, InstanceTransform, /*bWorldSpace=*/false);
						FTransformSRT3d XForm(InstanceTransform);

						Mappings.Reset();
						Editor.AppendMesh(&InstancedMesh, Mappings,
							[&](int, const FVector3d& Position) { return XForm.TransformPosition(Position); },
							[&](int, const FVector3d& Normal) { return XForm.TransformNormal(Normal); });
					}
				}

				OutMesh = MoveTemp(AccumMesh);
			}

			if (OutAssetMaterials)
			{
				int32 NumMaterials = StaticMesh->GetStaticMaterials().Num();
				OutAssetMaterials->SetNum(NumMaterials);
				for (int32 k = 0; k < NumMaterials; ++k)
				{
					(*OutAssetMaterials)[k] = StaticMesh->GetMaterial(k);
				}
			}
		}
		else
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingStaticMesh", "StaticMeshComponent has a null StaticMesh");
		}
	}
	else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		UDynamicMesh* CopyDynamicMesh = DynamicMeshComponent->GetDynamicMesh();
		if (CopyDynamicMesh)
		{
			CopyDynamicMesh->ProcessMesh([&](const FDynamicMesh3& Mesh)
			{
				OutMesh = Mesh;
			});
			bSuccess = true;
		}
		else
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingDynamicMesh", "DynamicMeshComponent has a null DynamicMesh");
		}
	}
	else if (UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		FVolumeToMeshOptions VolOptions;
		VolOptions.bMergeVertices = true;
		VolOptions.bAutoRepairMesh = true;
		VolOptions.bOptimizeMesh = true;
		VolOptions.bSetGroups = true;

		OutMesh.EnableTriangleGroups();
		BrushComponentToDynamicMesh(BrushComponent, OutMesh, VolOptions);

		// compute normals for current polygroup topology
		OutMesh.EnableAttributes();
		if (Options.bWantNormals)
		{
			FDynamicMeshNormalOverlay* Normals = OutMesh.Attributes()->PrimaryNormals();
			FMeshNormals::InitializeOverlayTopologyFromFaceGroups(&OutMesh, Normals);
			FMeshNormals::QuickRecomputeOverlayNormals(OutMesh);
		}

		if (OutMesh.TriangleCount() > 0)
		{
			bSuccess = true;
		}
		else
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromComponent_InvalidBrushConversion", "BrushComponent conversion produced 0 triangles");
		}
	}
	else if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(Component))
	{
		if (const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection())
		{
			if (const FGeometryCollection* Collection = RestCollection->GetGeometryCollection().Get())
			{
				FTransform UnusedTransform;
				const TArray<FTransform3f>& DynamicTransforms = GeometryCollectionComponent->GetComponentSpaceTransforms3f();
				if (!DynamicTransforms.IsEmpty())
				{
					ConvertGeometryCollectionToDynamicMesh(OutMesh, UnusedTransform, false, *Collection, true, DynamicTransforms, false, Collection->TransformIndex.GetConstArray());
				}
				else
				{
					ConvertGeometryCollectionToDynamicMesh(OutMesh, UnusedTransform, false, *Collection, true, TArrayView<const FTransform3f>(Collection->Transform.GetConstArray()), true, Collection->TransformIndex.GetConstArray());
				}
				bSuccess = true;

				if (OutAssetMaterials)
				{
					//const TArray<TObjectPtr<UMaterialInterface>>& AssetMaterials = RestCollection->Materials;
					*OutAssetMaterials = RestCollection->Materials;
				}
			}
			else
			{
				OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingCollectionData", "GeometryCollectionComponent has null Geometry Collection data");
			}
		}
		else
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingRestCollection", "GeometryCollectionComponent has null Rest Collection object");
		}
	}
	else if (UGeometryCacheComponent* GeometryCacheComponent = Cast<UGeometryCacheComponent>(Component))
	{
		if (UGeometryCache* GeometryCache = GeometryCacheComponent->GetGeometryCache())
		{
			UE::Conversion::FGeometryCacheToDynamicMeshOptions GeometryCacheOptions;
			GeometryCacheOptions.Time = GeometryCacheComponent->GetAnimationTime();
			GeometryCacheOptions.bLooping = GeometryCacheComponent->IsLooping();
			GeometryCacheOptions.bReversed = GeometryCacheComponent->IsPlayingReversed();
			GeometryCacheOptions.bAllowInterpolation = true;
			GeometryCacheOptions.bWantTangents = Options.bWantTangents;
			bSuccess = UE::Conversion::GeometryCacheToDynamicMesh(*GeometryCache, OutMesh, GeometryCacheOptions);
			if (!bSuccess)
			{
				OutErrorMessage = LOCTEXT("CopyMeshFromComponent_GeometryCacheComponentFailed", "Conversion from Geometry Cache to Dynamic Mesh failed");
			}
		}
		else
		{
			OutErrorMessage = LOCTEXT("CopyMeshFromComponent_MissingGeometryCache", "GeometryCacheComponent has null Geometry Cache object");
		}
	}
	else
	{
		OutErrorMessage = FText::FormatOrdered(LOCTEXT("CopyMeshFromComponent_UnsupportedComponentType", "Scene Component \"{0}\" has unsupported type"), FText::FromName(Component->GetFName()));
	}

	// transform mesh to world
	if (bSuccess && bTransformToWorld)
	{
		MeshTransforms::ApplyTransform(OutMesh, (FTransformSRT3d)OutLocalToWorld, true);
	}

	return bSuccess;
}

} // end namespace Conversion
} // end namespace UE


#undef LOCTEXT_NAMESPACE

