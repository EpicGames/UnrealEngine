// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/SkinnedMeshToDynamicMesh.h"


#include "Components/SkinnedMeshComponent.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/SkinnedAsset.h"
#include "SkeletalRenderPublic.h"
#include "ToDynamicMesh.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "BoneWeights.h"
#include "SkeletalMeshAttributes.h"

namespace UE 
{
namespace Conversion
{


// Wrapper that allows us to use ToDynamicMesh converter
class FSkinnedComponentWrapper
{
public:

	typedef int32 TriIDType;
	typedef int32 VertIDType;
	typedef int32 WedgeIDType;

	typedef int32 UVIDType;
	typedef int32 NormalIDType;
	typedef int32 ColorIDType;


	FSkinnedComponentWrapper(USkinnedMeshComponent& SkinnedMeshComponent, int32 RequestedLOD, bool UseDisabledSections, bool bWantTangents)
	: Component(&SkinnedMeshComponent)
	, LOD(RequestedLOD)
	, bUseDisabledSections(UseDisabledSections)
	, bHasTangents(bWantTangents)
	, Asset(SkinnedMeshComponent.GetSkinnedAsset())
	, SkeletalMeshRenderData(SkinnedMeshComponent.MeshObject->GetSkeletalMeshRenderData())
	{
		const int32 MinLODIndex = SkinnedMeshComponent.ComputeMinLOD();
		const int32 MaxLODIndex = SkinnedMeshComponent.GetNumLODs() - 1;
		LOD = FMath::Clamp<int32>(RequestedLOD, MinLODIndex, MaxLODIndex);
		if (LOD != RequestedLOD)
		{
			UE_LOGF(LogGeometry, Warning, "Requesting LOD %d within [%d, %d], clamping to %d", RequestedLOD, MinLODIndex, MaxLODIndex, LOD);
		}
		
		SrcLODInfo = Asset->GetLODInfo(LOD);
		const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData.LODRenderData[LOD];
		IndexBuffer = LODData.MultiSizeIndexContainer.GetIndexBuffer();
		
		// This step does the actual skinning (and isn't as const as the api suggests..)
		Component->GetCPUSkinnedVertices(SkinnedVertices, LOD);

		const int32 NumSections = LODData.RenderSections.Num();
		
		auto SkipSection = [&](const FSkelMeshRenderSection& SkelMeshSection)
							{
								return ( !bUseDisabledSections && !SkinnedMeshComponent.IsMaterialSectionShown(SkelMeshSection.MaterialIndex, LOD));
							};

		// pre-compute number of tris & verts
		int32 NumTris = 0;
		int32 NumVerts = 0;		
		{
					
			for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
			{
				const FSkelMeshRenderSection& SkelMeshSection = LODData.RenderSections[SectionIndex];

				if (SkipSection(SkelMeshSection))
				{
					continue;
				}
				NumTris += SkelMeshSection.NumTriangles;
				NumVerts += SkelMeshSection.NumVertices;
			}
		}
		
		// construct a list of all valid VertIDs for this mesh.
		bool bLogMeshInfoDueToErrors = false;
		
		VertIDs.Reserve(NumVerts);
		for (const FSkelMeshRenderSection& Section : LODData.RenderSections)
		{
			if (SkipSection(Section))
			{
				continue;
			}

			const int32 BaseVertexIndex = static_cast<int32>( Section.BaseVertexIndex);
			const int32 NumSectionVtx = static_cast<int32>( Section.NumVertices );
			for (int32 VtxIndex = BaseVertexIndex; VtxIndex < NumSectionVtx + BaseVertexIndex; ++VtxIndex)
			{
				if (ensure(VtxIndex < SkinnedVertices.Num()))
				{
					VertIDs.Add(VtxIndex);
				}
				else 
				{
					bLogMeshInfoDueToErrors = true;
				}
			}
		}

		if (bLogMeshInfoDueToErrors)
		{
			UE_LOGF(LogGeometry, Display, "LOD vertex mismatch with [%ls -> %ls]", *SkinnedMeshComponent.GetPathName(), *SkinnedMeshComponent.GetSkinnedAsset()->GetPathName());
			UE_LOGF(LogGeometry, Display, " - Total Vertex Count: %d", SkinnedVertices.Num());
			UE_LOGF(LogGeometry, Display, " - Render Data Vertex Count: %d", LODData.StaticVertexBuffers.PositionVertexBuffer.GetNumVertices());
			UE_LOGF(LogGeometry, Display, " - Total Index/Triangle Count: %d / %d ", IndexBuffer->Num(), IndexBuffer->Num() / 3);
		 
			for (int32 SectionIndex = 0; SectionIndex < LODData.RenderSections.Num(); SectionIndex++)
			{
				const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionIndex];
				UE_LOGF(LogGeometry, Display, " - Section[%d]: Skip? %ls", SectionIndex, SkipSection(Section) ? TEXT("Yes") : TEXT("No"));
				UE_LOGF(LogGeometry, Display, " - - Vertex Base: %d - Vertex Count: %d", Section.BaseVertexIndex, Section.NumVertices);
				UE_LOGF(LogGeometry, Display, " - - Index Base: %d - Triangle Base: %d - Triangle Count: %d",
					Section.BaseIndex, Section.BaseIndex / 3, Section.NumTriangles);
			}
		}

		// generate vertex weights and remap the indices
		const FSkinWeightVertexBuffer* SkinWeightBuffer = Component->GetSkinWeightBuffer(LOD);
		const bool bSkinWeightsValid = SkinWeightBuffer && SkinWeightBuffer->GetDataVertexBuffer() && SkinWeightBuffer->GetDataVertexBuffer()->IsInitialized() && SkinWeightBuffer->GetDataVertexBuffer()->GetNumVertices() > 0;

		if (bSkinWeightsValid)
		{
			SkinWeights.Reserve(NumVerts);
			for (const FSkelMeshRenderSection& Section : LODData.RenderSections)
			{
				if (SkipSection(Section))
				{
					continue;
				}

				const int32 BaseVertexIndex = static_cast<int32>(Section.BaseVertexIndex);
				const int32 NumSectionVtx = static_cast<int32>(Section.NumVertices);
				for (int32 VtxIndex = BaseVertexIndex; VtxIndex < NumSectionVtx + BaseVertexIndex; ++VtxIndex)
				{
					FSkinWeightInfo SrcWeights = SkinWeightBuffer->GetVertexSkinWeights(VtxIndex);

					for (int32 Idx = 0; Idx < MAX_TOTAL_INFLUENCES; ++Idx)
					{
						const int32 SectionBoneIdx = static_cast<int32>(SrcWeights.InfluenceBones[Idx]);
						if (ensure(SectionBoneIdx < Section.BoneMap.Num()))
						{
							SrcWeights.InfluenceBones[Idx] = Section.BoneMap[SectionBoneIdx];
						}
					}
					SkinWeights.Add(SrcWeights);
				}
			}
		}

		// construct list of all valid TriIDs for this mesh.  
		TriIDs.Reserve(NumTris);
		TriIDtoSectionID.Init(-1, NumTris);
		for (int32 s = 0; s < NumSections; ++s)
		{
			const FSkelMeshRenderSection& Section = LODData.RenderSections[s];
			if (SkipSection(Section))
			{
				continue; 
			}
			const int32 BaseIndex = Section.BaseIndex;
			const int32 BaseTriIdx = BaseIndex / 3;
			const int32 NumSectionTris = static_cast<int32>(Section.NumTriangles);
			for (int32 j = BaseTriIdx; j < BaseTriIdx + NumSectionTris; ++j)
			{
				TriIDs.Add(j);
				TriIDtoSectionID[j] = s;
			}
			
		}
	}

	int32 NumTris() const
	{
		return TriIDs.Num();
	}
	int32 NumVerts() const
	{
		return VertIDs.Num();
	}

	int32 NumUVLayers() const
	{
		const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData.LODRenderData[LOD];
		return static_cast<int32>( LODData.StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords() );
	}

	// --"Vertex Buffer" info
	const TArray<VertIDType>& GetVertIDs() const
	{
		return VertIDs;
	}

	FVector3d GetPosition(VertIDType VtxID) const
	{
		return static_cast<FVector3d>(SkinnedVertices[VtxID].Position);
	}

	// --"Index Buffer" info
	const TArray<TriIDType>& GetTriIDs() const
	{
		return TriIDs;
	}

	bool GetTri(TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
	{
		int32 Offset = TriID * 3;

		// index buffer for this LOD.
		const FRawStaticIndexBuffer16or32Interface& Indices = *IndexBuffer;
		auto GetVID = [&Indices](int32 i){ return static_cast<VertIDType>(Indices.Get(i));};

		VID0 = GetVID( Offset    );
		VID1 = GetVID( Offset + 1);
		VID2 = GetVID( Offset + 2);

		// maybe check IndexBuffer.Num() > Offset+2 ? 
		return true;
	}


	bool HasNormals() const { return true; }
	bool HasTangents() const { return bHasTangents; }
	bool HasBiTangents() const { return bHasTangents; }
	
	bool HasColors() const
	{
		const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData.LODRenderData[LOD];
		return (LODData.StaticVertexBuffers.ColorVertexBuffer.IsInitialized() && LODData.StaticVertexBuffers.ColorVertexBuffer.GetNumVertices() > 0);
	}

	//-- Access to per-wedge attributes --//
	void GetWedgeIDs(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const
	{
		int32 Offset = 3 * TriID;
		WID0 = Offset;
		WID1 = Offset + 1;
		WID2 = Offset + 2;
	}

	FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const
	{
		const FFinalSkinVertex& SkinnedVertex = GetWedgeVertexInstance(WID);
		return static_cast<FVector2f>( SkinnedVertex.TextureCoordinates[UVLayerIndex] );
	}

	FVector3f GetWedgeNormal(WedgeIDType WID) const
	{
		const FFinalSkinVertex& SkinnedVertex = GetWedgeVertexInstance(WID);
		return SkinnedVertex.TangentZ.ToFVector3f();
	}

	FVector3f GetWedgeTangent(WedgeIDType WID) const
	{
		const FFinalSkinVertex& SkinnedVertex = GetWedgeVertexInstance(WID);
		return SkinnedVertex.TangentX.ToFVector3f();
	}

	FVector3f GetWedgeBiTangent(WedgeIDType WID) const
	{
		const FFinalSkinVertex& SkinnedVertex = GetWedgeVertexInstance(WID);
		const FVector3f TangentX = SkinnedVertex.TangentX.ToFVector3f();
		const FVector3f TangentZ = SkinnedVertex.TangentZ.ToFVector3f();
		const float OrientationSign = SkinnedVertex.TangentZ.ToFVector4f().W;
		const FVector3f TangentY = (TangentZ.Cross(TangentX)).GetSafeNormal() * OrientationSign;
		return TangentY;
	}

	FVector4f GetWedgeColor(WedgeIDType WID) const
	{
		FLinearColor LinearColor = FLinearColor::White;
		if (HasColors())
		{ 
			const int32 VID = GetVertID(WID);
			const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData.LODRenderData[LOD];
			const FColor& Color = LODData.StaticVertexBuffers.ColorVertexBuffer.VertexColor(VID);
			LinearColor = Color.ReinterpretAsLinear();
		}
		return FVector4f(LinearColor.R, LinearColor.G, LinearColor.B, LinearColor.A);
	}


	int32 GetMaterialIndex(TriIDType TriID) const
	{
		int32 SectionID = TriIDtoSectionID[TriID];
		check(SectionID > -1);

		const FSkeletalMeshLODRenderData& LODData = SkeletalMeshRenderData.LODRenderData[LOD];
		const FSkelMeshRenderSection& Section = LODData.RenderSections[SectionID];
		
		int32 MaterialIndex = Section.MaterialIndex;
		// use the remapping of material indices if there is a valid value
		if (SrcLODInfo->LODMaterialMap.IsValidIndex(SectionID) && SrcLODInfo->LODMaterialMap[SectionID] != INDEX_NONE)
		{
			MaterialIndex = FMath::Clamp<int32>(SrcLODInfo->LODMaterialMap[SectionID], 0, Asset->GetMaterials().Num() - 1);
		}

		return MaterialIndex;
	}

	int32 NumSkinWeightAttributes() const 
	{ 
		const bool bSkinWeightsValid = SkinWeights.Num() == SkinnedVertices.Num();

		return static_cast<int32>(bSkinWeightsValid);
	}

	UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType VertexID) const
	{
		checkfSlow(SkinWeightAttributeIndex == 0, TEXT("USkinnedMeshComponent can have at most one skin weight attribute"));
		checkfSlow(VertexID < SkinWeights.Num() && VertexID >= 0, TEXT("Invalid vertex id"));

		const FSkinWeightInfo& WeightInfo = SkinWeights[VertexID];

		const UE::AnimationCore::FBoneWeights Weights = UE::AnimationCore::FBoneWeights::Create(WeightInfo.InfluenceBones, WeightInfo.InfluenceWeights);

		return Weights;
	}

	FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const
	{
		checkfSlow(SkinWeightAttributeIndex == 0, TEXT("USkinnedMeshComponent can have at most one skin weight attribute"));
		
		return FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
	};
	

	int32 GetNumBones() const 
	{ 
		return Asset->GetRefSkeleton().GetRawBoneNum(); 
	}

    FName GetBoneName(int32 BoneIdx) const 
	{
		if (ensure(BoneIdx >= 0 && BoneIdx < GetNumBones()))
		{
			return Asset->GetRefSkeleton().GetRawRefBoneInfo()[BoneIdx].Name;
		}

		return NAME_None;
	}

	int32 GetBoneParentIndex(int32 BoneIdx) const
	{
		if (ensure(BoneIdx >= 0 && BoneIdx < GetNumBones()))
		{
			return Asset->GetRefSkeleton().GetRawRefBoneInfo()[BoneIdx].ParentIndex;
		}

		return INDEX_NONE;
	}

	FTransform GetBonePose(int32 BoneIdx) const
	{
		if (ensure(BoneIdx >= 0 && BoneIdx < GetNumBones()))
		{
			return Asset->GetRefSkeleton().GetRawRefBonePose()[BoneIdx];
		}

		return FTransform::Identity;
	}

	FVector4f GetBoneColor(int32 BoneIdx) const
	{
		return FVector4f::One();
	}

	//-- null implementation of shared attributes: Skeletal mesh model doesn't use these --//
	const TArray<int32>& GetUVIDs(int32 LayerID) const { return EmptyArray; }
	FVector2f GetUV(int32 LayerID, UVIDType UVID) const { check(0); return FVector2f(); }
	bool GetUVTri(int32 LayerID, const TriIDType&, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const { ID0 = ID1 = ID2 = UVIDType(-1); return false; }

	const TArray<int32>& GetNormalIDs() const { return EmptyArray; }
	FVector3f GetNormal(NormalIDType ID) const { check(0); return FVector3f(); }
	bool GetNormalTri(const TriIDType&, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { ID0 = ID1 = ID2 = NormalIDType(-1); return false; }

	const TArray<int32>& GetTangentIDs() const { return EmptyArray; }
	FVector3f GetTangent(NormalIDType ID) const { check(0); return FVector3f(); }
	bool GetTangentTri(const TriIDType&, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { ID0 = ID1 = ID2 = NormalIDType(-1); return false; }


	const TArray<int32>& GetBiTangentIDs() const { return EmptyArray; }
	FVector3f GetBiTangent(NormalIDType ID) const { check(0); return FVector3f(); }
	bool GetBiTangentTri(const TriIDType&, NormalIDType& ID0, NormalIDType& ID1, NormalIDType& ID2) const { ID0 = ID1 = ID2 = NormalIDType(-1); return false; }

	const TArray<int32>& GetColorIDs() const { return EmptyArray; }
	FVector4f GetColor(ColorIDType ID) const { check(0); return FVector4f(); }
	bool GetColorTri(const TriIDType&, ColorIDType& ID0, ColorIDType& ID1, ColorIDType& ID2) const { ID0 = ID1 = ID2 = ColorIDType(-1); return false; }

	int32 NumWeightMapLayers() const { return 0; }
	float GetVertexWeight(int32 WeightMapIndex, int32 SrcVertID) const { return 0; }
	FName GetWeightMapName(int32 WeightMapIndex) const { return FName(); }



private:

	inline const FFinalSkinVertex& GetWedgeVertexInstance(WedgeIDType WID) const
	{
		int32 VertID = GetVertID(WID);
		return SkinnedVertices[VertID];
	}

	// Convert the WedgeID the ID of the corresponding position vertex.
	inline const VertIDType GetVertID(int32 WedgeID) const
	{
		const FRawStaticIndexBuffer16or32Interface& Indices = *IndexBuffer;
		return static_cast<VertIDType>(Indices.Get(WedgeID));
	}

private:

	TArray<FFinalSkinVertex> SkinnedVertices;
	USkinnedMeshComponent* Component;
	int32                  LOD;
	bool                   bUseDisabledSections;
	bool                   bHasTangents;

	USkinnedAsset* Asset;
	const FSkeletalMeshLODInfo* SrcLODInfo;
	const FSkeletalMeshRenderData& SkeletalMeshRenderData;
	const FRawStaticIndexBuffer16or32Interface* IndexBuffer;
	

	TArray<int32> VertIDs;
	TArray<int32> TriIDs;

	TArray<int32> TriIDtoSectionID;
	TArray<int32> EmptyArray;

	TArray<FSkinWeightInfo> SkinWeights;
};





bool SkinnedMeshComponentToDynamicMesh(USkinnedMeshComponent& SkinnedMeshComponent, Geometry::FDynamicMesh3& MeshOut, int32 RequestedLOD, bool bWantTangents)
{

	MeshOut.Clear();

	const int32 NumLODs = SkinnedMeshComponent.GetNumLODs();
	const bool bIsValidSkinnedMeshComponent = SkinnedMeshComponent.MeshObject && SkinnedMeshComponent.IsVisible();

	if (!bIsValidSkinnedMeshComponent)
	{
		return false;
	}

	if (RequestedLOD < 0 || RequestedLOD > NumLODs - 1)
	{
		return false;
	}

	if (!SkinnedMeshComponent.GetSkinnedAsset())
	{
		return false;
	}


	constexpr bool bUseDisabledSections = false;
	FSkinnedComponentWrapper SkinnedComponentWrapper(SkinnedMeshComponent, RequestedLOD, bUseDisabledSections, bWantTangents);

	// do the actual conversion.
	{
		constexpr bool bCopyTangents = true;
		auto TriToGroupID    = [&SkinnedComponentWrapper](const int32& SrcTriID)->int32 { return (SkinnedComponentWrapper.GetMaterialIndex(SrcTriID) + 1); };
		auto TriToMaterialID = [&SkinnedComponentWrapper](const int32& SrcTriID)->int32 { return SkinnedComponentWrapper.GetMaterialIndex(SrcTriID); };
		UE::Geometry::TToDynamicMesh<FSkinnedComponentWrapper> SkinnedComponentConverter;

		SkinnedComponentConverter.Convert(MeshOut, SkinnedComponentWrapper, TriToGroupID, TriToMaterialID, bCopyTangents);
	}

	return true;
}

} // end namespace Conversion
} // end namespace UE

