// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/SplineComponentDeformDynamicMesh.h"


#include "Components/SplineMeshComponent.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "Async/ParallelFor.h"

namespace UE {
namespace Geometry {

void SplineDeformDynamicMesh(USplineMeshComponent& SplineMeshComponent, FDynamicMesh3& Mesh, bool bUpdateTangentSpace)
{
	

	const ESplineMeshAxis::Type ForwardAxis = SplineMeshComponent.ForwardAxis;
	const FVector Mask = SplineMeshComponent.GetAxisMask(ForwardAxis);
	const int32 ForwardComponent = [&] { switch (SplineMeshComponent.ForwardAxis)
											{
											case ESplineMeshAxis::X: return 0;
											case ESplineMeshAxis::Y: return 1;
											case ESplineMeshAxis::Z: return 2;
											default: check(0); return -1;
											}}();

		// the transform is a function of position
		auto SliceTransform = [&SplineMeshComponent, ForwardComponent](const FVector& Position)-> FTransform
								{
									const float ForwardDistance = static_cast<float>(Position[ForwardComponent]);
									return SplineMeshComponent.CalcSliceTransform(ForwardDistance);
								};
		// position transform. note specialized because SliceTranform has the forward axis component of position baked in.
		auto TransformPosition = [&SliceTransform, Mask](const FVector& Position)->FVector3d
									{
										const FVector MaskedVert = Position * Mask;
										FVector3d OutPos = SliceTransform(Position).TransformPosition(MaskedVert);
										return OutPos;
									};


		// Since the transform is a function of position, we need to update the tangent space first.
		// For consistency with the way a static mesh tangent space is deformed by the spline on the GPU (see LocalVertexFactory.ush ) the vectors are simply rotated by 
		// the rotational component of the transform.  Strictly speaking this is wrong... 
		if (bUpdateTangentSpace)
		{
			// update tangent space attributes layers if they exist
			if (FDynamicMeshAttributeSet* Attributes = Mesh.Attributes())
			{
				auto TransformVectorOverlay = [&SliceTransform, &Mesh](auto* Overlay)
												{
													const int32 NumElements = Overlay->MaxElementID();
													ParallelFor(NumElements, [&](int32 elid)
														{
															if (Overlay->IsElement(elid))
															{
																// transform at position where this element lives
																const  FTransform Xform = [&] {  const int32 vid = Overlay->GetParentVertex(elid);
																								 const FVector3d Pos = Mesh.GetVertex(vid);
																								 return SliceTransform(Pos); } ();

																const FVector3f Vector3f = Overlay->GetElement(elid);
																const FVector3d TransformedVector3d = Xform.GetRotation() * static_cast<FVector3d>(Vector3f);
																const FVector3f TransformedVector3f = static_cast<FVector3f>(TransformedVector3d);

																Overlay->SetElement(elid, TransformedVector3f);
															}
														});
												};

				// update tangents
				if (FDynamicMeshNormalOverlay* TangentOverlay = Attributes->PrimaryTangents())
				{
					TransformVectorOverlay(TangentOverlay);
				}
				// update bi-tangents
				if (FDynamicMeshNormalOverlay* BiTangentOverlay = Attributes->PrimaryBiTangents())
				{
					TransformVectorOverlay(BiTangentOverlay);
				}
				// update normals
				if (FDynamicMeshNormalOverlay* NormalOverlay = Attributes->PrimaryNormals())
				{
					TransformVectorOverlay(NormalOverlay);
				}
			}
			else if (Mesh.HasVertexNormals())
			{
				const int32 NumVIDs = Mesh.MaxVertexID();
				ParallelFor(NumVIDs, [&](int32 vid)
					{
						if (Mesh.IsVertex(vid))
						{
							const  FTransform Xform = [&] {  const FVector3d Pos = Mesh.GetVertex(vid);
							return SliceTransform(Pos); } ();
							const FVector3f VertexNormal3f = Mesh.GetVertexNormal(vid);
							const FVector3f TransformedVertexNormal3f = static_cast<FVector3f>(Xform.GetRotation() * static_cast<FVector3d>(VertexNormal3f));
							Mesh.SetVertexNormal(vid, TransformedVertexNormal3f);
						}
					});
			}
		}

		// update vertex locations.  Note, tangents should be updated before position
		const int32 NumVertices = Mesh.MaxVertexID();
		ParallelFor(NumVertices, [&](int32 vid)
			{
				if (Mesh.IsVertex(vid))
				{
					FVector3d Position = Mesh.GetVertex(vid);
					Position = TransformPosition(Position);
					Mesh.SetVertex(vid, Position);
				}
			});
	}

};
};