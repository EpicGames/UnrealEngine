// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/VolumeToDynamicMesh.h"

#include "CompGeom/PolygonTriangulation.h"
#include "ConstrainedDelaunay2.h"
#include "Polygon2.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "GameFramework/Volume.h"
#include "MeshBoundaryLoops.h"
#include "MeshQueries.h"
#include "MeshRegionBoundaryLoops.h"
#include "Model.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "Operations/MinimalHoleFiller.h"
#include "Operations/PlanarFlipsOptimization.h"
#include "Components/BrushComponent.h"

#if WITH_EDITOR
#include "Engine/Polys.h"
#endif

using namespace UE::Geometry;

namespace VolumeToDynamicMeshLocals
{
	/** Attempts to use delaunay triangulation to triangulate the polygon, and falls back to ear clipping if that fails. */
	TArray<FIndex3i> TriangulatePolygon(const FPolygon2d& ToTriangulate)
	{
		// This first part is equivalent to calling  ConstrainedDelaunayTriangulate<double>(ToTriangulate), except
		// it allows us to properly check for failure.
		TConstrainedDelaunay2<double> DelaunayTriangulation;
		DelaunayTriangulation.FillRule = TConstrainedDelaunay2<double>::EFillRule::Positive;
		DelaunayTriangulation.Add(ToTriangulate);
		if (DelaunayTriangulation.Triangulate() && DelaunayTriangulation.Triangles.Num() > 0)
		{
			return DelaunayTriangulation.Triangles;
		}
		
		// If delaunay triangulation fails, use ear clipping.
		TArray<FIndex3i> TrianglesOut;
		PolygonTriangulation::TriangulateSimplePolygon(ToTriangulate.GetVertices(), TrianglesOut);
		return TrianglesOut;
	}
}

namespace UE {
namespace Conversion {

void VolumeToDynamicMesh(AVolume* Volume, FDynamicMesh3& Mesh, 
	const FVolumeToMeshOptions& Options)
{
	UModel* Model = Volume->Brush;
	if (Model == nullptr)		// model is null in some cases, eg default physics volume
	{
		return;
	}

	BrushToDynamicMesh(*Model, Mesh, Options);

	if (Options.bInWorldSpace)
	{
		FTransform3d XForm(Volume->GetTransform());
		MeshTransforms::ApplyTransform(Mesh, XForm, true);
	}
}


void BrushComponentToDynamicMesh(UBrushComponent* Component, UE::Geometry::FDynamicMesh3& Mesh, const FVolumeToMeshOptions& Options)
{
	UModel* Model = Component->Brush;
	if (Model == nullptr)		// model is null in some cases, eg default physics volume
	{
		return;
	}

	BrushToDynamicMesh(*Model, Mesh, Options);

	if (Options.bInWorldSpace)
	{
		FTransform3d XForm(Component->GetComponentTransform());
		MeshTransforms::ApplyTransform(Mesh, XForm, true);
	}
}



void BrushToDynamicMesh(UModel& BrushModel, UE::Geometry::FDynamicMesh3& Mesh, const FVolumeToMeshOptions& Options)
{
	using namespace VolumeToDynamicMeshLocals;

	Mesh.Clear();
	// Note: Groups are also used to compute normals matching the input polygons
	// If normals are requested but groups are not, groups will be discarded after normals are computed
	if (Options.bSetGroups || Options.bGenerateNormals)
	{
		Mesh.EnableTriangleGroups();
	}

#if WITH_EDITOR
	// In the editor, the preferred source of geometry for a volume is the Polys array,
	// which the bsp nodes are generated from, because the polys may be broken up into
	// pieces unnecessarily as bsp nodes.
	// Polys are planar polygons.

	// We do not try to merge any vertices yet.

	const TArray<FPoly>& Polygons = BrushModel.Polys->Element;
	for (FPoly Poly : Polygons)
	{
		int32 NumVerts = Poly.Vertices.Num();
		if (NumVerts < 3)
		{
			continue;
		}

		FVector3d Normal = (FVector3d)Poly.Normal;
		FFrame3d Plane((FVector3d)Poly.Base, Normal);

		TArray<int32> Vids;
		FPolygon2d ToTriangulate;

		Vids.SetNum(NumVerts);
		for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
		{
			FVector3d Point = (FVector3d)Poly.Vertices[VertexIndex];
			Vids[VertexIndex] = Mesh.AppendVertex(Point);
			ToTriangulate.AppendVertex(Plane.ToPlaneUV(Point, 2));
		}

		// Note that this call gives triangles with the reverse orientation compared to the
		// polygon, but the polygons we get are oriented opposite of what we want (they
		// are clockwise if the normal is towards us), so this ends up giving us the triangle
		// orientation that we want.
		TArray<FIndex3i> PolyTriangles = TriangulatePolygon(ToTriangulate);

		int32 GroupID = FDynamicMesh3::InvalidID;
		if (Options.bSetGroups)
		{
			GroupID = Mesh.AllocateTriangleGroup();
		}

		for (FIndex3i Tri : PolyTriangles)
		{
			Mesh.AppendTriangle(Vids[Tri.A], Vids[Tri.B], Vids[Tri.C], GroupID);
		}
	}

#else
	// Each "BspNode" is a planar polygon, triangulate each polygon and accumulate in a mesh.
	// Note that this does not make any attempt to weld vertices/edges
	for (const FBspNode& Node : BrushModel.Nodes)
	{
		FVector3d Normal = (FVector3d)Node.Plane;
		FFrame3d Plane(Node.Plane.W * Normal, Normal);

		int32 NumVerts = (Node.NodeFlags & PF_TwoSided) ? Node.NumVertices / 2 : Node.NumVertices;  // ??
		if (NumVerts > 0)
		{
			TArray<int32> Vids;
			FPolygon2d ToTriangulate;
			Vids.SetNum(NumVerts);
			for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
			{
				const FVert& Vert = BrushModel.Verts[Node.iVertPool + VertexIndex];
				FVector3d Point = (FVector3d)BrushModel.Points[Vert.pVertex];
				Vids[VertexIndex] = Mesh.AppendVertex(Point);
				ToTriangulate.AppendVertex(Plane.ToPlaneUV(Point, 2));
			}

			// Note that this call gives triangles with the reverse orientation compared to the
			// polygon, but the polygons we get are oriented opposite of what we want (they
			// are clockwise if the normal is towards us), so this ends up giving us the triangle
			// orientation that we want.
			TArray<FIndex3i> PolyTriangles = TriangulatePolygon(ToTriangulate);

			int32 GroupID = FDynamicMesh3::InvalidID;
			if (Options.bSetGroups)
			{
				GroupID = Mesh.AllocateTriangleGroup();
			}

			for (FIndex3i Tri : PolyTriangles)
			{
				Mesh.AppendTriangle(Vids[Tri.A], Vids[Tri.B], Vids[Tri.C], GroupID);
			}
	}
}
#endif

	if (Options.bMergeVertices)
	{
		// Merge the mesh edges to create a closed solid
		double MinLen, MaxLen, AvgLen;
		TMeshQueries<FDynamicMesh3>::EdgeLengthStats(Mesh, MinLen, MaxLen, AvgLen);
		FMergeCoincidentMeshEdges Merge(&Mesh);
		Merge.MergeVertexTolerance = FMathd::Max(Merge.MergeVertexTolerance, MinLen * 0.1);
		Merge.Apply();

		// If the mesh is not closed, the merge failed or the volume had cracks/holes. 
		// Do trivial hole fills to ensure the output is solid   (really want autorepair here)
		if (Mesh.IsClosed() == false && Options.bAutoRepairMesh)
		{
			FMeshBoundaryLoops BoundaryLoops(&Mesh, true);
			for (FEdgeLoop& Loop : BoundaryLoops.Loops)
			{
				FMinimalHoleFiller Filler(&Mesh, Loop);
				Filler.Fill();
			}
		}


		// try to flip towards better triangles in planar areas, should reduce/remove degenerate geo
		if (Options.bOptimizeMesh)
		{
			FPlanarFlipsOptimization(&Mesh, 5).Apply(); // Do five passes
		}

		Mesh.CompactInPlace();
	}

	if (Options.bGenerateNormals)
	{
		Mesh.EnableAttributes();
		FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
		FMeshNormals::InitializeOverlayTopologyFromFaceGroups(&Mesh, Normals);
		FMeshNormals::QuickRecomputeOverlayNormals(Mesh);
		if (!Options.bSetGroups)
		{
			Mesh.DiscardTriangleGroups();
		}
	}
}





}}//end namespace UE::Conversion
