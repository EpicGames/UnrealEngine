// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/LineSetComponent.h"

#include "Engine/CollisionProfile.h"
#include "LocalVertexFactory.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "DynamicMeshBuilder.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LineSetComponent)

struct FLineMeshBatchData
{
	FLineMeshBatchData()
		: MaterialProxy(nullptr)
	{}

	FMaterialRenderProxy* MaterialProxy;
	int32 StartIndex;
	int32 NumPrimitives;
	int32 MinVertexIndex;
	int32 MaxVertexIndex;
};

/** Class for the LineSetComponent data passed to the render thread. */
class FLineSetSceneProxy final : public FPrimitiveSceneProxy
{
public:

	FLineSetSceneProxy(ULineSetComponent* Component)
		: FPrimitiveSceneProxy(Component),
		MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform())),
		VertexFactory(GetScene().GetFeatureLevel(), "FPointSetSceneProxy")
	{
		const int32 NumLineVertices = Component->Lines.Num() * 4;
		const int32 NumLineIndices = Component->Lines.Num() * 6;
		const int32 NumTextureCoordinates = 1;

		VertexBuffers.PositionVertexBuffer.Init(NumLineVertices);
		VertexBuffers.StaticMeshVertexBuffer.Init(NumLineVertices, NumTextureCoordinates);
		VertexBuffers.ColorVertexBuffer.Init(NumLineVertices);
		IndexBuffer.Indices.SetNumUninitialized(NumLineIndices);

		int32 VertexBufferIndex = 0;
		int32 IndexBufferIndex = 0;
			
		// Initialize lines.
		// Lines are represented as two tris of zero thickness. The UV's stored at vertices are actually (lineThickness, depthBias), 
		// which the material unpacks and uses to thicken the polygons and set the pixel depth bias.
		if (Component->Lines.Num() > 0)
		{
			MeshBatchDatas.Emplace();
			FLineMeshBatchData& MeshBatchData = MeshBatchDatas.Last();
			MeshBatchData.MinVertexIndex = VertexBufferIndex;
			MeshBatchData.MaxVertexIndex = VertexBufferIndex + NumLineVertices - 1;
			MeshBatchData.StartIndex = IndexBufferIndex;
			MeshBatchData.NumPrimitives = Component->Lines.Num() * 2;
			if (Component->GetMaterial(0) != nullptr)
			{
				MeshBatchData.MaterialProxy = Component->GetMaterial(0)->GetRenderProxy();
			}
			else
			{
				MeshBatchData.MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			}

			for (const FRenderableLine& OverlayLine : Component->Lines)
			{
				const FVector LineDirection = (OverlayLine.End - OverlayLine.Start).GetSafeNormal();
				const FVector2f UV(OverlayLine.Thickness, OverlayLine.DepthBias);

				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 0) = (FVector3f)OverlayLine.Start;
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 1) = (FVector3f)OverlayLine.End;
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 2) = (FVector3f)OverlayLine.End;
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 3) = (FVector3f)OverlayLine.Start;

				// On each end of the line, we store the line direction pointing in opposite directions so that the verts
				// are moved in oppositie directions to make the rectangle
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 0, FVector3f::ZeroVector, FVector3f::ZeroVector, (FVector3f)-LineDirection);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 1, FVector3f::ZeroVector, FVector3f::ZeroVector, (FVector3f)-LineDirection);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 2, FVector3f::ZeroVector, FVector3f::ZeroVector, (FVector3f)LineDirection);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 3, FVector3f::ZeroVector, FVector3f::ZeroVector, (FVector3f)LineDirection);

				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 0, 0, UV);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 1, 0, UV);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 2, 0, UV);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 3, 0, UV);

				// The color stored in the vertices actually gets interpreted as a linear color by the material,
				// whereas it is more convenient for the user of the LineSet to specify colors as sRGB. So we actually
				// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
				FColor color = FLinearColor::FromSRGBColor(OverlayLine.Color).ToFColor(false);
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 0) = color;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 1) = color;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 2) = color;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 3) = color;

				IndexBuffer.Indices[IndexBufferIndex + 0] = VertexBufferIndex + 0;
				IndexBuffer.Indices[IndexBufferIndex + 1] = VertexBufferIndex + 1;
				IndexBuffer.Indices[IndexBufferIndex + 2] = VertexBufferIndex + 2;
				IndexBuffer.Indices[IndexBufferIndex + 3] = VertexBufferIndex + 2;
				IndexBuffer.Indices[IndexBufferIndex + 4] = VertexBufferIndex + 3;
				IndexBuffer.Indices[IndexBufferIndex + 5] = VertexBufferIndex + 0;

				VertexBufferIndex += 4;
				IndexBufferIndex += 6;
			}
		}

		ENQUEUE_RENDER_COMMAND(LineSetVertexBuffersInit)(
			[this](FRHICommandListImmediate& RHICmdList)
		{
			VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
			VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
			VertexFactory.SetData(RHICmdList, Data);

			VertexFactory.InitResource(RHICmdList);
			IndexBuffer.InitResource(RHICmdList);
		});
	}

	virtual ~FLineSetSceneProxy()
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, 
		const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				for (const FLineMeshBatchData& MeshBatchData : MeshBatchDatas)
				{
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshBatchData.MaterialProxy;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), false, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = MeshBatchData.StartIndex;
					BatchElement.NumPrimitives = MeshBatchData.NumPrimitives;
					BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
					BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }

	uint32 GetAllocatedSize() const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	virtual SIZE_T GetTypeHash() const override
	{
		static SIZE_T UniquePointer;
		return reinterpret_cast<SIZE_T>(&UniquePointer);
	}

private:
	TArray<FLineMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};


ULineSetComponent::ULineSetComponent()
{
	CastShadow = false;
	bSelectable = false;
	PrimaryComponentTick.bCanEverTick = false;
	bBoundsDirty = true;

	UPrimitiveComponent::SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

void ULineSetComponent::SetLineMaterial(UMaterialInterface* InLineMaterial)
{
	LineMaterial = InLineMaterial;
	SetMaterial(0, InLineMaterial);
}

void ULineSetComponent::Clear()
{
	Lines.Reset();
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void ULineSetComponent::ReserveLines(const int32 MaxID)
{
	Lines.Reserve(MaxID);
}

int32 ULineSetComponent::AddLine(const FRenderableLine& OverlayLine)
{
	MarkRenderStateDirty();
	return AddLineInternal(OverlayLine);
}

int32 ULineSetComponent::AddLineInternal(const FRenderableLine& Line)
{
	const int32 ID(Lines.Add(Line));
	bBoundsDirty = true;
	return ID;
}

int32 ULineSetComponent::AddLines(const TArray<FVector>& InStart, const TArray<FVector>& InEnd, const FColor& InColor, const float InThickness, const float InDepthBias)
{
	if(InStart.Num() != InEnd.Num())
	{
		FFrame::KismetExecutionMessage(TEXT("InStart and InEnd arrays are not the same size."), ELogVerbosity::Error);
		return 0;
	}

	const int NumLines = InStart.Num();
	for (int Index = 0; Index < NumLines; ++Index)
	{
		AddLine(InStart[Index], InEnd[Index], InColor, InThickness, InDepthBias);
	}
	return NumLines;
}

int32 ULineSetComponent::AddLines(const TArray<FRenderableLine>& InLines)
{
	const int NumLines = InLines.Num();

	MarkRenderStateDirty();

	for (int32 Idx = 0; Idx < InLines.Num(); ++Idx)
	{
		AddLine(InLines[Idx]);
	}

	return NumLines;
}

void ULineSetComponent::InsertLine(const int32 ID, const FRenderableLine& OverlayLine)
{
	Lines.Insert(ID, OverlayLine);
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void ULineSetComponent::SetLineStart(const int32 ID, const FVector& NewPostion)
{
	FRenderableLine& OverlayLine = Lines[ID];
	OverlayLine.Start = NewPostion;
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void ULineSetComponent::SetLineEnd(const int32 ID, const FVector& NewPostion)
{
	FRenderableLine& OverlayLine = Lines[ID];
	OverlayLine.End = NewPostion;
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void ULineSetComponent::SetLineColor(const int32 ID, const FColor& NewColor)
{
	FRenderableLine& OverlayLine = Lines[ID];
	OverlayLine.Color = NewColor;
	MarkRenderStateDirty();
}

void ULineSetComponent::SetLineThickness(const int32 ID, const float NewThickness)
{
	FRenderableLine& OverlayLine = Lines[ID];
	OverlayLine.Thickness = NewThickness;
	MarkRenderStateDirty();
}

void ULineSetComponent::SetAllLinesThickness(const float NewThickness)
{
	for (FRenderableLine& Line : Lines)
	{
		Line.Thickness = NewThickness;
	}
	MarkRenderStateDirty();
}

void ULineSetComponent::SetAllLinesDepthBias(const float NewDepthBias)
{
	for (FRenderableLine& Line : Lines)
	{
		Line.DepthBias = NewDepthBias;
	}
	MarkRenderStateDirty();
}

void ULineSetComponent::SetAllLinesLength(const float NewLength, bool bUpdateBounds)
{
	float UseSign = (NewLength < 0) ? -1.0f : 1.0f;
	float UseLength = UseSign * FMath::Max(FMath::Abs(NewLength), 0.001f);
	for (FRenderableLine& Line : Lines)
	{
		FVector Direction = Line.End - Line.Start;
		if (Direction.SizeSquared() > 0)
		{
			Direction.Normalize();
			Line.End = Line.Start + UseLength * Direction;
		}
	}
	MarkRenderStateDirty();
	bBoundsDirty = bUpdateBounds;
}


void ULineSetComponent::SetAllLinesColor(const FColor& NewColor)
{
	for (FRenderableLine& Line : Lines)
	{
		Line.Color = NewColor;
	}
	MarkRenderStateDirty();
}



void ULineSetComponent::RemoveLine(const int32 ID)
{
	Lines.RemoveAt(ID);
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

bool ULineSetComponent::IsLineValid(const int32 ID) const
{
	return Lines.IsValidIndex(ID);
}

FPrimitiveSceneProxy* ULineSetComponent::CreateSceneProxy()
{
	if (Lines.Num() > 0)
	{
		return new FLineSetSceneProxy(this);
	}
	return nullptr;
}

int32 ULineSetComponent::GetNumMaterials() const
{
	return 1;
}

FBoxSphereBounds ULineSetComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (bBoundsDirty)
	{
		FBox Box(ForceInit);
		for (const FRenderableLine& Line : Lines)
		{
			Box += Line.Start;
			Box += Line.End;
		}
		Bounds = FBoxSphereBounds(Box);
		bBoundsDirty = false;
	}
	return Bounds.TransformBy(LocalToWorld);
}



void ULineSetComponent::AddLines(
	int32 NumIndices,
	TFunctionRef<void(int32 Index, TArray<FRenderableLine>& LinesOut)> LineGenFunc,
	int32 LinesPerIndexHint,
	bool bDeferRenderStateDirty)
{
	TArray<FRenderableLine> TempLines;
	if (LinesPerIndexHint > 0)
	{
		ReserveLines(Lines.Num() + NumIndices*LinesPerIndexHint);
		TempLines.Reserve(LinesPerIndexHint);
	}

	for (int32 k = 0; k < NumIndices; ++k)
	{
		TempLines.Reset();
		LineGenFunc(k, TempLines);
		for (const FRenderableLine& Line : TempLines)
		{
			AddLineInternal(Line);
		}
	}

	if (!bDeferRenderStateDirty)
	{
		MarkRenderStateDirty();
	}
}

