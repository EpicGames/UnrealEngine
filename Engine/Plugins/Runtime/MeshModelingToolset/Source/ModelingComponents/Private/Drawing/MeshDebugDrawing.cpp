// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/MeshDebugDrawing.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "FrameTypes.h"
#include "ToolSceneQueriesUtil.h"

#include "MathUtil.h"

#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface

using namespace UE::Geometry;

void MeshDebugDraw::DrawNormals(
	const FDynamicMeshNormalOverlay* Overlay,
	float Length, FColor Color, float Thickness, bool bScreenSpace,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{
	const FDynamicMesh3* Mesh = Overlay->GetParentMesh();
	for (int ElementID : Overlay->ElementIndicesItr())
	{
		FVector3f Normal = Overlay->GetElement(ElementID);
		int ParentVID = Overlay->GetParentVertex(ElementID);
		FVector3d ParentPos = Mesh->GetVertex(ParentVID);

		FVector A = (FVector)ParentPos, B = (FVector)(ParentPos + (double)Length * (FVector3d)Normal);
		PDI->DrawLine(Transform.TransformPosition(A), Transform.TransformPosition(B),
			Color, 0, Thickness, 0, bScreenSpace);
	}
}





void MeshDebugDraw::DrawVertices(
	const FDynamicMesh3* Mesh, const TArray<int>& Indices,
	float PointSize, FColor Color,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{
	for (int VertID : Indices)
	{
		FVector3d Pos = Mesh->GetVertex(VertID);
		PDI->DrawPoint(Transform.TransformPosition((FVector)Pos), Color, PointSize, SDPG_World);
	}
}

void MeshDebugDraw::DrawVertices(
	const FDynamicMesh3* Mesh, const TSet<int>& Indices,
	float PointSize, FColor Color,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{
	for (int VertID : Indices)
	{
		FVector3d Pos = Mesh->GetVertex(VertID);
		PDI->DrawPoint(Transform.TransformPosition((FVector)Pos), Color, PointSize, SDPG_World);
	}
}



void MeshDebugDraw::DrawTriCentroids(
	const FDynamicMesh3* Mesh, const TArray<int>& Indices,
	float PointSize, FColor Color,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{
	for (int TriID : Indices)
	{
		FVector3d Pos = Mesh->GetTriCentroid(TriID);
		PDI->DrawPoint(Transform.TransformPosition((FVector)Pos), Color, PointSize, SDPG_World);
	}
}



void MeshDebugDraw::DrawSimpleGrid(	
	const FFrame3d& LocalFrame, int GridLines, double GridLineSpacing,
	float LineWidth, FColor Color, bool bDepthTested,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{
	ESceneDepthPriorityGroup DepthPriority = (bDepthTested) ? SDPG_World : SDPG_Foreground;

	FFrame3d WorldFrame = LocalFrame;
	WorldFrame.Transform(Transform);

	double Width = (double)(GridLines-1) * GridLineSpacing;
	double Extent = Width * 0.5;

	FVector3d Origin = WorldFrame.Origin;
	FVector3d X = WorldFrame.X();
	FVector3d Y = WorldFrame.Y();
	FVector3d A, B;

	int LineSteps = GridLines / 2;
	for (int i = 0; i < LineSteps; i++)
	{
		double dx = (double)i * GridLineSpacing;
		A = Origin - Extent * Y - dx * X;
		B = Origin + Extent * Y - dx * X;
		PDI->DrawLine((FVector)A, (FVector)B, Color, DepthPriority, LineWidth, 0, true);
		A = Origin - Extent * Y + dx * X;
		B = Origin + Extent * Y + dx * X;
		PDI->DrawLine((FVector)A, (FVector)B, Color, DepthPriority, LineWidth, 0, true);

		A = Origin - Extent * X - dx * Y;
		B = Origin + Extent * X - dx * Y;
		PDI->DrawLine((FVector)A, (FVector)B, Color, DepthPriority, LineWidth, 0, true);
		A = Origin - Extent * X + dx * Y;
		B = Origin + Extent * X + dx * Y;
		PDI->DrawLine((FVector)A, (FVector)B, Color, DepthPriority, LineWidth, 0, true);
	}
}


void MeshDebugDraw::DrawSimpleFixedScreenAreaGrid(
	const FViewCameraState& CameraState,
	const FFrame3d& LocalFrame, int32 NumGridLines, double VisualAngleSpan,
	float LineWidth, FColor Color, bool bDepthTested,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{
	FVector WorldOrigin = Transform.TransformPosition((FVector)LocalFrame.Origin);
	double GridWidth = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, (FVector3d)WorldOrigin, VisualAngleSpan);
	double GridLineSpacing = GridWidth / (float)NumGridLines;
	DrawSimpleGrid(LocalFrame, NumGridLines, GridLineSpacing, LineWidth, Color, bDepthTested, PDI, Transform);
}


void MeshDebugDraw::DrawHierarchicalGrid(
	double BaseScale, double GridZoomFactor, int32 MaxLevelDensity,
	const FVector& WorldMaxBounds, const FVector& WorldMinBounds,
	int32 Levels, int32 Subdivisions, TArray<FColor>& Colors,
	const FFrame3d& LocalFrame, float LineWidth, bool bDepthTested,
	FPrimitiveDrawInterface* PDI, const FTransform& Transform)
{

	// Determine the logrithmic scaling factor based on linear zoom factor.
    // This allows us to track and discretely shift grid resolutions at certain zoom levels.
	// This code assumes that we want the grid to remain stable with one logrithmic "unit"
	// around the base zoom factor of 1.0
	double LogZoom = FMath::LogX(Subdivisions, GridZoomFactor );
	double LogZoomDirection = FMath::Sign(LogZoom);
	LogZoom = FMath::Abs(LogZoom);
	LogZoom = FMathf::Floor(LogZoom);
	LogZoom = LogZoomDirection * LogZoom;

	// Adjust grid scales based on current zoom levels
	TArray<double> GridScales;
	GridScales.SetNum(Levels);
	for (int32 Level = 0; Level < Levels; ++Level)
	{
		GridScales[Level] = BaseScale * FMathd::Pow(Subdivisions, LogZoom - Level);
		ensure(!FMath::IsNearlyZero(GridScales[Level]));
	}
	
	//Determine the center of the drawing area for the grid, snapping to grid positions
	FVector3d GridOrigin(
		FMath::GridSnap(LocalFrame.Origin.X, GridScales[0]),
		FMath::GridSnap(LocalFrame.Origin.Y, GridScales[0]),
		LocalFrame.Origin.Z);
	UE::Geometry::FFrame3d GridFrame(GridOrigin, LocalFrame.Rotation);

	// Draw each level of the grid
	for (int32 Level = 0; Level < Levels; ++Level)
	{
		// We automatically adjust down each level's thickness by half each time
		float AdjustedLineWidth = LineWidth * FMathf::Pow(2, -Level);

		// Compute the number of needed grid lines based on the coarsest grid's scale, this way we never "run out" of lines as we pan about
		int32 GridLines = FMathd::Ceil(FMathd::Max(((WorldMaxBounds.X - WorldMinBounds.X + GridScales[0]*2) / GridScales[Level]),
			                                        ((WorldMaxBounds.Y - WorldMinBounds.Y + GridScales[0]*2) / GridScales[Level])));

		// If we ever have too many lines to draw, just bail. This preserves performance for large, deep grids.
		if (GridLines > MaxLevelDensity)
		{
			break;
		}

		// Select our color, using the last grid color over again if we don't have enough.
		FColor GridColor = Colors.Num() - 1 > Level ? Colors[Level] : Colors.Last();

		// Finally draw one grid level.
		MeshDebugDraw::DrawSimpleGrid(GridFrame, GridLines, GridScales[Level], AdjustedLineWidth,
			GridColor, bDepthTested, PDI, Transform);
	}	

}