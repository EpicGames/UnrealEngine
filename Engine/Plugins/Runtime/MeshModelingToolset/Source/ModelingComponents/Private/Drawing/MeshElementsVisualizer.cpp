// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/MeshElementsVisualizer.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Drawing/MeshWireframeComponent.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshElementsVisualizer)

using namespace UE::Geometry;


/**
 * IMeshWireframeSource implementation for a FDynamicMesh3
 */
class FTemporaryDynamicMeshWireframeSource : public IMeshWireframeSource
{
public:
	const FDynamicMesh3& Mesh;

	FTemporaryDynamicMeshWireframeSource(const FDynamicMesh3& MeshIn) : Mesh(MeshIn) {}

	virtual bool IsValid() const { return true; }

	virtual FBoxSphereBounds GetBounds() const 
	{ 
		FAxisAlignedBox3d Bounds = Mesh.GetBounds();

		if (Bounds.IsEmpty())
		{
			return FBoxSphereBounds(EForceInit::ForceInit);
		}
		else
		{
			return FBoxSphereBounds((FBox)Bounds);
		}
	}

	virtual FVector GetVertex(int32 Index) const
	{
		return (FVector)Mesh.GetVertex(Index);
	}

	virtual int32 GetEdgeCount() const
	{
		return Mesh.EdgeCount();
	}

	virtual int32 GetMaxEdgeIndex() const
	{
		return Mesh.MaxEdgeID();
	}

	virtual bool IsEdge(int32 Index) const
	{
		return Mesh.IsEdge(Index);
	}

	virtual void GetEdge(int32 EdgeIndex, int32& VertIndexAOut, int32& VertIndexBOut, EMeshEdgeType& TypeOut) const
	{
		FIndex2i EdgeV = Mesh.GetEdgeV(EdgeIndex);
		VertIndexAOut = EdgeV.A;
		VertIndexBOut = EdgeV.B;
		int32 EdgeType = (int32)EMeshEdgeType::Regular;
		if (Mesh.IsBoundaryEdge(EdgeIndex))
		{
			EdgeType |= (int32)EMeshEdgeType::MeshBoundary;
		}
		if (Mesh.HasTriangleGroups() && Mesh.IsGroupBoundaryEdge(EdgeIndex))
		{
			EdgeType |= (int32)EMeshEdgeType::GroupBoundary;
		}
		if (Mesh.HasAttributes())
		{
			bool bIsUVSeam = false, bIsNormalSeam = false, bIsColorSeam = false, bIsTangentSeam = false;
			if (Mesh.Attributes()->IsSeamEdge(EdgeIndex, bIsUVSeam, bIsNormalSeam, bIsColorSeam, bIsTangentSeam))
			{
				if (bIsUVSeam)
				{
					EdgeType |= (int32)EMeshEdgeType::UVSeam;
				}
				if (bIsNormalSeam)
				{
					EdgeType |= (int32)EMeshEdgeType::NormalSeam;
				}
				if (bIsTangentSeam)
				{
					EdgeType |= (int32)EMeshEdgeType::TangentSeam;
				}
				if (bIsColorSeam)
				{
					EdgeType |= (int32)EMeshEdgeType::ColorSeam;
				}
			}
		}
		TypeOut = (EMeshEdgeType)EdgeType;
	}
};


class FDynamicMeshWireframeSourceProvider : public IMeshWireframeSourceProvider
{
public:
	TUniqueFunction<void(UMeshElementsVisualizer::ProcessDynamicMeshFunc)> MeshAccessFunction;

	FDynamicMeshWireframeSourceProvider(TUniqueFunction<void(UMeshElementsVisualizer::ProcessDynamicMeshFunc)>&& MeshAccessFuncIn)
	{
		MeshAccessFunction = MoveTemp(MeshAccessFuncIn);
	}

	virtual void AccessMesh(TFunctionRef<void(const IMeshWireframeSource&)> ProcessingFunc) override
	{
		MeshAccessFunction([&](const FDynamicMesh3& ReadMesh)
		{
			FTemporaryDynamicMeshWireframeSource WireSource(ReadMesh);
			ProcessingFunc(WireSource);
		});
	}
};




void UMeshElementsVisualizer::OnCreated()
{
	Settings = NewObject<UMeshElementsVisualizerProperties>(this);
	Settings->WatchProperty(Settings->bVisible, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowWireframe, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowBorders, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowUVSeams, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowNormalSeams, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowTangentSeams, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowColorSeams, [this](bool){ bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowGroupBoundaries, [this](bool){ bSettingsModified = true; });
	Settings->WatchProperty(Settings->ThicknessScale, [this](float) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->DepthBias, [this](float) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bAdjustDepthBiasUsingMeshSize, [this](bool) {
		UpdateLineDepthBiasScale(); // A little expensive, so only want to do this when necessary.
		bSettingsModified = true; }); // Still mark this so we mark the wireframe as dirty.
	Settings->WatchProperty(Settings->WireframeColor, [this](FColor) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->BoundaryEdgeColor, [this](FColor) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->UVSeamColor, [this](FColor) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->NormalSeamColor, [this](FColor) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->TangentSeamColor, [this](FColor) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->ColorSeamColor, [this](FColor) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->GroupBoundaryColor, [this](FColor) { bSettingsModified = true; });
	bSettingsModified = false;

	WireframeComponent = NewObject<UMeshWireframeComponent>(GetActor());
	WireframeComponent->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(nullptr));
	WireframeComponent->SetupAttachment(GetActor()->GetRootComponent());	
	WireframeComponent->RegisterComponent();
}


void UMeshElementsVisualizer::SetMeshAccessFunction(TUniqueFunction<void(ProcessDynamicMeshFunc)>&& MeshAccessFunctionIn)
{
	WireframeSourceProvider = MakeShared<FDynamicMeshWireframeSourceProvider>(MoveTemp(MeshAccessFunctionIn));

	UpdateLineDepthBiasScale();

	WireframeComponent->SetWireframeSourceProvider(WireframeSourceProvider);
}

void UMeshElementsVisualizer::UpdateLineDepthBiasScale()
{
	if (Settings->bAdjustDepthBiasUsingMeshSize && WireframeSourceProvider)
	{
		WireframeSourceProvider->AccessMesh(
			[this](const IMeshWireframeSource& WireframeSource) {
				// Scale is 0.01 of diameter of bounding sphere (diagonal of box)
				WireframeComponent->LineDepthBiasSizeScale = WireframeSource.GetBounds().SphereRadius * 2 * 0.01;
			});
	}
	else
	{
		WireframeComponent->LineDepthBiasSizeScale = 1;
	}
}


void UMeshElementsVisualizer::OnTick(float DeltaTime)
{
	if (bSettingsModified)
	{
		UpdateVisibility();
		bSettingsModified = false;
	}
}


void UMeshElementsVisualizer::UpdateVisibility()
{
	if (Settings->bVisible == false)
	{
		WireframeComponent->SetVisibility(false);
		return;
	}

	WireframeComponent->SetVisibility(true);

	WireframeComponent->LineDepthBias = Settings->DepthBias;
	WireframeComponent->ThicknessScale = Settings->ThicknessScale;

	WireframeComponent->bEnableWireframe = Settings->bShowWireframe;
	WireframeComponent->bEnableBoundaryEdges = Settings->bShowBorders;
	WireframeComponent->bEnableUVSeams = Settings->bShowUVSeams;
	WireframeComponent->bEnableNormalSeams = Settings->bShowNormalSeams;
	WireframeComponent->bEnableTangentSeams = Settings->bShowTangentSeams;
	WireframeComponent->bEnableColorSeams = Settings->bShowColorSeams;
	WireframeComponent->bEnableGroupBoundaries = Settings->bShowGroupBoundaries;

	WireframeComponent->WireframeColor = Settings->WireframeColor;
	WireframeComponent->BoundaryEdgeColor = Settings->BoundaryEdgeColor;
	WireframeComponent->UVSeamColor = Settings->UVSeamColor;
	WireframeComponent->NormalSeamColor = Settings->NormalSeamColor;
	WireframeComponent->TangentSeamColor = Settings->TangentSeamColor;
	WireframeComponent->ColorSeamColor = Settings->ColorSeamColor;
	WireframeComponent->GroupBoundaryColor = Settings->GroupBoundaryColor;

	WireframeComponent->UpdateWireframe();
}


void UMeshElementsVisualizer::NotifyMeshChanged()
{
	WireframeComponent->UpdateWireframe();
}

void UMeshElementsVisualizerProperties::ShowAllElements(bool bShow)
{
	bShowWireframe = bShow;
	bShowBorders = bShow;
	bShowUVSeams = bShow;
	bShowNormalSeams = bShow;
	bShowTangentSeams = bShow;
	bShowColorSeams = bShow;
	bShowGroupBoundaries = bShow;
}
