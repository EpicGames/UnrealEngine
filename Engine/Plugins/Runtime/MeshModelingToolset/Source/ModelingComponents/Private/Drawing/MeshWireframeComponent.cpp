// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/MeshWireframeComponent.h"

#include "Engine/CollisionProfile.h"
#include "LocalVertexFactory.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "DynamicMeshBuilder.h"
#include "SceneInterface.h"
#include "StaticMeshResources.h"
#include "IndexTypes.h"
#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshWireframeComponent)

using UE::Geometry::FIndex4i;

struct FWireframeLinesMeshBatchData
{
	FWireframeLinesMeshBatchData()
		: MaterialProxy(nullptr)
	{}

	FMaterialRenderProxy* MaterialProxy;
	int32 StartIndex;
	int32 NumPrimitives;
	int32 MinVertexIndex;
	int32 MaxVertexIndex;
};





/** Class for the MeshWireframeComponent data passed to the render thread. */
class FMeshWireframeSceneProxy final : public FPrimitiveSceneProxy
{
public:

	FMeshWireframeSceneProxy(UMeshWireframeComponent* Component, const IMeshWireframeSource* WireSource)
		: FPrimitiveSceneProxy(Component),
		MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform())),
		VertexFactory(GetScene().GetFeatureLevel(), "FPointSetSceneProxy")
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshWireframeSceneProxy);

		if (!ensure(WireSource && WireSource->IsValid())) return;

		if (WireSource->GetEdgeCount() <= 0) return;

		// Create local copies to avoid repeated indirect look-up through the Component pointer in loops. 
		const bool bEnableWireframe = Component->bEnableWireframe;
		const bool bEnableBoundaryEdges = Component->bEnableBoundaryEdges;
		const bool bEnableUVSeams = Component->bEnableUVSeams;
		const bool bEnableNormalSeams = Component->bEnableNormalSeams;
		const bool bEnableTangentSeams = Component->bEnableTangentSeams;
		const bool bEnableColorSeams = Component->bEnableColorSeams;
		const bool bEnableGroupBoundaries = Component->bEnableGroupBoundaries;

		// Setup edge batches to process in parallel.
		const int32 NumSourceEdges = WireSource->GetMaxEdgeIndex();
		constexpr int32 BatchSize = 16384;
		const int32 NumBatches = FMath::DivideAndRoundUp<int32>(NumSourceEdges, BatchSize);
		
		struct EdgeBatch
		{
			int32 Offset;
			TArray<UE::Geometry::FIndex3i> Edges;
		};
		
		TArray<EdgeBatch> VisibleEdgeBatches;
		VisibleEdgeBatches.SetNum(NumBatches);

		// count visible edges and remap so we can build in parallel below
		int32 NumVisibleEdges = 0;
		ParallelFor(NumBatches, [&](int32 BatchIndex)
		{
			TArray<UE::Geometry::FIndex3i>& VisibleEdgeBatch = VisibleEdgeBatches[BatchIndex].Edges;
			const int32 CurrentBatchSize = (BatchIndex < NumBatches - 1) || (NumSourceEdges % BatchSize == 0) ? BatchSize : NumSourceEdges % BatchSize;
			VisibleEdgeBatch.Reserve(CurrentBatchSize);
			
			for (int32 EdgeIndex = BatchIndex * BatchSize, BatchEnd = BatchIndex * BatchSize + CurrentBatchSize; EdgeIndex < BatchEnd; ++EdgeIndex)
			{
				if (WireSource->IsEdge(EdgeIndex) == false) continue;

				int32 VertIndexA, VertIndexB;
				IMeshWireframeSource::EMeshEdgeType EdgeType;
				WireSource->GetEdge(EdgeIndex, VertIndexA, VertIndexB, EdgeType);

				if (bEnableWireframe ||
					(bEnableBoundaryEdges     && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::MeshBoundary )) != 0) ||
					(bEnableGroupBoundaries   && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::GroupBoundary)) != 0) ||
					(bEnableUVSeams           && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::UVSeam       )) != 0) ||
					(bEnableNormalSeams       && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::NormalSeam   )) != 0) ||
					(bEnableTangentSeams      && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::TangentSeam  )) != 0) ||
					(bEnableColorSeams        && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::ColorSeam    )) != 0))
				{
					VisibleEdgeBatch.Add(UE::Geometry::FIndex3i(VertIndexA, VertIndexB, static_cast<int>(EdgeType)));
				}
			}
		});

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			VisibleEdgeBatches[BatchIndex].Offset = NumVisibleEdges;
			NumVisibleEdges += VisibleEdgeBatches[BatchIndex].Edges.Num();
		}

		if (NumVisibleEdges == 0)
		{
			return;
		}

		const int32 NumLineVertices = NumVisibleEdges * 4;
		const int32 NumLineIndices = NumVisibleEdges * 6;
		constexpr int32 NumTextureCoordinates = 1;

		VertexBuffers.PositionVertexBuffer.Init(NumLineVertices);
		VertexBuffers.StaticMeshVertexBuffer.Init(NumLineVertices, NumTextureCoordinates);
		VertexBuffers.ColorVertexBuffer.Init(NumLineVertices);
		IndexBuffer.Indices.SetNumUninitialized(NumLineIndices);

		MeshBatchDatas.Emplace();
		FWireframeLinesMeshBatchData& MeshBatchData = MeshBatchDatas.Last();
		MeshBatchData.MinVertexIndex = 0;
		MeshBatchData.MaxVertexIndex = 0 + NumLineVertices - 1;
		MeshBatchData.StartIndex = 0;
		MeshBatchData.NumPrimitives = NumVisibleEdges * 2;
		if (Component->GetMaterial(0) != nullptr)
		{
			MeshBatchData.MaterialProxy = Component->GetMaterial(0)->GetRenderProxy();
		}
		else
		{
			MeshBatchData.MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		}

		const FColor RegularEdgeColor = FLinearColor::FromSRGBColor(Component->WireframeColor).ToFColor(false);
		const float RegularEdgeThickness = Component->ThicknessScale * Component->WireframeThickness;
		const FColor BoundaryEdgeColor = FLinearColor::FromSRGBColor(Component->BoundaryEdgeColor).ToFColor(false);
		const float BoundaryEdgeThickness = Component->ThicknessScale * Component->BoundaryEdgeThickness;
		const FColor UVSeamColor = FLinearColor::FromSRGBColor(Component->UVSeamColor).ToFColor(false);
		const float UVSeamThickness = Component->ThicknessScale * Component->UVSeamThickness;
		const FColor NormalSeamColor = FLinearColor::FromSRGBColor(Component->NormalSeamColor).ToFColor(false);
		const float NormalSeamThickness = Component->ThicknessScale * Component->NormalSeamThickness;
		const FColor TangentSeamColor = FLinearColor::FromSRGBColor(Component->TangentSeamColor).ToFColor(false);
		const float TangentSeamThickness = Component->ThicknessScale * Component->TangentSeamThickness;
		const FColor ColorSeamColor = FLinearColor::FromSRGBColor(Component->ColorSeamColor).ToFColor(false);
		const float ColorSeamThickness = Component->ThicknessScale * Component->ColorSeamThickness;
		const FColor GroupBoundaryColor = FLinearColor::FromSRGBColor(Component->GroupBoundaryColor).ToFColor(false);
		const float GroupBoundaryThickness = Component->ThicknessScale * Component->GroupBoundaryThickness;

		const float LineDepthBias = Component->LineDepthBias * Component->LineDepthBiasSizeScale;

		// Initialize lines.
		// Lines are represented as two tris of zero thickness. The UV's stored at vertices are actually (lineThickness, depthBias), 
		// which the material unpacks and uses to thicken the polygons and set the pixel depth bias.
		ParallelFor(NumBatches, [&](int32 BatchIndex)
		{
			const int32 Offset = VisibleEdgeBatches[BatchIndex].Offset;
			const TArray<UE::Geometry::FIndex3i>& VisibleEdgeBatch = VisibleEdgeBatches[BatchIndex].Edges;

			for (int32 VisibleEdgeIndex = 0, Num = VisibleEdgeBatch.Num(); VisibleEdgeIndex < Num; ++VisibleEdgeIndex)
			{
				const int32 VertexBufferIndex = (Offset + VisibleEdgeIndex) * 4;
				const int32 IndexBufferIndex = (Offset + VisibleEdgeIndex) * 6;

				const UE::Geometry::FIndex3i EdgeInfo = VisibleEdgeBatch[VisibleEdgeIndex];
				IMeshWireframeSource::EMeshEdgeType EdgeType = static_cast<IMeshWireframeSource::EMeshEdgeType>(EdgeInfo.C);

				float UseThickness = RegularEdgeThickness;
				FColor UseColor = RegularEdgeColor;

				if (EdgeType != IMeshWireframeSource::EMeshEdgeType::Regular)
				{
					const bool bIsBoundaryEdge = ((static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::MeshBoundary)) != 0);

					if (bEnableBoundaryEdges && bIsBoundaryEdge)
					{
						UseThickness = BoundaryEdgeThickness;
						UseColor = BoundaryEdgeColor;
					}
					else if (bEnableGroupBoundaries && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::GroupBoundary)) != 0)
					{
						UseThickness = GroupBoundaryThickness;
						UseColor = GroupBoundaryColor;
					}
					else if (bEnableUVSeams && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::UVSeam)) != 0)
					{
						UseThickness = (bIsBoundaryEdge) ? BoundaryEdgeThickness : UVSeamThickness;
						UseColor = UVSeamColor;
					}
					else if (bEnableNormalSeams && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::NormalSeam)) != 0)
					{
						UseThickness = (bIsBoundaryEdge) ? BoundaryEdgeThickness : NormalSeamThickness;
						UseColor = NormalSeamColor;
					}
					else if (bEnableTangentSeams && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::TangentSeam)) != 0)
					{
						UseThickness = (bIsBoundaryEdge) ? BoundaryEdgeThickness : TangentSeamThickness;
						UseColor = TangentSeamColor;
					}
					else if (bEnableColorSeams && (static_cast<int>(EdgeType) & static_cast<int>(IMeshWireframeSource::EMeshEdgeType::ColorSeam)) != 0)
					{
						UseThickness = (bIsBoundaryEdge) ? BoundaryEdgeThickness : ColorSeamThickness;
						UseColor = ColorSeamColor;
					}
				}

				const FVector A = WireSource->GetVertex(EdgeInfo.A);
				const FVector B = WireSource->GetVertex(EdgeInfo.B);
				const FVector LineDirection = (B - A).GetSafeNormal();
				const FVector2f UV(UseThickness, LineDepthBias);

				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 0) = static_cast<FVector3f>(A);
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 1) = static_cast<FVector3f>(B);
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 2) = static_cast<FVector3f>(B);
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 3) = static_cast<FVector3f>(A);

				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 0, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(-LineDirection));
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 1, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(-LineDirection));
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 2, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(LineDirection));
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 3, FVector3f::ZeroVector, FVector3f::ZeroVector, static_cast<FVector3f>(LineDirection));

				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 0, 0, UV);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 1, 0, UV);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 2, 0, UV);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 3, 0, UV);

				// The color stored in the vertices actually gets interpreted as a linear color by the material,
				// whereas it is more convenient for the user of the MeshWireframe to specify colors as sRGB. So we actually
				// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 0) = UseColor;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 1) = UseColor;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 2) = UseColor;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 3) = UseColor;

				IndexBuffer.Indices[IndexBufferIndex + 0] = VertexBufferIndex + 0;
				IndexBuffer.Indices[IndexBufferIndex + 1] = VertexBufferIndex + 1;
				IndexBuffer.Indices[IndexBufferIndex + 2] = VertexBufferIndex + 2;
				IndexBuffer.Indices[IndexBufferIndex + 3] = VertexBufferIndex + 2;
				IndexBuffer.Indices[IndexBufferIndex + 4] = VertexBufferIndex + 3;
				IndexBuffer.Indices[IndexBufferIndex + 5] = VertexBufferIndex + 0;
			}
		});

		ENQUEUE_RENDER_COMMAND(MeshWireframeVertexBuffersInit)(
			[this](FRHICommandListImmediate& RHICmdList)
		{
			VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
			VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
			VertexFactory.SetData(RHICmdList, Data);

			VertexFactory.InitResource(RHICmdList);
			IndexBuffer.InitResource(RHICmdList);
		});
	}

	virtual ~FMeshWireframeSceneProxy() override
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, 
		const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				for (const FWireframeLinesMeshBatchData& MeshBatchData : MeshBatchDatas)
				{
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshBatchData.MaterialProxy;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), false, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = MeshBatchData.StartIndex;
					BatchElement.NumPrimitives = MeshBatchData.NumPrimitives;
					BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
					BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }

	uint32 GetAllocatedSize() const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	virtual SIZE_T GetTypeHash() const override
	{
		static SIZE_T UniquePointer;
		return reinterpret_cast<SIZE_T>(&UniquePointer);
	}

private:
	TArray<FWireframeLinesMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};


UMeshWireframeComponent::UMeshWireframeComponent()
{
	CastShadow = false;
	bSelectable = false;
	PrimaryComponentTick.bCanEverTick = false;

	UPrimitiveComponent::SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

void UMeshWireframeComponent::SetWireframeSourceProvider(TSharedPtr<IMeshWireframeSourceProvider> Provider)
{
	SourceProvider = Provider;
	UpdateWireframe();
}

void UMeshWireframeComponent::UpdateWireframe()
{
	if (SourceProvider)
	{
		SourceProvider->AccessMesh([&](const IMeshWireframeSource& Source)
		{
			this->LocalBounds = Source.GetBounds();
		});
	}
	MarkRenderStateDirty();
}

void UMeshWireframeComponent::SetLineMaterial(UMaterialInterface* InLineMaterial)
{
	LineMaterial = InLineMaterial;
	SetMaterial(0, InLineMaterial);
}



FPrimitiveSceneProxy* UMeshWireframeComponent::CreateSceneProxy()
{
	if (SourceProvider)
	{
		FMeshWireframeSceneProxy* NewProxy = nullptr;
		SourceProvider->AccessMesh([this, &NewProxy](const IMeshWireframeSource& Source)
		{
			NewProxy = new FMeshWireframeSceneProxy(this, &Source);
		});
		return NewProxy;
	}
	return nullptr;
}

int32 UMeshWireframeComponent::GetNumMaterials() const
{
	return 1;
}

FBoxSphereBounds UMeshWireframeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return LocalBounds.TransformBy(LocalToWorld);
}


