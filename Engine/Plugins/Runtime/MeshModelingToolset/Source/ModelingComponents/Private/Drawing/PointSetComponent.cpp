// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/PointSetComponent.h"
#include "RenderingThread.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "VertexFactory.h"
#include "MaterialShared.h"
#include "Engine/CollisionProfile.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "LocalVertexFactory.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMeshBuilder.h"
#include "StaticMeshResources.h"
#include "Algo/Accumulate.h"
#include "Async/ParallelFor.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PointSetComponent)



struct FPointSetMeshBatchData
{
	FPointSetMeshBatchData()
		: MaterialProxy(nullptr)
	{}

	FMaterialRenderProxy* MaterialProxy;
	int32 StartIndex;
	int32 NumPrimitives;
	int32 MinVertexIndex;
	int32 MaxVertexIndex;
};





class FPointSetSceneProxy final : public FPrimitiveSceneProxy
{
public:

	FPointSetSceneProxy(UPointSetComponent* Component)
		: FPrimitiveSceneProxy(Component),
		MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform())),
		VertexFactory(GetScene().GetFeatureLevel(), "FPointSetSceneProxy")
	{
		const int32 NumPointVertices = Component->Points.Num() * 4;
		const int32 NumPointIndices = Component->Points.Num() * 6;
		const int32 TotalNumVertices = NumPointVertices;
		const int32 TotalNumIndices = NumPointIndices;
		const int32 NumTextureCoordinates = 1;

		VertexBuffers.PositionVertexBuffer.Init(TotalNumVertices);
		VertexBuffers.StaticMeshVertexBuffer.Init(TotalNumVertices, NumTextureCoordinates);
		VertexBuffers.ColorVertexBuffer.Init(TotalNumVertices);
		IndexBuffer.Indices.SetNumUninitialized(TotalNumIndices);

		int32 VertexBufferIndex = 0;
		int32 IndexBufferIndex = 0;
			
		// Initialize points.
		// Points are represented as two tris, all of whose vertices are coincident.
		// The material then offsets them according to the signs of the vertex normals in a camera facing orientation.
		// Size of the point is given by U0.
		if (Component->Points.Num() > 0)
		{
			MeshBatchDatas.Emplace();
			FPointSetMeshBatchData& MeshBatchData = MeshBatchDatas.Last();
			MeshBatchData.MinVertexIndex = VertexBufferIndex;
			MeshBatchData.MaxVertexIndex = VertexBufferIndex + NumPointVertices - 1;
			MeshBatchData.StartIndex = IndexBufferIndex;
			MeshBatchData.NumPrimitives = Component->Points.Num() * 2;
			if (Component->GetMaterial(0) != nullptr)
			{
				MeshBatchData.MaterialProxy = Component->GetMaterial(0)->GetRenderProxy();
			}
			else
			{
				MeshBatchData.MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
			}

			const FVector TangentVectors[4] = {
				FVector(1.0f, -1.0f, 0.0f),
				FVector(1.0f, 1.0f, 0.0f),
				FVector(-1.0f, 1.0f, 0.0f),
				FVector(-1.0f, -1.0f, 0.0f)
			};

			// flatten to list of linear points because the set might be very large
			// and we will benefit from parallel processing
			TArray<const FRenderablePoint*> LinearPoints;
			LinearPoints.Reserve(Component->Points.Num());
			for (FRenderablePoint& Point : Component->Points)
			{
				LinearPoints.Add(&Point);
			}

			// assemble the render buffers
			int NumLinearPoints = LinearPoints.Num();
			ParallelFor(NumLinearPoints, [&](int32 i)
			{
				const FRenderablePoint& Point = *LinearPoints[i];
				int UseVertexBufferIndex = 4 * i;
				int UseIndexBufferIndex = 6 * i;

				// The color stored in the vertices actually gets interpreted as a linear color by the material,
				// whereas it is more convenient for the user of the LineSet to specify colors as sRGB. So we actually
				// have to convert it back to linear. The ToFColor(false) call just scales back into 0-255 space.
				FColor color = FLinearColor::FromSRGBColor(Point.Color).ToFColor(false);

				const FVector2f UV(Point.Size, Point.DepthBias);
				for (int j = 0; j < 4; ++j)
				{
					VertexBuffers.PositionVertexBuffer.VertexPosition(UseVertexBufferIndex + j) = (FVector3f)Point.Position;
					VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(UseVertexBufferIndex + j, 0, UV);
					VertexBuffers.ColorVertexBuffer.VertexColor(UseVertexBufferIndex + j) = color;
					VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(UseVertexBufferIndex + j, FVector3f::ZeroVector, FVector3f::ZeroVector, (FVector3f)TangentVectors[j]);
				}

				IndexBuffer.Indices[UseIndexBufferIndex + 0] = UseVertexBufferIndex + 0;
				IndexBuffer.Indices[UseIndexBufferIndex + 1] = UseVertexBufferIndex + 1;
				IndexBuffer.Indices[UseIndexBufferIndex + 2] = UseVertexBufferIndex + 2;
				IndexBuffer.Indices[UseIndexBufferIndex + 3] = UseVertexBufferIndex + 2;
				IndexBuffer.Indices[UseIndexBufferIndex + 4] = UseVertexBufferIndex + 3;
				IndexBuffer.Indices[UseIndexBufferIndex + 5] = UseVertexBufferIndex + 0;
			});
		}

		ENQUEUE_RENDER_COMMAND(OverlayVertexBuffersInit)(
			[this](FRHICommandListImmediate& RHICmdList)
		{
			VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
			VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
			VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
			VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
			VertexFactory.SetData(RHICmdList, Data);

			VertexFactory.InitResource(RHICmdList);
			IndexBuffer.InitResource(RHICmdList);
		});
	}

	virtual ~FPointSetSceneProxy()
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				for (const FPointSetMeshBatchData& MeshBatchData : MeshBatchDatas)
				{
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshBatchData.MaterialProxy;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = MeshBatchData.StartIndex;
					BatchElement.NumPrimitives = MeshBatchData.NumPrimitives;
					BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
					BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }

	uint32 GetAllocatedSize() const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	virtual SIZE_T GetTypeHash() const override
	{
		static SIZE_T UniquePointer;
		return reinterpret_cast<SIZE_T>(&UniquePointer);
	}

private:
	TArray<FPointSetMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};





UPointSetComponent::UPointSetComponent()
{
	CastShadow = false;
	bSelectable = false;
	PrimaryComponentTick.bCanEverTick = false;
	bBoundsDirty = true;

	UPrimitiveComponent::SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

void UPointSetComponent::SetPointMaterial(UMaterialInterface* InPointMaterial)
{
	PointMaterial = InPointMaterial;
	UMeshComponent::SetMaterial(0, InPointMaterial);
}


void UPointSetComponent::Clear()
{
	Points.Reset();
	MarkRenderStateDirty();
	bBoundsDirty = true;
}


void UPointSetComponent::ReservePoints(const int32 MaxID)
{
	Points.Reserve(MaxID);
}



int32 UPointSetComponent::AddPoint(const FRenderablePoint& OverlayPoint)
{
	MarkRenderStateDirty();
	return AddPointInternal(OverlayPoint);
}

int32 UPointSetComponent::AddPointInternal(const FRenderablePoint& Point)
{
	const int32 ID(Points.Add(Point));
	bBoundsDirty = true;
	return ID;
}

int32 UPointSetComponent::AddPoints(
	const TArray<FVector>& Positions,
	const FColor& InColor,
	const float InSize,
	const float InDepthBias
)
{
	const int NumPoints = Positions.Num();
	if (NumPoints == 0)
	{
		return 0;
	}
	MarkRenderStateDirty();
	bBoundsDirty = true;
	for (FVector P : Positions)
	{
		Points.Add(FRenderablePoint(P, InColor, InSize, InDepthBias));
	}
	return NumPoints;
}

int32 UPointSetComponent::AddPoints(const TArray<FRenderablePoint>& InPoints)
{
	const int NumPoints = InPoints.Num();
	if (NumPoints == 0)
	{
		return 0;
	}

	MarkRenderStateDirty();
	bBoundsDirty = true;

	for (int32 Idx = 0; Idx < NumPoints; ++Idx)
	{
		Points.Add(InPoints[Idx]);
	}

	return NumPoints;
}

void UPointSetComponent::InsertPoint(const int32 ID, const FRenderablePoint& OverlayPoint)
{
	Points.Insert(ID, OverlayPoint);
	MarkRenderStateDirty();
	bBoundsDirty = true;
}


const FRenderablePoint& UPointSetComponent::GetPoint(const int32 ID)
{
	return Points[ID];
}


void UPointSetComponent::SetPointColor(const int32 ID, const FColor& NewColor)
{
	FRenderablePoint& OverlayPoint = Points[ID];
	OverlayPoint.Color = NewColor;
	MarkRenderStateDirty();
}


void UPointSetComponent::SetPointSize(const int32 ID, const float NewSize)
{
	FRenderablePoint& OverlayPoint = Points[ID];
	OverlayPoint.Size = NewSize;
	MarkRenderStateDirty();
}


void UPointSetComponent::SetPointPosition(const int32 ID, const FVector& NewPosition)
{
	FRenderablePoint& OverlayPoint = Points[ID];
	OverlayPoint.Position = NewPosition;
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void UPointSetComponent::SetAllPointsColor(const FColor& NewColor)
{
	for (FRenderablePoint& Point : Points)
	{
		Point.Color = NewColor;
	}
	MarkRenderStateDirty();
}

void UPointSetComponent::SetAllPointsSize(float NewSize)
{
	for (FRenderablePoint& Point : Points)
	{
		Point.Size = NewSize;
	}
	MarkRenderStateDirty();
}

void UPointSetComponent::SetAllPointsDepthBias(float NewDepthBias)
{
	for (FRenderablePoint& Point : Points)
	{
		Point.DepthBias = NewDepthBias;
	}
	MarkRenderStateDirty();
}


void UPointSetComponent::RemovePoint(const int32 ID)
{
	Points.RemoveAt(ID);
	MarkRenderStateDirty();
	bBoundsDirty = true;
}


bool UPointSetComponent::IsPointValid(const int32 ID) const
{
	return Points.IsValidIndex(ID);
}


FPrimitiveSceneProxy* UPointSetComponent::CreateSceneProxy()
{
	if (Points.Num() > 0)
	{
		return new FPointSetSceneProxy(this);
	}

	return nullptr;
}


int32 UPointSetComponent::GetNumMaterials() const
{
	return 1;
}


FBoxSphereBounds UPointSetComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (bBoundsDirty)
	{
		FBox Box(ForceInit);
		for (const FRenderablePoint& Point : Points)
		{
			Box += Point.Position;
		}
		Bounds = FBoxSphereBounds(Box);
		bBoundsDirty = false;
	}
	FBoxSphereBounds TransformBounds = Bounds.TransformBy(LocalToWorld);

	// TODO: This next bit is not ideal because the point size is specified in onscreen pixels,
	// so the true amount by which we would need to expand bounds depends on camera location, FOV, etc.
	// We mainly do this as a hack against a problem in ortho viewports, which cull small items
	// based on their bounds, and a set consisting of a single point will always be culled due
	// to having 0-sized bounds. It's worth noting that when zooming out sufficiently far, the
	// point will still be culled even with this hack, however.
	// The proper solution is to be able to opt out of the ortho culling behavior, which is something
	// we need to add.
	if (Points.Num() > 0)
	{
		TransformBounds = TransformBounds.ExpandBy(Points[0].Size);
	}

	return TransformBounds;
}


void UPointSetComponent::AddPoints(
	int32 NumIndices,
	TFunctionRef<void(int32 Index, TArray<FRenderablePoint>& PointsOut)> PointGenFunc,
	int32 PointsPerIndexHint,
	bool bDeferRenderStateDirty)
{
	TArray<FRenderablePoint> TempPoints;
	if (PointsPerIndexHint > 0)
	{
		ReservePoints(Points.Num() + NumIndices*PointsPerIndexHint);
		TempPoints.Reserve(PointsPerIndexHint);
	}

	for (int32 k = 0; k < NumIndices; ++k)
	{
		TempPoints.Reset();
		PointGenFunc(k, TempPoints);
		for (const FRenderablePoint& Point : TempPoints)
		{
			AddPointInternal(Point);
		}
	}

	if (!bDeferRenderStateDirty)
	{
		MarkRenderStateDirty();
	}
}
