// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/PolyEditPreviewMesh.h"
#include "DynamicSubmesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Operations/ExtrudeMesh.h"
#include "Operations/InsetMeshRegion.h"
#include "Selections/MeshVertexSelection.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyEditPreviewMesh)

using namespace UE::Geometry;

void UPolyEditPreviewMesh::InitializeExtrudeType(
	const FDynamicMesh3* SourceMesh, const TArray<int32>& Triangles,
	const FVector3d& TransformedOffsetDirection,
	const FTransform3d* MeshTransformIn,
	bool bDeleteExtrudeBaseFaces)
{
	// extract submesh
	ActiveSubmesh = MakeUnique<FDynamicSubmesh3>(SourceMesh, Triangles, (int32)(EMeshComponents::FaceGroups) | (int32)(EMeshComponents::VertexNormals), true);
	FDynamicMesh3& EditPatch = ActiveSubmesh->GetSubmesh();

	check(EditPatch.IsCompact());

	// do we want to apply a transform?
	bHaveMeshTransform = (MeshTransformIn != nullptr);
	if (bHaveMeshTransform)
	{
		MeshTransform = *MeshTransformIn;
		MeshTransforms::ApplyTransform(EditPatch, MeshTransform, true);
	}
	//FMeshNormals::QuickComputeVertexNormals(EditPatch);

	// save copy of initial patch
	InitialEditPatch = EditPatch;

	// save initial tri normals
	InitialTriNormals.SetNum(InitialEditPatch.MaxTriangleID());
	for ( int32 tid = 0; tid < InitialEditPatch.MaxTriangleID(); ++tid)
	{
		InitialTriNormals[tid] = InitialEditPatch.GetTriNormal(tid);
	}

	// extrude initial patch by a tiny amount so that we get normals
	FExtrudeMesh Extruder(&EditPatch);
	Extruder.DefaultExtrudeDistance = 0.01;
	Extruder.Apply();

	// get set of extrude vertices
	ExtrudeToInitialVerts.Reset();
	EditVertices.Reset();
	FMeshVertexSelection Vertices(&EditPatch);
	for (const FExtrudeMesh::FExtrusionInfo& Extrusion : Extruder.Extrusions)
	{
		Vertices.SelectTriangleVertices(Extrusion.OffsetTriangles);

		for (TPair<int32, int32> pair : Extrusion.InitialToOffsetMapV)
		{
			ExtrudeToInitialVerts.Add(pair.Value, pair.Key);
		}
	}
	EditVertices = Vertices.AsArray();

	// save initial extrude positions
	InitialPositions.Reset();
	InitialNormals.Reset();
	for (int32 vid : EditVertices)
	{
		InitialPositions.Add(EditPatch.GetVertex(vid));
		InitialNormals.Add((FVector3d)EditPatch.GetVertexNormal(vid));
	}

	if (bDeleteExtrudeBaseFaces)
	{
		FDynamicMeshEditor Editor(&EditPatch);
		for (const FExtrudeMesh::FExtrusionInfo& Extrusion : Extruder.Extrusions)
		{
			Editor.RemoveTriangles(Extrusion.InitialTriangles, false);
		}
	}

	InputDirection = TransformedOffsetDirection;

	// initialize the preview mesh
	UpdatePreview(&EditPatch);
}

void UPolyEditPreviewMesh::InitializeExtrudeType(FDynamicMesh3&& BaseMesh,
	const FVector3d& TransformedOffsetDirection,
	const FTransform3d* MeshTransformIn,
	bool bDeleteExtrudeBaseFaces)
{
	InitialEditPatch = MoveTemp(BaseMesh);

	// do we want to apply a transform?
	bHaveMeshTransform = (MeshTransformIn != nullptr);
	if (bHaveMeshTransform)
	{
		MeshTransform = *MeshTransformIn;
		MeshTransforms::ApplyTransform(InitialEditPatch, MeshTransform, true);
	}

	// save initial tri normals
	InitialTriNormals.SetNum(InitialEditPatch.MaxTriangleID());
	for ( int32 tid : InitialEditPatch.TriangleIndicesItr() )
	{
		InitialTriNormals[tid] = InitialEditPatch.GetTriNormal(tid);
	}

	// extrude initial patch by a tiny amount so that we get normals
	FDynamicMesh3 EditPatch(InitialEditPatch);
	FExtrudeMesh Extruder(&EditPatch);
	Extruder.DefaultExtrudeDistance = 0.01;
	Extruder.Apply();

	// get set of extrude vertices
	ExtrudeToInitialVerts.Reset();
	EditVertices.Reset();
	FMeshVertexSelection Vertices(&EditPatch);
	for (const FExtrudeMesh::FExtrusionInfo& Extrusion : Extruder.Extrusions)
	{
		Vertices.SelectTriangleVertices(Extrusion.OffsetTriangles);

		for (TPair<int32,int32> pair : Extrusion.InitialToOffsetMapV)
		{
			ExtrudeToInitialVerts.Add(pair.Value, pair.Key);
		}
	}
	EditVertices = Vertices.AsArray();

	// save initial extrude positions
	InitialPositions.Reset();
	InitialNormals.Reset();
	for (int32 vid : EditVertices)
	{
		InitialPositions.Add(EditPatch.GetVertex(vid));
		InitialNormals.Add((FVector3d)EditPatch.GetVertexNormal(vid));
	}

	if (bDeleteExtrudeBaseFaces)
	{
		FDynamicMeshEditor Editor(&EditPatch);
		for (const FExtrudeMesh::FExtrusionInfo& Extrusion : Extruder.Extrusions)
		{
			Editor.RemoveTriangles(Extrusion.InitialTriangles, false);
		}
	}

	InputDirection = TransformedOffsetDirection;

	// initialize the preview mesh
	UpdatePreview(&EditPatch);
}




void UPolyEditPreviewMesh::UpdateExtrudeType(double NewOffset, bool bUseNormalDirection)
{
	EditMesh([&](FDynamicMesh3& Mesh)
	{
		int32 NumVertices = EditVertices.Num();
		for (int32 k = 0; k < NumVertices; ++k)
		{
			int vid = EditVertices[k];
			FVector3d InitialPos = InitialPositions[k];
			FVector3d NewPos = InitialPos + NewOffset * (bUseNormalDirection ? InitialNormals[k] : InputDirection);
			Mesh.SetVertex(vid, NewPos);
		}
	});
}



void UPolyEditPreviewMesh::UpdateExtrudeType_FaceNormalAvg(double NewOffset)
{
	EditMesh([&](FDynamicMesh3& Mesh)
	{
		int32 NumVertices = EditVertices.Num();
		TArray<FVector3d> NewPositions;
		NewPositions.SetNum(NumVertices);
		for (int32 k = 0; k < NumVertices; ++k)
		{
			int vid = EditVertices[k];
			FVector3d InitialPos = InitialPositions[k];
			FVector3d AccumV = FVector3d::Zero();
			int32 Count = 0;
			int32 InitialVID = ExtrudeToInitialVerts[vid];
			InitialEditPatch.EnumerateVertexTriangles(InitialVID, [&](int32 InitialTID)
			{
				AccumV += (InitialPos + NewOffset * InitialTriNormals[InitialTID]);
				Count++;
			});
			NewPositions[k] = (Count == 0) ? InitialPos : (AccumV / (double)Count);
		}
		for (int32 k = 0; k < NumVertices; ++k)
		{
			Mesh.SetVertex(EditVertices[k], NewPositions[k]);
		}
	});
}



void UPolyEditPreviewMesh::UpdateExtrudeType(TFunctionRef<void(FDynamicMesh3&)> UpdateMeshFunc, bool bFullRecalculate)
{
	if (bFullRecalculate)
	{
		FDynamicMesh3 TempMesh(InitialEditPatch);
		UpdateMeshFunc(TempMesh);
		ReplaceMesh(MoveTemp(TempMesh));
	}
	else
	{
		EditMesh(UpdateMeshFunc);
	}
}


void UPolyEditPreviewMesh::MakeExtrudeTypeHitTargetMesh(FDynamicMesh3& TargetMesh, bool bUseNormalDirection)
{
	FVector3d ExtrudeDirection = InputDirection;
	double Length = 99999.0;

	TargetMesh = InitialEditPatch;
	MeshTransforms::Translate(TargetMesh, -Length * ExtrudeDirection);

	FExtrudeMesh Extruder(&TargetMesh);
	Extruder.ExtrudedPositionFunc = [&](const FVector3d& Position, const FVector3f& Normal, int VertexID)
	{
		return Position + 2.0 * Length * (bUseNormalDirection ? (FVector3d)Normal : ExtrudeDirection);
	};
	Extruder.Apply();
}



void UPolyEditPreviewMesh::InitializeInsetType(const FDynamicMesh3* SourceMesh, const TArray<int32>& Triangles,
	const FTransform3d* MeshTransformIn)
{
	// extract submesh
	ActiveSubmesh = MakeUnique<FDynamicSubmesh3>(SourceMesh, Triangles, (int32)EMeshComponents::FaceGroups, true);
	FDynamicMesh3& EditPatch = ActiveSubmesh->GetSubmesh();

	check(EditPatch.IsCompact());

	// do we want to apply a transform?
	bHaveMeshTransform = (MeshTransformIn != nullptr);
	if (bHaveMeshTransform)
	{
		MeshTransform = *MeshTransformIn;
		MeshTransforms::ApplyTransform(EditPatch, MeshTransform, true);
	}

	// save copy of initial patch
	InitialEditPatch = EditPatch;

	// initialize the preview mesh
	UpdatePreview(&EditPatch);
}


void UPolyEditPreviewMesh::UpdateInsetType(double NewOffset, bool bReproject, double Softness, double AreaScaleT, bool bBoundaryOnly)
{
	FDynamicMesh3 EditPatch(InitialEditPatch);
	FInsetMeshRegion Inset(&EditPatch);
	for (int32 tid : EditPatch.TriangleIndicesItr())
	{
		Inset.Triangles.Add(tid);
	}
	Inset.InsetDistance = NewOffset;
	Inset.bReproject = bReproject;
	Inset.Softness = Softness;
	Inset.AreaCorrection = AreaScaleT;
	Inset.bSolveRegionInteriors = !bBoundaryOnly;
	Inset.Apply();

	FMeshNormals::QuickRecomputeOverlayNormals(EditPatch);

	UpdatePreview(&EditPatch);
}


void UPolyEditPreviewMesh::MakeInsetTypeTargetMesh(FDynamicMesh3& TargetMesh)
{
	TargetMesh = InitialEditPatch;
	MeshTransforms::ApplyTransform(TargetMesh, GetTransform());
}





void UPolyEditPreviewMesh::InitializeStaticType(const FDynamicMesh3* SourceMesh, const TArray<int32>& Triangles,
	const FTransform3d* MeshTransformIn)
{
	// extract submesh
	ActiveSubmesh = MakeUnique<FDynamicSubmesh3>(SourceMesh, Triangles, (int32)EMeshComponents::FaceGroups, true);
	FDynamicMesh3& EditPatch = ActiveSubmesh->GetSubmesh();

	check(EditPatch.IsCompact());

	// do we want to apply a transform?
	bHaveMeshTransform = (MeshTransformIn != nullptr);
	if (bHaveMeshTransform)
	{
		MeshTransform = *MeshTransformIn;
		MeshTransforms::ApplyTransform(EditPatch, MeshTransform, true);
	}

	// save copy of initial patch
	InitialEditPatch = EditPatch;

	// initialize the preview mesh
	UpdatePreview(&EditPatch);
}


void UPolyEditPreviewMesh::UpdateStaticType(TFunctionRef<void(FDynamicMesh3&)> UpdateMeshFunc, bool bFullRecalculate)
{
	if (bFullRecalculate)
	{
		FDynamicMesh3 TempMesh(InitialEditPatch);
		UpdateMeshFunc(TempMesh);
		ReplaceMesh(MoveTemp(TempMesh));
	}
	else
	{
		EditMesh(UpdateMeshFunc);
	}
}

void UPolyEditPreviewMesh::MakeStaticTypeTargetMesh(FDynamicMesh3& TargetMesh)
{
	TargetMesh = InitialEditPatch;
}
