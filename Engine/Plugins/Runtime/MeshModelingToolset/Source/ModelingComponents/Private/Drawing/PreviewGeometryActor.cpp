// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/PreviewGeometryActor.h"
#include "ToolSetupUtil.h"

// to create sphere root component
#include "Components/SphereComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/World.h"
#include "Engine/CollisionProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewGeometryActor)


UPreviewGeometry::~UPreviewGeometry()
{
	ensureMsgf(ParentActor == nullptr, TEXT("You must explicitly Disconnect() UPreviewGeometry before it is GCd"));
}

void UPreviewGeometry::CreateInWorld(UWorld* World, const FTransform& WithTransform)
{
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	ParentActor = World->SpawnActor<APreviewGeometryActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	// root component is a hidden sphere
	USphereComponent* SphereComponent = NewObject<USphereComponent>(ParentActor);
	ParentActor->SetRootComponent(SphereComponent);
	SphereComponent->InitSphereRadius(1.0f);
	SphereComponent->SetVisibility(false);
	SphereComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	ParentActor->SetActorTransform(WithTransform);

	OnCreated();
}


void UPreviewGeometry::Disconnect()
{
	OnDisconnected();

	if (ParentActor != nullptr)
	{
		ParentActor->Destroy();
		ParentActor = nullptr;
	}
}



FTransform UPreviewGeometry::GetTransform() const
{
	if (ParentActor != nullptr)
	{
		return ParentActor->GetTransform();
	}
	return FTransform();
}

void UPreviewGeometry::SetTransform(const FTransform& UseTransform)
{
	if (ParentActor != nullptr)
	{
		ParentActor->SetActorTransform(UseTransform);
	}
}


void UPreviewGeometry::SetAllVisible(bool bVisible)
{
	for (TPair<FString, TObjectPtr<UTriangleSetComponent>> Entry : TriangleSets)
	{
		if (Entry.Value)
		{
			Entry.Value->SetVisibility(bVisible);
		}
	}
	for (TPair<FString, TObjectPtr<ULineSetComponent>> Entry : LineSets)
	{
		if (Entry.Value)
		{
			Entry.Value->SetVisibility(bVisible);
		}
	}
	for (TPair<FString, TObjectPtr<UPointSetComponent>> Entry : PointSets)
	{
		if (Entry.Value)
		{
			Entry.Value->SetVisibility(bVisible);
		}
	}
}


UTriangleSetComponent* UPreviewGeometry::AddTriangleSet(const FString& SetIdentifier)
{
	if (!ParentActor)
	{
		return nullptr;
	}
	
	if (TriangleSets.Contains(SetIdentifier))
	{
		check(false);
		return nullptr;
	}

	UTriangleSetComponent* TriangleSet = NewObject<UTriangleSetComponent>(ParentActor);
	TriangleSet->SetupAttachment(ParentActor->GetRootComponent());
	TriangleSet->RegisterComponent();

	TriangleSets.Add(SetIdentifier, TriangleSet);
	return TriangleSet;
}


UTriangleSetComponent* UPreviewGeometry::FindTriangleSet(const FString& TriangleSetIdentifier)
{
	TObjectPtr<UTriangleSetComponent>* Found = TriangleSets.Find(TriangleSetIdentifier);
	if (Found != nullptr)
	{
		return *Found;
	}
	return nullptr;
}



UTriangleSetComponent* UPreviewGeometry::CreateOrUpdateTriangleSet(const FString& TriangleSetIdentifier, int32 NumIndices,
	TFunctionRef<void(int32 Index, TArray<FRenderableTriangle>& TrianglesOut)> TriangleGenFunc,
	int32 TrianglesPerIndexHint)
{
	UTriangleSetComponent* TriangleSet = FindTriangleSet(TriangleSetIdentifier);
	if (TriangleSet == nullptr)
	{
		TriangleSet = AddTriangleSet(TriangleSetIdentifier);
		if (TriangleSet == nullptr)
		{
			check(false);
			return nullptr;
		}
	}

	TriangleSet->Clear();
	TriangleSet->AddTriangles(NumIndices, TriangleGenFunc, TrianglesPerIndexHint);
	return TriangleSet;
}


ULineSetComponent* UPreviewGeometry::AddLineSet(const FString& SetIdentifier)
{
	if (!ParentActor)
	{
		return nullptr;
	}
	
	if (LineSets.Contains(SetIdentifier))
	{
		check(false);
		return nullptr;
	}

	ULineSetComponent* LineSet = NewObject<ULineSetComponent>(ParentActor);
	LineSet->SetupAttachment(ParentActor->GetRootComponent());

	UMaterialInterface* LineMaterial = ToolSetupUtil::GetDefaultLineComponentMaterial(nullptr);
	if (LineMaterial != nullptr)
	{
		LineSet->SetLineMaterial(LineMaterial);
	}

	LineSet->RegisterComponent();

	LineSets.Add(SetIdentifier, LineSet);
	return LineSet;
}


ULineSetComponent* UPreviewGeometry::FindLineSet(const FString& LineSetIdentifier)
{
	TObjectPtr<ULineSetComponent>* Found = LineSets.Find(LineSetIdentifier);
	if (Found != nullptr)
	{
		return *Found;
	}
	return nullptr;
}

bool UPreviewGeometry::RemoveLineSet(const FString& LineSetIdentifier, bool bDestroy)
{
	TObjectPtr<ULineSetComponent>* Found = LineSets.Find(LineSetIdentifier);
	if (Found != nullptr)
	{
		ULineSetComponent* LineSet = *Found;
		LineSets.Remove(LineSetIdentifier);
		if (bDestroy && LineSet)
		{
			LineSet->UnregisterComponent();
			LineSet->DestroyComponent();
			LineSet = nullptr;
		}
		return true;
	}
	return false;
}



void UPreviewGeometry::RemoveAllLineSets(bool bDestroy)
{
	if (bDestroy)
	{
		for (TPair<FString, TObjectPtr<ULineSetComponent>> Entry : LineSets)
		{
			if (Entry.Value)
			{
				Entry.Value->UnregisterComponent();
				Entry.Value->DestroyComponent();
			}
		}
	}
	LineSets.Reset();
}


bool UPreviewGeometry::RemoveTriangleSet(const FString& TriangleSetIdentifier, bool bDestroy)
{
	TObjectPtr<UTriangleSetComponent>* Found = TriangleSets.Find(TriangleSetIdentifier);
	if (Found != nullptr)
	{
		UTriangleSetComponent* TriangleSet = *Found;

		TriangleSets.Remove(TriangleSetIdentifier);
		if (bDestroy && TriangleSet)
		{
			TriangleSet->UnregisterComponent();
			TriangleSet->DestroyComponent();
			TriangleSet = nullptr;
		}
		return true;
	}
	return false;
}



void UPreviewGeometry::RemoveAllTriangleSets(bool bDestroy)
{
	if (bDestroy)
	{
		for (TPair<FString, TObjectPtr<UTriangleSetComponent>> Entry : TriangleSets)
		{
			if (Entry.Value)
			{
				Entry.Value->UnregisterComponent();
				Entry.Value->DestroyComponent();
			}
		}
	}
	TriangleSets.Reset();
}



bool UPreviewGeometry::SetLineSetVisibility(const FString& LineSetIdentifier, bool bVisible)
{
	TObjectPtr<ULineSetComponent>* Found = LineSets.Find(LineSetIdentifier);
	if (Found != nullptr)
	{
		(*Found)->SetVisibility(bVisible);
		return true;
	}
	return false;
}


bool UPreviewGeometry::SetLineSetMaterial(const FString& LineSetIdentifier, UMaterialInterface* NewMaterial)
{
	TObjectPtr<ULineSetComponent>* Found = LineSets.Find(LineSetIdentifier);
	if (Found != nullptr)
	{
		(*Found)->SetLineMaterial(NewMaterial);
		return true;
	}
	return false;
}


void UPreviewGeometry::SetAllLineSetsMaterial(UMaterialInterface* Material)
{
	for (TPair<FString, TObjectPtr<ULineSetComponent>> Entry : LineSets)
	{
		if (Entry.Value)
		{
			Entry.Value->SetLineMaterial(Material);
		}
	}
}



ULineSetComponent* UPreviewGeometry::CreateOrUpdateLineSet(const FString& LineSetIdentifier, int32 NumIndices,
	TFunctionRef<void(int32 Index, TArray<FRenderableLine>& LinesOut)> LineGenFunc,
	int32 LinesPerIndexHint)
{
	ULineSetComponent* LineSet = FindLineSet(LineSetIdentifier);
	if (LineSet == nullptr)
	{
		LineSet = AddLineSet(LineSetIdentifier);
		if (LineSet == nullptr)
		{
			check(false);
			return nullptr;
		}
	}

	LineSet->Clear();
	LineSet->AddLines(NumIndices, LineGenFunc, LinesPerIndexHint);
	return LineSet;
}


UPointSetComponent* UPreviewGeometry::AddPointSet(const FString& SetIdentifier)
{
	if (!ParentActor)
	{
		return nullptr;
	}
	
	if (PointSets.Contains(SetIdentifier))
	{
		check(false);
		return nullptr;
	}

	UPointSetComponent* PointSet = NewObject<UPointSetComponent>(ParentActor);
	PointSet->SetupAttachment(ParentActor->GetRootComponent());

	UMaterialInterface* PointMaterial = ToolSetupUtil::GetDefaultPointComponentMaterial(nullptr);
	if (PointMaterial != nullptr)
	{
		PointSet->SetPointMaterial(PointMaterial);
	}

	PointSet->RegisterComponent();

	PointSets.Add(SetIdentifier, PointSet);
	return PointSet;
}

UPointSetComponent* UPreviewGeometry::FindPointSet(const FString& PointSetIdentifier)
{
	TObjectPtr<UPointSetComponent>* Found = PointSets.Find(PointSetIdentifier);
	if (Found != nullptr)
	{
		return *Found;
	}
	return nullptr;
}

void UPreviewGeometry::CreateOrUpdatePointSet(const FString& PointSetIdentifier, int32 NumIndices,
	TFunctionRef<void(int32 Index, TArray<FRenderablePoint>& PointsOut)> PointGenFunc,
	int32 PointsPerIndexHint)
{
	UPointSetComponent* PointSet = FindPointSet(PointSetIdentifier);
	if (PointSet == nullptr)
	{
		PointSet = AddPointSet(PointSetIdentifier);
		if (PointSet == nullptr)
		{
			check(false);
			return;
		}
	}

	PointSet->Clear();
	PointSet->AddPoints(NumIndices, PointGenFunc, PointsPerIndexHint);
}

bool UPreviewGeometry::RemovePointSet(const FString& PointSetIdentifier, bool bDestroy)
{
	TObjectPtr<UPointSetComponent>* Found = PointSets.Find(PointSetIdentifier);
	if (Found != nullptr)
	{
		UPointSetComponent* PointSet = *Found;
		PointSets.Remove(PointSetIdentifier);
		if (bDestroy && PointSet)
		{
			PointSet->UnregisterComponent();
			PointSet->DestroyComponent();
			PointSet = nullptr;
		}
		return true;
	}
	return false;
}

void UPreviewGeometry::RemoveAllPointSets(bool bDestroy)
{
	if (bDestroy)
	{
		for (TPair<FString, TObjectPtr<UPointSetComponent>> Entry : PointSets)
		{
			if (Entry.Value)
			{
				Entry.Value->UnregisterComponent();
				Entry.Value->DestroyComponent();
			}
		}
	}
	PointSets.Reset();
}

bool UPreviewGeometry::SetPointSetVisibility(const FString& PointSetIdentifier, bool bVisible)
{
	TObjectPtr<UPointSetComponent>* Found = PointSets.Find(PointSetIdentifier);
	if (Found != nullptr)
	{
		(*Found)->SetVisibility(bVisible);
		return true;
	}
	return false;
}

bool UPreviewGeometry::SetPointSetMaterial(const FString& PointSetIdentifier, UMaterialInterface* NewMaterial)
{
	TObjectPtr<UPointSetComponent>* Found = PointSets.Find(PointSetIdentifier);
	if (Found != nullptr)
	{
		(*Found)->SetPointMaterial(NewMaterial);
		return true;
	}
	return false;
}

void UPreviewGeometry::SetAllPointSetsMaterial(UMaterialInterface* Material)
{
	for (TPair<FString, TObjectPtr<UPointSetComponent>> Entry : PointSets)
	{
		if (Entry.Value)
		{
			Entry.Value->SetPointMaterial(Material);
		}
	}
}
