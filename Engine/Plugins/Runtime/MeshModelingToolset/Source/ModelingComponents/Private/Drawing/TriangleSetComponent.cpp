// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/TriangleSetComponent.h"

#include "DynamicMeshBuilder.h"
#include "Engine/CollisionProfile.h"
#include "Materials/MaterialRelevance.h"
#include "LocalVertexFactory.h"
#include "PrimitiveSceneProxy.h"
#include "SceneInterface.h"
#include "ShaderCompiler.h"
#include "StaticMeshResources.h"

#include "Algo/Accumulate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TriangleSetComponent)

struct FTriangleSetMeshBatchData
{
	FTriangleSetMeshBatchData()
		: MaterialProxy(nullptr)
	{}

	FMaterialRenderProxy* MaterialProxy;
	int32 StartIndex;
	int32 NumPrimitives;
	int32 MinVertexIndex;
	int32 MaxVertexIndex;
};

/** Class for the TriangleSetComponent data passed to the render thread. */
class FTriangleSetSceneProxy final : public FPrimitiveSceneProxy
{
public:

	FTriangleSetSceneProxy(UTriangleSetComponent* Component)
		: FPrimitiveSceneProxy(Component),
		MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform())),
		VertexFactory(GetScene().GetFeatureLevel(), "FTriangleSetSceneProxy")
	{
		const int32 NumTriangleVertices = Algo::Accumulate(Component->TrianglesByMaterial, 0, [](int32 Acc, const TSparseArray<FRenderableTriangle>& Tris) { return Acc + Tris.Num() * 3; });
		const int32 NumTriangleIndices = NumTriangleVertices;

		const int32 NumTextureCoordinates = 1;

		VertexBuffers.PositionVertexBuffer.Init(NumTriangleVertices);
		VertexBuffers.StaticMeshVertexBuffer.Init(NumTriangleVertices, NumTextureCoordinates);
		VertexBuffers.ColorVertexBuffer.Init(NumTriangleVertices);
		IndexBuffer.Indices.SetNumUninitialized(NumTriangleIndices);

		int32 VertexBufferIndex = 0;
		int32 IndexBufferIndex = 0;

		// Triangles
		for (TPair<UMaterialInterface*, int32> MaterialAndMaterialIndex : Component->MaterialToIndex)
		{
			UMaterialInterface* Material = MaterialAndMaterialIndex.Get<0>();
			if (Material == nullptr)
			{
				continue;
			}
			
			const int32 MaterialIndex = MaterialAndMaterialIndex.Get<1>();
			const TSparseArray<FRenderableTriangle>& MaterialTriangles = Component->TrianglesByMaterial[MaterialIndex];

			MeshBatchDatas.Emplace();
			FTriangleSetMeshBatchData& MeshBatchData = MeshBatchDatas.Last();
			MeshBatchData.MinVertexIndex = VertexBufferIndex;
			MeshBatchData.MaxVertexIndex = VertexBufferIndex + MaterialTriangles.Num() * 3 - 1;
			MeshBatchData.StartIndex = IndexBufferIndex;
			MeshBatchData.NumPrimitives = MaterialTriangles.Num();
			MeshBatchData.MaterialProxy = Material->GetRenderProxy();

			for (const FRenderableTriangle& Triangle : MaterialTriangles)
			{
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 0) = (FVector3f)Triangle.Vertex0.Position;
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 1) = (FVector3f)Triangle.Vertex1.Position;
				VertexBuffers.PositionVertexBuffer.VertexPosition(VertexBufferIndex + 2) = (FVector3f)Triangle.Vertex2.Position;

				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 0, FVector3f(1, 0, 0), FVector3f(0, 1, 0), (FVector3f)Triangle.Vertex0.Normal);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 1, FVector3f(1, 0, 0), FVector3f(0, 1, 0), (FVector3f)Triangle.Vertex1.Normal);
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertexBufferIndex + 2, FVector3f(1, 0, 0), FVector3f(0, 1, 0), (FVector3f)Triangle.Vertex2.Normal);

				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 0, 0, FVector2f(Triangle.Vertex0.UV));
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 1, 0, FVector2f(Triangle.Vertex1.UV));
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertexBufferIndex + 2, 0, FVector2f(Triangle.Vertex2.UV));

				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 0) = Triangle.Vertex0.Color;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 1) = Triangle.Vertex1.Color;
				VertexBuffers.ColorVertexBuffer.VertexColor(VertexBufferIndex + 2) = Triangle.Vertex2.Color;

				IndexBuffer.Indices[IndexBufferIndex + 0] = VertexBufferIndex + 0;
				IndexBuffer.Indices[IndexBufferIndex + 1] = VertexBufferIndex + 1;
				IndexBuffer.Indices[IndexBufferIndex + 2] = VertexBufferIndex + 2;

				VertexBufferIndex += 3;
				IndexBufferIndex += 3;
			}
		}

		ENQUEUE_RENDER_COMMAND(LineSetVertexBuffersInit)(
			[this](FRHICommandListImmediate& RHICmdList)
			{
				VertexBuffers.PositionVertexBuffer.InitResource(RHICmdList);
				VertexBuffers.StaticMeshVertexBuffer.InitResource(RHICmdList);
				VertexBuffers.ColorVertexBuffer.InitResource(RHICmdList);

				FLocalVertexFactory::FDataType Data;
				VertexBuffers.PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
				VertexBuffers.StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
				VertexBuffers.StaticMeshVertexBuffer.BindTexCoordVertexBuffer(&VertexFactory, Data);
				VertexBuffers.ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);
				VertexFactory.SetData(RHICmdList, Data);

				VertexFactory.InitResource(RHICmdList);
				IndexBuffer.InitResource(RHICmdList);
			});
	}

	virtual ~FTriangleSetSceneProxy()
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views,
		const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_OverlaySceneProxy_GetDynamicMeshElements);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				for (const FTriangleSetMeshBatchData& MeshBatchData : MeshBatchDatas)
				{
					FMeshBatch& Mesh = Collector.AllocateMesh();
					FMeshBatchElement& BatchElement = Mesh.Elements[0];
					BatchElement.IndexBuffer = &IndexBuffer;
					Mesh.bWireframe = false;
					Mesh.VertexFactory = &VertexFactory;
					Mesh.MaterialRenderProxy = MeshBatchData.MaterialProxy;

					FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
					DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), GetLocalToWorld(), GetLocalToWorld(), GetBounds(), GetLocalBounds(), true, false, AlwaysHasVelocity());
					BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

					BatchElement.FirstIndex = MeshBatchData.StartIndex;
					BatchElement.NumPrimitives = MeshBatchData.NumPrimitives;
					BatchElement.MinVertexIndex = MeshBatchData.MinVertexIndex;
					BatchElement.MaxVertexIndex = MeshBatchData.MaxVertexIndex;
					Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
					Mesh.Type = PT_TriangleList;
					Mesh.DepthPriorityGroup = SDPG_World;
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bDynamicRelevance = true;
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;
		return Result;
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }

	uint32 GetAllocatedSize() const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	virtual SIZE_T GetTypeHash() const override
	{
		static SIZE_T UniquePointer;
		return reinterpret_cast<SIZE_T>(&UniquePointer);
	}

private:
	TArray<FTriangleSetMeshBatchData> MeshBatchDatas;
	FMaterialRelevance MaterialRelevance;
	FLocalVertexFactory VertexFactory;
	FStaticMeshVertexBuffers VertexBuffers;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};


UTriangleSetComponent::UTriangleSetComponent()
{
	CastShadow = false;
	bSelectable = false;
	PrimaryComponentTick.bCanEverTick = false;
	bBoundsDirty = true;

	UPrimitiveComponent::SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

int32 UTriangleSetComponent::FindOrAddMaterialIndex(UMaterialInterface* Material)
{
	const int32* MaterialIndexPtr = MaterialToIndex.Find(Material);
	if (MaterialIndexPtr == nullptr)
	{
		const int32 MaterialIndex = TrianglesByMaterial.Add(TSparseArray<FRenderableTriangle>());
		MaterialToIndex.Add(Material, MaterialIndex);
		SetMaterial(MaterialIndex, Material);
		return MaterialIndex;
	}

	return *MaterialIndexPtr;
}

void UTriangleSetComponent::Clear()
{
	Triangles.Reset();
	for (int32 Index = 0; Index < TrianglesByMaterial.Num(); Index++)
	{
		SetMaterial(Index, nullptr);
	}
	TrianglesByMaterial.Reset();
	MaterialToIndex.Reset();
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void UTriangleSetComponent::ReserveTriangles(const int32 MaxID)
{
	Triangles.Reserve(MaxID);
}

int32 UTriangleSetComponent::AddTriangle(const FRenderableTriangle& OverlayTriangle)
{
	MarkRenderStateDirty();
	return AddTriangleInternal(OverlayTriangle);
}

int32 UTriangleSetComponent::AddTriangleInternal(const FRenderableTriangle& Triangle)
{
	const int32 MaterialIndex = FindOrAddMaterialIndex(Triangle.Material);
	const int32 IndexByMaterial = TrianglesByMaterial[MaterialIndex].Add(Triangle);
	const int32 ID = Triangles.Add(MakeTuple(MaterialIndex, IndexByMaterial));
	MarkRenderStateDirty();
	bBoundsDirty = true;
	return ID;
}

void UTriangleSetComponent::AddTriangles(
	int32 NumIndices,
	TFunctionRef<void(int32 Index, TArray<FRenderableTriangle>& TrianglesOut)> TriangleGenFunc,
	int32 TrianglesPerIndexHint,
	bool bDeferRenderStateDirty)
{
	TArray<FRenderableTriangle> Temp;
	if (TrianglesPerIndexHint > 0)
	{
		ReserveTriangles(Triangles.Num() + NumIndices*TrianglesPerIndexHint);
		Temp.Reserve(TrianglesPerIndexHint);
	}

	for (int32 k = 0; k < NumIndices; ++k)
	{
		Temp.Reset();
		TriangleGenFunc(k, Temp);
		for (const FRenderableTriangle& Triangle : Temp)
		{
			AddTriangleInternal(Triangle);
		}
	}

	if (!bDeferRenderStateDirty)
	{
		MarkRenderStateDirty();
	}
}

void UTriangleSetComponent::InsertTriangle(const int32 ID, const FRenderableTriangle& OverlayTriangle)
{
	const int32 MaterialIndex = FindOrAddMaterialIndex(OverlayTriangle.Material);
	const int32 IndexByMaterial = TrianglesByMaterial[MaterialIndex].Add(OverlayTriangle);
	Triangles.Insert(ID, MakeTuple(MaterialIndex, IndexByMaterial));
	MarkRenderStateDirty();
	bBoundsDirty = true;
}

void UTriangleSetComponent::RemoveTriangle(const int32 ID)
{
	const TTuple<int32, int32> MaterialAndTriangleIndex = Triangles[ID];
	const int32 MaterialIndex = MaterialAndTriangleIndex.Get<0>();
	const int32 IndexByMaterial = MaterialAndTriangleIndex.Get<1>();
	TSparseArray<FRenderableTriangle>& Container = TrianglesByMaterial[MaterialIndex];
	Container.RemoveAt(IndexByMaterial);
	if (Container.Num() == 0)
	{
		TrianglesByMaterial.RemoveAt(MaterialIndex);

		// Find the MaterialInterface* for the given MaterialIndex. TMaps doesn't support finding values matching a
		// given key so we just visit all the pairs
		UMaterialInterface* MaterialToRemove = nullptr;
		for (TPair<UMaterialInterface*, int32> MaterialAndMaterialIndex : MaterialToIndex)
		{
			if (MaterialAndMaterialIndex.Get<1>() == MaterialIndex)
			{
				MaterialToRemove = MaterialAndMaterialIndex.Get<0>();
				break;
			}
		}
		if (ensure(MaterialToRemove))
		{
			int32 NumRemoved = MaterialToIndex.Remove(MaterialToRemove);
			ensure(NumRemoved == 1);
		}

		SetMaterial(MaterialIndex, nullptr);
	}
	Triangles.RemoveAt(ID);
	MarkRenderStateDirty();
	bBoundsDirty = true;
}



int32 UTriangleSetComponent::AddTriangle(const FVector& A, const FVector& B, const FVector& C, const FVector& Normal, const FColor& Color, UMaterialInterface* Material)
{
	FRenderableTriangle NewTriangle;
	NewTriangle.Material = Material;

	NewTriangle.Vertex0 = { A, FVector2D(0,0), Normal, Color };
	NewTriangle.Vertex1 = { B, FVector2D(1,0), Normal, Color };
	NewTriangle.Vertex2 = { C, FVector2D(1,1), Normal, Color };

	return AddTriangle(NewTriangle);
}

UE::Geometry::FIndex2i UTriangleSetComponent::AddQuad(const FVector& A, const FVector& B, const FVector& C, const FVector& D, const FVector& Normal, const FColor& Color, UMaterialInterface* Material)
{
	FRenderableTriangle NewTriangle0;
	NewTriangle0.Material = Material;

	NewTriangle0.Vertex0 = { A, FVector2D(0,0), Normal, Color };
	NewTriangle0.Vertex1 = { B, FVector2D(1,0), Normal, Color };
	NewTriangle0.Vertex2 = { C, FVector2D(1,1), Normal, Color };

	FRenderableTriangle NewTriangle1 = NewTriangle0;
	NewTriangle1.Vertex1 = NewTriangle1.Vertex2;
	NewTriangle1.Vertex2 = { D, FVector2D(0,1), Normal, Color };

	int32 Index0 = AddTriangle(NewTriangle0);
	int32 Index1 = AddTriangle(NewTriangle1);
	return UE::Geometry::FIndex2i(Index0, Index1);
}


bool UTriangleSetComponent::IsTriangleValid(const int32 ID) const
{
	return Triangles.IsValidIndex(ID);
}

void UTriangleSetComponent::SetAllTrianglesColor(const FColor& NewColor)
{
	for (TSparseArray<FRenderableTriangle>& TriangleArray : TrianglesByMaterial)
	{
		for (FRenderableTriangle& Triangle : TriangleArray)
		{
			Triangle.Vertex0.Color = NewColor;
			Triangle.Vertex1.Color = NewColor;
			Triangle.Vertex2.Color = NewColor;
		}
	}
}

void UTriangleSetComponent::SetAllTrianglesMaterial(UMaterialInterface* Material)
{
	int32 MaterialIndex = FindOrAddMaterialIndex(Material);
	for (TSparseArray<TTuple<int32, int32>>::TIterator It = Triangles.CreateIterator(); It; ++It)
	{
		const int32 OldMaterialIndex = It->Get<0>();
		if (OldMaterialIndex != MaterialIndex)
		{
			const int32 OldIndexByMaterial = It->Get<1>();

			FRenderableTriangle Triangle = TrianglesByMaterial[OldMaterialIndex][OldIndexByMaterial];
			Triangle.Material = Material;

			int32 TrianglesIndex = It.GetIndex();
			RemoveTriangle(TrianglesIndex);
			InsertTriangle(TrianglesIndex, Triangle);
		}
	}

	// TODO RemoveTriangle and InsertTriangle also do this... for every triangle! we should fix that 
	MarkRenderStateDirty();
}

FPrimitiveSceneProxy* UTriangleSetComponent::CreateSceneProxy()
{
	if (Triangles.Num() > 0)
	{
		return new FTriangleSetSceneProxy(this);
	}
	return nullptr;
}

int32 UTriangleSetComponent::GetNumMaterials() const
{
	return TrianglesByMaterial.GetMaxIndex();
}

FBoxSphereBounds UTriangleSetComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (bBoundsDirty)
	{
		FBox Box(ForceInit);

		for (const TSparseArray<FRenderableTriangle>& TriangleArray : TrianglesByMaterial)
		{
			for (const FRenderableTriangle& Triangle : TriangleArray)
			{
				Box += Triangle.Vertex0.Position;
				Box += Triangle.Vertex1.Position;
				Box += Triangle.Vertex2.Position;
			}
		}

		Bounds = FBoxSphereBounds(Box);
		bBoundsDirty = false;
	}

	return Bounds.TransformBy(LocalToWorld);
}
