// Copyright Epic Games, Inc. All Rights Reserved.

#include "Drawing/UVLayoutPreview.h"
#include "PrimitiveDrawingUtils.h"
#include "ToolSetupUtil.h"
#include "Drawing/MeshElementsVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVLayoutPreview)

using namespace UE::Geometry;

UUVLayoutPreview::~UUVLayoutPreview()
{
	checkf(PreviewMesh == nullptr, TEXT("You must explicitly Disconnect() UVLayoutPreview before it is GCd"));
}



void UUVLayoutPreview::CreateInWorld(UWorld* World)
{
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(World, FTransform::Identity);
	PreviewMesh->SetShadowsEnabled(false);

	MeshElementsVisualizer = NewObject<UMeshElementsVisualizer>(this);
	MeshElementsVisualizer->CreateInWorld(World, FTransform::Identity);
	MeshElementsVisualizer->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
		PreviewMesh->ProcessMesh(ProcessFunc);
	});	

	TriangleComponent = NewObject<UTriangleSetComponent>(PreviewMesh->GetActor());
	TriangleComponent->SetupAttachment(PreviewMesh->GetRootComponent());
	TriangleComponent->RegisterComponent();

	Settings = NewObject<UUVLayoutPreviewProperties>(this);
	Settings->WatchProperty(Settings->bEnabled, [this](bool) { bSettingsModified = true; });
	Settings->WatchProperty(Settings->bShowWireframe, [this](bool) { bSettingsModified = true; });
	//Settings->WatchProperty(Settings->bWireframe, [this](bool) { bSettingsModified = true; });
	bSettingsModified = true;

	MeshElementsVisualizer->Settings->bVisible = Settings->bEnabled;
	MeshElementsVisualizer->Settings->bShowWireframe = Settings->bShowWireframe;
	MeshElementsVisualizer->Settings->bShowBorders = Settings->bShowWireframe;
	MeshElementsVisualizer->Settings->bShowUVSeams = false;
	MeshElementsVisualizer->Settings->bShowNormalSeams = false;
	MeshElementsVisualizer->Settings->bShowColorSeams = false;
	MeshElementsVisualizer->Settings->ThicknessScale = 0.5;
	MeshElementsVisualizer->Settings->DepthBias = 0.5;
	MeshElementsVisualizer->Settings->bAdjustDepthBiasUsingMeshSize = false;
	MeshElementsVisualizer->Settings->CheckAndUpdateWatched();

	BackingRectangleMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor::White, nullptr);
	if (BackingRectangleMaterial == nullptr)
	{
		BackingRectangleMaterial = ToolSetupUtil::GetDefaultMaterial();
	}
}


void UUVLayoutPreview::Disconnect()
{
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	MeshElementsVisualizer->Disconnect();
	MeshElementsVisualizer = nullptr;
}


void UUVLayoutPreview::SetSourceMaterials(const FComponentMaterialSet& MaterialSet)
{
	SourceMaterials = MaterialSet;

	PreviewMesh->SetMaterials(SourceMaterials.Materials);
}


void UUVLayoutPreview::SetSourceWorldPosition(FTransform WorldTransform, FBox WorldBounds)
{
	SourceObjectWorldBounds = FAxisAlignedBox3d(WorldBounds);

	SourceObjectFrame = FFrame3d(WorldTransform);
}


void UUVLayoutPreview::SetCurrentCameraState(const FViewCameraState& CameraStateIn)
{
	CameraState = CameraStateIn;
}


void UUVLayoutPreview::SetTransform(const FTransform& UseTransform)
{
	PreviewMesh->SetTransform(UseTransform);
	MeshElementsVisualizer->SetTransform(UseTransform);
}


void UUVLayoutPreview::SetVisible(bool bVisible)
{
	PreviewMesh->SetVisible(bVisible);
	MeshElementsVisualizer->Settings->bVisible = bVisible;
}


void UUVLayoutPreview::Render(IToolsContextRenderAPI* RenderAPI)
{
	SetCurrentCameraState(RenderAPI->GetCameraState());

	RecalculatePosition();

	if (Settings->bEnabled)
	{
		float ScaleFactor = GetCurrentScale();
		FVector Origin = (FVector)CurrentWorldFrame.Origin;
		FVector DX = ScaleFactor * (FVector)CurrentWorldFrame.X();
		FVector DY = ScaleFactor * (FVector)CurrentWorldFrame.Y();

		RenderAPI->GetPrimitiveDrawInterface()->DrawLine(
			Origin, Origin+DX, FLinearColor::Black, SDPG_Foreground, 0.5f, 0.0f, true);
		RenderAPI->GetPrimitiveDrawInterface()->DrawLine(
			Origin+DX, Origin+DX+DY, FLinearColor::Black, SDPG_Foreground, 0.5f, 0.0f, true);
		RenderAPI->GetPrimitiveDrawInterface()->DrawLine(
			Origin+DX+DY, Origin+DY, FLinearColor::Black, SDPG_Foreground, 0.5f, 0.0f, true);
		RenderAPI->GetPrimitiveDrawInterface()->DrawLine(
			Origin+DY, Origin, FLinearColor::Black, SDPG_Foreground, 0.5f, 0.0f, true);
	}
}



void UUVLayoutPreview::OnTick(float DeltaTime)
{
	if (bSettingsModified)
	{
		SetVisible(Settings->bEnabled);

		MeshElementsVisualizer->Settings->bShowWireframe = Settings->bShowWireframe;
		MeshElementsVisualizer->Settings->bShowBorders = Settings->bShowWireframe;
		MeshElementsVisualizer->Settings->CheckAndUpdateWatched();

		bSettingsModified = false;
	}

	MeshElementsVisualizer->OnTick(DeltaTime);
}


float UUVLayoutPreview::GetCurrentScale()
{
	return Settings->Scale * SourceObjectWorldBounds.Height();
}


void UUVLayoutPreview::UpdateUVMesh(const FDynamicMesh3* SourceMesh, int32 SourceUVLayer)
{
	FDynamicMesh3 UVMesh;
	UVMesh.EnableAttributes();
	FDynamicMeshUVOverlay* NewUVOverlay = UVMesh.Attributes()->GetUVLayer(0);
	FDynamicMeshNormalOverlay* NewNormalOverlay = UVMesh.Attributes()->PrimaryNormals();

	FAxisAlignedBox2f Bounds = FAxisAlignedBox2f(FVector2f::Zero(), FVector2f::One());
	if (SourceMesh && SourceMesh->Attributes())
	{
		if (const FDynamicMeshUVOverlay* UVOverlay = SourceMesh->Attributes()->GetUVLayer(SourceUVLayer))
		{
			for (int32 tid : SourceMesh->TriangleIndicesItr())
			{
				if (UVOverlay->IsSetTriangle(tid))
				{
					FIndex3i UVTri = UVOverlay->GetTriangle(tid);

					FVector2f UVs[3];
					for (int32 j = 0; j < 3; ++j)
					{
						UVs[j] = UVOverlay->GetElement(UVTri[j]);
						Bounds.Contain(UVs[j]);
					}

					FIndex3i NewTri, NewUVTri, NewNormalTri;
					for (int32 j = 0; j < 3; ++j)
					{
						NewUVTri[j] = NewUVOverlay->AppendElement(UVs[j]);
						NewTri[j] = UVMesh.AppendVertex(FVector3d(UVs[j].X, UVs[j].Y, 0));
						NewNormalTri[j] = NewNormalOverlay->AppendElement( FVector3f::UnitZ() );
					}

					FVector2f EdgeUV1 = UVs[1] - UVs[0];
					FVector2f EdgeUV2 = UVs[2] - UVs[0];
					float SignedUVArea = 0.5f * (EdgeUV1.X * EdgeUV2.Y - EdgeUV1.Y * EdgeUV2.X);

					if (SignedUVArea > 0)
					{
						Swap(NewTri.A, NewTri.B);
						Swap(NewUVTri.A, NewUVTri.B);
					}


					int32 NewTriID = UVMesh.AppendTriangle(NewTri);
					NewUVOverlay->SetTriangle(NewTriID, NewUVTri);
					NewNormalOverlay->SetTriangle(NewTriID, NewNormalTri);
				}
			}
		}
	}

	Bounds.Expand(0.01f);

	float BackZ = -0.01f;
	TriangleComponent->Clear();
	if (bShowBackingRectangle)
	{
		TriangleComponent->AddQuad(
			FVector(Bounds.Min.X, Bounds.Min.Y, BackZ), 
			FVector(Bounds.Min.X, Bounds.Max.Y, BackZ), 
			FVector(Bounds.Max.X, Bounds.Max.Y, BackZ), 
			FVector(Bounds.Max.X, Bounds.Min.Y, BackZ),
			FVector(0, 0, -1), FColor::White, BackingRectangleMaterial);
	}

	PreviewMesh->UpdatePreview(MoveTemp(UVMesh));
	MeshElementsVisualizer->NotifyMeshChanged();
}




void UUVLayoutPreview::RecalculatePosition()
{
	FFrame3d ObjFrame;
	ObjFrame.AlignAxis(2, -(FVector3d)CameraState.Forward());
	ObjFrame.ConstrainedAlignAxis(0, (FVector3d)CameraState.Right(), ObjFrame.Z());
	ObjFrame.Origin = SourceObjectWorldBounds.Center(); // SourceObjectFrame.Origin;

	FAxisAlignedBox2d ProjectedBounds = FAxisAlignedBox2d::Empty();
	for (int32 k = 0; k < 8; ++k)
	{
		ProjectedBounds.Contain(ObjFrame.ToPlaneUV(SourceObjectWorldBounds.GetCorner(k)));
	}

	double UseScale = GetCurrentScale();

	double ShiftRight = Settings->Offset.X * (ProjectedBounds.Max.X + (ProjectedBounds.Width() * 0.1));
	if (Settings->Side == EUVLayoutPreviewSide::Left)
	{
		ShiftRight = ProjectedBounds.Min.X - Settings->Offset.X * (UseScale + (ProjectedBounds.Width() * 0.1));
	}

	double ShiftUp = Settings->Offset.Y * UseScale;
	ObjFrame.Origin += ShiftRight * ObjFrame.X() - ShiftUp * ObjFrame.Y();

	CurrentWorldFrame = ObjFrame;

	FTransformSRT3d Transform(ObjFrame.Rotation, ObjFrame.Origin);
	Transform.SetScale(UseScale * FVector3d::One());

	SetTransform((FTransform)Transform);
}
