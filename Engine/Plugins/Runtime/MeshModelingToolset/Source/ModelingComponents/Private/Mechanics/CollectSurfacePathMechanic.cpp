// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/CollectSurfacePathMechanic.h"
#include "ToolSceneQueriesUtil.h"
#include "MeshQueries.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Distance/DistLine3Ray3.h"
#include "Util/ColorConstants.h"
#include "DynamicMesh/MeshNormals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollectSurfacePathMechanic)

using namespace UE::Geometry;

UCollectSurfacePathMechanic::UCollectSurfacePathMechanic()
{
	SpatialSnapPointsFunc = [this](FVector3d A, FVector3d B) { return DistanceSquared(A, B) < (ConstantSnapDistance * ConstantSnapDistance); };

	PathColor = LinearColors::DarkOrange3f();
	PreviewColor = LinearColors::Green3f();
	PathCompleteColor = LinearColors::Yellow3f();

	PathDrawer.LineColor = PathColor;
	PathDrawer.LineThickness = 4.0f;
	PathDrawer.PointSize = 8.0f;
	PathDrawer.bDepthTested = false;
}


void UCollectSurfacePathMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);
}

void UCollectSurfacePathMechanic::Shutdown()
{
	UInteractionMechanic::Shutdown();


}

void UCollectSurfacePathMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (bDrawPath == true)
	{
		PathDrawer.BeginFrame(RenderAPI);

		if (HitPath.Num() > 0)
		{
			const FLinearColor& DrawPathColor = (bCurrentPreviewWillComplete || bGeometricCloseOccurred) ? PathCompleteColor : PathColor;
			int32 NumPoints = HitPath.Num() - 1;
			for (int32 k = 0; k < NumPoints; ++k)
			{
				PathDrawer.DrawLine(HitPath[k].Origin, HitPath[k + 1].Origin, DrawPathColor);
			}

			if (!bGeometricCloseOccurred)
			{
				if (bPreviewPathPointValid)
				{
					// This draws the line to the current hover point as a preview
					const FLinearColor& DrawPreviewColor = (bCurrentPreviewWillComplete || bGeometricCloseOccurred) ? PathCompleteColor : PreviewColor;
					PathDrawer.DrawLine(HitPath[NumPoints].Origin, PreviewPathPoint.Origin, DrawPreviewColor);
				}
			}
			else if (LoopWasClosed())
			{
				// Draw a line to the first point
				PathDrawer.DrawLine(HitPath[0].Origin, HitPath[HitPath.Num() - 1].Origin, DrawPathColor);
			}
		}

		if (!bGeometricCloseOccurred && bPreviewPathPointValid)
		{
			PathDrawer.DrawPoint(PreviewPathPoint.Origin, PreviewColor, PathDrawer.PointSize, PathDrawer.bDepthTested);
		}

		PathDrawer.EndFrame();
	}
}


void UCollectSurfacePathMechanic::InitializeMeshSurface(FDynamicMesh3&& TargetSurfaceMesh)
{
	TargetSurface = MoveTemp(TargetSurfaceMesh);
	TargetSurfaceAABB.SetMesh(&TargetSurface);
}

void UCollectSurfacePathMechanic::InitializePlaneSurface(const FFrame3d& TargetPlaneIn)
{
	TargetPlane = TargetPlaneIn;
	bHaveTargetPlane = true;
}


void UCollectSurfacePathMechanic::SetFixedNumPointsMode(int32 NumPoints)
{
	check(NumPoints > 0 && NumPoints < 100);
	DoneMode = ECollectSurfacePathDoneMode::FixedNumPoints;
	FixedPointTargetCount = NumPoints;
}

void UCollectSurfacePathMechanic::SetCloseWithLambdaMode()
{
	check(IsDoneFunc);
	DoneMode = ECollectSurfacePathDoneMode::ExternalLambda;
}

void UCollectSurfacePathMechanic::SetDrawClosedLoopMode()
{
	DoneMode = ECollectSurfacePathDoneMode::SnapCloseLoop;
}

void UCollectSurfacePathMechanic::SetDoubleClickOrCloseLoopMode()
{
	DoneMode = ECollectSurfacePathDoneMode::SnapDoubleClickOrCloseLoop;
}


bool UCollectSurfacePathMechanic::IsHitByRay(const FRay3d& Ray, FFrame3d& HitPoint)
{
	return RayToPathPoint(Ray, HitPoint, false);
}


bool UCollectSurfacePathMechanic::UpdatePreviewPoint(const FRay3d& Ray)
{
	FFrame3d PreviewPoint;
	bPreviewPathPointValid = RayToPathPoint(Ray, PreviewPoint, true);
	if (!bPreviewPathPointValid)
	{
		return false;
	}

	PreviewPathPoint = PreviewPoint;

	bCurrentPreviewWillComplete = CheckGeometricClosure(PreviewPathPoint);

	return true;
}


bool UCollectSurfacePathMechanic::TryAddPointFromRay(const FRay3d& Ray)
{
	FFrame3d NewPoint;
	if (RayToPathPoint(Ray, NewPoint, true) == false)
	{
		return false;
	}

	if (CheckGeometricClosure(NewPoint, &bLoopWasClosed)) // update bLoopWasClosed as we do this
	{
		bGeometricCloseOccurred = true;
	}
	else
	{
		HitPath.Add(NewPoint);
	}

	bCurrentPreviewWillComplete = false;
	return true;
}


bool UCollectSurfacePathMechanic::PopLastPoint()
{
	if (bGeometricCloseOccurred)
	{
		// Undoing the closure is effectively "popping" the results of the last TryAddPointFromRay call.
		bGeometricCloseOccurred = false;
		bLoopWasClosed = false;
		return true;
	} 
	else if (HitPath.Num() > 0)
	{
		HitPath.RemoveAt(HitPath.Num() - 1);
		return true;
	}
	return false;
}


bool UCollectSurfacePathMechanic::RayToPathPoint(const FRay3d& Ray, FFrame3d& PointOut, bool bEnableSnapping)
{
	bool bHaveHit = false;
	FFrame3d NearestHitFrame;
	double NearestHitDistSqr = TNumericLimits<double>::Max();

	if (TargetSurface.TriangleCount() > 0)
	{
		int32 HitTri = TargetSurfaceAABB.FindNearestHitTriangle(Ray);
		if (HitTri != FDynamicMesh3::InvalidID)
		{
			FIntrRay3Triangle3d Hit = TMeshQueries<FDynamicMesh3>::RayTriangleIntersection(TargetSurface, HitTri, Ray);
			NearestHitFrame = TargetSurface.GetTriFrame(HitTri);
			NearestHitFrame.Origin = Hit.Triangle.BarycentricPoint(Hit.TriangleBaryCoords);
			NearestHitDistSqr = Ray.GetParameter(NearestHitFrame.Origin);
			bHaveHit = true;
		}
	}

	if (bHaveTargetPlane)
	{
		FFrame3d PlaneHit(TargetPlane);
		if (TargetPlane.RayPlaneIntersection(Ray.Origin, Ray.Direction, 2, PlaneHit.Origin))
		{
			double HitDistSqr = Ray.GetParameter(PlaneHit.Origin);
			if (HitDistSqr < NearestHitDistSqr)
			{
				NearestHitFrame = PlaneHit;
				bHaveHit = true;
			}
		}
	}

	if (bHaveHit == false)
	{
		return false;
	}

	PointOut = NearestHitFrame;

	// try snapping to close/end
	bool bHaveSnapped = false;
	bool bWouldCloseLoop = false;
	if (CheckGeometricClosure(PointOut, &bWouldCloseLoop))
	{
		bHaveSnapped = true;
		PointOut = bWouldCloseLoop ? HitPath[0] : HitPath.Last();
	}

	// try snapping to other things, if we haven't yet
	if (bEnableSnapping && bHaveSnapped == false)
	{
		if (bSnapToTargetMeshVertices && TargetSurface.TriangleCount() > 0)
		{
			double NearDistSqr;
			int32 NearestVID = TargetSurfaceAABB.FindNearestVertex(PointOut.Origin, NearDistSqr);
			if (NearestVID != FDynamicMesh3::InvalidID)
			{
				FVector3d NearestVertexPos = TargetSurface.GetVertex(NearestVID);
				if (SpatialSnapPointsFunc(PointOut.Origin, NearestVertexPos))
				{
					PointOut.Origin = NearestVertexPos;
					PointOut.AlignAxis(2, FMeshNormals::ComputeVertexNormal(TargetSurface, NearestVID));
				}
			}
		}

		if (bSnapToWorldGrid && ParentTool.IsValid())
		{
			FVector3d WorldGridSnapPos;
			if (ToolSceneQueriesUtil::FindWorldGridSnapPoint(ParentTool.Get(), PointOut.Origin, WorldGridSnapPos))
			{
				PointOut.Origin = WorldGridSnapPos;
			}
		}

	}

	return true;
}


bool UCollectSurfacePathMechanic::IsDone() const
{
	if (DoneMode == ECollectSurfacePathDoneMode::FixedNumPoints)
	{
		return HitPath.Num() >= FixedPointTargetCount;
	}
	else if (DoneMode == ECollectSurfacePathDoneMode::ExternalLambda)
	{
		return IsDoneFunc();
	}
	else if (DoneMode == ECollectSurfacePathDoneMode::SnapCloseLoop || DoneMode == ECollectSurfacePathDoneMode::SnapDoubleClick || DoneMode == ECollectSurfacePathDoneMode::SnapDoubleClickOrCloseLoop)
	{
		return bGeometricCloseOccurred;
	}
	ensure(false);
	return false;
}


bool UCollectSurfacePathMechanic::CheckGeometricClosure(const FFrame3d& Point, bool* bLoopWasClosedOut)
{
	if (bLoopWasClosedOut != nullptr)
	{
		// There's multiple places we might return, and in most of them, the loop is not closed
		*bLoopWasClosedOut = false; 
	}

	if (HitPath.Num() == 0)
	{
		return false;
	}

	if (DoneMode == ECollectSurfacePathDoneMode::SnapCloseLoop || DoneMode == ECollectSurfacePathDoneMode::SnapDoubleClickOrCloseLoop)
	{
		if (HitPath.Num() > 2)
		{
			// See if we clicked on the first point
			const FFrame3d& FirstPoint = HitPath[0];
			if (SpatialSnapPointsFunc(Point.Origin, FirstPoint.Origin))
			{
				if (bLoopWasClosedOut != nullptr)
				{
					*bLoopWasClosedOut = true;
				}
				return true;
			}
		}
	}

	if (DoneMode == ECollectSurfacePathDoneMode::SnapDoubleClick || DoneMode == ECollectSurfacePathDoneMode::SnapDoubleClickOrCloseLoop)
	{
		if (HitPath.Num() > 1)
		{
			const FFrame3d& LastPoint = HitPath[HitPath.Num() - 1];
			if (SpatialSnapPointsFunc(Point.Origin, LastPoint.Origin))
			{
				return true;
			}
		}
	}

	return false;
}

