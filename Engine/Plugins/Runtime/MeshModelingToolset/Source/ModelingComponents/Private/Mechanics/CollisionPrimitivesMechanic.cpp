// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/CollisionPrimitivesMechanic.h"

#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/IntervalGizmo.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/LineSetComponent.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "Transforms/MultiTransformer.h"
#include "Physics/PhysicsDataCollection.h"
#include "Util/ColorConstants.h"
#include "Generators/LineSegmentGenerators.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CollisionPrimitivesMechanic)

#define LOCTEXT_NAMESPACE "UCollisionPrimitivesMechanic"

namespace CollisionPrimitivesMechanicLocals
{
	static const FText CollisionPrimitiveDeselectionTransactionText = LOCTEXT("CollisionPrimitiveDeselection", "Collision Primitive Deselection");
	static const FText CollisionPrimitiveSelectionTransactionText = LOCTEXT("CollisionPrimitiveSelection", "Collision Primitive Selection");
	static const FText CollisionPrimitiveMovementTransactionText = LOCTEXT("CollisionPrimitiveGeometryChange", "Collision Primitive Geometry Change");

	void Toggle(TSet<int>& Selection, const TSet<int>& NewSelection)
	{
		for (const int& NewElement : NewSelection)
		{
			if (Selection.Remove(NewElement) == 0)
			{
				Selection.Add(NewElement);
			}
		}
	}

	// TODO Move to SetUtil namespace and call it in FGroupTopologySelection::operator==
	bool IsEqual(const TSet<int32>& A, const TSet<int32>& B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}

		// No need to also check B.Includes(A) here since counts match and sets contain unique elements
		return A.Includes(B);
	}
}


void UCollisionPrimitivesMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	MarqueeMechanic = NewObject<URectangleMarqueeMechanic>(this);
	MarqueeMechanic->bUseExternalClickDragBehavior = true;
	MarqueeMechanic->Setup(ParentToolIn);
	MarqueeMechanic->OnDragRectangleStarted.AddUObject(this, &UCollisionPrimitivesMechanic::OnDragRectangleStarted);
	MarqueeMechanic->OnDragRectangleChanged.AddUObject(this, &UCollisionPrimitivesMechanic::OnDragRectangleChanged);
	MarqueeMechanic->OnDragRectangleFinished.AddUObject(this, &UCollisionPrimitivesMechanic::OnDragRectangleFinished);

	USingleClickOrDragInputBehavior* ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, MarqueeMechanic);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(ClickOrDragBehavior);

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	NormalSegmentColor = FColor::Red;
	HoverColor = FColor::Green;
	SelectedColor = FColor::Yellow;
	SegmentsThickness = 1.0f;

	float UseSegmentRadius = 5.0f;
	GeometrySetToleranceTest = [UseSegmentRadius](const FVector3d& A, const FVector3d& B) { return UE::Geometry::DistanceSquared(A, B) < UseSegmentRadius * UseSegmentRadius; };

	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	TranslateTransformProxy = NewObject<UTransformProxy>(this);
	SphereTransformProxy = NewObject<UTransformProxy>(this);
	BoxTransformProxy = NewObject<UTransformProxy>(this);
	CapsuleTransformProxy = NewObject<UTransformProxy>(this);
	FullTransformProxy = NewObject<UTransformProxy>(this);
	
	// TODO: Maybe don't have the gizmo's axes flip around when it crosses the origin, if possible?
	TranslateTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes, this);

	SphereTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes | ETransformGizmoSubElements::ScaleUniform, this);

	BoxTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes | ETransformGizmoSubElements::RotateAllAxes, this);

	// create sources for the interval parameters
	BoxXIntervalSource = NewObject< UGizmoLocalFloatParameterSource >(this);
	BoxYIntervalSource = NewObject< UGizmoLocalFloatParameterSource >(this);
	BoxZIntervalSource = NewObject< UGizmoLocalFloatParameterSource >(this);

	CapsuleRadiusIntervalSource = NewObject< UGizmoLocalFloatParameterSource >(this);
	CapsuleLengthIntervalSource = NewObject< UGizmoLocalFloatParameterSource >(this);

	// add the interval gizmo
	GizmoManager->RegisterGizmoType(UIntervalGizmo::GizmoName, NewObject<UIntervalGizmoBuilder>());
	BoxIntervalGizmo = GizmoManager->CreateGizmo<UIntervalGizmo>(UIntervalGizmo::GizmoName, TEXT("BoxSizeInterval"), this);

	UIntervalGizmo::FParameterSources BoxIntervalSources;
	BoxIntervalSources.RightInterval = BoxXIntervalSource;
	BoxIntervalSources.ForwardInterval = BoxYIntervalSource;
	BoxIntervalSources.UpInterval = BoxZIntervalSource;
	BoxIntervalGizmo->SetActiveTarget(BoxTransformProxy, BoxIntervalSources);

	CapsuleTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::TranslateAllPlanes | ETransformGizmoSubElements::RotateAllAxes, this);

	CapsuleIntervalGizmo = GizmoManager->CreateGizmo<UIntervalGizmo>(UIntervalGizmo::GizmoName, TEXT("CapsuleSizeInterval"), this);
	UIntervalGizmo::FParameterSources CapsuleIntervalSources;
	CapsuleIntervalSources.RightInterval = CapsuleRadiusIntervalSource;
	CapsuleIntervalSources.UpInterval = CapsuleLengthIntervalSource;
	CapsuleIntervalGizmo->SetActiveTarget(CapsuleTransformProxy, CapsuleIntervalSources);

	FullTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::FullTranslateRotateScale, this);

	const bool bShouldUseContextCoordinateSystem = true;

	TranslateTransformProxy->OnTransformChanged.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformChanged);
	TranslateTransformProxy->OnBeginTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformStarted);
	TranslateTransformProxy->OnEndTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformEnded);

	TranslateTransformGizmo->SetActiveTarget(TranslateTransformProxy);
	TranslateTransformGizmo->SetVisibility(false);
	TranslateTransformGizmo->bUseContextCoordinateSystem = bShouldUseContextCoordinateSystem;
	TranslateTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;

	SphereTransformProxy->OnTransformChanged.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformChanged);
	SphereTransformProxy->OnBeginTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformStarted);
	SphereTransformProxy->OnEndTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformEnded);

	SphereTransformGizmo->SetActiveTarget(SphereTransformProxy);
	SphereTransformGizmo->SetVisibility(false);
	SphereTransformGizmo->bUseContextCoordinateSystem = bShouldUseContextCoordinateSystem;
	SphereTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;

	BoxTransformProxy->OnTransformChanged.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformChanged);
	BoxTransformProxy->OnBeginTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformStarted);
	BoxTransformProxy->OnEndTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformEnded);
	BoxIntervalGizmo->OnIntervalChanged.AddUObject(this, &UCollisionPrimitivesMechanic::IntervalGizmoValueChanged);

	BoxTransformGizmo->SetActiveTarget(BoxTransformProxy);
	BoxTransformGizmo->SetVisibility(false);
	BoxTransformGizmo->bUseContextCoordinateSystem = bShouldUseContextCoordinateSystem;
	BoxTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
	BoxIntervalGizmo->SetVisibility(false);

	CapsuleTransformProxy->OnTransformChanged.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformChanged);
	CapsuleTransformProxy->OnBeginTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformStarted);
	CapsuleTransformProxy->OnEndTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformEnded);
	CapsuleIntervalGizmo->OnIntervalChanged.AddUObject(this, &UCollisionPrimitivesMechanic::IntervalGizmoValueChanged);

	CapsuleTransformGizmo->SetActiveTarget(CapsuleTransformProxy);
	CapsuleTransformGizmo->SetVisibility(false);
	CapsuleTransformGizmo->bUseContextCoordinateSystem = bShouldUseContextCoordinateSystem;
	CapsuleTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
	CapsuleIntervalGizmo->SetVisibility(false);

	FullTransformProxy->OnTransformChanged.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformChanged);
	FullTransformProxy->OnBeginTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformStarted);
	FullTransformProxy->OnEndTransformEdit.AddUObject(this, &UCollisionPrimitivesMechanic::GizmoTransformEnded);

	FullTransformGizmo->SetActiveTarget(FullTransformProxy);
	FullTransformGizmo->SetVisibility(false);
	FullTransformGizmo->bUseContextCoordinateSystem = bShouldUseContextCoordinateSystem;
	FullTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;

	// Hold the active proxy, so we can capture undo information from whatever gizmo got moved by the user.
	CurrentActiveProxy = TranslateTransformProxy;
}

void UCollisionPrimitivesMechanic::Initialize(TSharedPtr<FPhysicsDataCollection> InPhysicsData,
											   const UE::Geometry::FAxisAlignedBox3d& MeshBoundsIn,
											   const FTransform3d& InLocalToWorldTransform)
{
	LocalToWorldTransform = InLocalToWorldTransform;
	PhysicsData = InPhysicsData;
	MeshBounds = MeshBoundsIn;
	SelectedPrimitiveIDs.Empty();
	UpdateGizmoLocation();
	RebuildDrawables();
	++CurrentChangeStamp;
}

void UCollisionPrimitivesMechanic::SetWorld(UWorld* World)
{
	// It may be unreasonable to worry about SetWorld being called more than once, but let's be safe anyway
	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
		PreviewGeometry = nullptr;
	} 

	// We need the world so we can create the geometry actor in the right place
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(World, FTransform());

	DrawnPrimitiveEdges = PreviewGeometry->AddLineSet("CollisionPrimitives");
	DrawnPrimitiveEdges->SetLineMaterial(ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), false));

}

void UCollisionPrimitivesMechanic::Shutdown()
{
	LongTransactions.CloseAll(GetParentTool()->GetToolManager());

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
		PreviewGeometry = nullptr;
	}

	// Calls shutdown on gizmo and destroys it.
	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	GizmoManager->DestroyAllGizmosByOwner(this);
	GizmoManager->DeregisterGizmoType(UIntervalGizmo::GizmoName);
}


void UCollisionPrimitivesMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	// Cache the camera state
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CachedCameraState);

	MarqueeMechanic->Render(RenderAPI);
}

void UCollisionPrimitivesMechanic::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	MarqueeMechanic->DrawHUD(Canvas, RenderAPI);
}

void UCollisionPrimitivesMechanic::RebuildDrawables(bool bRegenerateCurveLists)
{
	// TODO: This code replicates a lot of the functionality found in CollisionGeometryVisualization.cpp, but with additional behavior needed for this class. Unification in the future would be good to explore.

	int32 CircleSteps = FMath::Max(24, 24);

	typedef TFunction<void(FPrimitiveRenderData& RenderData, TArray<FRenderableLine>& LinesOut, int32 PrimIndex, int32 CurveIndex, const FTransform& Transform, const FVector& A, const FVector& B, const FColor& Color)> WriteCurveFunc;
	WriteCurveFunc StoreCurve;
	if (bRegenerateCurveLists)
	{
		GeometrySet.Reset();
		DrawnPrimitiveEdges->Clear();
		PrimitiveRenderData.Empty();
		PrimitiveToCurveLookup.Empty();
		CurveToPrimitiveLookup.Empty();
		StoreCurve = [this](FPrimitiveRenderData& RenderData, TArray<FRenderableLine>& LinesOut, int32 PrimIndex, int32 CurveIndex, const FTransform& Transform, const FVector& A, const FVector& B, const FColor& Color) -> void
		{
			FVector A_World, B_World;
			A_World = Transform.TransformPosition(A);
			B_World = Transform.TransformPosition(B);
			LinesOut.Add(FRenderableLine(A_World, B_World, Color, SegmentsThickness, DepthBias));
			GeometrySet.AddCurve(CurveIndex, UE::Geometry::FPolyline3d(A_World, B_World));

			PrimitiveToCurveLookup.FindOrAdd(PrimIndex).Add(CurveIndex);
			CurveToPrimitiveLookup.FindOrAdd(CurveIndex) = PrimIndex;
		};
	}
	else
	{
		StoreCurve = [this](FPrimitiveRenderData& RenderData, TArray<FRenderableLine>& LinesOut, int32 PrimIndex, int32 CurveIndex, const FTransform& Transform, const FVector& A, const FVector& B, const FColor& Color) -> void
		{
			FVector A_World, B_World;
			A_World = Transform.TransformPosition(A);
			B_World = Transform.TransformPosition(B);

			DrawnPrimitiveEdges->SetLineStart(CurveIndex, A_World);
			DrawnPrimitiveEdges->SetLineEnd(CurveIndex, B_World);
			GeometrySet.UpdateCurve(CurveIndex, UE::Geometry::FPolyline3d(A_World, B_World));

			ensure(PrimitiveToCurveLookup[PrimIndex].Contains(CurveIndex));
			ensure(CurveToPrimitiveLookup[CurveIndex] == PrimIndex);
		};
	}


	const FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	int32 PrimIndex = 0;
	int32 GobalCurveIndex = 0;
	// spheres are draw as 3 orthogonal circles
	for (int32 Index = 0; Index < AggGeom.SphereElems.Num(); Index++)
	{
		FPrimitiveRenderData RenderData;
		RenderData.PrimIndex = PrimIndex;
		RenderData.LineIndexStart = GobalCurveIndex;
		FColor Color = SelectedPrimitiveIDs.Contains(PrimIndex) ? SelectedColor : NormalSegmentColor;
		RenderData.RenderColor = Color;
		RenderData.ShapeType = EAggCollisionShape::Sphere;

		int32 CurveIndex = 0;
		DrawnPrimitiveEdges->AddLines(1, [this, bRegenerateCurveLists, &StoreCurve, &RenderData, &AggGeom, &Color, CircleSteps, PrimIndex, &GobalCurveIndex, Index](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
			{	
				const FKSphereElem& Sphere = AggGeom.SphereElems[Index];
				FTransform ElemTransform = Sphere.GetTransform();
				ElemTransform.ScaleTranslation(PhysicsData->ExternalScale3D);
				if (bRegenerateCurveLists)
				{					
					RenderData.Transform = ElemTransform;
				}
				float Radius = PhysicsData->ExternalScale3D.GetAbsMin() * Sphere.Radius;
				UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, FVector3f::Zero(), FVector3f::UnitX(), FVector3f::UnitY(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) {
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform * LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
				UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, FVector3f::Zero(), FVector3f::UnitX(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) {
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform * LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
				UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, FVector3f::Zero(), FVector3f::UnitY(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform * LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
			});
		RenderData.LineIndexEnd = GobalCurveIndex;
		if (bRegenerateCurveLists)
		{
			PrimitiveRenderData.Add(RenderData);
		}
		PrimIndex++;
	}


	// boxes are drawn as boxes
	for (int32 Index = 0; Index < AggGeom.BoxElems.Num(); Index++)
	{
		FPrimitiveRenderData RenderData;
		RenderData.PrimIndex = PrimIndex;
		RenderData.LineIndexStart = GobalCurveIndex;
		FColor Color = SelectedPrimitiveIDs.Contains(PrimIndex) ? SelectedColor : NormalSegmentColor;
		RenderData.RenderColor = Color;
		RenderData.ShapeType = EAggCollisionShape::Box;

		int32 CurveIndex = 0;
		DrawnPrimitiveEdges->AddLines(1, [this, bRegenerateCurveLists, &StoreCurve, &RenderData, &AggGeom, &Color, CircleSteps, PrimIndex, &GobalCurveIndex, Index](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
			{
				const FKBoxElem& Box = AggGeom.BoxElems[Index];
				FTransform ElemTransform = Box.GetTransform();
				ElemTransform.ScaleTranslation(PhysicsData->ExternalScale3D);
				if (bRegenerateCurveLists)
				{
					RenderData.Transform = ElemTransform;
				}								
				FVector3f HalfDimensions(
					PhysicsData->ExternalScale3D.X * Box.X * 0.5f,
					PhysicsData->ExternalScale3D.Y * Box.Y * 0.5f,
					PhysicsData->ExternalScale3D.Z * Box.Z * 0.5f);
				UE::Geometry::GenerateBoxSegments<float>(HalfDimensions, FVector3f::Zero(), FVector3f::UnitX(), FVector3f::UnitY(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) {
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
			});
		RenderData.LineIndexEnd = GobalCurveIndex;
		if (bRegenerateCurveLists)
		{
			PrimitiveRenderData.Add(RenderData);
		}
		PrimIndex++;
	}


	// capsules are draw as two hemispheres (with 3 intersecting arcs/circles) and connecting lines
	for (int32 Index = 0; Index < AggGeom.SphylElems.Num(); Index++)
	{
		FPrimitiveRenderData RenderData;
		RenderData.PrimIndex = PrimIndex;
		RenderData.LineIndexStart = GobalCurveIndex;
		FColor Color = SelectedPrimitiveIDs.Contains(PrimIndex) ? SelectedColor : NormalSegmentColor;
		RenderData.RenderColor = Color;
		RenderData.ShapeType = EAggCollisionShape::Sphyl;

		int32 CurveIndex = 0;
		DrawnPrimitiveEdges->AddLines(1, [this, bRegenerateCurveLists, &StoreCurve, &RenderData, &AggGeom, &Color, CircleSteps, PrimIndex, &GobalCurveIndex, Index](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
			{
				const FKSphylElem& Capsule = AggGeom.SphylElems[Index];
				FTransform ElemTransform = Capsule.GetTransform();
				ElemTransform.ScaleTranslation(PhysicsData->ExternalScale3D);
				if (bRegenerateCurveLists)
				{
					RenderData.Transform = ElemTransform;
				}					const float HalfLength = Capsule.GetScaledCylinderLength(PhysicsData->ExternalScale3D) * .5f;
				const float Radius = Capsule.GetScaledRadius(PhysicsData->ExternalScale3D);
				FVector3f Top(0, 0, HalfLength), Bottom(0, 0, -HalfLength);

				// top and bottom circles
				UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, Top, FVector3f::UnitX(), FVector3f::UnitY(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
				UE::Geometry::GenerateCircleSegments<float>(CircleSteps, Radius, Bottom, FVector3f::UnitX(), FVector3f::UnitY(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});

				// top dome
				UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, PI, Top, FVector3f::UnitY(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
				UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, PI, Top, FVector3f::UnitX(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});

				// bottom dome
				UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, -PI, Bottom, FVector3f::UnitY(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});
				UE::Geometry::GenerateArcSegments<float>(CircleSteps, Radius, 0.0, -PI, Bottom, FVector3f::UnitX(), FVector3f::UnitZ(), UE::Geometry::FTransformSRT3f(),
					[&](const FVector3f& A, const FVector3f& B) { 
						StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					});

				// connecting lines
				for (int k = 0; k < 2; ++k)
				{
					FVector A, B;

					FVector DX = (k < 1) ? FVector(-Radius, 0, 0) : FVector(Radius, 0, 0);
					A = (FVector)Top + DX;
					B = (FVector)Bottom + DX;
					StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);

					FVector DY = (k < 1) ? FVector(0, -Radius, 0) : FVector(0, Radius, 0);
					A = (FVector)Top + DY;
					B = (FVector)Bottom + DY;
					StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
				}
			});
		RenderData.LineIndexEnd = GobalCurveIndex;
		if (bRegenerateCurveLists)
		{
			PrimitiveRenderData.Add(RenderData);
		}
		PrimIndex++;
	}

	// convexes are drawn as mesh edges
	for (int32 Index = 0; Index < AggGeom.ConvexElems.Num(); Index++)
	{
		FPrimitiveRenderData RenderData;
		RenderData.PrimIndex = PrimIndex;
		RenderData.LineIndexStart = GobalCurveIndex;
		FColor Color = SelectedPrimitiveIDs.Contains(PrimIndex) ? SelectedColor : NormalSegmentColor;
		RenderData.RenderColor = Color;
		RenderData.ShapeType = EAggCollisionShape::Convex;

		int32 CurveIndex = 0;
		DrawnPrimitiveEdges->AddLines(1, [this, bRegenerateCurveLists, &StoreCurve, &RenderData, &AggGeom, &Color, CircleSteps, PrimIndex, &GobalCurveIndex, Index](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
			{
				const FKConvexElem& Convex = AggGeom.ConvexElems[Index];
				FTransform ElemTransform = Convex.GetTransform();
				ElemTransform.ScaleTranslation(PhysicsData->ExternalScale3D);
				ElemTransform.MultiplyScale3D(PhysicsData->ExternalScale3D);
				if (bRegenerateCurveLists)
				{
					RenderData.Transform = ElemTransform;
				}				int32 NumTriangles = Convex.IndexData.Num() / 3;
				for (int32 k = 0; k < NumTriangles; ++k)
				{
					FVector A = Convex.VertexData[Convex.IndexData[3 * k]];
					FVector B = Convex.VertexData[Convex.IndexData[3 * k + 1]];
					FVector C = Convex.VertexData[Convex.IndexData[3 * k + 2]];
					StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
					StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)B, (FVector)C, Color);
					StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)C, (FVector)A, Color);
				}
			});
		RenderData.LineIndexEnd = GobalCurveIndex;
		if (bRegenerateCurveLists)
		{
			PrimitiveRenderData.Add(RenderData);
		}
		PrimIndex++;
	}

	// for Level Sets draw the grid cells where phi < 0
	for (int32 Index = 0; Index < AggGeom.LevelSetElems.Num(); Index++)
	{
		FPrimitiveRenderData RenderData;
		RenderData.PrimIndex = PrimIndex;
		RenderData.LineIndexStart = GobalCurveIndex;
		FColor Color = SelectedPrimitiveIDs.Contains(PrimIndex) ? SelectedColor : NormalSegmentColor;
		RenderData.RenderColor = Color;
		RenderData.ShapeType = EAggCollisionShape::LevelSet;

		int32 CurveIndex = 0;
		DrawnPrimitiveEdges->AddLines(1, [this, bRegenerateCurveLists, &StoreCurve, &RenderData, &AggGeom, &Color, CircleSteps, PrimIndex, &GobalCurveIndex, Index](int32 UnusedIndex, TArray<FRenderableLine>& LinesOut)
			{
				const FKLevelSetElem& LevelSet = AggGeom.LevelSetElems[Index];

				FTransform ElemTransform = LevelSet.GetTransform();
				ElemTransform.ScaleTranslation(PhysicsData->ExternalScale3D);
				ElemTransform.MultiplyScale3D(PhysicsData->ExternalScale3D);
				if (bRegenerateCurveLists)
				{
					RenderData.Transform = ElemTransform;
				}
				auto GenerateBoxSegmentsFromFBox = [&](const FBox& Box)
				{
					const FVector3d Center = Box.GetCenter();
					const FVector3d HalfDimensions = 0.5 * (Box.Max - Box.Min);

					UE::Geometry::GenerateBoxSegments<double>(HalfDimensions, Center, FVector3d::UnitX(), FVector3d::UnitY(), FVector3d::UnitZ(), UE::Geometry::FTransformSRT3d(),
						[&](const FVector3d& A, const FVector3d& B) {
							StoreCurve(RenderData, LinesOut, PrimIndex, GobalCurveIndex++, ElemTransform* LocalToWorldTransform, (FVector)A, (FVector)B, Color);
						});
				};

				const FBox TotalGridBox = LevelSet.UntransformedAABB();
				GenerateBoxSegmentsFromFBox(TotalGridBox);

				TArray<FBox> CellBoxes;
				const double Threshold = UE_KINDA_SMALL_NUMBER;		// allow slightly greater than zero for visualization purposes
				LevelSet.GetInteriorGridCells(CellBoxes, Threshold);

				for (const FBox& CellBox : CellBoxes)
				{
					GenerateBoxSegmentsFromFBox(CellBox);
				}

			});
		RenderData.LineIndexEnd = GobalCurveIndex;
		if (bRegenerateCurveLists)
		{
			PrimitiveRenderData.Add(RenderData);
		}
		PrimIndex++;
	}

	//  TODO: Unclear whether we actually use these in the Engine, for UBodySetup? Does not appear to be supported by UxX import system,
	// and online documentation suggests they may only be supported for cloth?
	ensure(AggGeom.TaperedCapsuleElems.Num() == 0);

}


void UCollisionPrimitivesMechanic::UpdateDrawables()
{
	UpdateGizmoLocation();
	UpdateGizmoVisibility();

	for (int32 PrimIndex = 0; PrimIndex < PrimitiveRenderData.Num(); ++PrimIndex)
	{
		PrimitiveRenderData[PrimIndex].RenderColor = SelectedPrimitiveIDs.Contains(PrimIndex) ? SelectedColor : NormalSegmentColor;

		for (int32 CurveId : PrimitiveToCurveLookup[PrimIndex])
		{
			DrawnPrimitiveEdges->SetLineColor(CurveId, PrimitiveRenderData[PrimIndex].RenderColor);
		}
	}
}


void UCollisionPrimitivesMechanic::IntervalGizmoValueChanged(UIntervalGizmo* IntervalGizmo, const FVector& Direction, float Value)
{
	if (SelectedPrimitiveIDs.Num() == 0 || !bGizmoBeingDragged)
	{
		return;
	}

	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;

	for (int32 PrimitiveID : SelectedPrimitiveIDs)
	{
		FKShapeElem* Prim = AggGeom.GetElement(PrimitiveID);

		switch (Prim->GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			break;
		case EAggCollisionShape::Box:
			SetBoxShapeFromIntervals(StaticCast<FKBoxElem*>(Prim));
			break;
		case EAggCollisionShape::Sphyl:
			SetCapsuleShapeFromIntervals(StaticCast<FKSphylElem*>(Prim));
			break;
		case EAggCollisionShape::Convex:
			break;
		case EAggCollisionShape::LevelSet:
			break;
		default:
			ensure(false); // If we encounter a non-supported primitive type,
			// we should catch this and figure out how to handle it when needed.
		};
	}

	RebuildDrawables(false);

	OnCollisionGeometryChanged.Broadcast();
}


void UCollisionPrimitivesMechanic::GizmoTransformStarted(UTransformProxy* Proxy)
{
	ParentTool->GetToolManager()->BeginUndoTransaction(CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);
	PrimitivePreTransform = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);

	GizmoStartPosition = Proxy->GetTransform().GetTranslation();
	GizmoStartRotation = Proxy->GetTransform().GetRotation();
	GizmoStartScale = Proxy->GetTransform().GetScale3D();

	bGizmoBeingDragged = true;
}

void UCollisionPrimitivesMechanic::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (SelectedPrimitiveIDs.Num() == 0 || !bGizmoBeingDragged)
	{
		return;
	}

	if (Proxy->bSetPivotMode)
	{
		return;
	}

	FTransform LocalTransform = Transform * LocalToWorldTransform.Inverse();


	UE::Geometry::FQuaterniond DeltaRotation = (UE::Geometry::FQuaterniond) ( LocalToWorldTransform.InverseTransformRotation( Transform.GetRotation() ) * LocalToWorldTransform.InverseTransformRotation( GizmoStartRotation) .Inverse() );
	FVector DeltaScale = Transform.GetScale3D() / GizmoStartScale;

	UE::Geometry::FTransformSRT3d DeltaTransform;
	DeltaTransform.SetScale((FVector3d)DeltaScale);
	DeltaTransform.SetRotation((UE::Geometry::FQuaterniond)DeltaRotation);
	DeltaTransform.SetTranslation( (FVector3d)LocalToWorldTransform.InverseTransformPosition(Transform.GetLocation()) - (FVector3d)LocalToWorldTransform.InverseTransformPosition(GizmoStartPosition));

	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	FKAggregateGeom& PreTransformAggGeom = *PrimitivePreTransform;

	for (int32 PrimitiveID : SelectedPrimitiveIDs)
	{
		FKShapeElem* Prim = AggGeom.GetElement(PrimitiveID);
		FKShapeElem* PreTransformPrim = PreTransformAggGeom.GetElement(PrimitiveID);
		
		FTransform& ElemTM = PrimitiveRenderData[PrimitiveID].Transform;

		switch (Prim->GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			ElemTM = StaticCast<FKSphereElem*>(PreTransformPrim)->GetTransform();
			ElemTM.Accumulate(DeltaTransform, ScalarRegister(1.0));
			ElemTM.NormalizeRotation();
			StaticCast<FKSphereElem*>(Prim)->Radius = StaticCast<FKSphereElem*>(PreTransformPrim)->Radius * ElemTM.GetScale3D().GetAbsMax();
			ElemTM.SetScale3D(FVector::One());
			StaticCast<FKSphereElem*>(Prim)->SetTransform(ElemTM);
			break;
		case EAggCollisionShape::Box:
			ElemTM = StaticCast<FKBoxElem*>(PreTransformPrim)->GetTransform();
			ElemTM.Accumulate(DeltaTransform, ScalarRegister(1.0));
			ElemTM.NormalizeRotation();
			SetBoxShapeFromIntervals(StaticCast<FKBoxElem*>(Prim));
			ElemTM.SetScale3D(FVector::One());
			StaticCast<FKBoxElem*>(Prim)->SetTransform(ElemTM);
			break;
		case EAggCollisionShape::Sphyl:
			ElemTM = StaticCast<FKSphylElem*>(PreTransformPrim)->GetTransform();
			ElemTM.Accumulate(DeltaTransform, ScalarRegister(1.0));
			ElemTM.NormalizeRotation();
			SetCapsuleShapeFromIntervals(StaticCast<FKSphylElem*>(Prim));
			ElemTM.SetScale3D(FVector::One());
			StaticCast<FKSphylElem*>(Prim)->SetTransform(ElemTM);
			break;
		case EAggCollisionShape::Convex:
			ElemTM = StaticCast<FKConvexElem*>(PreTransformPrim)->GetTransform();
			ElemTM.Accumulate(DeltaTransform, ScalarRegister(1.0));
			ElemTM.NormalizeRotation();
			StaticCast<FKConvexElem*>(Prim)->SetTransform(ElemTM);
			break;
		case EAggCollisionShape::LevelSet:
			ElemTM = StaticCast<FKLevelSetElem*>(PreTransformPrim)->GetTransform();
			ElemTM.Accumulate(DeltaTransform, ScalarRegister(1.0));
			ElemTM.NormalizeRotation();
			StaticCast<FKLevelSetElem*>(Prim)->SetTransform(ElemTM);
			break;
		default:
			ensure(false); // If we encounter a non-supported primitive type,
			// we should catch this and figure out how to handle it when needed.
		};
	}

	RebuildDrawables(false);

	OnCollisionGeometryChanged.Broadcast();
}

void UCollisionPrimitivesMechanic::SetBoxShapeFromIntervals(FKBoxElem* BoxElem) const
{
	FVector ExternalScale = GetSafeAbsScale(LocalToWorldTransform.GetScale3D());
	BoxElem->X = FMath::Abs(BoxXIntervalSource->GetParameter() * 2 / ExternalScale.X);
	BoxElem->Y = FMath::Abs(BoxYIntervalSource->GetParameter() * 2 / ExternalScale.Y);
	BoxElem->Z = FMath::Abs(BoxZIntervalSource->GetParameter() * 2 / ExternalScale.Z);
}

void UCollisionPrimitivesMechanic::SetCapsuleShapeFromIntervals(FKSphylElem* CapsuleElem) const
{
	FVector ExternalScale = GetSafeAbsScale(LocalToWorldTransform.GetScale3D());
	CapsuleElem->Radius = FMath::Abs(CapsuleRadiusIntervalSource->GetParameter() / FMath::Max(ExternalScale.X, ExternalScale.Y));
	CapsuleElem->Length = FMath::Abs(CapsuleLengthIntervalSource->GetParameter() * 2 / ExternalScale.Z);
}

FVector3d UCollisionPrimitivesMechanic::GetSafeAbsScale(FVector3d Scale3D) const
{
	Scale3D.X = FMath::Max(FMathd::ZeroTolerance, FMath::Abs(Scale3D.X));
	Scale3D.Y = FMath::Max(FMathd::ZeroTolerance, FMath::Abs(Scale3D.Y));
	Scale3D.Z = FMath::Max(FMathd::ZeroTolerance, FMath::Abs(Scale3D.Z));
	return Scale3D;
}

void UCollisionPrimitivesMechanic::GizmoTransformEnded(UTransformProxy* Proxy)
{
	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	for (int32 PrimitiveID : SelectedPrimitiveIDs)
	{
		FKShapeElem* Prim = AggGeom.GetElement(PrimitiveID);
		FTransform& ElemTM = PrimitiveRenderData[PrimitiveID].Transform;
		switch (Prim->GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			StaticCast<FKSphereElem*>(Prim)->SetTransform(ElemTM);
			StaticCast<FKSphereElem*>(Prim)->Radius *= ElemTM.GetScale3D().GetAbsMax();
			break;
		case EAggCollisionShape::Box:
			StaticCast<FKBoxElem*>(Prim)->SetTransform(ElemTM);
			SetBoxShapeFromIntervals(StaticCast<FKBoxElem*>(Prim));
			break;
		case EAggCollisionShape::Sphyl:
			StaticCast<FKSphylElem*>(Prim)->SetTransform(ElemTM);
			SetCapsuleShapeFromIntervals(StaticCast<FKSphylElem*>(Prim));
			break;
		case EAggCollisionShape::Convex:
			StaticCast<FKConvexElem*>(Prim)->SetTransform(ElemTM);
			break;
		case EAggCollisionShape::LevelSet:
			StaticCast<FKLevelSetElem*>(Prim)->SetTransform(ElemTM);
			break;
		default:
			ensure(false); // If we encounter a non-supported primitive type,
			// we should catch this and figure out how to handle it when needed.
		};
	}

	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PrimitivePreTransform, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);

	// This just lets the tool know that the gizmo has finished moving and we've added it to the undo stack.
	// TODO: Add a different callback? "OnGizmoTransformChanged"?
	OnCollisionGeometryChanged.Broadcast();

	// TODO: When we implement snapping
	// We may need to reset the gizmo if our snapping caused the final Primitive position to differ from the gizmo position
	// UpdateGizmoLocation();

	ParentTool->GetToolManager()->EndUndoTransaction();		// was started in GizmoTransformStarted above
	
	RebuildDrawables(true);



	PrimitivePreTransform.Reset();
	bGizmoBeingDragged = false;
}

bool UCollisionPrimitivesMechanic::HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut)
{
	FGeometrySet3::FNearest Nearest;

	// See if we hit a curve for selection
	if (GeometrySet.FindNearestCurveToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ResultOut = FInputRayHit(Nearest.RayParam);
		return true;
	}
	return false;
}

FInputRayHit UCollisionPrimitivesMechanic::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Result;
	if (HitTest(ClickPos, Result))
	{
		return Result;
	}

	// Return a hit so we always capture and can clear the selection
	return FInputRayHit(TNumericLimits<float>::Max());
}

void UCollisionPrimitivesMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	using namespace CollisionPrimitivesMechanicLocals;
	
	ParentTool->GetToolManager()->BeginUndoTransaction(CollisionPrimitiveSelectionTransactionText);
	
	const TSet<int32> PreClickSelection = SelectedPrimitiveIDs;
	
	FGeometrySet3::FNearest Nearest;
	if (GeometrySet.FindNearestCurveToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		if (ShouldAddToSelectionFunc())
		{
			if (ShouldRemoveFromSelectionFunc())
			{
				Toggle(SelectedPrimitiveIDs, TSet<int>{CurveToPrimitiveLookup[Nearest.ID]});
			}
			else
			{
				SelectedPrimitiveIDs.Add(CurveToPrimitiveLookup[Nearest.ID]);
			}
		}
		else if (ShouldRemoveFromSelectionFunc())
		{
			SelectedPrimitiveIDs.Remove(CurveToPrimitiveLookup[Nearest.ID]);
		}
		else
		{
			// Neither key pressed.
			SelectedPrimitiveIDs = TSet<int>{ CurveToPrimitiveLookup[Nearest.ID] };
		}
	}
	else
	{
		SelectedPrimitiveIDs.Empty();
	}
	
	UpdateDrawables();

	if (!IsEqual(PreClickSelection, SelectedPrimitiveIDs))
	{
		checkSlow(CurrentActiveProxy);
		const FTransform Transform = CurrentActiveProxy->GetTransform();
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
			PreClickSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitiveDeselectionTransactionText);
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
			SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitiveSelectionTransactionText);
		UpdateGizmoLocation();
		OnSelectionChanged.Broadcast();
	}

	ParentTool->GetToolManager()->EndUndoTransaction();
}

void UCollisionPrimitivesMechanic::UpdateGizmoLocation()
{
	if (!TranslateTransformGizmo || !SphereTransformGizmo || !BoxTransformGizmo || !CapsuleTransformGizmo || !FullTransformGizmo)
	{
		return;
	}

	FVector3d NewGizmoLocation(0, 0, 0);

	auto ForAllGizmos = [this](TFunction<void(UCombinedTransformGizmo& Gizmo)> GizmoFunc)
	{
		GizmoFunc(*TranslateTransformGizmo);
		GizmoFunc(*SphereTransformGizmo);
		GizmoFunc(*BoxTransformGizmo);
		GizmoFunc(*CapsuleTransformGizmo);
		GizmoFunc(*FullTransformGizmo);
	};

	if (SelectedPrimitiveIDs.Num() == 0)
	{
		ForAllGizmos([](UCombinedTransformGizmo& Gizmo)
		{
			Gizmo.ReinitializeGizmoTransform(FTransform());
		});
	}
	else
	{
		for (int32 PrimitiveID : SelectedPrimitiveIDs)
		{
			NewGizmoLocation += PrimitiveRenderData[PrimitiveID].Transform.GetLocation();
		}
		NewGizmoLocation /= (double)SelectedPrimitiveIDs.Num();
	}

	FQuat GizmoRotation = LocalToWorldTransform.GetRotation();

	// We are handling one selected primitive specially because multiple selected primitives don't get rotation or interval controls
	if (SelectedPrimitiveIDs.Num() == 1)
	{
		const int32 SelectedPrimitiveID = *SelectedPrimitiveIDs.CreateConstIterator();
		GizmoRotation = (PrimitiveRenderData[SelectedPrimitiveID].Transform * LocalToWorldTransform).GetRotation();

		FKShapeElem* Shape = PhysicsData->AggGeom.GetElement(SelectedPrimitiveID);
		if (Shape->GetShapeType() == EAggCollisionShape::Box)
		{
			FKBoxElem* BoxElem = StaticCast<FKBoxElem*>(Shape);
			FVector ExternalScale = GetSafeAbsScale(LocalToWorldTransform.GetScale3D());
			BoxXIntervalSource->SetParameter(BoxElem->X * .5 * ExternalScale.X);
			BoxYIntervalSource->SetParameter(BoxElem->Y * .5 * ExternalScale.Y);
			BoxZIntervalSource->SetParameter(BoxElem->Z * .5 * ExternalScale.Z);
		}
		if (Shape->GetShapeType() == EAggCollisionShape::Sphyl)
		{
			FKSphylElem* CapsuleElem = StaticCast<FKSphylElem*>(Shape);
			FVector ExternalScaleAbs = GetSafeAbsScale(LocalToWorldTransform.GetScale3D());
			const float RadiusScale = FMath::Max(ExternalScaleAbs.X, ExternalScaleAbs.Y);
			CapsuleRadiusIntervalSource->SetParameter(CapsuleElem->Radius * RadiusScale);
			CapsuleLengthIntervalSource->SetParameter(CapsuleElem->Length * .5 * ExternalScaleAbs.Z);
		}
	}

	UpdateGizmoVisibility();

	FTransform NewTransform(GizmoRotation, LocalToWorldTransform.TransformPosition((FVector)NewGizmoLocation));
	ForAllGizmos([&NewTransform](UCombinedTransformGizmo& Gizmo)
	{
		Gizmo.ReinitializeGizmoTransform(NewTransform);
	});

	// Clear the child scale
	ForAllGizmos([](UCombinedTransformGizmo& Gizmo)
	{
		FVector GizmoScale{ 1.0, 1.0, 1.0 };
		Gizmo.SetNewChildScale(GizmoScale);
	});
}


void UCollisionPrimitivesMechanic::UpdateGizmoVisibility()
{
	if (!TranslateTransformGizmo || !SphereTransformGizmo || !BoxTransformGizmo || !CapsuleTransformGizmo || !FullTransformGizmo)
	{
		return;
	}

	auto ForAllGizmos = [this](TFunction<void(UCombinedTransformGizmo& Gizmo)> GizmoFunc)
	{
		GizmoFunc(*TranslateTransformGizmo);
		GizmoFunc(*SphereTransformGizmo);
		GizmoFunc(*BoxTransformGizmo);
		GizmoFunc(*CapsuleTransformGizmo);
		GizmoFunc(*FullTransformGizmo);
	};
	ForAllGizmos([](UCombinedTransformGizmo& Gizmo)
	{
		Gizmo.SetVisibility(false);
	});
	// Note: Interval gizmos are not transform gizmos so we handle them separately from the above 'forall' helper
	BoxIntervalGizmo->SetVisibility(false);
	CapsuleIntervalGizmo->SetVisibility(false);
	CurrentActiveProxy = TranslateTransformProxy;

	if(!(bIsDraggingRectangle || SelectedPrimitiveIDs.IsEmpty() || (ShouldHideGizmo.IsBound() && ShouldHideGizmo.Execute())))
	{
		// We are handling one selected primitive specially because each primitive type gets its own defined gizmo
		if (SelectedPrimitiveIDs.Num() == 1)
		{
			const int32 PrimitiveID = *SelectedPrimitiveIDs.CreateConstIterator();
			switch (PrimitiveRenderData[PrimitiveID].ShapeType)
			{
			case EAggCollisionShape::Sphere:
				SphereTransformGizmo->SetVisibility(true);
				CurrentActiveProxy = SphereTransformProxy;
				break;
			case EAggCollisionShape::Box:
				BoxTransformGizmo->SetVisibility(true);
				BoxIntervalGizmo->SetVisibility(true);
				CurrentActiveProxy = BoxTransformProxy;
				break;
			case EAggCollisionShape::Sphyl:
				CapsuleTransformGizmo->SetVisibility(true);
				CapsuleIntervalGizmo->SetVisibility(true);
				CurrentActiveProxy = CapsuleTransformProxy;
				break;
			case EAggCollisionShape::Convex:
				FullTransformGizmo->SetVisibility(true);
				CurrentActiveProxy = FullTransformProxy;
				break;
			case EAggCollisionShape::LevelSet:
				FullTransformGizmo->SetVisibility(true);
				CurrentActiveProxy = FullTransformProxy;
				break;
			default:
				ensure(false);
			}
		}
		else
		{
			TranslateTransformGizmo->SetVisibility(true);
		}
	}
}


bool UCollisionPrimitivesMechanic::DeselectPrimitive(int32 PrimitiveID)
{
	return SelectedPrimitiveIDs.Remove(PrimitiveID) != 0; // true if a primitive was removed
}

void UCollisionPrimitivesMechanic::SelectPrimitive(int32 PrimitiveID)
{
	SelectedPrimitiveIDs.Add(PrimitiveID);
}

FInputRayHit UCollisionPrimitivesMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Result;
	HitTest(PressPos, Result);
	return Result;
}

void UCollisionPrimitivesMechanic::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

void UCollisionPrimitivesMechanic::ClearHover()
{
	if (HoveredPrimitiveID >= 0)
	{
		for (int32 CurveId = PrimitiveRenderData[HoveredPrimitiveID].LineIndexStart; CurveId < PrimitiveRenderData[HoveredPrimitiveID].LineIndexEnd; ++CurveId)
		{
			DrawnPrimitiveEdges->SetLineColor(CurveId, PreHoverPrimitiveColor);
		}
		HoveredPrimitiveID = -1;
	}
}

bool UCollisionPrimitivesMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	FGeometrySet3::FNearest Nearest;

	// see if we're hovering a curve for selection
	if (GeometrySet.FindNearestCurveToRay((FRay3d)DevicePos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		// Only need to update the hover if we changed the curve
		if (Nearest.ID != HoveredPrimitiveID)
		{
			ClearHover();
			HoveredPrimitiveID = CurveToPrimitiveLookup[Nearest.ID];
			if (SelectedPrimitiveIDs.Contains(HoveredPrimitiveID))
			{
				PreHoverPrimitiveColor = SelectedColor;
			}
			else
			{
				PreHoverPrimitiveColor = PrimitiveRenderData[HoveredPrimitiveID].RenderColor;
			}
			for (int32 CurveId = PrimitiveRenderData[HoveredPrimitiveID].LineIndexStart; CurveId < PrimitiveRenderData[HoveredPrimitiveID].LineIndexEnd; ++CurveId)
			{
				DrawnPrimitiveEdges->SetLineColor(CurveId, HoverColor);
			}
		}
	}
	else
	{
		// Not hovering anything, so done hovering
		return false;
	}

	return true;
}

void UCollisionPrimitivesMechanic::OnEndHover()
{
	ClearHover();
}

// Detects Ctrl key state
void UCollisionPrimitivesMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	switch (ModifierID)
	{
	case ShiftModifierID:
		bShiftToggle = bIsOn;
		break;
	case CtrlModifierID:
		bCtrlToggle = bIsOn;
		break;
	default:
		break;
	}
}

// These get bound to the delegates on the marquee mechanic.
void UCollisionPrimitivesMechanic::OnDragRectangleStarted()
{
	using namespace CollisionPrimitivesMechanicLocals;

	PreDragSelection = SelectedPrimitiveIDs;
	bIsDraggingRectangle = true;
	UpdateGizmoVisibility();
	LongTransactions.Open(CollisionPrimitiveSelectionTransactionText, GetParentTool()->GetToolManager());
}

void UCollisionPrimitivesMechanic::OnDragRectangleChanged(const FCameraRectangle& Rectangle)
{
	using namespace CollisionPrimitivesMechanicLocals;
	
	TSet<int> CurveSelection, PrimitiveSelection;
	GeometrySet.ParallelFindAllCurvesSatisfying([&](const UE::Geometry::FPolyline3d& Segment)
	{
		return Rectangle.IsProjectedSegmentIntersectingRectangle(Segment.Start(), Segment.End());
	}, CurveSelection);

	for (int& Entry : CurveSelection)
	{
		PrimitiveSelection.Add(CurveToPrimitiveLookup[Entry]);
	}

	if (ShouldAddToSelectionFunc())
	{
		SelectedPrimitiveIDs = PreDragSelection;
		if (ShouldRemoveFromSelectionFunc())
		{
			Toggle(SelectedPrimitiveIDs, PrimitiveSelection);
		}
		else
		{
			SelectedPrimitiveIDs.Append(PrimitiveSelection);
		}
	}
	else if (ShouldRemoveFromSelectionFunc())
	{
		SelectedPrimitiveIDs = PreDragSelection.Difference(PrimitiveSelection);
	}
	else
	{
		// Neither key pressed.
		SelectedPrimitiveIDs = PrimitiveSelection;
	}

	UpdateDrawables();
}

void UCollisionPrimitivesMechanic::OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{
	using namespace CollisionPrimitivesMechanicLocals;

	bIsDraggingRectangle = false;

	if (!IsEqual(PreDragSelection, SelectedPrimitiveIDs))
	{
		checkSlow(CurrentActiveProxy);
		const FTransform Transform = CurrentActiveProxy->GetTransform();
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
			PreDragSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitiveDeselectionTransactionText);
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
			SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitiveSelectionTransactionText);
		
		OnSelectionChanged.Broadcast();
	}
	// We hid the gizmo at rectangle start, so it needs updating now.
	UpdateGizmoLocation();

	LongTransactions.Close(GetParentTool()->GetToolManager());

	UpdateDrawables();
}

void UCollisionPrimitivesMechanic::AddSphere()
{
	// TOOD: Refactor these Add<Shape> methods into a single templated solution

	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	TSharedPtr<FKAggregateGeom> PreAddGeometry = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);

	AggGeom.SphereElems.Add(FKSphereElem(MeshBounds.MaxDim()/2.0));

	TSet<int32> PreAddSelection = SelectedPrimitiveIDs;
	SelectedPrimitiveIDs.Empty();
	SelectedPrimitiveIDs.Add(AggGeom.SphereElems.Num() - 1);

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("AddSphereUndo", "Add Sphere Collision Primitive"));


	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PreAddGeometry, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);
	OnCollisionGeometryChanged.Broadcast();

	checkSlow(CurrentActiveProxy);
	const FTransform Transform = CurrentActiveProxy->GetTransform();
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		PreAddSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveDeselectionTransactionText);
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveSelectionTransactionText);

	OnSelectionChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();


	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

void UCollisionPrimitivesMechanic::AddBox()
{
	// TOOD: Refactor these Add<Shape> methods into a single templated solution

	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	TSharedPtr<FKAggregateGeom> PreAddGeometry = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);

	AggGeom.BoxElems.Add(FKBoxElem(MeshBounds.Dimension(0), MeshBounds.Dimension(1), MeshBounds.Dimension(2)));

	TSet<int32> PreAddSelection = SelectedPrimitiveIDs;
	SelectedPrimitiveIDs.Empty();
	SelectedPrimitiveIDs.Add(AggGeom.SphereElems.Num() + AggGeom.BoxElems.Num() - 1);

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("AddBoxUndo", "Add Box Collision Primitive"));


	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PreAddGeometry, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);
	OnCollisionGeometryChanged.Broadcast();

	checkSlow(CurrentActiveProxy);
	const FTransform Transform = CurrentActiveProxy->GetTransform();
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		PreAddSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveDeselectionTransactionText);
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveSelectionTransactionText);

	OnSelectionChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();


	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

void UCollisionPrimitivesMechanic::AddCapsule()
{
	// TOOD: Refactor these Add<Shape> methods into a single templated solution

	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	TSharedPtr<FKAggregateGeom> PreAddGeometry = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);

	AggGeom.SphylElems.Add(FKSphylElem(FMath::Max(MeshBounds.Dimension(0), MeshBounds.Dimension(1)) / 2.0, MeshBounds.Dimension(2)));

	TSet<int32> PreAddSelection = SelectedPrimitiveIDs;
	SelectedPrimitiveIDs.Empty();
	SelectedPrimitiveIDs.Add(AggGeom.SphereElems.Num() + AggGeom.BoxElems.Num() + AggGeom.SphylElems.Num() - 1);

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("AddCapsuleUndo", "Add Capsule Collision Primitive"));

	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PreAddGeometry, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);
	OnCollisionGeometryChanged.Broadcast();

	checkSlow(CurrentActiveProxy);
	const FTransform Transform = CurrentActiveProxy->GetTransform();
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		PreAddSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveDeselectionTransactionText);
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveSelectionTransactionText);

	OnSelectionChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();


	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

void UCollisionPrimitivesMechanic::DuplicateSelectedPrimitive()
{
	using namespace CollisionPrimitivesMechanicLocals;

	TArray<FKSphereElem> NewSphereElems;
	TArray<FKBoxElem> NewBoxElems;
	TArray<FKSphylElem> NewSphylElems;
	TArray<FKConvexElem> NewConvexElems;
	TArray<FKLevelSetElem> NewLevelSetElems;

	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	TSharedPtr<FKAggregateGeom> PreAddGeometry = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);
	
	if (SelectedPrimitiveIDs.IsEmpty())
	{
		return;
	}

	for (int32 PrimIndex : SelectedPrimitiveIDs)
	{
		const FKShapeElem* Prim = AggGeom.GetElement(PrimIndex);
		switch (Prim->GetShapeType())
		{
		case EAggCollisionShape::Sphere:
			NewSphereElems.Add(FKSphereElem(*StaticCast<const FKSphereElem*>(Prim)));
			break;
		case EAggCollisionShape::Box:
			NewBoxElems.Add(FKBoxElem(*StaticCast<const FKBoxElem*>(Prim)));
			break;
		case EAggCollisionShape::Sphyl:
			NewSphylElems.Add(FKSphylElem(*StaticCast<const FKSphylElem*>(Prim)));
			break;
		case EAggCollisionShape::Convex:
			NewConvexElems.Add(FKConvexElem(*StaticCast<const FKConvexElem*>(Prim)));
			break;
		case EAggCollisionShape::LevelSet:
			NewLevelSetElems.Add(FKLevelSetElem(*StaticCast<const FKLevelSetElem*>(Prim)));
			break;
		default:
			ensure(false); // If we encounter a non-supported primitive type,
			// we should catch this and figure out how to handle it when needed.
		};
	}

	TSet<int32> NewSelectedIDs;
	TSet<int32> PreDuplicateSelection = SelectedPrimitiveIDs;
	int32 TotalIDs = 0;

	for (int32 NewID = AggGeom.SphereElems.Num(); NewID < AggGeom.SphereElems.Num() + NewSphereElems.Num(); ++NewID)
	{
		NewSelectedIDs.Add(NewID);
	}
	AggGeom.SphereElems.Append(NewSphereElems);
	TotalIDs += AggGeom.SphereElems.Num();

	for (int32 NewID = AggGeom.BoxElems.Num()+TotalIDs; NewID < AggGeom.BoxElems.Num() + NewBoxElems.Num() + TotalIDs; ++NewID)
	{
		NewSelectedIDs.Add(NewID);
	}
	AggGeom.BoxElems.Append(NewBoxElems);
	TotalIDs += AggGeom.BoxElems.Num();

	for (int32 NewID = AggGeom.SphylElems.Num() + TotalIDs; NewID < AggGeom.SphylElems.Num() + NewSphylElems.Num() + TotalIDs; ++NewID)
	{
		NewSelectedIDs.Add(NewID);
	}
	AggGeom.SphylElems.Append(NewSphylElems);
	TotalIDs += AggGeom.SphylElems.Num();

	for (int32 NewID = AggGeom.ConvexElems.Num() + TotalIDs; NewID < AggGeom.ConvexElems.Num() + NewConvexElems.Num() + TotalIDs; ++NewID)
	{
		NewSelectedIDs.Add(NewID);
	}
	AggGeom.ConvexElems.Append(NewConvexElems);
	TotalIDs += AggGeom.ConvexElems.Num();

	for (int32 NewID = AggGeom.LevelSetElems.Num() + TotalIDs; NewID < AggGeom.LevelSetElems.Num() + NewLevelSetElems.Num() + TotalIDs; ++NewID)
	{
		NewSelectedIDs.Add(NewID);
	}
	AggGeom.LevelSetElems.Append(NewLevelSetElems);
	TotalIDs += AggGeom.LevelSetElems.Num();

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("DuplicateSelectionUndo", "Duplicate Selected Collision Primitives"));

	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PreAddGeometry, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitiveMovementTransactionText);
	OnCollisionGeometryChanged.Broadcast();

	SelectedPrimitiveIDs = NewSelectedIDs;

	checkSlow(CurrentActiveProxy);
	const FTransform Transform = CurrentActiveProxy->GetTransform();
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		PreDuplicateSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitiveDeselectionTransactionText);
    ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitiveSelectionTransactionText);

	OnSelectionChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();


	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

void UCollisionPrimitivesMechanic::DeleteSelectedPrimitive()
{
	int32 PrimsDeleted = 0;
	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	TSharedPtr<FKAggregateGeom> PreAddGeometry = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);


	TArray<int32> SortedSelectedIDs = SelectedPrimitiveIDs.Array();
	SortedSelectedIDs.Sort();

	for (int32 PrimIndex : SortedSelectedIDs)
	{
		int Index = PrimIndex - PrimsDeleted;
		ensure(Index < AggGeom.GetElementCount());
		PrimsDeleted++;
		if (Index < AggGeom.SphereElems.Num())
		{
			AggGeom.SphereElems.RemoveAt(Index);
			continue;
		}
		Index -= AggGeom.SphereElems.Num();
		if (Index < AggGeom.BoxElems.Num())
		{
			AggGeom.BoxElems.RemoveAt(Index);
			continue;
		}
		Index -= AggGeom.BoxElems.Num();
		if (Index < AggGeom.SphylElems.Num())
		{
			AggGeom.SphylElems.RemoveAt(Index);
			continue;
		}
		Index -= AggGeom.SphylElems.Num();
		if (Index < AggGeom.ConvexElems.Num())
		{
			AggGeom.ConvexElems.RemoveAt(Index);
			continue;
		}
		Index -= AggGeom.ConvexElems.Num();
		if (Index < AggGeom.TaperedCapsuleElems.Num())
		{
			AggGeom.TaperedCapsuleElems.RemoveAt(Index);
			continue;
		}
		Index -= AggGeom.TaperedCapsuleElems.Num();
		if (Index < AggGeom.LevelSetElems.Num())
		{
			AggGeom.LevelSetElems.RemoveAt(Index);
			continue;
		}
		Index -= AggGeom.LevelSetElems.Num();
		if (Index < AggGeom.SkinnedLevelSetElems.Num())
		{
			AggGeom.SkinnedLevelSetElems.RemoveAt(Index);
			continue;
		}		
	}

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("DeleteSelectionUndo", "Delete Selected Collision Primitives"));

	TSet<int32> PreDeleteSelection = SelectedPrimitiveIDs;
	SelectedPrimitiveIDs.Empty();

	checkSlow(CurrentActiveProxy);
	const FTransform Transform = CurrentActiveProxy->GetTransform();
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		PreDeleteSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveDeselectionTransactionText);
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveSelectionTransactionText);
	OnSelectionChanged.Broadcast();

	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PreAddGeometry, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);
	OnCollisionGeometryChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();


	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

void UCollisionPrimitivesMechanic::DeleteAllPrimitives()
{
	FKAggregateGeom& AggGeom = PhysicsData->AggGeom;
	TSharedPtr<FKAggregateGeom> PreAddGeometry = MakeShared<FKAggregateGeom>(PhysicsData->AggGeom);

	AggGeom.EmptyElements();

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("DeleteAllUndo", "Delete All Collision Primitives"));

	TSet<int32> PreDeleteSelection = SelectedPrimitiveIDs;
	SelectedPrimitiveIDs.Empty();

	checkSlow(CurrentActiveProxy);
	const FTransform Transform = CurrentActiveProxy->GetTransform();
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		PreDeleteSelection, false, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveDeselectionTransactionText);
	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicSelectionChange>(
		SelectedPrimitiveIDs, true, Transform, Transform, CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveSelectionTransactionText);
	OnSelectionChanged.Broadcast();

	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCollisionPrimitivesMechanicGeometryChange>(
		PreAddGeometry, MakeShared<FKAggregateGeom>(PhysicsData->AggGeom),
		CurrentChangeStamp), CollisionPrimitivesMechanicLocals::CollisionPrimitiveMovementTransactionText);
	OnCollisionGeometryChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();


	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

void UCollisionPrimitivesMechanic::UpdateCollisionGeometry(const FKAggregateGeom& NewGeometryIn)
{
	PhysicsData->AggGeom = NewGeometryIn;

	RebuildDrawables(true);
	UpdateGizmoLocation();
	UpdateGizmoVisibility();
}

// ==================== Undo/redo object functions ====================

UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicSelectionChange::FCollisionPrimitivesMechanicSelectionChange(int32 PrimitiveIDIn,
																						   bool bAddedIn,
																						   const FTransform& PreviousTransformIn,
																						   const FTransform& NewTransformIn,
																						   int32 ChangeStampIn)
	: PrimitiveIDs{ PrimitiveIDIn }
	, bAdded(bAddedIn)
	, PreviousTransform(PreviousTransformIn)
	, NewTransform(NewTransformIn)
	, ChangeStamp(ChangeStampIn)
{}

UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicSelectionChange::FCollisionPrimitivesMechanicSelectionChange(const TSet<int32>& PrimitiveIDsIn,
																						   bool bAddedIn, 
																						   const FTransform& PreviousTransformIn,
																						   const FTransform& NewTransformIn, 
																						   int32 ChangeStampIn)
	: PrimitiveIDs(PrimitiveIDsIn)
	, bAdded(bAddedIn)
	, PreviousTransform(PreviousTransformIn)
	, NewTransform(NewTransformIn)
	, ChangeStamp(ChangeStampIn)
{}

void UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicSelectionChange::Apply(UObject* Object)
{
	UCollisionPrimitivesMechanic* Mechanic = Cast<UCollisionPrimitivesMechanic>(Object);
	
	for (int32 PrimitiveID : PrimitiveIDs)
	{
		if (bAdded)
		{
			Mechanic->SelectPrimitive(PrimitiveID);
		}
		else
		{
			Mechanic->DeselectPrimitive(PrimitiveID);
		}
	}

	Mechanic->UpdateDrawables();
	Mechanic->OnSelectionChanged.Broadcast();
}

void UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicSelectionChange::Revert(UObject* Object)
{
	UCollisionPrimitivesMechanic* Mechanic = Cast<UCollisionPrimitivesMechanic>(Object);
	for (int32 PrimitiveID : PrimitiveIDs)
	{
		if (bAdded)
		{
			Mechanic->DeselectPrimitive(PrimitiveID);
		}
		else
		{
			Mechanic->SelectPrimitive(PrimitiveID);
		}
	}
	
	Mechanic->UpdateDrawables();
	Mechanic->OnSelectionChanged.Broadcast();
}

FString UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicSelectionChange::ToString() const
{
	return TEXT("FCollisionPrimitivesMechanicSelectionChange");
}


UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicGeometryChange::FCollisionPrimitivesMechanicGeometryChange(
	TSharedPtr<FKAggregateGeom> GeometryPreviousIn,
	TSharedPtr<FKAggregateGeom> GeometryNewIn,
	int32 ChangeStampIn) :
	GeometryPrevious{ GeometryPreviousIn }
	, GeometryNew{ GeometryNewIn }
	, ChangeStamp(ChangeStampIn)
{
}

void UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicGeometryChange::Apply(UObject* Object)
{
	UCollisionPrimitivesMechanic* Mechanic = Cast<UCollisionPrimitivesMechanic>(Object);
	Mechanic->UpdateCollisionGeometry(*GeometryNew);	
	Mechanic->OnCollisionGeometryChanged.Broadcast();
}

void UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicGeometryChange::Revert(UObject* Object)
{
	UCollisionPrimitivesMechanic* Mechanic = Cast<UCollisionPrimitivesMechanic>(Object);
	Mechanic->UpdateCollisionGeometry(*GeometryPrevious);
	Mechanic->OnCollisionGeometryChanged.Broadcast();
}

FString UCollisionPrimitivesMechanic::FCollisionPrimitivesMechanicGeometryChange::ToString() const
{
	return TEXT("FCollisionPrimitivesMechanicGeometryChange");
}

#undef LOCTEXT_NAMESPACE

