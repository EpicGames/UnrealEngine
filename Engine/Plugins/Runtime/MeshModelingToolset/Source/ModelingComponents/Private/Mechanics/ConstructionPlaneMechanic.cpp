// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/ConstructionPlaneMechanic.h"
#include "InteractiveToolManager.h"
#include "InteractiveGizmoManager.h"
#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "Drawing/MeshDebugDrawing.h"
#include "ToolSceneQueriesUtil.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ModelingComponentsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConstructionPlaneMechanic)

using namespace UE::Geometry;

void UConstructionPlaneMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);
}

void UConstructionPlaneMechanic::Shutdown()
{
	GetParentTool()->GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
}



void UConstructionPlaneMechanic::Initialize(UWorld* TargetWorld, const FFrame3d& InitialPlane)
{
	Plane = InitialPlane;
	
	// create proxy and gizmo
	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();

	PlaneTransformProxy = NewObject<UTransformProxy>(this);
	PlaneTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::StandardTranslateRotate, this);
	PlaneTransformProxy->OnTransformChanged.AddUObject(this, &UConstructionPlaneMechanic::TransformChanged);

	PlaneTransformGizmo->SetActiveTarget(PlaneTransformProxy, GetParentTool()->GetToolManager());
	PlaneTransformGizmo->ReinitializeGizmoTransform(Plane.ToFTransform());

	// click to set plane behavior
	SetPlaneCtrlClickBehaviorTarget = MakeUnique<FSelectClickedAction>();
	SetPlaneCtrlClickBehaviorTarget->SnapManager = USceneSnappingManager::Find(GetParentTool()->GetToolManager());
	SetPlaneCtrlClickBehaviorTarget->OnClickedPositionFunc = [this](const FHitResult& Hit)
	{
		SetDrawPlaneFromWorldPos(FVector3d(Hit.ImpactPoint), FVector3d(Hit.ImpactNormal), SetPlaneCtrlClickBehaviorTarget->bShiftModifierToggle);
	};
	SetPlaneCtrlClickBehaviorTarget->ExternalCanClickPredicate = [this]() { return this->CanUpdatePlaneFunc(); };

	ClickToSetPlaneBehavior = NewObject<USingleClickInputBehavior>();
	ClickToSetPlaneBehavior->ModifierCheckFunc = FInputDeviceState::IsCtrlKeyDown;
	ClickToSetPlaneBehavior->Modifiers.RegisterModifier(FSelectClickedAction::ShiftModifier, FInputDeviceState::IsShiftKeyDown);
	ClickToSetPlaneBehavior->Initialize(SetPlaneCtrlClickBehaviorTarget.Get());

	GetParentTool()->AddInputBehavior(ClickToSetPlaneBehavior);

	// click to set gizmo position behavior
	MiddleClickToSetGizmoBehavior = NewObject<ULocalSingleClickInputBehavior>();
	MiddleClickToSetGizmoBehavior->IsHitByClickFunc = [this](const FInputDeviceRay& InputRay) {
		FInputRayHit OutResult;
		FVector3d HitPointOut;
		OutResult.bHit = Plane.RayPlaneIntersection(InputRay.WorldRay.Origin, InputRay.WorldRay.Direction, 2, HitPointOut);
		return OutResult;
	};
	MiddleClickToSetGizmoBehavior->OnClickedFunc = [this](const FInputDeviceRay& ClickPos)
	{
		FVector3d HitPointOut;
		if (Plane.RayPlaneIntersection(ClickPos.WorldRay.Origin, ClickPos.WorldRay.Direction, 2, HitPointOut))
		{
			FTransform NewTransform = PlaneTransformGizmo->GetGizmoTransform();
			NewTransform.SetLocation(HitPointOut);

			// Note: The plane of this mechanic is not being updated and OnPlaneChanged is not being broadcasted here, but the
			// gizmo's position is. Anything listening for the gizmo's OnTransformChanged delegate however will be notified.
			PlaneTransformGizmo->SetNewGizmoTransform(NewTransform);
		}
	};
	MiddleClickToSetGizmoBehavior->ModifierCheckFunc = FInputDeviceState::IsCtrlKeyDown;
	MiddleClickToSetGizmoBehavior->SetUseMiddleMouseButton();
	MiddleClickToSetGizmoBehavior->Initialize();	// This takes no target because this behavior is its own target.
	
	GetParentTool()->AddInputBehavior(MiddleClickToSetGizmoBehavior);
}



void UConstructionPlaneMechanic::UpdateClickPriority(FInputCapturePriority NewPriority)
{
	ensure(ClickToSetPlaneBehavior != nullptr);
	if (ClickToSetPlaneBehavior != nullptr)
	{
		ClickToSetPlaneBehavior->SetDefaultPriority(NewPriority);
	}
}

void UConstructionPlaneMechanic::TransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	Plane.Rotation = FQuaterniond(Transform.GetRotation());
	Plane.Origin = FVector3d(Transform.GetTranslation());

	OnPlaneChanged.Broadcast();
}


void UConstructionPlaneMechanic::SetDrawPlaneFromWorldPos(const FVector3d& Position, const FVector3d& Normal, bool bIgnoreNormal)
{
	Plane.Origin = Position;
	if (!bIgnoreNormal)
	{
		// The normal will be our frame Z, but the other axes are unconstrained. If the normal
		// is aligned with world Z, then the entire frame might as well be aligned with world.
		if (1 - Normal.Dot(FVector3d::UnitZ()) < KINDA_SMALL_NUMBER)
		{
			Plane.Rotation = FQuaterniond::Identity();
		}
		else
		{
			// Otherwise, let's place one of the other axes into the XY plane so that the frame is more
			// useful for translation. We somewhat arbitrarily choose Y for this. 
			FVector3d FrameY = UE::Geometry::Normalized( Normal.Cross(FVector3d::UnitZ()) ); // orthogonal to world Z and frame Z 
			FVector3d FrameX = FrameY.Cross(Normal); // safe to not normalize because already orthogonal
			Plane = FFrame3d(Position, FrameX, FrameY, Normal);
		}
	}
	PlaneTransformGizmo->SetActiveTarget(PlaneTransformProxy, GetParentTool()->GetToolManager());
	PlaneTransformGizmo->SetNewGizmoTransform(Plane.ToFTransform());

	OnPlaneChanged.Broadcast();
}

void UConstructionPlaneMechanic::SetPlaneWithoutBroadcast(const FFrame3d &PlaneIn)
{
	Plane = PlaneIn;
	PlaneTransformGizmo->ReinitializeGizmoTransform(Plane.ToFTransform());
}


void UConstructionPlaneMechanic::Tick(float DeltaTime)
{
	if (PlaneTransformGizmo != nullptr)
	{
		PlaneTransformGizmo->SetVisibility(CanUpdatePlaneFunc());
	}
}
 
void UConstructionPlaneMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	const UModelingComponentsEditorSettings* Settings = GetMutableDefault<UModelingComponentsEditorSettings>();
	if (bShowGrid && Settings != nullptr)
	{
		const FViewCameraState CameraState = RenderAPI->GetCameraState();
		const float PDIScale = CameraState.GetPDIScalingFactor();
		
		const FFrame3d DrawFrame(PlaneTransformProxy->GetTransform().GetLocation(), Plane.Rotation);
		const int32 NumGridLines = Settings->NumGridLines;
		const float GridThickness = 0.75f * PDIScale;
		constexpr FColor GridColor(128, 128, 128, 32);
		static constexpr bool bDepthTested = false;
		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		const FTransform DrawTransform = FTransform::Identity;

		switch (Settings->GridMode)
		{
		case EModelingComponentsPlaneVisualizationMode::SimpleGrid:
		{
			const double GridSpacing = Settings->GridSpacing;
			MeshDebugDraw::DrawSimpleGrid(DrawFrame, NumGridLines, GridSpacing, GridThickness, GridColor, bDepthTested, PDI, DrawTransform);
			break;
		}
			
		case EModelingComponentsPlaneVisualizationMode::HierarchicalGrid:
		{
			const double ZoomFactor = Distance(CameraState.Position, DrawFrame.Origin);
			const FVector WorldMaxBounds(ZoomFactor, ZoomFactor, 0);
			const FVector WorldMinBounds(-WorldMaxBounds);
			constexpr int32 Levels = 2;

			TArray<FColor> Colors;
			Colors.Add(GridColor);
				
			const double Scale = Settings->GridScale;
			constexpr int32 MaxLevelDensity = 500;
			constexpr int32 Subdivision = 5;
				
			MeshDebugDraw::DrawHierarchicalGrid(Scale, ZoomFactor / Scale, MaxLevelDensity, WorldMaxBounds, WorldMinBounds, Levels, Subdivision, Colors, DrawFrame, GridThickness, bDepthTested, PDI, DrawTransform);
			break;
		}
			
		case EModelingComponentsPlaneVisualizationMode::FixedScreenAreaGrid:
		{
			const double VisualAngleSpan = Settings->GridSize * 90.;
			MeshDebugDraw::DrawSimpleFixedScreenAreaGrid(CameraState, DrawFrame, NumGridLines, VisualAngleSpan, GridThickness, GridColor, bDepthTested, PDI, DrawTransform);
			break;
		}
		}
	}
}