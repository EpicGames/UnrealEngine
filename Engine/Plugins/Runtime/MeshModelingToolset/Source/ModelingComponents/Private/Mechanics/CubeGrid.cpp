// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/CubeGrid.h"

#include "BaseGizmos/GizmoMath.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "CubeGrid"

namespace CubeGridLocals {
	FVector3d Floor(const FVector3d& VectorIn)
	{
		return FVector3d(
			FMath::Floor(VectorIn.X),
			FMath::Floor(VectorIn.Y),
			FMath::Floor(VectorIn.Z));
	}

	double GetMultiplierForGridPower(bool bPowerOfTwo, uint8 GridPower)
	{
		if (bPowerOfTwo)
		{
			// This is split up into two statements to avoid a static analysis warning about
			// shifting a 32 bit value and casting to a 64 bit value.
			uint32 ShiftedResult = static_cast<uint32>(1) << GridPower;
			return ShiftedResult;
		}
		else
		{
			// For FiveAndTen, we multiply  by 2 half the time and by 5 the second half, rounding up for 2's.
			uint8 FloorHalfGridPower = GridPower / 2;
			uint32 TwoMultiplier = static_cast<uint32>(1) << (GridPower - FloorHalfGridPower);
			return TwoMultiplier * FMath::Pow(5.0, static_cast<double>(FloorHalfGridPower));
		}
	}
}

FCubeGrid::FCubeFace::FCubeFace(const FVector3d& PointOnFace, EFaceDirection DirectionIn, uint8 SourceCubeGridPower)
	: Direction(DirectionIn)
	, CubeGridPower(SourceCubeGridPower)
{
	FVector3d MinCorner = CubeGridLocals::Floor(PointOnFace);

	Center = FVector3d(0.5, 0.5, 0.5);
	Center[DirToFlatDim(DirectionIn)] = 0;
	Center += MinCorner;
}

FVector3d FCubeGrid::FCubeFace::GetMinCorner() const
{
	return CubeGridLocals::Floor(Center);
}
FVector3d FCubeGrid::FCubeFace::GetMaxCorner() const
{
	return FVector3d(
		FMath::CeilToDouble(Center.X),
		FMath::CeilToDouble(Center.Y),
		FMath::CeilToDouble(Center.Z));
}

void FCubeGrid::SetCurrentGridCellSize(double SizeIn)
{
	using namespace CubeGridLocals;

	double Multiplier = GetMultiplierForGridPower(GridPowerMode == EPowerMode::PowerOfTwo, CurrentGridPower);
	double BaseGridSize = SizeIn / Multiplier;
	SetBaseGridCellSize(BaseGridSize);
}

double FCubeGrid::GetCellSize(uint8 GridPower) const
{
	using namespace CubeGridLocals;

	return BaseGridCellSize * GetMultiplierForGridPower(GridPowerMode == EPowerMode::PowerOfTwo, GridPower);
}

bool FCubeGrid::GetHitGridFaceBasedOnRay(const FVector3d& WorldHitPoint, const FVector3d& NormalOrTowardCameraRay,
	FCubeGrid::FCubeFace& FaceOut, bool bPierceToBack, double PlaneTolerance) const
{
	// TODO: Scale plane tolerance to grid space

	FVector3d GridSpaceHitPoint = ToGridPoint(WorldHitPoint);
	FVector3d GridSpaceRayDirection = GridFrame.ToFrameVector(NormalOrTowardCameraRay);

	FVector3d CellMinCorner = CubeGridLocals::Floor(GridSpaceHitPoint);

	// Generally, we start from the point in grid space and hit test the faces of the enclosing cell using the
	// given ray direction to figure out which face to select. However, there is ambiguity about which cell to
	// place our point when it lies on a face. In those cases, if the ray direction is inward into the current
	// cell, we need to choose the adjacent one so that the direction is outward. We also clamp the point to
	// the face because it makes it easier to treat the case properly based on what we do to t=0 during our
	// ray cast later, and guarantees that the point is inside (boundary-inclusive) our cell.
	for (int32 i = 0; i < 3; ++i)
	{
		double LowerFace = CellMinCorner[i];
		double UpperFace = LowerFace + 1;

		if ((UpperFace - GridSpaceHitPoint[i]) * CurrentCellSize <= PlaneTolerance)
		{
			GridSpaceHitPoint[i] = UpperFace;

			// Shift the chosen cell up if ray is going in the negative direction
			CellMinCorner[i] += GridSpaceRayDirection[i] < 0 ? 1 : 0;
		}
		else if ((GridSpaceHitPoint[i] - LowerFace) * CurrentCellSize <= PlaneTolerance)
		{
			GridSpaceHitPoint[i] = LowerFace;

			// Shift the chosen cell down if ray is going in the positive direction
			CellMinCorner[i] += GridSpaceRayDirection[i] > 0 ? -1 : 0;
		}
	}

	// Now do ray cast:
	GridSpaceRayDirection *= bPierceToBack ? -1 : 1;
	double BestT = TNumericLimits<double>::Max();

	// We're going iterate over XYZ dimensions and the min and max face in each dimension.
	for (int Dimension = 0; Dimension < 3; ++Dimension)
	{
		for (int Face = 0; Face < 2; ++Face) // 0 for min, 1 for other side
		{
			if (GridSpaceRayDirection[Dimension] == 0)
			{
				continue; // ray is parallel
			}

			// Since all of our planes are axis-aligned, getting the intersection T is just a matter of dividing
			// the difference of the relevant coordinate by our dt in that direction (i.e. ray direction coordinate)
			double PlaneCoordInDimension = CellMinCorner[Dimension] + Face;
			double PlaneT = (PlaneCoordInDimension - GridSpaceHitPoint[Dimension]) / GridSpaceRayDirection[Dimension];

			if (PlaneT == 0 && bPierceToBack)
			{
				// If on a face and piercing back, we don't want to select that same face
				continue;
			}

			if (PlaneT >= 0 && PlaneT < BestT)
			{
				BestT = PlaneT;

				FVector3d ArbitraryPointOnPlane = GridSpaceHitPoint;
				ArbitraryPointOnPlane[Dimension] = PlaneCoordInDimension;

				EFaceDirection FaceDirection = static_cast<EFaceDirection>(Dimension + 1);

				// Face direction should actually be flipped either if this is the lower face or if it's
				// a pierce to back, and the two flips can cancel themselves out.
				FaceDirection = (bPierceToBack ^ (Face == 0)) ? FlipDir(FaceDirection) : FaceDirection;

				FaceOut = FCubeGrid::FCubeFace(ArbitraryPointOnPlane, FaceDirection, GetGridPower());
			}
		}
	}

	return BestT >= 0 && BestT != TNumericLimits<double>::Max();
}

#undef LOCTEXT_NAMESPACE
