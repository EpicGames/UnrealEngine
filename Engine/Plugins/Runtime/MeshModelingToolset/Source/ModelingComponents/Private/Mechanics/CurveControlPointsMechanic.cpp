// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/CurveControlPointsMechanic.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"
#include "Engine/World.h"
#include "PrimitiveDrawingUtils.h"
#include "InteractiveToolManager.h"
#include "Polyline3.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "Transforms/MultiTransformer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveControlPointsMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UCurveControlPointsMechanic"

const FText PointAdditionTransactionText = LOCTEXT("PointAddition", "Point Addition");
const FText PointDeletionTransactionText = LOCTEXT("PointDeletion", "Point Deletion");
const FText PointDeselectionTransactionText = LOCTEXT("PointDeselection", "Point Deselection");
const FText PointSelectionTransactionText = LOCTEXT("PointSelection", "Point Selection");
const FText PointMovementTransactionText = LOCTEXT("PointMovement", "Point Movement");
const FText InitializationCompletedTransactionText = LOCTEXT("InitializationCompleted", "InitializationCompleted");

UCurveControlPointsMechanic::~UCurveControlPointsMechanic()
{
	checkf(PreviewGeometryActor == nullptr, TEXT("Shutdown() should be called before UCurveControlPointsMechanic is destroyed."));
}

void UCurveControlPointsMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ClickBehavior->Modifiers.RegisterModifier(ShiftModifierId, FInputDeviceState::IsShiftKeyDown);
	ClickBehavior->Modifiers.RegisterModifier(CtrlModifierId, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(ClickBehavior);

	HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->Modifiers.RegisterModifier(ShiftModifierId, FInputDeviceState::IsShiftKeyDown);
	HoverBehavior->Modifiers.RegisterModifier(CtrlModifierId, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(HoverBehavior);

	// We use custom materials that are visible through other objects.
	// TODO: This probably should be configurable. For instance, we may want the segments to be dashed, or not visible at all.
	DrawnControlPoints = NewObject<UPointSetComponent>();
	DrawnControlPoints->SetPointMaterial(
		ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ false));
	DrawnControlSegments = NewObject<ULineSetComponent>();
	DrawnControlSegments->SetLineMaterial(
		ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ false));
	PreviewPoint = NewObject<UPointSetComponent>();
	PreviewPoint->SetPointMaterial(
		ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ false));
	PreviewSegment = NewObject<ULineSetComponent>();
	PreviewSegment->SetLineMaterial(
		ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ false));

	InitializationCurveColor = FColor::Orange;
	NormalCurveColor = FColor::Red;
	CurrentSegmentsColor = bInteractiveInitializationMode ? InitializationCurveColor : NormalCurveColor;
	SegmentsThickness = 4.0f;
	CurrentPointsColor = bInteractiveInitializationMode ? InitializationCurveColor : NormalCurveColor;
	PointsSize = 8.0f;
	HoverColor = FColor::Green;
	SelectedColor = FColor::Yellow;
	PreviewColor = HoverColor;
	SnapLineColor = FColor::Yellow;
	HighlightColor = FColor::Yellow;
	DepthBias = 1.0f;

	GeometrySetToleranceTest = [this](const FVector3d& Position1, const FVector3d& Position2) {
		if (CameraState.bIsOrthographic)
		{
			// We could just always use ToolSceneQueriesUtil::PointSnapQuery. But in ortho viewports, we happen to know
			// that the only points that we will ever give this function will be the closest points between a ray and
			// some geometry, meaning that the vector between them will be orthogonal to the view ray. With this knowledge,
			// we can do the tolerance computation more efficiently than PointSnapQuery can, since we don't need to project
			// down to the view plane.
			// As in PointSnapQuery, we convert our angle-based tolerance to one we can use in an ortho viewport (instead of
			// dividing our field of view into 90 visual angle degrees, we divide the plane into 90 units).
			float OrthoTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * CameraState.OrthoWorldCoordinateWidth / 90.0;
			return DistanceSquared(Position1, Position2) < OrthoTolerance * OrthoTolerance;
		}
		else
		{
			return ToolSceneQueriesUtil::PointSnapQuery(CameraState, Position1, Position2);
		}
	};

	SnapEngine.SnapMetricFunc = [this](const FVector3d& Position1, const FVector3d& Position2) {
		return ToolSceneQueriesUtil::PointSnapMetric(this->CameraState, Position1, Position2);
	};
	SnapEngine.SnapMetricTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();

	// So far, snapping to known lengths has seemed more inconvenient than useful, but we
	// can enable this if we decide we want it. It will require a bit of extra code to make it
	// obvious what we're snapping to in that case.
	SnapEngine.bEnableSnapToKnownLengths = false;

	LineSnapIDMin = FPointPlanarSnapSolver::BaseExternalLineID;
	LineSnapPriority = SnapEngine.MinInternalPriority() - 1; // lower is more important

	// We need to be able to detect when we snap to first/last points in the curve for leaving initialization mode
	FirstPointSnapID = FPointPlanarSnapSolver::BaseExternalPointID;
	LastPointSnapID = FirstPointSnapID + 1;
	EndpointSnapPriority = LineSnapPriority - SnapEngine.IntersectionPriorityDelta - 1; // more important than user lines, or intersections with user lines

	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	PointTransformProxy = NewObject<UTransformProxy>(this);
	PointTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::TranslateAxisX | ETransformGizmoSubElements::TranslateAxisY | ETransformGizmoSubElements::TranslatePlaneXY,
		this);
	PointTransformProxy->OnTransformChanged.AddUObject(this, &UCurveControlPointsMechanic::GizmoTransformChanged);
	PointTransformProxy->OnBeginTransformEdit.AddUObject(this, &UCurveControlPointsMechanic::GizmoTransformStarted);
	PointTransformProxy->OnEndTransformEdit.AddUObject(this, &UCurveControlPointsMechanic::GizmoTransformEnded);
	PointTransformGizmo->SetActiveTarget(PointTransformProxy, GetParentTool()->GetToolManager());
	PointTransformGizmo->SetVisibility(false);

	// We force the coordinate system to be local so that the gizmo only moves in the plane we specify
	PointTransformGizmo->bUseContextCoordinateSystem = false;
	PointTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
}

void UCurveControlPointsMechanic::SetWorld(UWorld* World)
{
	// It may be unreasonable to worry about SetWorld being called more than once, but let's be safe anyway
	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
	}

	// We need the world so we can create the geometry actor in the right place.
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	PreviewGeometryActor = World->SpawnActor<APreviewGeometryActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	// Attach the rendering components to the actor
	DrawnControlPoints->Rename(nullptr, PreviewGeometryActor); // Changes the "outer"
	PreviewGeometryActor->SetRootComponent(DrawnControlPoints);
	if (DrawnControlPoints->IsRegistered())
	{
		DrawnControlPoints->ReregisterComponent();
	}
	else
	{
		DrawnControlPoints->RegisterComponent();
	}

	DrawnControlSegments->Rename(nullptr, PreviewGeometryActor); // Changes the "outer"
	DrawnControlSegments->AttachToComponent(DrawnControlPoints, FAttachmentTransformRules::KeepWorldTransform);
	if (DrawnControlSegments->IsRegistered())
	{
		DrawnControlSegments->ReregisterComponent();
	}
	else
	{
		DrawnControlSegments->RegisterComponent();
	}

	PreviewPoint->Rename(nullptr, PreviewGeometryActor); // Changes the "outer"
	PreviewPoint->AttachToComponent(DrawnControlPoints, FAttachmentTransformRules::KeepWorldTransform);
	if (PreviewPoint->IsRegistered())
	{
		PreviewPoint->ReregisterComponent();
	}
	else
	{
		PreviewPoint->RegisterComponent();
	}

	PreviewSegment->Rename(nullptr, PreviewGeometryActor); // Changes the "outer"
	PreviewSegment->AttachToComponent(DrawnControlPoints, FAttachmentTransformRules::KeepWorldTransform);
	if (PreviewSegment->IsRegistered())
	{
		PreviewSegment->ReregisterComponent();
	}
	else
	{
		PreviewSegment->RegisterComponent();
	}
}

void UCurveControlPointsMechanic::Shutdown()
{
	// Calls shutdown on gizmo and destroys it.
	GetParentTool()->GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);

	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
		PreviewGeometryActor = nullptr;
	}
}

void UCurveControlPointsMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	// TODO: Should we cache the camera state here or somewhere else?
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	// Draw the lines that we're currently snapped to
	if (bSnappingEnabled && SnapEngine.HaveActiveSnap())
	{
		const int SNAP_LINE_HALF_LENGTH = 9999;

		FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();
		float PDIScale = RenderAPI->GetCameraState().GetPDIScalingFactor();
		if (SnapEngine.HaveActiveSnapLine())
		{
			FLine3d SnapLine = SnapEngine.GetActiveSnapLine();
			PDI->DrawLine((FVector)SnapLine.PointAt(-SNAP_LINE_HALF_LENGTH), (FVector)SnapLine.PointAt(SNAP_LINE_HALF_LENGTH),
				SnapLineColor, SDPG_Foreground, 0.5 * PDIScale, 0.0f, true);
		}
		else
		{
			// See if we snapped to an intersection of two lines
			if (SnapEngine.HaveActiveSnapIntersection())
			{
				// Draw both of the lines that are intersecting
				FLine3d SnapLine = SnapEngine.GetActiveSnapLine();
				PDI->DrawLine((FVector)SnapLine.PointAt(-SNAP_LINE_HALF_LENGTH), (FVector)SnapLine.PointAt(SNAP_LINE_HALF_LENGTH),
					SnapLineColor, SDPG_Foreground, 0.5 * PDIScale, 0.0f, true);
				SnapLine = SnapEngine.GetIntersectionSecondLine();
				PDI->DrawLine((FVector)SnapLine.PointAt(-SNAP_LINE_HALF_LENGTH), (FVector)SnapLine.PointAt(SNAP_LINE_HALF_LENGTH),
					SnapLineColor, SDPG_Foreground, 0.5 * PDIScale, 0.0f, true);
			}
		}
	}

	if (SnapEngine.HaveActiveSnap() && (SnapEngine.GetActiveSnapTargetID() == FirstPointSnapID || SnapEngine.GetActiveSnapTargetID() == LastPointSnapID))
	{
		DrawnControlSegments->SetAllLinesColor(HighlightColor);
	}
	else
	{
		DrawnControlSegments->SetAllLinesColor(CurrentSegmentsColor);
	}
}

void UCurveControlPointsMechanic::Initialize(const TArray<FVector3d>& Points, bool bIsLoopIn)
{
	// This also clears selection and hover
	ClearPoints();

	for (const FVector3d& Point : Points)
	{
		AppendPoint(Point);
	}

	bIsLoop = bIsLoopIn;

	SnapEngine.UpdatePointHistory(Points);
	UpdateSnapTargetsForHover();
}

void UCurveControlPointsMechanic::ClearPoints()
{
	ClearSelection();
	ClearHover();

	ControlPoints.Empty();
	GeometrySet.Reset();
	DrawnControlSegments->Clear();
	DrawnControlPoints->Clear();
	SnapEngine.Reset();
}

int32 UCurveControlPointsMechanic::AppendPoint(const FVector3d& Point)
{
	return InsertPointAt(ControlPoints.Num(), Point);
}

int32 UCurveControlPointsMechanic::InsertPointAt(int32 SequencePosition, const FVector3d& NewPointCoordinates, const int32* KnownPointID)
{
	// Add the point
	int32 NewPointID = ControlPoints.InsertPointAt(SequencePosition, NewPointCoordinates, KnownPointID);
	GeometrySet.AddPoint(NewPointID, NewPointCoordinates);
	FRenderablePoint RenderablePoint((FVector)NewPointCoordinates, CurrentPointsColor, PointsSize, DepthBias);
	DrawnControlPoints->InsertPoint(NewPointID, RenderablePoint);

	// See if we need to add some segments
	if (ControlPoints.Num() > 1)
	{
		if (bIsLoop || SequencePosition != 0)
		{
			// Alter (or add) the preceding segment to go to the new point.
			int32 PreviousSequencePosition = (SequencePosition + ControlPoints.Num() - 1) % ControlPoints.Num();
			int32 PreviousID = ControlPoints.GetPointIDAt(PreviousSequencePosition);

			FPolyline3d SegmentPolyline(ControlPoints.GetPointCoordinates(PreviousID), NewPointCoordinates);

			if (DrawnControlSegments->IsLineValid(PreviousID))
			{
				DrawnControlSegments->SetLineEnd(PreviousID, (FVector)NewPointCoordinates);

				GeometrySet.UpdateCurve(PreviousID, SegmentPolyline);
			}
			else
			{
				FRenderableLine RenderableSegment((FVector)ControlPoints.GetPointCoordinates(PreviousID),
					(FVector)NewPointCoordinates, CurrentSegmentsColor, SegmentsThickness, DepthBias);
				DrawnControlSegments->InsertLine(PreviousID, RenderableSegment);

				GeometrySet.AddCurve(PreviousID, SegmentPolyline);
			}
		}
		if (bIsLoop || SequencePosition != ControlPoints.Num() - 1)
		{
			// Create a segment going to the next point
			int32 NextSequencePosition = (SequencePosition + 1) % ControlPoints.Num();

			FPolyline3d SegmentPolyline(ControlPoints.GetPointCoordinatesAt(NextSequencePosition), NewPointCoordinates);
			GeometrySet.AddCurve(NewPointID, SegmentPolyline);

			FRenderableLine RenderableSegment((FVector)NewPointCoordinates, (FVector)ControlPoints.GetPointCoordinatesAt(NextSequencePosition),
				CurrentSegmentsColor, SegmentsThickness, DepthBias);
			DrawnControlSegments->InsertLine(NewPointID, RenderableSegment);
		}
	}

	// Update the snapping support
	SnapEngine.InsertHistoryPoint(NewPointCoordinates, SequencePosition);
	if (bInteractiveInitializationMode)
	{
		// See if we have a new endpoint for interactive mode to snap to
		if (SequencePosition == ControlPoints.Num() - 1)
		{
			SnapEngine.RemovePointTargetsByID(LastPointSnapID);
			SnapEngine.AddPointTarget(NewPointCoordinates, LastPointSnapID, EndpointSnapPriority);

			// Also need to go ahead and regenerate snap targets, since we don't rely on a selection in initialization mode.
			UpdateSnapTargetsForHover();
		}
		if (SequencePosition == 0)
		{
			SnapEngine.RemovePointTargetsByID(FirstPointSnapID);
			SnapEngine.AddPointTarget(NewPointCoordinates, FirstPointSnapID, EndpointSnapPriority);
		}
	}

	return NewPointID;
}

void UCurveControlPointsMechanic::SetIsLoop(bool bIsLoopIn)
{
	// Only do stuff if things changed
	if (bIsLoopIn != bIsLoop)
	{
		// Add/remove closing segment
		if (ControlPoints.Num() > 1)
		{
			if (bIsLoopIn)
			{
				FPolyline3d SegmentPolyline(ControlPoints.GetPointCoordinates(ControlPoints.Last()), ControlPoints.GetPointCoordinates(ControlPoints.First()));
				GeometrySet.AddCurve(ControlPoints.Last(), SegmentPolyline);

				FRenderableLine RenderableSegment((FVector)ControlPoints.GetPointCoordinates(ControlPoints.Last()),
					(FVector)ControlPoints.GetPointCoordinates(ControlPoints.First()), CurrentSegmentsColor, SegmentsThickness, DepthBias);
				DrawnControlSegments->InsertLine(ControlPoints.Last(), RenderableSegment);
			}
			else
			{
				// Need to remove the loop closing segment
				GeometrySet.RemoveCurve(ControlPoints.Last());
				DrawnControlSegments->RemoveLine(ControlPoints.Last());
			}
		}

		bIsLoop = bIsLoopIn;
	}
}

void UCurveControlPointsMechanic::ExtractPointPositions(TArray<FVector3d>& PositionsOut)
{
	for (int32 PointID : ControlPoints.PointIDItr())
	{
		PositionsOut.Add(ControlPoints.GetPointCoordinates(PointID));
	}
}

void UCurveControlPointsMechanic::SetInteractiveInitialization(bool bOn)
{
	// Only do things if things actually changed
	if (bInteractiveInitializationMode != bOn)
	{
		bInteractiveInitializationMode = bOn;

		// The visualization is different colors in different modes.
		CurrentSegmentsColor = bInteractiveInitializationMode ? InitializationCurveColor : NormalCurveColor;
		CurrentPointsColor = bInteractiveInitializationMode ? InitializationCurveColor : NormalCurveColor;
		DrawnControlPoints->SetAllPointsColor(CurrentPointsColor);
		DrawnControlSegments->SetAllLinesColor(CurrentSegmentsColor);

		if (bOn)
		{
			// If we're in initialization mode, we can't have any selection, and we can't be in a loop.
			ClearSelection();
			ClearHover();
			SetIsLoop(false);

			// We now need to be able to snap the new segment, especially against the first and last points, as
			// this brings us out of initialization mode.
			if (ControlPoints.Num() > 0)
			{
				SnapEngine.AddPointTarget(ControlPoints.GetPointCoordinates(ControlPoints.First()), FirstPointSnapID, EndpointSnapPriority);
				SnapEngine.AddPointTarget(ControlPoints.GetPointCoordinates(ControlPoints.Last()), LastPointSnapID, EndpointSnapPriority);
			}

			SnapEngine.RegenerateTargetLinesAround(ControlPoints.Num());
		}
		else
		{
			// If we've left initialization mode, we no longer need to snap against the endpoints
			SnapEngine.RemovePointTargetsByID(FirstPointSnapID);
			SnapEngine.RemovePointTargetsByID(LastPointSnapID);
		}
	}
}

void UCurveControlPointsMechanic::GizmoTransformStarted(UTransformProxy* Proxy)
{
	ParentTool->GetToolManager()->BeginUndoTransaction(PointMovementTransactionText);

	GizmoStartPosition = Proxy->GetTransform().GetTranslation();

	SelectedPointStartPositions.SetNum(SelectedPointIDs.Num());
	for (int32 i = 0; i < SelectedPointIDs.Num(); ++i)
	{
		SelectedPointStartPositions[i] = ControlPoints.GetPointCoordinates(SelectedPointIDs[i]);
	}

	if (SelectedPointIDs.Num() == 1)
	{
		SnapEngine.RegenerateTargetLinesAround(ControlPoints.GetSequencePosition(SelectedPointIDs[0]), bIsLoop);
	}
	
	bGizmoBeingDragged = true;
}

void UCurveControlPointsMechanic::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (SelectedPointIDs.Num() == 0 || !bGizmoBeingDragged)
	{
		return;
	}

	bool bPointsChanged = false;

	FVector Displacement = Transform.GetTranslation() - GizmoStartPosition;
	if (Displacement != FVector::ZeroVector)
	{
		// Do snapping only if we have a single point selected.
		if (SelectedPointIDs.Num() == 1 && (bSnappingEnabled ^ bSnapToggle))
		{
			FVector3d NewLocation = SelectedPointStartPositions[0] + (FVector3d)Displacement;
			if (bSnappingEnabled ^ bSnapToggle)
			{
				SnapEngine.UpdateSnappedPoint(NewLocation);
				if (SnapEngine.HaveActiveSnap())
				{
					NewLocation = SnapEngine.GetActiveSnapToPoint();
				}
			}

			// When snapping, it is possible that a point didn't actually end up moving, and doesn't need updating.
			if (ControlPoints.GetPointCoordinates(SelectedPointIDs[0]) != NewLocation)
			{
				UpdatePointLocation(SelectedPointIDs[0], NewLocation);
				bPointsChanged = true;
			}
		}
		else
		{
			for (int32 i = 0; i < SelectedPointIDs.Num(); ++i)
			{
				UpdatePointLocation(SelectedPointIDs[i], SelectedPointStartPositions[i] + (FVector3d)Displacement);
			}
			bPointsChanged = true;
		}
	}

	if (bPointsChanged)
	{
		OnPointsChanged.Broadcast();
	}
}

void UCurveControlPointsMechanic::GizmoTransformEnded(UTransformProxy* Proxy)
{
	for (int32 i = 0; i < SelectedPointIDs.Num(); ++i)
	{
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicMovementChange>(
			SelectedPointIDs[i], SelectedPointStartPositions[i], ControlPoints.GetPointCoordinates(SelectedPointIDs[i]),
			CurrentChangeStamp), PointMovementTransactionText);

		// Note that we can't do this in UpdatePointLocation despite that being conceptually nice, because we don't
		// want to change what we're snapping to as we drag a point around (technically we could make it work if we
		// made the snap engine not reset snaps at each update, since the points that we're snapping to remain stationary,
		// but best not to).
		UpdateSnapHistoryPoint(ControlPoints.GetSequencePosition(SelectedPointIDs[i]), ControlPoints.GetPointCoordinates(SelectedPointIDs[i]));
	}

	SelectedPointStartPositions.Reset();

	// We may need to reset the gizmo if our snapping caused the final point position to differ from the gizmo position.
	UpdateGizmoLocation();

	// No need to draw the snap line anymore
	SnapEngine.ResetActiveSnap();

	UpdateSnapTargetsForHover();

	ParentTool->GetToolManager()->EndUndoTransaction();

	bGizmoBeingDragged = false;
}

void UCurveControlPointsMechanic::UpdatePointLocation(int32 PointID, const FVector3d& NewLocation)
{
	ControlPoints.SetPointCoordinates(PointID, NewLocation);
	GeometrySet.UpdatePoint(PointID, NewLocation);
	DrawnControlPoints->SetPointPosition(PointID, (FVector)NewLocation);

	int32 SequencePosition = ControlPoints.GetSequencePosition(PointID);
	
	// Update the segment going to this point.
	if (bIsLoop || PointID != ControlPoints.First())
	{
		int32 PreviousSequencePosition = (SequencePosition + ControlPoints.Num() - 1) % ControlPoints.Num();
		DrawnControlSegments->SetLineEnd(ControlPoints.GetPointIDAt(PreviousSequencePosition), (FVector)NewLocation);

		FPolyline3d SegmentPolyline(ControlPoints.GetPointCoordinatesAt(PreviousSequencePosition), NewLocation);
		GeometrySet.UpdateCurve(ControlPoints.GetPointIDAt(PreviousSequencePosition), SegmentPolyline);
	}

	// Update the segment going from this point.
	if (bIsLoop || PointID != ControlPoints.Last())
	{
		DrawnControlSegments->SetLineStart(PointID, (FVector)NewLocation);

		FPolyline3d SegmentPolyline(NewLocation, ControlPoints.GetPointCoordinatesAt((SequencePosition + 1) % ControlPoints.Num()));
		GeometrySet.UpdateCurve(PointID, SegmentPolyline);
	}
}


bool UCurveControlPointsMechanic::HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut)
{
	ResultOut = FInputRayHit();
	FGeometrySet3::FNearest Nearest;

	// See if we are adding a new point (either in interactive initialization, or by adding a point on the end)
	if (bInteractiveInitializationMode ||
		(bInsertPointToggle && !bIsLoop && SelectedPointIDs.Num() == 1
		&& (SelectedPointIDs[0] == ControlPoints.First() || SelectedPointIDs[0] == ControlPoints.Last())))
	{
		FVector3d HitPoint;
		bool bHit = DrawPlane.RayPlaneIntersection((FVector3d)ClickPos.WorldRay.Origin, (FVector3d)ClickPos.WorldRay.Direction, 2, HitPoint);
		if (bHit)
		{
			ResultOut = FInputRayHit(ClickPos.WorldRay.GetParameter((FVector)HitPoint));
		}
		return bHit;
	}
	// Otherwise, see if we are in insert mode and hitting a segment
	else if (bInsertPointToggle)
	{
		if (GeometrySet.FindNearestCurveToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
		{
			ResultOut = FInputRayHit(Nearest.RayParam);
			return true;
		}
	}
	// See if we hit a point for selection
	else if (GeometrySet.FindNearestPointToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ResultOut = FInputRayHit(Nearest.RayParam);
		return true;
	}
	return false;
}

FInputRayHit UCurveControlPointsMechanic::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Result;
	HitTest(ClickPos, Result);
	return Result;
}

void UCurveControlPointsMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	FGeometrySet3::FNearest Nearest;

	if (bInteractiveInitializationMode)
	{
		FVector3d NewPointCoordinates;
		if (!DrawPlane.RayPlaneIntersection((FVector3d)ClickPos.WorldRay.Origin, (FVector3d)ClickPos.WorldRay.Direction, 2, NewPointCoordinates))
		{
			// Missed the plane entirely (probably in ortho mode). Nothing to do.
			return;
		}

		// When snapping is disabled, we still need to be able to snap to the first/last points to get out of the mode.
		bool bLeavingMode = false;
		SnapEngine.UpdateSnappedPoint(NewPointCoordinates);
		if (SnapEngine.HaveActiveSnap())
		{
			if ((ControlPoints.Num() >= MinPointsForLoop && SnapEngine.GetActiveSnapTargetID() == FirstPointSnapID)
				|| (ControlPoints.Num() >= MinPointsForNonLoop && SnapEngine.GetActiveSnapTargetID() == LastPointSnapID))
			{
				ParentTool->GetToolManager()->BeginUndoTransaction(InitializationCompletedTransactionText);

				bool bClosedLoop = SnapEngine.GetActiveSnapTargetID() == FirstPointSnapID;
				SetIsLoop(bClosedLoop);
				SetInteractiveInitialization(false);
				ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicModeChange>(
					true, bClosedLoop, CurrentChangeStamp), InitializationCompletedTransactionText);
				
				ParentTool->GetToolManager()->EndUndoTransaction();

				bLeavingMode = true;
			}
			else if (bSnappingEnabled ^ bSnapToggle)
			{
				NewPointCoordinates = SnapEngine.GetActiveSnapToPoint();
			}

			// Don't want to draw the snap line after the click
			SnapEngine.ResetActiveSnap();
		}

		if (bLeavingMode)
		{
			OnModeChanged.Broadcast();
		}
		else
		{
			// Adding a point in initialization mode
			ParentTool->GetToolManager()->BeginUndoTransaction(PointAdditionTransactionText);

			int32 NewPointID = InsertPointAt(ControlPoints.Num(), NewPointCoordinates);
			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicInsertionChange>(
				ControlPoints.Num() - 1, NewPointID, NewPointCoordinates, true, CurrentChangeStamp), PointAdditionTransactionText);

			ParentTool->GetToolManager()->EndUndoTransaction();
			OnPointsChanged.Broadcast();

			// Prepare snap targets for next point
			UpdateSnapTargetsForHover();
		}
	}

	else if (bInsertPointToggle)
	{
		// Adding on an existing edge takes priority to adding to the end.
		if (GeometrySet.FindNearestCurveToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
		{
			ParentTool->GetToolManager()->BeginUndoTransaction(PointAdditionTransactionText);

			int32 SequencePosition = ControlPoints.GetSequencePosition(Nearest.ID);
			int32 NewPointID = InsertPointAt(SequencePosition + 1, Nearest.NearestGeoPoint);
			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicInsertionChange>(
				SequencePosition + 1, NewPointID, Nearest.NearestGeoPoint, true, CurrentChangeStamp), PointAdditionTransactionText);

			ChangeSelection(NewPointID, false);

			ParentTool->GetToolManager()->EndUndoTransaction();
			OnPointsChanged.Broadcast();
		}

		// Try to add to one of the ends
		else if (SelectedPointIDs.Num() == 1 && !bIsLoop
			&& (SelectedPointIDs[0] == ControlPoints.First() || SelectedPointIDs[0] == ControlPoints.Last()))
		{
			ParentTool->GetToolManager()->BeginUndoTransaction(PointAdditionTransactionText);

			FVector3d NewPointCoordinates;
			DrawPlane.RayPlaneIntersection((FVector3d)ClickPos.WorldRay.Origin, (FVector3d)ClickPos.WorldRay.Direction, 2, NewPointCoordinates);

			if (bSnappingEnabled ^ bSnapToggle)
			{
				SnapEngine.UpdateSnappedPoint(NewPointCoordinates);
				if (SnapEngine.HaveActiveSnap())
				{
					NewPointCoordinates = SnapEngine.GetActiveSnapToPoint();
					SnapEngine.ResetActiveSnap();
				}
			}

			// Do the actual insertion
			int32 NewPointID;
			if (SelectedPointIDs[0] == ControlPoints.First())
			{
				NewPointID = InsertPointAt(0, NewPointCoordinates);
				ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicInsertionChange>(
					0, NewPointID, NewPointCoordinates, true, CurrentChangeStamp), PointAdditionTransactionText);
			}
			else
			{
				NewPointID = InsertPointAt(ControlPoints.Num(), NewPointCoordinates);
				ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicInsertionChange>(
					ControlPoints.Num() - 1, NewPointID, NewPointCoordinates, true, CurrentChangeStamp), PointAdditionTransactionText);
			}
			ChangeSelection(NewPointID, false);

			ParentTool->GetToolManager()->EndUndoTransaction();
			OnPointsChanged.Broadcast();
		}
	}
	// Otherwise, check for plain old selection
	else if (GeometrySet.FindNearestPointToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ParentTool->GetToolManager()->BeginUndoTransaction(PointSelectionTransactionText);

		ChangeSelection(Nearest.ID, bAddToSelectionToggle);

		ParentTool->GetToolManager()->EndUndoTransaction();
	}
}

void UCurveControlPointsMechanic::ChangeSelection(int32 NewPointID, bool bAddToSelection)
{
	// If not adding to selection, clear it
	if (!bAddToSelection && SelectedPointIDs.Num() > 0)
	{
		for (int32 PointID : SelectedPointIDs)
		{
			// We check for validity here because we'd like to be able to use this function to deselect points after
			// deleting them.
			if (DrawnControlPoints->IsPointValid(PointID))
			{
				DrawnControlPoints->SetPointColor(PointID, CurrentPointsColor);

				ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicSelectionChange>(
					PointID, false, CurrentChangeStamp), PointDeselectionTransactionText);
			}
		}

		SelectedPointIDs.Empty();
	}

	// We check for validity here because giving an invalid id (such as -1) with bAddToSelection == false
	// is an easy way to clear the selection.
	if (ControlPoints.IsValidPoint(NewPointID))
	{
		if (bAddToSelection && DeselectPoint(NewPointID))
		{
			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicSelectionChange>(
				NewPointID, false, CurrentChangeStamp), PointDeselectionTransactionText);
		}
		else
		{
			SelectPoint(NewPointID);

			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicSelectionChange>(
				NewPointID, true, CurrentChangeStamp), PointSelectionTransactionText);
		}
	}

	UpdateGizmoLocation();
}

void UCurveControlPointsMechanic::UpdateGizmoVisibility()
{
	if (PointTransformGizmo)
	{
		PointTransformGizmo->SetVisibility( (SelectedPointIDs.Num() > 0) && (!bInsertPointToggle) );
	}
}

void UCurveControlPointsMechanic::UpdateGizmoLocation()
{
	if (!PointTransformGizmo)
	{
		return;
	}

	if (SelectedPointIDs.Num() > 0)
	{
		FVector3d NewGizmoLocation = FVector3d::Zero();
		for (int32 PointID : SelectedPointIDs)
		{
			NewGizmoLocation += ControlPoints.GetPointCoordinates(PointID);
		}
		NewGizmoLocation /= (double)SelectedPointIDs.Num();

		PointTransformGizmo->ReinitializeGizmoTransform(FTransform((FQuat)DrawPlane.Rotation, (FVector)NewGizmoLocation));
	}

	UpdateGizmoVisibility();
}

void UCurveControlPointsMechanic::SetPlane(const FFrame3d& DrawPlaneIn)
{
	DrawPlane = DrawPlaneIn;
	UpdateGizmoLocation(); // Gizmo is constrained to plane

	SnapEngine.Plane = DrawPlane;
}

void UCurveControlPointsMechanic::SetSnappingEnabled(bool bOn)
{
	bSnappingEnabled = bOn;
}

void UCurveControlPointsMechanic::AddSnapLine(int32 LineID, const FLine3d& Line)
{
	SnapEngine.AddLineTarget(Line, LineSnapIDMin + LineID, LineSnapPriority);
}

void UCurveControlPointsMechanic::RemoveSnapLine(int32 LineID)
{
	SnapEngine.RemoveLineTargetsByID(LineSnapIDMin + LineID);
}

bool UCurveControlPointsMechanic::DeselectPoint(int32 PointID)
{
	bool PointFound = false;
	int32 IndexInSelection;
	if (SelectedPointIDs.Find(PointID, IndexInSelection))
	{
		SelectedPointIDs.RemoveAt(IndexInSelection);
		DrawnControlPoints->SetPointColor(PointID, CurrentPointsColor);

		PointFound = true;

		UpdateSnapTargetsForHover();
	}
	
	return PointFound;
}

void UCurveControlPointsMechanic::SelectPoint(int32 PointID)
{
	SelectedPointIDs.Add(PointID);
	DrawnControlPoints->SetPointColor(PointID, SelectedColor);

	UpdateSnapTargetsForHover();
}

void UCurveControlPointsMechanic::ClearSelection()
{
	ChangeSelection(-1, false);
}

FInputRayHit UCurveControlPointsMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Result;
	HitTest(PressPos, Result);
	return Result;
}

void UCurveControlPointsMechanic::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

void UCurveControlPointsMechanic::ClearHover()
{
	if (HoveredPointID >= 0)
	{
		DrawnControlPoints->SetPointColor(HoveredPointID, PreHoverPointColor);
		HoveredPointID = -1;
	}
	PreviewPoint->Clear();
	PreviewSegment->Clear();
}

void UCurveControlPointsMechanic::UpdateSnapTargetsForHover()
{
	if (ControlPoints.Num() == 0)
	{
		// Clear targets
		SnapEngine.RegenerateTargetLines(false, false);
		return;
	}

	if (bInteractiveInitializationMode)
	{
		SnapEngine.RegenerateTargetLinesAround(ControlPoints.Num());
	}
	else if (SelectedPointIDs.Num() == 1)
	{
		if (SelectedPointIDs[0] == ControlPoints.First())
		{
			SnapEngine.RegenerateTargetLinesAround(-1);
		}
		else if (SelectedPointIDs[0] == ControlPoints.Last())
		{
			SnapEngine.RegenerateTargetLinesAround(ControlPoints.Num());
		}
		else
		{
			SnapEngine.RegenerateTargetLines(false, false); // clear targets
		}
	}
	else
	{
		SnapEngine.RegenerateTargetLines(false, false); // clear targets
	}
}

void UCurveControlPointsMechanic::UpdateSnapHistoryPoint(int32 Index, FVector3d NewPosition)
{
	SnapEngine.RemoveHistoryPoint(Index);
	SnapEngine.InsertHistoryPoint(NewPosition, Index);
}

bool UCurveControlPointsMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	FGeometrySet3::FNearest Nearest;

	// In edit mode, point insertion on an edge has priority.
	if (!bInteractiveInitializationMode && bInsertPointToggle && GeometrySet.FindNearestCurveToRay((FRay3d)DevicePos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ClearHover();
		FRenderablePoint RenderablePoint((FVector)Nearest.NearestGeoPoint, PreviewColor, PointsSize, DepthBias);
		PreviewPoint->InsertPoint(0, RenderablePoint);
	}

	// Otherwise, see if we are hovering an insertion on the end. This is always the case in interactive initialization mode, 
	// and requires having a point on the end selected in edit mode.
	else if (bInteractiveInitializationMode ||
		(bInsertPointToggle && SelectedPointIDs.Num() == 1 && (SelectedPointIDs[0] == ControlPoints.First() || SelectedPointIDs[0] == ControlPoints.Last())))
	{
		FVector3d HitPoint;
		if (!DrawPlane.RayPlaneIntersection((FVector3d)DevicePos.WorldRay.Origin, (FVector3d)DevicePos.WorldRay.Direction, 2, HitPoint))
		{
			// We're probably looking in an ortho viewport and missing the plane. End the hover.
			return false;
		}
		else
		{
			FColor LastSegmentColor = PreviewColor;

			// Snap the hitpoint if applicable
			if (bInteractiveInitializationMode || (bSnappingEnabled ^ bSnapToggle))
			{
				SnapEngine.UpdateSnappedPoint(HitPoint);
				if (SnapEngine.HaveActiveSnap())
				{
					// whether the next click would complete the path
					bool bEndingInitMode = (ControlPoints.Num() >= MinPointsForLoop && SnapEngine.GetActiveSnapTargetID() == FirstPointSnapID)
					                    || (ControlPoints.Num() >= MinPointsForNonLoop && SnapEngine.GetActiveSnapTargetID() == LastPointSnapID);

					if (bEndingInitMode)
					{
						LastSegmentColor = HighlightColor;
					}

					// We always snap to the start/end points because that's how we get out of initialization mode, and we don't want to
					// risk the user not knowing what to do if they set snapping to be disabled.
					if ((bSnappingEnabled ^ bSnapToggle) || bEndingInitMode)
					{
						HitPoint = SnapEngine.GetActiveSnapToPoint();
					}
					else
					{
						// If we're not snapping, we don't want to render the snap line
						SnapEngine.ResetActiveSnap();
					}
				}
			}

			// Redraw preview
			ClearHover();
			FRenderablePoint RenderablePoint((FVector)HitPoint, PreviewColor, PointsSize, DepthBias);
			PreviewPoint->InsertPoint(0, RenderablePoint);

			if (ControlPoints.Num() > 0)
			{
				int32 OriginPointID = bInteractiveInitializationMode ? ControlPoints.Last() : SelectedPointIDs[0];

				FRenderableLine RenderableLine((FVector)ControlPoints.GetPointCoordinates(OriginPointID),
					(FVector)HitPoint, LastSegmentColor, SegmentsThickness, DepthBias);
				PreviewSegment->InsertLine(0, RenderableLine);
			}
		}
	}

	// Otherwise, see if we're hovering a point for selection
	else if (!bInsertPointToggle && GeometrySet.FindNearestPointToRay((FRay3d)DevicePos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		// Only need to update the hover if we changed the point
		if (Nearest.ID != HoveredPointID)
		{
			ClearHover();
			HoveredPointID = Nearest.ID;
			PreHoverPointColor = DrawnControlPoints->GetPoint(HoveredPointID).Color;
			DrawnControlPoints->SetPointColor(HoveredPointID, HoverColor);
		}
	}
	else
	{
		// Not hovering anything, so done hovering
		return false;
	}

	return true;
}

void UCurveControlPointsMechanic::OnEndHover()
{
	ClearHover();
	SnapEngine.ResetActiveSnap();
}

// Detects Ctrl and Shift key states
void UCurveControlPointsMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == ShiftModifierId)
	{
		bAddToSelectionToggle = bIsOn;
		bSnapToggle = bIsOn;
	}
	else if (ModifierID == CtrlModifierId)
	{
		bInsertPointToggle = bIsOn;
		UpdateGizmoVisibility();
	}
}


void UCurveControlPointsMechanic::DeleteSelectedPoints()
{
	if (SelectedPointIDs.Num() == 0)
	{
		return;
	}

	ParentTool->GetToolManager()->BeginUndoTransaction(PointDeletionTransactionText);

	// There are minor inefficiencies in the way we delete multiple points since we sometimes do edge updates
	// for edges that get deleted later in the loop, and we upate the map inside ControlPoints each
	// time, but avoiding these would make the code more cumbersome.

	// We are deselecting, deleting, and (potentially) reverting back to interactive initialization. 
	// When undoing all of this, we want to add the points first, potentially switch modes second 
	// (now that we have enough points), and reselect third (now that the points exist). So, we want
	// to perform the operations in reverse order (deselect, switch modes, delete).

	// Deselect
	TArray<int32> PointsToDelete = SelectedPointIDs;
	ClearSelection();

	// Revert back to interactive intialization if needed.
	if (bAutoRevertToInteractiveInitialization)
	{
		int32 NumPointsRemaining = ControlPoints.Num() - PointsToDelete.Num();

		if ((bIsLoop && NumPointsRemaining < MinPointsForLoop)
			|| (!bIsLoop && NumPointsRemaining < MinPointsForNonLoop))
		{
			// We emit first just out of convenience of not having to temporarily store bIsLoop, 
			// which gets reset during the SetInteractiveIntialization(true) call.
			ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicModeChange>(
				false, bIsLoop, CurrentChangeStamp), InitializationCompletedTransactionText);

			SetInteractiveInitialization(true);
			OnModeChanged.Broadcast();
		}
	}

	// Delete
	for (int32 PointID : PointsToDelete)
	{
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FCurveControlPointsMechanicInsertionChange>(
			ControlPoints.GetSequencePosition(PointID), PointID, ControlPoints.GetPointCoordinates(PointID),
			false, CurrentChangeStamp), PointDeletionTransactionText);
		DeletePoint(PointID);
	}

	OnPointsChanged.Broadcast();

	ParentTool->GetToolManager()->EndUndoTransaction();
}

int32 UCurveControlPointsMechanic::DeletePoint(int32 PointID)
{
	int32 SequencePosition = ControlPoints.GetSequencePosition(PointID);

	// Deal with the segments:
	// See if there is a preceding point
	if (ControlPoints.Num() > 1 && (bIsLoop || SequencePosition > 0))
	{
		int32 PreviousPointID = ControlPoints.GetPointIDAt((SequencePosition + ControlPoints.Num() - 1) % ControlPoints.Num());

		// See if there is a point to connect to after the about-to-be-deleted one
		if (ControlPoints.Num() > 2 && (bIsLoop || SequencePosition < ControlPoints.Num() - 1))
		{
			// Move edge
			FVector3d NextPointCoordinates = ControlPoints.GetPointCoordinatesAt((SequencePosition + 1) % ControlPoints.Num());

			DrawnControlSegments->SetLineEnd(PreviousPointID, (FVector)NextPointCoordinates);
			FPolyline3d SegmentPolyline(ControlPoints.GetPointCoordinates(PreviousPointID), NextPointCoordinates);
			GeometrySet.UpdateCurve(PreviousPointID, SegmentPolyline);
		}
		else
		{
			// Delete edge
			GeometrySet.RemoveCurve(PreviousPointID);
			DrawnControlSegments->RemoveLine(PreviousPointID);
		}
	}

	// Delete outgoing edge if there is one.
	if (DrawnControlSegments->IsLineValid(PointID))
	{
		GeometrySet.RemoveCurve(PointID);
		DrawnControlSegments->RemoveLine(PointID);
	}

	// Delete the point itself.
	GeometrySet.RemovePoint(PointID);
	DrawnControlPoints->RemovePoint(PointID);
	ControlPoints.RemovePointAt(SequencePosition);

	SnapEngine.RemoveHistoryPoint(SequencePosition);

	// Even though initialization mode doesn't allow for explicit deletion, it could happen through undo.
	if (bInteractiveInitializationMode)
	{
		// Update start/end snap targets, and update snapping for hover if we deleted the last point.
		if (SequencePosition == ControlPoints.Num())
		{
			SnapEngine.RemovePointTargetsByID(LastPointSnapID);
			if (ControlPoints.Num() > 0)
			{
				SnapEngine.AddPointTarget(ControlPoints.GetPointCoordinates(ControlPoints.Last()), LastPointSnapID, EndpointSnapPriority);
			}

			UpdateSnapTargetsForHover();
		}
		if (SequencePosition == 0)
		{
			SnapEngine.RemovePointTargetsByID(FirstPointSnapID);
			if (ControlPoints.Num() > 0)
			{
				SnapEngine.AddPointTarget(ControlPoints.GetPointCoordinates(ControlPoints.First()), FirstPointSnapID, EndpointSnapPriority);
			}
		}
	}

	return PointID;
}


// ==================== Undo/redo object functions ====================

FCurveControlPointsMechanicSelectionChange::FCurveControlPointsMechanicSelectionChange(int32 PointIDIn, 
	bool AddedIn, int32 ChangeStampIn)
	: PointID(PointIDIn)
	, Added(AddedIn)
	, ChangeStamp(ChangeStampIn)
{}

void FCurveControlPointsMechanicSelectionChange::Apply(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	if (Added)
	{
		Mechanic->SelectPoint(PointID);
	}
	else
	{
		Mechanic->DeselectPoint(PointID);
	}
	Mechanic->UpdateGizmoLocation();
}

void FCurveControlPointsMechanicSelectionChange::Revert(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	if (Added)
	{
		Mechanic->DeselectPoint(PointID);
	}
	else
	{
		Mechanic->SelectPoint(PointID);
	}
	Mechanic->UpdateGizmoLocation();
}

FString FCurveControlPointsMechanicSelectionChange::ToString() const
{
	return TEXT("FCurveControlPointsMechanicSelectionChange");
}


FCurveControlPointsMechanicInsertionChange::FCurveControlPointsMechanicInsertionChange(int32 SequencePositionIn,
	int32 PointIDIn, const FVector3d& CoordinatesIn, bool AddedIn, int32 ChangeStampIn)
	: SequencePosition(SequencePositionIn)
	, PointID(PointIDIn)
	, Coordinates(CoordinatesIn)
	, Added(AddedIn)
	, ChangeStamp(ChangeStampIn)
{}

void FCurveControlPointsMechanicInsertionChange::Apply(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	if (Added)
	{
		Mechanic->InsertPointAt(SequencePosition, Coordinates, &PointID);
	}
	else
	{
		Mechanic->DeletePoint(PointID);
	}
	Mechanic->OnPointsChanged.Broadcast();
}

void FCurveControlPointsMechanicInsertionChange::Revert(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	if (Added)
	{
		Mechanic->DeletePoint(PointID);
	}
	else
	{
		Mechanic->InsertPointAt(SequencePosition, Coordinates, &PointID);
	}
	Mechanic->OnPointsChanged.Broadcast();
}

FString FCurveControlPointsMechanicInsertionChange::ToString() const
{
	return TEXT("FCurveControlPointsMechanicSelectionChange");
}

FCurveControlPointsMechanicModeChange::FCurveControlPointsMechanicModeChange(bool bDoneWithInitializationIn, bool bIsLoopIn, int32 ChangeStampIn)
	: bDoneWithInitialization(bDoneWithInitializationIn)
	, bIsLoop(bIsLoopIn)
	, ChangeStamp(ChangeStampIn)
{}

void FCurveControlPointsMechanicModeChange::Apply(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	if (bDoneWithInitialization)
	{
		Mechanic->SetInteractiveInitialization(false);
		Mechanic->SetIsLoop(bIsLoop);
	}
	else
	{
		Mechanic->SetInteractiveInitialization(true);
		Mechanic->SetIsLoop(false);
	}
	Mechanic->OnModeChanged.Broadcast();
}

void FCurveControlPointsMechanicModeChange::Revert(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	if (bDoneWithInitialization)
	{
		Mechanic->SetInteractiveInitialization(true);
		Mechanic->SetIsLoop(false);
	}
	else
	{
		Mechanic->SetInteractiveInitialization(false);
		Mechanic->SetIsLoop(bIsLoop);
	}
	Mechanic->OnModeChanged.Broadcast();
}

FString FCurveControlPointsMechanicModeChange::ToString() const
{
	return TEXT("FCurveControlPointsMechanicModeChange");
}


FCurveControlPointsMechanicMovementChange::FCurveControlPointsMechanicMovementChange(int32 PointIDIn,
	const FVector3d& OriginalPositionIn, const FVector3d& NewPositionIn, int32 ChangeStampIn)
	: PointID(PointIDIn)
	, OriginalPosition(OriginalPositionIn)
	, NewPosition(NewPositionIn)
	, ChangeStamp(ChangeStampIn)
{}

void FCurveControlPointsMechanicMovementChange::Apply(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	Mechanic->UpdatePointLocation(PointID, NewPosition);
	Mechanic->UpdateGizmoLocation();
	Mechanic->UpdateSnapHistoryPoint(Mechanic->ControlPoints.GetSequencePosition(PointID), NewPosition);
	Mechanic->UpdateSnapTargetsForHover(); // Only actually necessary if this was an end point, but easy enough to do
	Mechanic->OnPointsChanged.Broadcast();
}

void FCurveControlPointsMechanicMovementChange::Revert(UObject* Object)
{
	UCurveControlPointsMechanic* Mechanic = Cast<UCurveControlPointsMechanic>(Object);
	Mechanic->UpdatePointLocation(PointID, OriginalPosition);
	Mechanic->UpdateGizmoLocation();
	Mechanic->UpdateSnapHistoryPoint(Mechanic->ControlPoints.GetSequencePosition(PointID), OriginalPosition);
	Mechanic->UpdateSnapTargetsForHover();
	Mechanic->OnPointsChanged.Broadcast();
}

FString FCurveControlPointsMechanicMovementChange::ToString() const
{
	return TEXT("FCurveControlPointsMechanicMovementChange");
}




// ==================== FOrderedPoints functions ====================

UCurveControlPointsMechanic::FOrderedPoints::FOrderedPoints(const FOrderedPoints& ToCopy)
	: Vertices(ToCopy.Vertices)
	, Sequence(ToCopy.Sequence)
{
}

UCurveControlPointsMechanic::FOrderedPoints::FOrderedPoints(const TArray<FVector3d>& PointSequence)
{
	ReInitialize(PointSequence);
}

int32 UCurveControlPointsMechanic::FOrderedPoints::AppendPoint(const FVector3d& PointCoordinates)
{
	int32 PointID = Vertices.Add(PointCoordinates);
	Sequence.Add(PointID);
	PointIDToSequencePosition.Add(PointID, Sequence.Num()-1);
	return PointID;
}

int32 UCurveControlPointsMechanic::FOrderedPoints::InsertPointAt(int32 SequencePosition, 
	const FVector3d& VertCoordinates, const int32* KnownPointID)
{
	// Everything from this point onward moves further in the sequence, so update map
	for (int32 i = SequencePosition; i < Sequence.Num(); ++i)
	{
		++PointIDToSequencePosition[Sequence[i]];
	}

	int32 PointID;
	if (KnownPointID)
	{
		Vertices.Insert(*KnownPointID, VertCoordinates);
		PointID = *KnownPointID;
	}
	else
	{
		PointID = Vertices.Add(VertCoordinates);
	}

	Sequence.Insert(PointID, SequencePosition);
	PointIDToSequencePosition.Add(PointID, SequencePosition);
	return PointID;
}

int32 UCurveControlPointsMechanic::FOrderedPoints::RemovePointAt(int32 SequencePosition)
{
	check(SequencePosition >= 0 && SequencePosition < Sequence.Num());

	// Everything past this point moves back in sequence, so update map
	for (int32 i = SequencePosition + 1; i < Sequence.Num(); ++i)
	{
		--PointIDToSequencePosition[Sequence[i]];
	}

	int32 PointID = Sequence[SequencePosition];
	Vertices.RemoveAt(PointID);
	Sequence.RemoveAt(SequencePosition);
	PointIDToSequencePosition.Remove(PointID);

	return PointID;
}

void UCurveControlPointsMechanic::FOrderedPoints::Empty()
{
	Vertices.Empty();
	Sequence.Empty();
	PointIDToSequencePosition.Empty();
}

void UCurveControlPointsMechanic::FOrderedPoints::ReInitialize(const TArray<FVector3d>& PointSequence)
{
	Empty();

	Vertices.Reserve(PointSequence.Num());
	Sequence.Reserve(PointSequence.Num());
	PointIDToSequencePosition.Reserve(PointSequence.Num());
	for (int i = 0; i < PointSequence.Num(); ++i)
	{
		int32 PointID = Vertices.Add(PointSequence[i]);
		Sequence.Add(PointID);
		PointIDToSequencePosition.Add(PointID, i);
	}
}

#undef LOCTEXT_NAMESPACE

