// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/DragAlignmentMechanic.h"

#include "BaseBehaviors/KeyAsModifierInputBehavior.h"
#include "BaseGizmos/IntervalGizmo.h"
#include "BaseGizmos/RepositionableTransformGizmo.h"
#include "BaseGizmos/CombinedTransformGizmo.h"
#include "BaseGizmos/TransformProxy.h"
#include "Components/PrimitiveComponent.h"
#include "Operations/GroupTopologyDeformer.h"
#include "PrimitiveDrawingUtils.h" //FPrimitiveDrawInterface
#include "ToolDataVisualizer.h"
#include "ToolSceneQueriesUtil.h"
#include "SceneQueries/SceneSnappingManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DragAlignmentMechanic)

using namespace UE::Geometry;


void FDragAlignmentBase::AddToGizmo(UCombinedTransformGizmo* TransformGizmo, const TArray<const UPrimitiveComponent*>* ComponentsToIgnoreInAlignment, 
	const TArray<const UPrimitiveComponent*>* InvisibleComponentsToIncludeInAlignment)
{
	if (!ensureMsgf(TransformGizmo && TransformGizmo->ActiveTarget, TEXT("Drag Alignment AddToGizmo method must "
		"be called on a gizmo that is already initialized.")))
	{
		return;
	}

	// We need a copy of the components to ignore, potentially to be shared across multiple lambdas. For pivot
	//  adjustment, the ignored components will be just those of the gizmo itself, but for regular gizmo operation
	//  we add ComponentsToIgnoreInAlignment.
	TSharedPtr<TArray<const UPrimitiveComponent*>> GizmoComponentsToIgnore = MakeShared<TArray<const UPrimitiveComponent*>>();
	if (ensure(TransformGizmo->GetGizmoActor()))
	{
		TransformGizmo->GetGizmoActor()->GetComponents<const UPrimitiveComponent*>(*GizmoComponentsToIgnore);
	}

	TSharedPtr<TArray<const UPrimitiveComponent*>> CombinedComponentsToIgnore = GizmoComponentsToIgnore;
	if (ComponentsToIgnoreInAlignment)
	{
		// If we had some extra components to ignore, we create a separate combined array so that
		//  GizmoComponentsToIgnore continues to only include gizmo components for pivot alignment.
		CombinedComponentsToIgnore = MakeShared<TArray<const UPrimitiveComponent*>>(*ComponentsToIgnoreInAlignment);
		CombinedComponentsToIgnore->Append(*GizmoComponentsToIgnore);
	}

	// Similarly, create an array to hold the extra component to include in raycasting.
	TSharedPtr<TArray<const UPrimitiveComponent*>> ComponentsToInclude;
	if (InvisibleComponentsToIncludeInAlignment)
	{
		ComponentsToInclude = MakeShared<TArray<const UPrimitiveComponent*>>(*InvisibleComponentsToIncludeInAlignment);
	}


	// Set the alignment functions.
	TransformGizmo->SetWorldAlignmentFunctions(
		[this]() { return GetAlignmentModeEnabled(); },
		[this, CombinedComponentsToIgnore, ComponentsToInclude](const FRay& WorldRay, FVector& OutputPoint) {
			return CastRay(WorldRay, OutputPoint, CombinedComponentsToIgnore.Get(), ComponentsToInclude.Get(), true);
		});

	// If we're adding to a repositionable gizmo, we probably don't want to ignore components other
	// than the gizmo components themselves, since ignoring components is mainly used to avoid hitting
	// the moved object, which we actually do want to hit in moving moving the pivot.
	// If we someday do want to ignore things in a pivot reposition function, we'll have to
	// make this caller-configurable somehow.
	if (URepositionableTransformGizmo* RepositionableGizmo = Cast<URepositionableTransformGizmo>(TransformGizmo))
	{
		RepositionableGizmo->SetPivotAlignmentFunctions(
			[this]() { return GetAlignmentModeEnabled(); },
			[this, GizmoComponentsToIgnore, ComponentsToInclude](const FRay& WorldRay, FVector& OutputPoint) {
				return CastRay(WorldRay, OutputPoint, GizmoComponentsToIgnore.Get(), ComponentsToInclude.Get(), /*bUseFilter*/ false);
			});
	}

	// Register listeners to help with rendering. They get us the location that the gizmo ended
	// up being placed, so that we can draw a line from the hit point to that location.
	TransformGizmo->ActiveTarget->OnTransformChanged.AddWeakLambda(GetUObjectContainer(), [this](UTransformProxy*, FTransform NewTransform) {
		OnGizmoTransformChanged(NewTransform);
	});
	TransformGizmo->ActiveTarget->OnPivotChanged.AddWeakLambda(GetUObjectContainer(), [this](UTransformProxy*, FTransform NewTransform) {
		OnGizmoTransformChanged(NewTransform);
	});

	// Register a listener to stop drawing once the gizmo is done moving.
	TransformGizmo->ActiveTarget->OnEndTransformEdit.AddWeakLambda(GetUObjectContainer(), [this](UTransformProxy*) { 
		bPreviewEndpointsValid = false; 
		bWaitingOnProjectedResult = false;
	});
	TransformGizmo->ActiveTarget->OnEndPivotEdit.AddWeakLambda(GetUObjectContainer(), [this](UTransformProxy*) {
		bPreviewEndpointsValid = false;
		bWaitingOnProjectedResult = false;
		});
}

void FDragAlignmentBase::AddToGizmo(UIntervalGizmo* IntervalGizmo, const TArray<const UPrimitiveComponent*>* ComponentsToIgnoreInAlignment,
	const TArray<const UPrimitiveComponent*>* InvisibleComponentsToIncludeInAlignment)
{
	// If we have components to ignore/include, we need a copy of the array so that the alignment
	// functions can use them later.
	TSharedPtr<TArray<const UPrimitiveComponent*>> ComponentsToIgnorePersistent;
	if (ComponentsToIgnoreInAlignment)
	{
		ComponentsToIgnorePersistent = MakeShared<TArray<const UPrimitiveComponent*>>(*ComponentsToIgnoreInAlignment);
	}
	TSharedPtr<TArray<const UPrimitiveComponent*>> ComponentsToIncludePersistent;
	if (InvisibleComponentsToIncludeInAlignment)
	{
		ComponentsToIncludePersistent = MakeShared<TArray<const UPrimitiveComponent*>>(*InvisibleComponentsToIncludeInAlignment);
	}

	// Set the alignment functions.
	IntervalGizmo->SetWorldAlignmentFunctions(
		[this]() { return GetAlignmentModeEnabled(); },
		[this, ComponentsToIgnorePersistent, ComponentsToIncludePersistent](const FRay& WorldRay, FVector& OutputPoint) {
			return CastRay(WorldRay, OutputPoint, ComponentsToIgnorePersistent.Get(), ComponentsToIncludePersistent.Get(), true);
		});

	// Register listener to help with rendering. It gets us the final location of the interval endpoint
	// so that we can draw a line from the hit point to that location.
	IntervalGizmo->OnIntervalChanged.AddWeakLambda(GetUObjectContainer(), [this](UIntervalGizmo* Gizmo, FVector Direction, float NewParam) {
		FTransform Transform = Gizmo->GetGizmoTransform();
		Transform.AddToTranslation(Transform.Rotator().RotateVector(Direction) * NewParam);
		OnGizmoTransformChanged(Transform);
		});

	// Register a listener to stop drawing once the gizmo is done moving.
	IntervalGizmo->OnEndIntervalGizmoEdit.AddWeakLambda(GetUObjectContainer(), [this](UIntervalGizmo*) {
		bPreviewEndpointsValid = false;
		bWaitingOnProjectedResult = false;
		});
}

void FDragAlignmentBase::OnGizmoTransformChanged(FTransform NewTransform)
{
	// Ignore changes that weren't preceded by a successful query (likely programatic)
	if (bWaitingOnProjectedResult) 
	{
		LastProjectedResult = (FVector3d)NewTransform.GetLocation();
		bPreviewEndpointsValid = true;
	}
	bWaitingOnProjectedResult = false;
}

void FDragAlignmentBase::InitializeDeformedMeshRayCast(TFunction<FDynamicMeshAABBTree3* ()> GetSpatialIn, 
	const FTransform3d &TargetTransform, const FGroupTopologyDeformer* LinearDeformer)
{
	if (LinearDeformer)
	{
		// Use the deformer to create a way to filter out triangles we don't want to be hitting (the ones we're manipulating)
		const FDynamicMesh3* Mesh = LinearDeformer->GetMesh();
		MeshRayCastTriangleFilter = [Mesh, LinearDeformer](int32 Tid)
		{
			FIndex3i Vids = Mesh->GetTriangle(Tid);
			for (int32 i = 0; i < 3; ++i)
			{
				if (LinearDeformer->GetModifiedVertices().Contains(Vids[i]))
				{
					return false;
				}
			}
			return true;
		};
	}

	// Set up the actual function that casts rays into the mesh.
	MeshRayCast = [this, GetSpatialIn, TargetTransform](const FRay& WorldRay, FHitResult& HitResult, bool bUseFilter)
	{
		// Transform ray into local space
		FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
			TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
		UE::Geometry::Normalize(LocalRay.Direction);

		FDynamicMeshAABBTree3* Spatial = GetSpatialIn();
		const FDynamicMesh3* Mesh = Spatial->GetMesh();

		// Do the actual ray cast. Note that if we're using the filter, then we're presumably deforming the
		// mesh, and the aabb tree will not be updated. In that case, we want to tell it to not perform
		// that check, since we'll be relying on the filter to make sure that we're not hitting anything
		// invalid.
		double RayTValue;
		int32 HitTid;
		IMeshSpatial::FQueryOptions Options;
		if (bUseFilter)
		{
			Options.TriangleFilterF = [this](int32 Tid) { return MeshRayCastTriangleFilter(Tid); };
			Options.bAllowUnsafeModifiedMeshQueries = true;
		}
		if (!Spatial->FindNearestHitTriangle(LocalRay, RayTValue, HitTid, Options))
		{
			return false;
		}

		// See if we're close enough to one of the triangle's vertices to snap to it. We have
		// to do this check in world space since scaling will affect things.
		FVector3d WorldHitPoint = (FVector3d)WorldRay.PointAt(RayTValue);

		FVector3d Vert1, Vert2, Vert3;
		Mesh->GetTriVertices(HitTid, Vert1, Vert2, Vert3);
		Vert1 = TargetTransform.TransformPosition(Vert1);
		Vert2 = TargetTransform.TransformPosition(Vert2);
		Vert3 = TargetTransform.TransformPosition(Vert3);

		FVector3d* ClosestVert = &Vert1;
		double MinDistSquared = DistanceSquared(WorldHitPoint, Vert1);
		double DistSquared = DistanceSquared(WorldHitPoint, Vert2);
		if (DistSquared < MinDistSquared)
		{
			ClosestVert = &Vert2;
			MinDistSquared = DistSquared;
		}
		DistSquared = DistanceSquared(WorldHitPoint, Vert3);
		if (DistSquared < MinDistSquared)
		{
			ClosestVert = &Vert3;
		}

		// Check if we're close enough to snap.
		if (ToolSceneQueriesUtil::PointSnapQuery(CachedCameraState, WorldHitPoint, *ClosestVert, VisualAngleSnapThreshold))
		{
			HitResult = FHitResult(WorldRay.Origin, (FVector)*ClosestVert);
			HitResult.ImpactPoint = (FVector)*ClosestVert;
			HitResult.Item = VERT_HIT_TYPE_ID;
		}
		else
		{
			HitResult = FHitResult(WorldRay.Origin, (FVector)WorldHitPoint);
			HitResult.ImpactPoint = (FVector)WorldHitPoint;
			HitResult.Item = TRIANGLE_HIT_TYPE_ID;
		}
		HitResult.Distance = WorldRay.GetParameter(HitResult.ImpactPoint);
		HitResult.ImpactNormal = (FVector)Mesh->GetTriNormal(HitTid);

		return true;
	};
}

/**
 * Casts a ray into the scene to find an alignment point. Used in the functions bound
 * to gizmos.
 *
 * @param bUseFilter If true, and if there is a dynamic mesh to hit, its triangles are
 *  filtered to avoid hitting deformed triangles.
 * @return true if something was hit.
 */
bool FDragAlignmentBase::CastRay(const FRay& WorldRay, FVector& OutputPoint,
	const TArray<const UPrimitiveComponent*>* ComponentsToIgnore, 
	const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude,
	bool bUseFilter)
{
	bool bHit = false;
	FHitResult MeshHitResult;
	if (MeshRayCast && MeshRayCast(WorldRay, MeshHitResult, bUseFilter))
	{
		bHit = true;
		LastHitResult = MeshHitResult;
		OutputPoint = LastHitResult.ImpactPoint;
	}

	FHitResult WorldHitResult;
	if (WorldRayCast 
		&& WorldRayCast(WorldRay, WorldHitResult, ComponentsToIgnore, InvisibleComponentsToInclude) 
		&& (!bHit || WorldHitResult.Distance < MeshHitResult.Distance))
	{
		bHit = true;
		LastHitResult = WorldHitResult;
		OutputPoint = LastHitResult.ImpactPoint;
	}

	// We keep track of the last position we served the user, and the position
	// that they moved a transform. When we have both, we can render a visualization
	// connecting the two. Right now we (may) have the first of the two.
	bWaitingOnProjectedResult = bHit;
	bPreviewEndpointsValid = false;
	return bHit;
}






void FDragAlignmentBase::SetupInternal()
{
	// Set up the function that casts rays into the world.
	WorldRayCast = [this](const FRay& WorldRay, FHitResult& HitResult, 
		const TArray<const UPrimitiveComponent*>* ComponentsToIgnore,
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)
	{
		bool bSuccess = false;
		FVector3d QueryPoint = (FVector3d)WorldRay.PointAt(1);
		FVector3d SnapPoint;
		ToolSceneQueriesUtil::FSnapGeometry SnapGeometry;

		// Try to snap to visible vertices/edges first
		ToolSceneQueriesUtil::FFindSceneSnapPointParams SnapParams;
		SnapParams.SnappingManager = GetSnappingManager();
		SnapParams.CameraState = CachedCameraState;
		SnapParams.Point = &QueryPoint;
		SnapParams.SnapPointOut = &SnapPoint;
		SnapParams.bEdges = true;
		SnapParams.bVertices = true;
		SnapParams.VisualAngleThreshold = VisualAngleSnapThreshold;
		SnapParams.SnapGeometryOut = &SnapGeometry;
		SnapParams.ComponentsToIgnore = ComponentsToIgnore;
		SnapParams.InvisibleComponentsToInclude = InvisibleComponentsToInclude;

		if (ToolSceneQueriesUtil::FindSceneSnapPoint(SnapParams))
		{
			HitResult = FHitResult(WorldRay.Origin, (FVector)SnapPoint);
			HitResult.ImpactPoint = (FVector)SnapPoint;
			HitResult.Distance = WorldRay.GetParameter(HitResult.ImpactPoint);
			HitResult.Item = SnapGeometry.PointCount == 1 ? VERT_HIT_TYPE_ID : EDGE_HIT_TYPE_ID;
			bSuccess = true;
		}
		// If we failed, try to hit a simple triangle
		else if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(GetSnappingManager(), HitResult, WorldRay, ComponentsToIgnore))
		{
			HitResult.Item = TRIANGLE_HIT_TYPE_ID;
			bSuccess = true;
		}

		return bSuccess;
	};
}


void FDragAlignmentBase::RenderInternal(IToolsContextRenderAPI* RenderAPI)
{
	// Cache the camera state
	CachedCameraState = GetCameraState();

	if (!bPreviewEndpointsValid)
	{
		return;
	}

	FColor Color = FColor::Orange;
	FViewCameraState CameraState = RenderAPI->GetCameraState();
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

	float PDIScale = CameraState.GetPDIScalingFactor();
	double VisualAngleRenderThreshold = 0.5;

	FToolDataVisualizer Renderer;
	Renderer.BeginFrame(RenderAPI, CameraState);

	if (LastHitResult.Item == TRIANGLE_HIT_TYPE_ID)
	{
		// We'll draw a circle with an X in it lying on the face we hit.
		FFrame3d HitFrame = FFrame3d((FVector3d)LastHitResult.ImpactPoint, (FVector3d)LastHitResult.ImpactNormal);

		float TickLength = (float)ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, HitFrame.Origin, 0.8);
		float TickThickness = 4 * PDIScale;

		Renderer.DrawLine(
			HitFrame.PointAt(-TickLength, -TickLength, 0),
			HitFrame.PointAt(TickLength, TickLength, 0), Color, TickThickness, false);
		Renderer.DrawLine(
			HitFrame.PointAt(-TickLength, TickLength, 0),
			HitFrame.PointAt(TickLength, -TickLength, 0), Color, TickThickness, false);

		Renderer.DrawCircle(
			HitFrame.Origin, HitFrame.Z(), 2.0 * TickLength, 8, Color, 1.0, false);
	}
	else
	{
		PDI->DrawPoint(LastHitResult.ImpactPoint, Color, 10.0f * PDIScale, SDPG_Foreground);
	}

	// If the two endpoints are far enough apart, draw a line connecting them.
	if (ToolSceneQueriesUtil::CalculateNormalizedViewVisualAngleD(
		CameraState, (FVector3d)LastHitResult.ImpactPoint, LastProjectedResult) > VisualAngleRenderThreshold)
	{
		PDI->DrawLine(LastHitResult.ImpactPoint, (FVector)LastProjectedResult, Color,
			SDPG_Foreground, 0.5f * PDIScale, 0.0f, true);
	}

	Renderer.EndFrame();
}




void UDragAlignmentMechanic::Setup(UInteractiveTool* ParentToolIn)
{	
	UInteractionMechanic::Setup(ParentToolIn);

	// Register modifier listeners.
	UKeyAsModifierInputBehavior* AlignmentToggleBehavior = NewObject<UKeyAsModifierInputBehavior>();
	AlignmentToggleBehavior->Initialize(this, AlignmentModifierID, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(AlignmentToggleBehavior);

	SetupInternal();
}

void UDragAlignmentMechanic::Shutdown()
{
}


void UDragAlignmentMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	RenderInternal(RenderAPI);
}

USceneSnappingManager* UDragAlignmentMechanic::GetSnappingManager()
{
	return USceneSnappingManager::Find(GetParentTool()->GetToolManager());
}

FViewCameraState UDragAlignmentMechanic::GetCameraState()
{
	FViewCameraState CurState;
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CurState);
	return CurState;
}

void UDragAlignmentMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == AlignmentModifierID)
	{
		bAlignmentToggle = bIsOn;
	}
	if (!bAlignmentToggle)
	{
		bPreviewEndpointsValid = false;
	}
}




void UDragAlignmentInteraction::Setup(USceneSnappingManager* SnappingManager)
{
	this->SceneSnappingManager = SnappingManager;
	SetupInternal();
}

void UDragAlignmentInteraction::RegisterAsBehaviorTarget(UKeyAsModifierInputBehavior* KeyModifierBehavior)
{
	KeyModifierBehavior->Initialize(this, AlignmentModifierID, FInputDeviceState::IsCtrlKeyDown);
}


void UDragAlignmentInteraction::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == AlignmentModifierID)
	{
		bAlignmentToggle = bIsOn;
	}
	if (!bAlignmentToggle)
	{
		bPreviewEndpointsValid = false;
	}
}
void UDragAlignmentInteraction::Render(IToolsContextRenderAPI* RenderAPI)
{
	LastRenderCameraState = RenderAPI->GetCameraState();
	RenderInternal(RenderAPI);
}
