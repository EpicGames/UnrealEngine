// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/LatticeControlPointsMechanic.h"

#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "Transforms/MultiTransformer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LatticeControlPointsMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "ULatticeControlPointsMechanic"

static const FText LatticePointDeselectionTransactionText = LOCTEXT("LatticePointDeselection", "Lattice Point Deselection");
static const FText LatticePointSelectionTransactionText = LOCTEXT("LatticePointSelection", "Lattice Point Selection");
static const FText LatticePointMovementTransactionText = LOCTEXT("LatticePointMovement", "Lattice Point Movement");

namespace LatticeControlPointMechanicLocals
{
	void Toggle(TSet<int>& Selection, const TSet<int>& NewSelection)
	{
		for (const int& NewElement : NewSelection)
		{
			if (Selection.Remove(NewElement) == 0)
			{
				Selection.Add(NewElement);
			}
		}
	}

	// TODO Move to SetUtil namespace and call it in FGroupTopologySelection::operator==
	bool IsEqual(const TSet<int32>& A, const TSet<int32>& B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}

		// No need to also check B.Includes(A) here since counts match and sets contain unique elements
		return A.Includes(B);
	}
}


void ULatticeControlPointsMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	MarqueeMechanic = NewObject<URectangleMarqueeMechanic>(this);
	MarqueeMechanic->bUseExternalClickDragBehavior = true;
	MarqueeMechanic->Setup(ParentToolIn);
	MarqueeMechanic->OnDragRectangleStarted.AddUObject(this, &ULatticeControlPointsMechanic::OnDragRectangleStarted);
	MarqueeMechanic->OnDragRectangleChanged.AddUObject(this, &ULatticeControlPointsMechanic::OnDragRectangleChanged);
	MarqueeMechanic->OnDragRectangleFinished.AddUObject(this, &ULatticeControlPointsMechanic::OnDragRectangleFinished);

	USingleClickOrDragInputBehavior* ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, MarqueeMechanic);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(ClickOrDragBehavior);

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior);

	DrawnControlPoints = NewObject<UPointSetComponent>();
	DrawnControlPoints->SetPointMaterial(
		ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ true));
	DrawnLatticeEdges = NewObject<ULineSetComponent>();
	DrawnLatticeEdges->SetLineMaterial(
		ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ true));

	NormalPointColor = FColor::Red;
	NormalSegmentColor = FColor::Red;
	HoverColor = FColor::Green;
	SelectedColor = FColor::Yellow;
	SegmentsThickness = 1.0f;
	PointsSize = 6.0f;

	GeometrySetToleranceTest = [this](const FVector3d& Position1, const FVector3d& Position2) {
		if (CachedCameraState.bIsOrthographic)
		{
			// We could just always use ToolSceneQueriesUtil::PointSnapQuery. But in ortho viewports, we happen to know
			// that the only points that we will ever give this function will be the closest points between a ray and
			// some geometry, meaning that the vector between them will be orthogonal to the view ray. With this knowledge,
			// we can do the tolerance computation more efficiently than PointSnapQuery can, since we don't need to project
			// down to the view plane.
			// As in PointSnapQuery, we convert our angle-based tolerance to one we can use in an ortho viewport (instead of
			// dividing our field of view into 90 visual angle degrees, we divide the plane into 90 units).
			float OrthoTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * CachedCameraState.OrthoWorldCoordinateWidth / 90.0;
			return DistanceSquared(Position1, Position2) < OrthoTolerance * OrthoTolerance;
		}
		else
		{
			return ToolSceneQueriesUtil::PointSnapQuery(CachedCameraState, Position1, Position2);
		}
	};

	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	PointTransformProxy = NewObject<UTransformProxy>(this);
	
	// TODO: Maybe don't have the gizmo's axes flip around when it crosses the origin, if possible?
	PointTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager,
		ETransformGizmoSubElements::FullTranslateRotateScale, this);

	PointTransformProxy->OnTransformChanged.AddUObject(this, &ULatticeControlPointsMechanic::GizmoTransformChanged);
	PointTransformProxy->OnBeginTransformEdit.AddUObject(this, &ULatticeControlPointsMechanic::GizmoTransformStarted);
	PointTransformProxy->OnEndTransformEdit.AddUObject(this, &ULatticeControlPointsMechanic::GizmoTransformEnded);
	PointTransformGizmo->SetActiveTarget(PointTransformProxy);
	PointTransformGizmo->SetVisibility(false);
	PointTransformGizmo->bUseContextCoordinateSystem = false;
	PointTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;
}

void ULatticeControlPointsMechanic::SetCoordinateSystem(EToolContextCoordinateSystem InCoordinateSystem)
{
	PointTransformGizmo->CurrentCoordinateSystem = InCoordinateSystem;
	UpdateGizmoLocation();
}

EToolContextCoordinateSystem ULatticeControlPointsMechanic::GetCoordinateSystem() const
{
	return PointTransformGizmo->CurrentCoordinateSystem;
}


void ULatticeControlPointsMechanic::UpdateSetPivotMode(bool bInSetPivotMode)
{
	PointTransformProxy->bSetPivotMode = bInSetPivotMode;
}


void ULatticeControlPointsMechanic::Initialize(const TArray<FVector3d>& Points, 
											   const TArray<FVector2i>& Edges,
											   const FTransform3d& InLocalToWorldTransform)
{
	LocalToWorldTransform = InLocalToWorldTransform;
	ControlPoints = Points;
	SelectedPointIDs.Empty();
	LatticeEdges = Edges;
	UpdateGizmoLocation();
	RebuildDrawables();
	++CurrentChangeStamp;		// If the lattice is potentially changing resolution, make this an undo barrier
	bHasChanged = false;
}

void ULatticeControlPointsMechanic::SetWorld(UWorld* World)
{
	// It may be unreasonable to worry about SetWorld being called more than once, but let's be safe anyway
	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
	} 

	// We need the world so we can create the geometry actor in the right place
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	PreviewGeometryActor = World->SpawnActor<APreviewGeometryActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	// Attach the rendering components to the actor
	DrawnControlPoints->Rename(nullptr, PreviewGeometryActor);
	PreviewGeometryActor->SetRootComponent(DrawnControlPoints);
	if (DrawnControlPoints->IsRegistered())
	{
		DrawnControlPoints->ReregisterComponent();
	}
	else
	{
		DrawnControlPoints->RegisterComponent();
	}

	DrawnLatticeEdges->Rename(nullptr, PreviewGeometryActor); 
	DrawnLatticeEdges->AttachToComponent(DrawnControlPoints, FAttachmentTransformRules::KeepWorldTransform);
	if (DrawnLatticeEdges->IsRegistered())
	{
		DrawnLatticeEdges->ReregisterComponent();
	}
	else
	{
		DrawnLatticeEdges->RegisterComponent();
	}
}

void ULatticeControlPointsMechanic::Shutdown()
{
	LongTransactions.CloseAll(GetParentTool()->GetToolManager());

	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
		PreviewGeometryActor = nullptr;
	}

	// Calls shutdown on gizmo and destroys it.
	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	GizmoManager->DestroyAllGizmosByOwner(this);
}


void ULatticeControlPointsMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	// Cache the camera state
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CachedCameraState);

	MarqueeMechanic->Render(RenderAPI);
}

void ULatticeControlPointsMechanic::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	MarqueeMechanic->DrawHUD(Canvas, RenderAPI);
}


void ULatticeControlPointsMechanic::RebuildDrawables()
{
	DrawnControlPoints->Clear();
	GeometrySet.Reset();
	for (int32 PointIndex = 0; PointIndex < ControlPoints.Num(); ++PointIndex)
	{
		const FVector3d& P = ControlPoints[PointIndex];
		DrawnControlPoints->InsertPoint(PointIndex, FRenderablePoint(static_cast<FVector>(P), NormalPointColor, PointsSize));
		GeometrySet.AddPoint(PointIndex, P);
	}

	for (int32 PointID : SelectedPointIDs)
	{
		if (DrawnControlPoints->IsPointValid(PointID))
		{
			DrawnControlPoints->SetPointColor(PointID, SelectedColor);
		}
	}

	for (const TPair<int32, FColor>& Override : ColorOverrides)
	{
		DrawnControlPoints->SetPointColor(Override.Key, Override.Value);
	}

	DrawnLatticeEdges->Clear();
	for (int32 EdgeIndex = 0; EdgeIndex < LatticeEdges.Num(); ++EdgeIndex)
	{
		const FVector3d& A = ControlPoints[LatticeEdges[EdgeIndex].X];
		const FVector3d& B = ControlPoints[LatticeEdges[EdgeIndex].Y];
		int32 SegmentID = DrawnLatticeEdges->AddLine(FVector(A), FVector(B), NormalSegmentColor, SegmentsThickness);
		check(SegmentID == EdgeIndex);
	}
}


void ULatticeControlPointsMechanic::UpdateDrawables()
{
	UpdateGizmoVisibility();

	for (int32 PointIndex = 0; PointIndex < ControlPoints.Num(); ++PointIndex)
	{
		const FVector3d& P = ControlPoints[PointIndex];
		DrawnControlPoints->SetPointPosition(PointIndex, static_cast<FVector>(P));
		DrawnControlPoints->SetPointColor(PointIndex, NormalPointColor);

		GeometrySet.UpdatePoint(PointIndex, P);
	}

	for (int32 PointID : SelectedPointIDs)
	{
		if (DrawnControlPoints->IsPointValid(PointID))
		{
			DrawnControlPoints->SetPointColor(PointID, SelectedColor);
		}
	}

	for (const TPair<int32, FColor>& Override : ColorOverrides)
	{
		DrawnControlPoints->SetPointColor(Override.Key, Override.Value);
	}

	for (int32 EdgeIndex = 0; EdgeIndex < LatticeEdges.Num(); ++EdgeIndex)
	{
		const FVector3d& A = ControlPoints[LatticeEdges[EdgeIndex].X];
		const FVector3d& B = ControlPoints[LatticeEdges[EdgeIndex].Y];
		DrawnLatticeEdges->SetLineStart(EdgeIndex, FVector(A));
		DrawnLatticeEdges->SetLineEnd(EdgeIndex, FVector(B));
	}
}


void ULatticeControlPointsMechanic::GizmoTransformStarted(UTransformProxy* Proxy)
{
	ParentTool->GetToolManager()->BeginUndoTransaction(LatticePointMovementTransactionText);

	GizmoStartPosition = Proxy->GetTransform().GetTranslation();
	GizmoStartRotation = Proxy->GetTransform().GetRotation();
	GizmoStartScale = Proxy->GetTransform().GetScale3D();

	SelectedPointStartPositions.Empty();
	for (int32 PointID : SelectedPointIDs)
	{
		SelectedPointStartPositions.Add(PointID, ControlPoints[PointID]);
	}

	bGizmoBeingDragged = true;
}

void ULatticeControlPointsMechanic::GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	if (SelectedPointIDs.Num() == 0 || !bGizmoBeingDragged)
	{
		return;
	}

	if (PointTransformProxy->bSetPivotMode)
	{
		return;
	}

	FQuaterniond DeltaRotation = FQuaterniond(Transform.GetRotation() * GizmoStartRotation.Inverse());
	FVector DeltaScale = Transform.GetScale3D() / GizmoStartScale;

	FTransformSRT3d DeltaTransform;
	DeltaTransform.SetRotation((FQuaterniond)DeltaRotation);
	DeltaTransform.SetTranslation((FVector3d)Transform.GetTranslation());

	for (int32 PointID : SelectedPointIDs)
	{
		FVector3d PointPosition = SelectedPointStartPositions[PointID];

		// Translate to origin, scale, rotate, and translate back (DeltaTransform has "translate back" baked in.)
		PointPosition -= (FVector3d)GizmoStartPosition;
		// Align the scale to the gizmo orientation
		PointPosition = GizmoStartRotation * (DeltaScale * (GizmoStartRotation.Inverse() * PointPosition));
		PointPosition = DeltaTransform.TransformPosition(PointPosition);

		ControlPoints[PointID] = PointPosition;
	}

	UpdateDrawables();

	OnPointsChanged.Broadcast();
}

void ULatticeControlPointsMechanic::GizmoTransformEnded(UTransformProxy* Proxy)
{
	TMap<int32, FVector3d> SelectedPointNewPositions;
	for (int32 PointID : SelectedPointIDs)
	{
		SelectedPointNewPositions.Add(PointID, ControlPoints[PointID]);
	}

	bool bFirstMovement = !bHasChanged;
	bHasChanged = true;

	ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeControlPointsMechanicMovementChange>(
		SelectedPointStartPositions, SelectedPointNewPositions,
		CurrentChangeStamp, bFirstMovement), LatticePointMovementTransactionText);

	SelectedPointStartPositions.Reset();

	// TODO: When we implement snapping
	// We may need to reset the gizmo if our snapping caused the final point position to differ from the gizmo position
	// UpdateGizmoLocation();

	ParentTool->GetToolManager()->EndUndoTransaction();		// was started in GizmoTransformStarted above

	// This just lets the tool know that the gizmo has finished moving and we've added it to the undo stack.
	// TODO: Add a different callback? "OnGizmoTransformChanged"?
	OnPointsChanged.Broadcast();

	bGizmoBeingDragged = false;
}

void ULatticeControlPointsMechanic::UpdatePointLocations(const TMap<int32, FVector3d>& NewLocations)
{
	for ( const TPair<int32, FVector3d>& NewLocationPair : NewLocations )
	{
		ControlPoints[NewLocationPair.Key] = NewLocationPair.Value;
	}
	UpdateDrawables();
}

void ULatticeControlPointsMechanic::UpdatePointLocations(const TArray<FVector3d>& NewLocations)
{
	if (ControlPoints.Num() == NewLocations.Num())
	{
		ControlPoints = NewLocations;
	}
	UpdateDrawables();
}

bool ULatticeControlPointsMechanic::HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut)
{
	FGeometrySet3::FNearest Nearest;

	// See if we hit a point for selection
	if (GeometrySet.FindNearestPointToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ResultOut = FInputRayHit(Nearest.RayParam);
		return true;
	}
	return false;
}

FInputRayHit ULatticeControlPointsMechanic::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Result;
	if (HitTest(ClickPos, Result))
	{
		return Result;
	}

	// Return a hit so we always capture and can clear the selection
	return FInputRayHit(TNumericLimits<float>::Max());
}

void ULatticeControlPointsMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	using namespace LatticeControlPointMechanicLocals;
	
	ParentTool->GetToolManager()->BeginUndoTransaction(LatticePointSelectionTransactionText);
	
	const TSet<int32> PreClickSelection = SelectedPointIDs;
	
	FGeometrySet3::FNearest Nearest;
	if (GeometrySet.FindNearestPointToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		if (ShouldAddToSelectionFunc())
		{
			if (ShouldRemoveFromSelectionFunc())
			{
				Toggle(SelectedPointIDs, TSet<int>{Nearest.ID});
			}
			else
			{
				SelectedPointIDs.Add(Nearest.ID);
			}
		}
		else if (ShouldRemoveFromSelectionFunc())
		{
			SelectedPointIDs.Remove(Nearest.ID);
		}
		else
		{
			// Neither key pressed.
			SelectedPointIDs = TSet<int>{Nearest.ID};
		}
	}
	else
	{
		SelectedPointIDs.Empty();
	}
	
	if (!IsEqual(PreClickSelection, SelectedPointIDs))
	{
		const FTransform Transform = PointTransformProxy->GetTransform();
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeControlPointsMechanicSelectionChange>(
			PreClickSelection, false, Transform, Transform, CurrentChangeStamp), LatticePointDeselectionTransactionText);
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeControlPointsMechanicSelectionChange>(
			SelectedPointIDs, true, Transform, Transform, CurrentChangeStamp), LatticePointSelectionTransactionText);
		UpdateGizmoLocation();
		OnSelectionChanged.Broadcast();
	}

	ParentTool->GetToolManager()->EndUndoTransaction();
	
	UpdateDrawables();
}

void ULatticeControlPointsMechanic::UpdateGizmoLocation()
{
	if (!PointTransformGizmo)
	{
		return;
	}

	FVector3d NewGizmoLocation(0, 0, 0);

	if (SelectedPointIDs.Num() == 0)
	{
		PointTransformGizmo->ReinitializeGizmoTransform(FTransform());
	}
	else
	{
		for (int32 PointID : SelectedPointIDs)
		{
			NewGizmoLocation += ControlPoints[PointID];
		}
		NewGizmoLocation /= (double)SelectedPointIDs.Num();
	}

	UpdateGizmoVisibility();

	FTransform NewTransform(FQuat(LocalToWorldTransform.GetRotation()), (FVector)NewGizmoLocation);
	PointTransformGizmo->ReinitializeGizmoTransform(NewTransform);

	// Clear the child scale
	FVector GizmoScale{ 1.0, 1.0, 1.0 };
	PointTransformGizmo->SetNewChildScale(GizmoScale);
}


void ULatticeControlPointsMechanic::UpdateGizmoVisibility()
{
	if (!PointTransformGizmo)
	{
		return;
	}

	if (SelectedPointIDs.Num() == 0)
	{
		PointTransformGizmo->SetVisibility(false);
	}
	else if (bIsDraggingRectangle)
	{
		PointTransformGizmo->SetVisibility(false);
	}
	else if (ShouldHideGizmo.IsBound() && ShouldHideGizmo.Execute())
	{
		PointTransformGizmo->SetVisibility(false);
	}
	else
	{
		PointTransformGizmo->SetVisibility(true);
	}
}


bool ULatticeControlPointsMechanic::DeselectPoint(int32 PointID)
{
	bool PointFound = false;
	
	if (SelectedPointIDs.Remove(PointID) != 0)
	{
		if (ColorOverrides.Contains(PointID))
		{
			DrawnControlPoints->SetPointColor(PointID, ColorOverrides[PointID]);
		}
		else
		{
			DrawnControlPoints->SetPointColor(PointID, NormalPointColor);
		}
		PointFound = true;
	}
	
	return PointFound;
}

void ULatticeControlPointsMechanic::SelectPoint(int32 PointID)
{
	SelectedPointIDs.Add(PointID);

	if (ColorOverrides.Contains(PointID))
	{
		DrawnControlPoints->SetPointColor(PointID, ColorOverrides[PointID]);
	}
	else
	{
		DrawnControlPoints->SetPointColor(PointID, SelectedColor);
	}
}

FInputRayHit ULatticeControlPointsMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Result;
	HitTest(PressPos, Result);
	return Result;
}

void ULatticeControlPointsMechanic::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

void ULatticeControlPointsMechanic::ClearHover()
{
	if (HoveredPointID >= 0)
	{
		DrawnControlPoints->SetPointColor(HoveredPointID, PreHoverPointColor);
		HoveredPointID = -1;
	}
}


bool ULatticeControlPointsMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	FGeometrySet3::FNearest Nearest;

	// see if we're hovering a point for selection
	if (GeometrySet.FindNearestPointToRay((FRay3d)DevicePos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		// Only need to update the hover if we changed the point
		if (Nearest.ID != HoveredPointID)
		{
			ClearHover();
			HoveredPointID = Nearest.ID;
			PreHoverPointColor = DrawnControlPoints->GetPoint(HoveredPointID).Color;
			DrawnControlPoints->SetPointColor(HoveredPointID, HoverColor);
		}
	}
	else
	{
		// Not hovering anything, so done hovering
		return false;
	}

	return true;
}

void ULatticeControlPointsMechanic::OnEndHover()
{
	ClearHover();
}

// Detects Ctrl key state
void ULatticeControlPointsMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	switch (ModifierID)
	{
	case ShiftModifierID:
		bShiftToggle = bIsOn;
		break;
	case CtrlModifierID:
		bCtrlToggle = bIsOn;
		break;
	default:
		break;
	}
}

// These get bound to the delegates on the marquee mechanic.
void ULatticeControlPointsMechanic::OnDragRectangleStarted()
{
	PreDragSelection = SelectedPointIDs;
	bIsDraggingRectangle = true;
	LongTransactions.Open(LatticePointSelectionTransactionText, GetParentTool()->GetToolManager());
	UpdateGizmoVisibility();
}

void ULatticeControlPointsMechanic::OnDragRectangleChanged(const FCameraRectangle& Rectangle)
{
	using namespace LatticeControlPointMechanicLocals;
	
	TSet<int> DragSelection;
	for (int32 PointID = 0; PointID < ControlPoints.Num(); ++PointID)
	{
		FVector PointPosition = DrawnControlPoints->GetPoint(PointID).Position;
		if (Rectangle.IsProjectedPointInRectangle(PointPosition))
		{
			DragSelection.Add(PointID);
		}
	}

	if (ShouldAddToSelectionFunc())
	{
		SelectedPointIDs = PreDragSelection;
		if (ShouldRemoveFromSelectionFunc())
		{
			Toggle(SelectedPointIDs, DragSelection);
		}
		else
		{
			SelectedPointIDs.Append(DragSelection);
		}
	}
	else if (ShouldRemoveFromSelectionFunc())
	{
		SelectedPointIDs = PreDragSelection.Difference(DragSelection);
	}
	else
	{
		// Neither key pressed.
		SelectedPointIDs = DragSelection;
	}

	UpdateDrawables();
}

void ULatticeControlPointsMechanic::OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{
	using namespace LatticeControlPointMechanicLocals;

	bIsDraggingRectangle = false;


	if (!IsEqual(PreDragSelection, SelectedPointIDs))
	{
		const FTransform Transform = PointTransformProxy->GetTransform();
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeControlPointsMechanicSelectionChange>(
			PreDragSelection, false, Transform, Transform, CurrentChangeStamp), LatticePointDeselectionTransactionText);
		ParentTool->GetToolManager()->EmitObjectChange(this, MakeUnique<FLatticeControlPointsMechanicSelectionChange>(
			SelectedPointIDs, true, Transform, Transform, CurrentChangeStamp), LatticePointSelectionTransactionText);
		
		OnSelectionChanged.Broadcast();
	}
	// We hid the gizmo at rectangle start, so it needs updating now.
	UpdateGizmoLocation();

	LongTransactions.Close(GetParentTool()->GetToolManager());

	UpdateDrawables();
}

const TArray<FVector3d>& ULatticeControlPointsMechanic::GetControlPoints() const
{
	return ControlPoints;
}

void ULatticeControlPointsMechanic::UpdateControlPointPositions(const TArray<FVector3d>& NewPoints)
{
	if (!ensure(NewPoints.Num() == ControlPoints.Num()))
	{
		return;
	}

	ControlPoints = NewPoints;
	UpdateDrawables();
}


// ==================== Undo/redo object functions ====================

FLatticeControlPointsMechanicSelectionChange::FLatticeControlPointsMechanicSelectionChange(int32 PointIDIn,
																						   bool bAddedIn,
																						   const FTransform& PreviousTransformIn,
																						   const FTransform& NewTransformIn,
																						   int32 ChangeStampIn)
	: PointIDs{ PointIDIn }
	, bAdded(bAddedIn)
	, PreviousTransform(PreviousTransformIn)
	, NewTransform(NewTransformIn)
	, ChangeStamp(ChangeStampIn)
{}

FLatticeControlPointsMechanicSelectionChange::FLatticeControlPointsMechanicSelectionChange(const TSet<int32>& PointIDsIn, 
																						   bool bAddedIn, 
																						   const FTransform& PreviousTransformIn,
																						   const FTransform& NewTransformIn, 
																						   int32 ChangeStampIn)
	: PointIDs(PointIDsIn)
	, bAdded(bAddedIn)
	, PreviousTransform(PreviousTransformIn)
	, NewTransform(NewTransformIn)
	, ChangeStamp(ChangeStampIn)
{}

void FLatticeControlPointsMechanicSelectionChange::Apply(UObject* Object)
{
	ULatticeControlPointsMechanic* Mechanic = Cast<ULatticeControlPointsMechanic>(Object);
	
	for (int32 PointID : PointIDs)
	{
		if (bAdded)
		{
			Mechanic->SelectPoint(PointID);
		}
		else
		{
			Mechanic->DeselectPoint(PointID);
		}
	}

	Mechanic->PointTransformGizmo->ReinitializeGizmoTransform(NewTransform);
	Mechanic->PointTransformGizmo->SetNewChildScale(FVector{ 1.0, 1.0, 1.0 });	// Clear the child scale

	Mechanic->UpdateGizmoVisibility();
	Mechanic->OnSelectionChanged.Broadcast();
}

void FLatticeControlPointsMechanicSelectionChange::Revert(UObject* Object)
{
	ULatticeControlPointsMechanic* Mechanic = Cast<ULatticeControlPointsMechanic>(Object);
	for (int32 PointID : PointIDs)
	{
		if (bAdded)
		{
			Mechanic->DeselectPoint(PointID);
		}
		else
		{
			Mechanic->SelectPoint(PointID);
		}
	}
	
	Mechanic->PointTransformGizmo->ReinitializeGizmoTransform(PreviousTransform);
	Mechanic->PointTransformGizmo->SetNewChildScale(FVector{ 1.0, 1.0, 1.0 });	// Clear the child scale

	Mechanic->UpdateGizmoVisibility();
	Mechanic->OnSelectionChanged.Broadcast();
}

FString FLatticeControlPointsMechanicSelectionChange::ToString() const
{
	return TEXT("FLatticeControlPointsMechanicSelectionChange");
}


FLatticeControlPointsMechanicMovementChange::FLatticeControlPointsMechanicMovementChange(
	const TMap<int32, FVector3d>& OriginalPositionsIn,
	const TMap<int32, FVector3d>& NewPositionsIn,
	int32 ChangeStampIn,
	bool bFirstMovementIn) :
	OriginalPositions{ OriginalPositionsIn }
	, NewPositions{ NewPositionsIn }
	, ChangeStamp(ChangeStampIn)
	, bFirstMovement(bFirstMovementIn)
{
}

void FLatticeControlPointsMechanicMovementChange::Apply(UObject* Object)
{
	ULatticeControlPointsMechanic* Mechanic = Cast<ULatticeControlPointsMechanic>(Object);
	Mechanic->UpdatePointLocations(NewPositions);
	Mechanic->bHasChanged = true;
	Mechanic->OnPointsChanged.Broadcast();
}

void FLatticeControlPointsMechanicMovementChange::Revert(UObject* Object)
{
	ULatticeControlPointsMechanic* Mechanic = Cast<ULatticeControlPointsMechanic>(Object);
	Mechanic->UpdatePointLocations(OriginalPositions);
	if (bFirstMovement)
	{
		// If we're undoing the first change, make it possible to change the lattice resolution again
		Mechanic->bHasChanged = false;
	}
	Mechanic->OnPointsChanged.Broadcast();
}

FString FLatticeControlPointsMechanicMovementChange::ToString() const
{
	return TEXT("FLatticeControlPointsMechanicMovementChange");
}

#undef LOCTEXT_NAMESPACE

