// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/PlaneDistanceFromHitMechanic.h"
#include "ToolSceneQueriesUtil.h"
#include "MeshQueries.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Distance/DistLine3Ray3.h"
#include "ToolDataVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PlaneDistanceFromHitMechanic)

using namespace UE::Geometry;


void UPlaneDistanceFromHitMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	//Properties = NewObject<UPolygonSelectionMechanicProperties>(this);
	//AddToolPropertySource(Properties);

	//// set up visualizers
	//PolyEdgesRenderer.LineColor = FLinearColor::Red;
	//PolyEdgesRenderer.LineThickness = 2.0;
	//HilightRenderer.LineColor = FLinearColor::Green;
	//HilightRenderer.LineThickness = 4.0f;
	//SelectionRenderer.LineColor = LinearColors::Gold3f<FLinearColor>();
	//SelectionRenderer.LineThickness = 4.0f;
}


void UPlaneDistanceFromHitMechanic::Initialize(FDynamicMesh3&& HitTargetMesh, const FFrame3d& PlaneFrameWorld, bool bMeshInWorldCoords)
{
	PreviewHeightFrame = PlaneFrameWorld;

	PreviewHeightTarget = MoveTemp(HitTargetMesh);
	if (bMeshInWorldCoords)
	{
		MeshTransforms::WorldToFrameCoords(PreviewHeightTarget, PreviewHeightFrame);
	}

	PreviewHeightTargetAABB.SetMesh(&PreviewHeightTarget);
}

// Alternative Initialize() that explicitly specifies what transform to bake
void UPlaneDistanceFromHitMechanic::Initialize(FDynamicMesh3&& HitTargetMesh, const FFrame3d& PlaneFrameWorld, const FTransform& MeshToPlaneFrame)
{
	PreviewHeightFrame = PlaneFrameWorld;

	PreviewHeightTarget = MoveTemp(HitTargetMesh);
	MeshTransforms::ApplyTransform(PreviewHeightTarget, MeshToPlaneFrame);

	PreviewHeightTargetAABB.SetMesh(&PreviewHeightTarget);
}



void UPlaneDistanceFromHitMechanic::UpdateCurrentDistance(const FRay& WorldRay)
{
	float NearestHitDist = TNumericLimits<float>::Max();
	float NearestHitHeight = 1.0f;
	FFrame3d NearestHitFrameWorld;
	bool bFoundHit = false;

	bCurrentHitIsWorldHit = false;

	// cast ray at target object
	FRay3d LocalRay = PreviewHeightFrame.ToFrame((FRay3d)WorldRay);
	int HitTID = PreviewHeightTargetAABB.FindNearestHitTriangle(LocalRay);
	if (HitTID >= 0)
	{
		FIntrRay3Triangle3d IntrQuery =
			TMeshQueries<FDynamicMesh3>::TriangleIntersection(PreviewHeightTarget, HitTID, LocalRay);
		FVector3d HitPosLocal = LocalRay.PointAt(IntrQuery.RayParameter);
		FVector3d HitNormalLocal = PreviewHeightTarget.GetTriNormal(HitTID);

		NearestHitFrameWorld = FFrame3d(
			PreviewHeightFrame.FromFramePoint(HitPosLocal),
			PreviewHeightFrame.FromFrameVector(HitNormalLocal));
		NearestHitHeight = HitPosLocal.Z;
		NearestHitDist = WorldRay.GetParameter((FVector)NearestHitFrameWorld.Origin);

		bFoundHit = true;
	}


	// cast ray into scene
	if (WorldHitQueryFunc)
	{
		FHitResult WorldHitResult;
		bool bHitWorld = WorldHitQueryFunc(WorldRay, WorldHitResult);
		if (bHitWorld)
		{
			float WorldHitDist = WorldRay.GetParameter(WorldHitResult.ImpactPoint);
			if (WorldHitDist < NearestHitDist)
			{
				NearestHitFrameWorld = FFrame3d((FVector3d)WorldHitResult.ImpactPoint, (FVector3d)WorldHitResult.ImpactNormal);
				FVector3d HitPosLocal = PreviewHeightFrame.ToFramePoint(FVector3d(WorldHitResult.ImpactPoint));
				NearestHitHeight = HitPosLocal.Z;
				NearestHitDist = WorldHitDist;
				LastActiveWorldHit = WorldHitResult;
				bFoundHit = true;
				bCurrentHitIsWorldHit = true;
			}
		}
	}

	if (bFoundHit == false && bFallbackToLineAxisPoint)
	{
		FDistLine3Ray3d Distance(FLine3d(PreviewHeightFrame.Origin, PreviewHeightFrame.Z()), FRay3d(WorldRay));
		Distance.ComputeResult();
		float WorldHitDist = WorldRay.GetParameter( (FVector)Distance.RayClosestPoint );
		NearestHitFrameWorld = FFrame3d(Distance.RayClosestPoint, UE::Geometry::Normalized(Distance.RayClosestPoint - Distance.LineClosestPoint) );
		FVector3d HitPosLocal = PreviewHeightFrame.ToFramePoint(Distance.RayClosestPoint);
		NearestHitHeight = HitPosLocal.Z;
		NearestHitDist = WorldHitDist;
		bFoundHit = true;
		bCurrentHitIsWorldHit = false;
	}

	if (bFoundHit)
	{
		if ( WorldPointSnapFunc)
		{
			FVector3d SnapPosWorld;
			if ( WorldPointSnapFunc(NearestHitFrameWorld.Origin, SnapPosWorld) )
			{
				NearestHitFrameWorld.Origin = SnapPosWorld;
				FVector3d LocalPos = PreviewHeightFrame.ToFramePoint((FVector3d)NearestHitFrameWorld.Origin);
				NearestHitHeight = (float)LocalPos.Z;
			}
		}

		CurrentHitPosFrameWorld = NearestHitFrameWorld;
		CurrentHeight = NearestHitHeight;
	}
	else
	{
		//CurrentHeight = DefaultHeight; 
	}

}



void UPlaneDistanceFromHitMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	FViewCameraState CameraState = RenderAPI->GetCameraState();
	float PDIScale = CameraState.GetPDIScalingFactor();

	FToolDataVisualizer Renderer;
	Renderer.BeginFrame(RenderAPI, CameraState);

	float TickLength = (float)ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, CurrentHitPosFrameWorld.Origin, 0.8);
	float TickThickness = 2.0;
	FColor HitFrameColor(0, 128, 128);
	FColor AxisColor(128, 128, 0);
	FColor HeightPosColor(128, 0, 128);

	if (bCurrentHitIsWorldHit)
	{
		HitFrameColor = FColor(255, 128, 0);
		TickThickness = 4 * PDIScale;
	}
	Renderer.DrawLine(
		CurrentHitPosFrameWorld.PointAt(-TickLength, -TickLength, 0),
		CurrentHitPosFrameWorld.PointAt(TickLength, TickLength, 0), HitFrameColor, TickThickness, false);
	Renderer.DrawLine(
		CurrentHitPosFrameWorld.PointAt(-TickLength, TickLength, 0),
		CurrentHitPosFrameWorld.PointAt(TickLength, -TickLength, 0), HitFrameColor, TickThickness, false);
	if (bCurrentHitIsWorldHit)
	{
		Renderer.DrawCircle(
			CurrentHitPosFrameWorld.Origin, CurrentHitPosFrameWorld.Z(), 2.0*TickLength, 8, HitFrameColor, 1.0, false);
	}


	FVector3d PreviewOrigin = PreviewHeightFrame.Origin;
	FVector3d DrawPlaneNormal = PreviewHeightFrame.Z();

	Renderer.DrawLine<FVector3d>(
		PreviewOrigin - 1000*DrawPlaneNormal, PreviewOrigin + 1000*DrawPlaneNormal, AxisColor, 1.0, false);
	Renderer.DrawLine<FVector3d>(
		PreviewOrigin + (double)CurrentHeight*DrawPlaneNormal, CurrentHitPosFrameWorld.Origin, HeightPosColor, 1.0, false);

	Renderer.EndFrame();
}
