// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/PolyLassoMarqueeMechanic.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "ToolSceneQueriesUtil.h"
#include "TransformTypes.h"

#include "SegmentTypes.h"
#include "LineTypes.h"
#include "Intersection/IntrLine2Line2.h"
#include "Intersection/IntrSegment2Segment2.h"
#include "Async/ParallelFor.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(PolyLassoMarqueeMechanic)

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "UPolyLassoMarqueeMechanic"



FCameraPolyLasso::FCameraPolyLasso(const FViewCameraState& CachedCameraState)
{
	// Create a plane just in front of the camera
	CameraOrigin = CachedCameraState.Position;
	CameraPlane = FPlane(CachedCameraState.Position + CachedCameraState.Forward(), CachedCameraState.Forward());
	bCameraIsOrthographic = CachedCameraState.bIsOrthographic;

	// Intersect the drag rays with the camera plane and compute their coordinates in the camera basis
	UBasisVector = CachedCameraState.Right();
	VBasisVector = CachedCameraState.Up();
}


void FCameraPolyLasso::AddPoint(const FRay& WorldRay)
{
	FVector Intersection = FMath::RayPlaneIntersection(WorldRay.Origin, WorldRay.Direction, CameraPlane);
	Polyline.Add(PlaneCoordinates(Intersection));
}


FVector2D FCameraPolyLasso::GetProjectedPoint(const FVector& Point) const
{
	FVector ProjectedPoint;
	if (bCameraIsOrthographic)
	{
		// project directly to plane
		ProjectedPoint = FVector::PointPlaneProject(Point, CameraPlane);
	}
	else
	{
		// intersect along the eye-to-point ray
		ProjectedPoint = FMath::RayPlaneIntersection(CameraOrigin,
			Point - CameraOrigin,
			CameraPlane);
	}

	return PlaneCoordinates(ProjectedPoint);
}



// ---------------------------------------

void UPolyLassoMarqueeMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
	ClickDragBehavior->SetDefaultPriority(BasePriority);
	ClickDragBehavior->Initialize(this);
	ParentTool->AddInputBehavior(ClickDragBehavior, this);

	HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->SetDefaultPriority(BasePriority);
	HoverBehavior->Initialize(this);
	ParentTool->AddInputBehavior(HoverBehavior, this);

	SetIsEnabled(true);
}

bool UPolyLassoMarqueeMechanic::IsEnabled() const
{
	return bIsEnabled;
}

void UPolyLassoMarqueeMechanic::SetIsEnabled(bool bOn)
{
	if ((bIsDragging || bIsInMultiClickPolygon) && !bOn)
	{
		OnTerminateDragSequence();
	}

	bIsEnabled = bOn;
}

void UPolyLassoMarqueeMechanic::SetBasePriority(const FInputCapturePriority& Priority)
{
	BasePriority = Priority;
	if (ClickDragBehavior)
	{
		ClickDragBehavior->SetDefaultPriority(Priority);
	}
}

TPair<FInputCapturePriority, FInputCapturePriority> UPolyLassoMarqueeMechanic::GetPriorityRange() const
{
	return TPair<FInputCapturePriority, FInputCapturePriority>(BasePriority, BasePriority);
}



FInputRayHit UPolyLassoMarqueeMechanic::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	if (bIsEnabled && (bEnableFreehandPolygons || bEnableMultiClickPolygons))
	{
		return FInputRayHit(TNumericLimits<float>::Max()); // bHit is true. Depth is max to lose the standard tiebreaker.
	}
	else
	{
		return FInputRayHit(); // bHit is false
	}
}


void UPolyLassoMarqueeMechanic::OnClickPress(const FInputDeviceRay& PressPos)
{
	if (!PressPos.bHas2D)
	{
		bIsDragging = false;
		return;
	}

	if (bIsInMultiClickPolygon)
	{
		// nothing to do on click-down
		DragCurrentScreenPosition = PressPos.ScreenPosition;
	}
	else
	{
		PolyPathPoints.Reset();
		PolyPathPoints.Add(PressPos);
		DragCurrentScreenPosition = PressPos.ScreenPosition;

		GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CachedCameraState);
		CurrentPolyLasso = FCameraPolyLasso(CachedCameraState);
		CurrentPolyLasso.AddPoint(PressPos.WorldRay);

		if (bEnableFreehandPolygons)
		{
			OnDrawPolyLassoStarted.Broadcast();
		}
	}
}

void UPolyLassoMarqueeMechanic::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!DragPos.bHas2D)
	{
		return;
	}

	if (bIsInMultiClickPolygon)
	{
		DragCurrentScreenPosition = DragPos.ScreenPosition;
	}
	else
	{

		bIsDragging = true;
		DragCurrentScreenPosition = DragPos.ScreenPosition;

		if (bEnableFreehandPolygons)
		{
			float DistSqr = FVector2D::DistSquared(DragPos.ScreenPosition, PolyPathPoints.Last().ScreenPosition);
			if (DistSqr > SpacingTolerance * SpacingTolerance)
			{
				PolyPathPoints.Add(DragPos);
				CurrentPolyLasso.AddPoint(DragPos.WorldRay);
			}

			OnDrawPolyLassoChanged.Broadcast(CurrentPolyLasso);
		}
	}
}

void UPolyLassoMarqueeMechanic::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	// if we are in multiclick draw polygon, we either close poly or continue  
	// (could also check for intersection?)
	if (bIsInMultiClickPolygon)
	{
		if (WillCloseCurrentLasso(ReleasePos))
		{
			CurrentPolyLasso.AddPoint(PolyPathPoints[0].WorldRay);
			OnDrawPolyLassoFinished.Broadcast(CurrentPolyLasso, false);
			bIsInMultiClickPolygon = false;
		}
		else
		{
			PolyPathPoints.Add(ReleasePos);
			CurrentPolyLasso.AddPoint(ReleasePos.WorldRay);
			OnDrawPolyLassoChanged.Broadcast(CurrentPolyLasso);
		}
	
	} 
	else
	{
		bIsDragging = false;

		// if we clicked down and up in (approximately) the same spot, begin multi-click polygon mode
		if (PolyPathPoints.Num() == 1 && this->bEnableMultiClickPolygons)
		{
			float DistSqr = FVector2D::DistSquared(ReleasePos.ScreenPosition, PolyPathPoints.Last().ScreenPosition);
			if (DistSqr < SpacingTolerance * SpacingTolerance)
			{
				bIsInMultiClickPolygon = true;
				return;
			}
		}

		if (bEnableFreehandPolygons)
		{
			OnDrawPolyLassoFinished.Broadcast(CurrentPolyLasso, false);
		}
	}
}

void UPolyLassoMarqueeMechanic::OnTerminateDragSequence()
{
	bIsDragging = false;
	if (bIsInMultiClickPolygon)
	{
		bIsInMultiClickPolygon = false;
		// do not emit lasso?
	}
	else
	{
		OnDrawPolyLassoFinished.Broadcast(CurrentPolyLasso, true);
	}
}



FInputRayHit UPolyLassoMarqueeMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	if (bIsInMultiClickPolygon)
	{
		// abort multiclick polygon if camera moves
		FViewCameraState CurCameraState;
		GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CurCameraState);
		if (CurCameraState.Position != CachedCameraState.Position || CurCameraState.Orientation != CachedCameraState.Orientation)
		{
			OnTerminateDragSequence();
			return FInputRayHit();
		}

		return FInputRayHit(0);
	}
	return FInputRayHit();
}


bool UPolyLassoMarqueeMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (bIsInMultiClickPolygon)
	{
		DragCurrentScreenPosition = DevicePos.ScreenPosition;
		bIsMultiClickPolygonClosed = WillCloseCurrentLasso(DevicePos);
		return true;
	}
	return false;
}


bool UPolyLassoMarqueeMechanic::WillCloseCurrentLasso(const FInputDeviceRay& DevicePos) const
{
	float UseTol = 2.0 * SpacingTolerance;
	float DistSqr = FVector2D::DistSquared(DevicePos.ScreenPosition, PolyPathPoints[0].ScreenPosition);
	return (DistSqr < UseTol* UseTol);
}


void UPolyLassoMarqueeMechanic::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	EViewInteractionState State = RenderAPI->GetViewInteractionState();
	bool bThisViewHasFocus = !!(State & EViewInteractionState::Focused);
	if (bThisViewHasFocus && (bIsDragging || bIsInMultiClickPolygon) )
	{
		FLinearColor UseColor = (bIsInMultiClickPolygon && bIsMultiClickPolygonClosed) ? this->ClosedColor : this->LineColor;

		float DPIScale = Canvas->GetDPIScale();
		int32 NumPoints = PolyPathPoints.Num();
		FVector2D LastPoint = PolyPathPoints[0].ScreenPosition / DPIScale;
		for (int32 k = 1; k < NumPoints; ++k)
		{
			FVector2D NextPoint = PolyPathPoints[k].ScreenPosition / DPIScale;
			FCanvasLineItem LineItem(LastPoint, NextPoint);
			LineItem.LineThickness = this->LineThickness;
			LineItem.SetColor(UseColor);
			Canvas->DrawItem(LineItem);
			LastPoint = NextPoint;
		}

		FVector2D CurPoint = DragCurrentScreenPosition / DPIScale;
		FCanvasLineItem CurLineItem(LastPoint, CurPoint);
		CurLineItem.LineThickness = this->LineThickness;
		CurLineItem.SetColor(UseColor);
		Canvas->DrawItem(CurLineItem);
	}
}




namespace UELocal
{


static bool FindPolylineSelfIntersection(
	const TArray<FVector2D>& Polyline, 
	FVector2D& IntersectionPointOut, 
	FIndex2i& IntersectionIndexOut,
	bool bParallel = true)
{
	int32 N = Polyline.Num();
	std::atomic<bool> bSelfIntersects(false);
	ParallelFor(N - 1, [&](int32 i)
	{
		FSegment2d SegA(Polyline[i], Polyline[i + 1]);
		for (int32 j = i + 2; j < N - 1 && bSelfIntersects == false; ++j)
		{
			FSegment2d SegB(Polyline[j], Polyline[j + 1]);
			if (SegA.Intersects(SegB) && bSelfIntersects == false)		
			{
				bool ExpectedValue = false;
				if (std::atomic_compare_exchange_strong(&bSelfIntersects, &ExpectedValue, true))
				{
					FIntrSegment2Segment2d Intersection(SegA, SegB);
					Intersection.Find();
					IntersectionPointOut = Intersection.Point0;
					IntersectionIndexOut = FIndex2i(i, j);
					return;
				}
			}
		}
	}, (bParallel) ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread );

	return bSelfIntersects;
}



static bool FindPolylineSegmentIntersection(
	const TArray<FVector2D>& Polyline,
	const UE::Geometry::FSegment2d& Segment,
	FVector2D& IntersectionPointOut,
	int& IntersectionIndexOut)
{
	int32 N = Polyline.Num();
	for (int32 i = 0; i < N - 1; ++i)
	{
		FSegment2d PolySeg(Polyline[i], Polyline[i + 1]);
		if (Segment.Intersects(PolySeg))
		{
			FIntrSegment2Segment2d Intersection(Segment, PolySeg);
			Intersection.Find();
			IntersectionPointOut = Intersection.Point0;
			IntersectionIndexOut = i;
			return true;
		}
	}
	return false;
}

}


bool UPolyLassoMarqueeMechanic::ApproximateSelfClipPolyline(TArray<FVector2D>& Polyline)
{
	int32 N = Polyline.Num();

	// handle already-closed polylines
	if (Distance(Polyline[0], Polyline[N - 1]) < 0.0001)
	{
		return true;
	}

	FVector2d IntersectPoint;
	FIndex2i IntersectionIndex(-1, -1);
	bool bSelfIntersects = UELocal::FindPolylineSelfIntersection(Polyline, IntersectPoint, IntersectionIndex);
	if (bSelfIntersects)
	{
		TArray<FVector2D> NewPolyline;
		NewPolyline.Add(IntersectPoint);
		for (int32 i = IntersectionIndex.A; i <= IntersectionIndex.B; ++i)
		{
			NewPolyline.Add(Polyline[i]);
		}
		NewPolyline.Add(IntersectPoint);
		Polyline = MoveTemp(NewPolyline);
		return true;
	}


	FVector2d StartDirOut = UE::Geometry::Normalized(Polyline[0] - Polyline[1]);
	FLine2d StartLine(Polyline[0], StartDirOut);
	FVector2d EndDirOut = UE::Geometry::Normalized(Polyline[N - 1] - Polyline[N - 2]);
	FLine2d EndLine(Polyline[N - 1], EndDirOut);
	FIntrLine2Line2d LineIntr(StartLine, EndLine);
	bool bIntersects = false;
	if (LineIntr.Find())
	{
		bIntersects = LineIntr.IsSimpleIntersection() && (LineIntr.Segment1Parameter > 0) && (LineIntr.Segment2Parameter > 0);
		if (bIntersects)
		{
			Polyline.Add(StartLine.PointAt(LineIntr.Segment1Parameter));
			Polyline.Add(StartLine.Origin);
			return true;
		}
	}


	FAxisAlignedBox2d Bounds;
	for (const FVector2d& P : Polyline)
	{
		Bounds.Contain(P);
	}
	double Size = Bounds.DiagonalLength();

	FVector2d StartPos = Polyline[0] + 0.001 * StartDirOut;
	if (UELocal::FindPolylineSegmentIntersection(Polyline, FSegment2d(StartPos, StartPos + 2 * Size * StartDirOut), IntersectPoint, IntersectionIndex.A))
	{
		//TArray<FVector2f> NewPolyline;
		//for (int32 i = 0; i <= IntersectionIndex.A; ++i)
		//{
		//	NewPolyline.Add(Polyline[i]);
		//}
		//NewPolyline.Add(IntersectPoint);
		//NewPolyline.Add(Polyline[0]);
		//Polyline = MoveTemp(NewPolyline);
		return true;
	}

	FVector2d EndPos = Polyline[N - 1] + 0.001 * EndDirOut;
	if (UELocal::FindPolylineSegmentIntersection(Polyline, FSegment2d(EndPos, EndPos + 2 * Size * EndDirOut), IntersectPoint, IntersectionIndex.A))
	{
		//TArray<FVector2f> NewPolyline;
		//NewPolyline.Add(IntersectPoint);
		//for (int32 i = IntersectionIndex.A+1; i < N; ++i)
		//{
		//	NewPolyline.Add(Polyline[i]);
		//}
		//NewPolyline.Add(Polyline[0]);
		//NewPolyline.Add(IntersectPoint);
		//Polyline = MoveTemp(NewPolyline);
		return true;
	}

	return false;
}


#undef LOCTEXT_NAMESPACE

