// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/RectangleMarqueeMechanic.h"
#include "BaseBehaviors/ClickDragBehavior.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "ConvexVolume.h"
#include "ToolSceneQueriesUtil.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include "Intersection/IntersectionQueries2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RectangleMarqueeMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "URectangleMarqueeMechanic"

void FCameraRectangle::Initialize()
{
	bIsInitialized = true;
	SelectionDomain.Plane = FPlane3(CameraState.Forward(), CameraState.Position + CameraState.Forward());
	SelectionDomain.Rectangle = ProjectSelectionDomain().Rectangle;
}

FCameraRectangle::FRectangleInPlane FCameraRectangle::ProjectSelectionDomain(double OffsetFromSelectionDomain) const
{
	ensure(bIsInitialized);
	
	FRectangleInPlane Result;

	Result.Plane = SelectionDomain.Plane;
	Result.Plane.Constant += OffsetFromSelectionDomain;
	
	FVector ProjectedStart = FMath::RayPlaneIntersection(RectangleStartRay.WorldRay.Origin, RectangleStartRay.WorldRay.Direction, (FPlane)Result.Plane);
	FVector ProjectedEnd = FMath::RayPlaneIntersection(RectangleEndRay.WorldRay.Origin, RectangleEndRay.WorldRay.Direction, (FPlane)Result.Plane);
	FVector2 StartUV = Point3DToPointUV(Result.Plane, ProjectedStart);
	FVector2 EndUV = Point3DToPointUV(Result.Plane, ProjectedEnd);

	// Initialize this way so we don't have to care about min/max
	Result.Rectangle.Contain(StartUV);
	Result.Rectangle.Contain(EndUV);

	return Result;
}

bool FCameraRectangle::IsProjectedPointInRectangle(const FVector& Point) const
{
	ensure(bIsInitialized);
	
	FVector ProjectedPoint;
	if (CameraState.bIsOrthographic)
	{
		ProjectedPoint = OrthographicProjection(SelectionDomain.Plane, Point);
	}
	else
	{
		// If it's not in front of the camera plane, its not contained in the camera rectangle
		if (SelectionDomain.Plane.DistanceTo(Point) <= 0)
		{
			return false;
		}
		ProjectedPoint = PerspectiveProjection(SelectionDomain.Plane, Point);
	}

	FVector2 PointUV = Point3DToPointUV(SelectionDomain.Plane, ProjectedPoint);
	return SelectionDomain.Rectangle.Contains(PointUV);
}

bool FCameraRectangle::IsProjectedSegmentIntersectingRectangle(const FVector& Endpoint1, const FVector& Endpoint2) const
{
	ensure(bIsInitialized);
	
	FVector ProjectedEndpoint1;
	FVector ProjectedEndpoint2;

	if (CameraState.bIsOrthographic)
	{
		ProjectedEndpoint1 = OrthographicProjection(SelectionDomain.Plane, Endpoint1);
		ProjectedEndpoint2 = OrthographicProjection(SelectionDomain.Plane, Endpoint2);
	}
	else
	{
		ProjectedEndpoint1 = Endpoint1;
		ProjectedEndpoint2 = Endpoint2;

		// We have to crop the segment to the portion in front of the camera plane
		FPlane3::EClipSegmentType ClipType = SelectionDomain.Plane.ClipSegment(ProjectedEndpoint1, ProjectedEndpoint2);
		
		// Since the selection plane is identical to the clipping plane there's no need to reproject clipped points
		switch (ClipType)
		{
			case FPlane3::FullyClipped:
				return false; // Segment is behind the camera plane hence not in the camera rectangle
			case FPlane3::FirstClipped:
				ProjectedEndpoint2 = PerspectiveProjection(SelectionDomain.Plane, ProjectedEndpoint2);
				break;
			case FPlane3::SecondClipped:
				ProjectedEndpoint1 = PerspectiveProjection(SelectionDomain.Plane, ProjectedEndpoint1);
				break;
			case FPlane3::NotClipped:
			default:
				ProjectedEndpoint1 = PerspectiveProjection(SelectionDomain.Plane, ProjectedEndpoint1);
				ProjectedEndpoint2 = PerspectiveProjection(SelectionDomain.Plane, ProjectedEndpoint2);
				break;
		}
	}

	FVector2 Endpoint1UV = Point3DToPointUV(SelectionDomain.Plane, ProjectedEndpoint1);
	FVector2 Endpoint2UV = Point3DToPointUV(SelectionDomain.Plane, ProjectedEndpoint2);
	FSegment2 ProjectedSegmentUV(Endpoint1UV, Endpoint2UV);

	return TestIntersection(ProjectedSegmentUV, SelectionDomain.Rectangle);
}

FConvexVolume FCameraRectangle::FrustumAsConvexVolume() const
{
	FConvexVolume OutFrustum;
	ensure(bIsInitialized);
	FVector2 Point[4]
	{
		SelectionDomain.Rectangle.Min,
		FVector2(SelectionDomain.Rectangle.Min.X, SelectionDomain.Rectangle.Max.Y),
		SelectionDomain.Rectangle.Max,
		FVector2(SelectionDomain.Rectangle.Max.X, SelectionDomain.Rectangle.Min.Y)
	};

	FVector BoxPoint[4];
	for (int32 i = 0; i < 4; i++)
	{
		BoxPoint[i] = PointUVToPoint3D(SelectionDomain.Plane, Point[i]);
	}

	if (!CameraState.bIsOrthographic)
	{
		FVector CamPoint = CameraState.Position;
		FVector WorldDir[4];
		for (int32 i = 0; i < 4; i++)
		{
			WorldDir[i] = (BoxPoint[i] - CamPoint).GetSafeNormal();
		}
		// Use the camera position and the selection box to create the bounding planes
		FPlane TopPlane(BoxPoint[0], BoxPoint[1], CamPoint); // Top Plane
		FPlane RightPlane(BoxPoint[1], BoxPoint[2], CamPoint); // Right Plane
		FPlane BottomPlane(BoxPoint[2], BoxPoint[3], CamPoint); // Bottom Plane
		FPlane LeftPlane(BoxPoint[3], BoxPoint[0], CamPoint); // Left Plane

		// Near plane is the selection plane flipped, as the convex construction wants the planes to be oriented away from the volume
		FPlane NearPlane(-SelectionDomain.Plane.Normal, -SelectionDomain.Plane.Constant);

		// TODO: currently no far plane because we haven't propagated that info down to here

		// The frustum is built with the first four planes corresponding to the sides of the frustum + a near plane at the front
		OutFrustum.Planes.Empty();
		OutFrustum.Planes.Add(TopPlane);
		OutFrustum.Planes.Add(RightPlane);
		OutFrustum.Planes.Add(BottomPlane);
		OutFrustum.Planes.Add(LeftPlane);
		OutFrustum.Planes.Add(NearPlane);
	}
	else
	{
		FVector CamOffset = -SelectionDomain.Plane.Normal;
		// Use the camera position and the selection box to create the bounding planes
		FPlane TopPlane(BoxPoint[0], BoxPoint[1], BoxPoint[0] + CamOffset); // Top Plane
		FPlane RightPlane(BoxPoint[1], BoxPoint[2], BoxPoint[1] + CamOffset); // Right Plane
		FPlane BottomPlane(BoxPoint[2], BoxPoint[3], BoxPoint[2] + CamOffset); // Bottom Plane
		FPlane LeftPlane(BoxPoint[3], BoxPoint[0], BoxPoint[3] + CamOffset); // Left Plane

		OutFrustum.Planes.Empty();
		OutFrustum.Planes.Add(TopPlane);
		OutFrustum.Planes.Add(RightPlane);
		OutFrustum.Planes.Add(BottomPlane);
		OutFrustum.Planes.Add(LeftPlane);

		// Note: Orthographic selections are unbounded in the plane normal direction
	}

	OutFrustum.Init();
	return OutFrustum;
}





// ---------------------------------------

FInputRayHit URectangleMarqueeInteraction::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	return FInputRayHit(TNumericLimits<float>::Max());
}

void URectangleMarqueeInteraction::OnClickPress(const FInputDeviceRay& PressPos)
{
	if (!PressPos.bHas2D)
	{
		bIsDragging = false;
		return;
	}

	CameraRectangle.RectangleStartRay = PressPos;
	CameraRectangle.RectangleEndRay = PressPos;
	CameraRectangle.Initialize();

	OnDragRectangleStarted.Broadcast();
	// consider broadcasting OnDragRectangleChanged as well here, and set bIsDragging=true here...
}

void URectangleMarqueeInteraction::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!DragPos.bHas2D)
	{
		return;
	}

	// should probably threshold based on movement here, ie if 2D position is the same, do not 
	// need to post an update... (no guarantees that this isn't called continuously w/ same DragPos!)

	bIsDragging = true;
	CameraRectangle.RectangleEndRay = DragPos;
	CameraRectangle.Initialize();
	
	OnDragRectangleChanged.Broadcast(CameraRectangle);
}

void URectangleMarqueeInteraction::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	bIsDragging = false;
	OnDragRectangleFinished.Broadcast(CameraRectangle, false);
}

void URectangleMarqueeInteraction::OnTerminateDragSequence()
{
	bIsDragging = false;
	OnDragRectangleFinished.Broadcast(CameraRectangle, true);
}


void URectangleMarqueeInteraction::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	CurrentCameraState = RenderAPI->GetCameraState();

	EViewInteractionState State = RenderAPI->GetViewInteractionState();
	bool bThisViewHasFocus = !!(State & EViewInteractionState::Focused);
	if (bThisViewHasFocus && bIsDragging)
	{
		FVector2D Start = CameraRectangle.RectangleStartRay.ScreenPosition;
		FVector2D Curr = CameraRectangle.RectangleEndRay.ScreenPosition;
		FCanvasBoxItem BoxItem(Start / Canvas->GetDPIScale(), (Curr - Start) / Canvas->GetDPIScale());
		BoxItem.SetColor(FLinearColor::White);
		Canvas->DrawItem(BoxItem);
	}
}







// ---------------------------------------

void URectangleMarqueeMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	if (bUseExternalClickDragBehavior == false)
	{
		ClickDragBehavior = NewObject<UClickDragInputBehavior>(this);
		ClickDragBehavior->SetDefaultPriority(BasePriority);
		ClickDragBehavior->Initialize(this);
		ParentTool->AddInputBehavior(ClickDragBehavior, this);
	}
	SetIsEnabled(true);
}

bool URectangleMarqueeMechanic::IsEnabled()
{
	return bIsEnabled;
}

void URectangleMarqueeMechanic::SetIsEnabled(bool bOn)
{
	if (bIsDragging && !bOn)
	{
		OnTerminateDragSequence();
	}

	bIsEnabled = bOn;
}

void URectangleMarqueeMechanic::SetBasePriority(const FInputCapturePriority& Priority)
{
	BasePriority = Priority;
	if (ClickDragBehavior)
	{
		ClickDragBehavior->SetDefaultPriority(Priority);
	}
}

TPair<FInputCapturePriority, FInputCapturePriority> URectangleMarqueeMechanic::GetPriorityRange() const
{
	return TPair<FInputCapturePriority, FInputCapturePriority>(BasePriority, BasePriority);
}

void URectangleMarqueeMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	// Cache the camera state
	if (bUseExternalUpdateCameraState && ensureMsgf(UpdateCameraStateFunc, TEXT("bUseExternalUpdateCameraState is requested, but no update function has been provided.")))
	{
		CameraRectangle.CameraState = UpdateCameraStateFunc();
	}
	else
	{
		GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraRectangle.CameraState);
	}
}

FInputRayHit URectangleMarqueeMechanic::CanBeginClickDragSequence(const FInputDeviceRay& PressPos)
{
	return bIsEnabled ? 
		FInputRayHit(TNumericLimits<float>::Max()) // bHit is true. Depth is max to lose the standard tiebreaker.
		: FInputRayHit(); // bHit is false
}

void URectangleMarqueeMechanic::OnClickPress(const FInputDeviceRay& PressPos)
{
	if (!PressPos.bHas2D)
	{
		bIsDragging = false;
		return;
	}

	CameraRectangle.RectangleStartRay = PressPos;
	CameraRectangle.RectangleEndRay = PressPos;
	CameraRectangle.Initialize();

	bIsOnDragRectangleChangedDeferred = false;
	OnDragRectangleStarted.Broadcast();
}

void URectangleMarqueeMechanic::OnClickDrag(const FInputDeviceRay& DragPos)
{
	if (!DragPos.bHas2D)
	{
		return;
	}

	bIsDragging = true;
	CameraRectangle.RectangleEndRay = DragPos;
	CameraRectangle.Initialize();
	
	if (bIsOnDragRectangleChangedDeferred)
	{
		return;
	}
	
	double Time = 0.0;
	FDurationTimer Timer(Time);
	OnDragRectangleChanged.Broadcast(CameraRectangle);
	Timer.Stop();
	
	if (Time > OnDragRectangleChangedDeferredThreshold)
	{
		bIsOnDragRectangleChangedDeferred = true;
	}
}

void URectangleMarqueeMechanic::OnClickRelease(const FInputDeviceRay& ReleasePos)
{
	if (bIsOnDragRectangleChangedDeferred)
	{
		bIsOnDragRectangleChangedDeferred = false;
		OnClickDrag(ReleasePos);
	}
	
	bIsDragging = false;
	OnDragRectangleFinished.Broadcast(CameraRectangle, false);
}

void URectangleMarqueeMechanic::OnTerminateDragSequence()
{
	bIsDragging = false;
	OnDragRectangleFinished.Broadcast(CameraRectangle, true);
}


void URectangleMarqueeMechanic::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	EViewInteractionState State = RenderAPI->GetViewInteractionState();
	bool bThisViewHasFocus = !!(State & EViewInteractionState::Focused);
	if (bThisViewHasFocus && bIsDragging)
	{
		FVector2D Start = CameraRectangle.RectangleStartRay.ScreenPosition;
		FVector2D Curr = CameraRectangle.RectangleEndRay.ScreenPosition;
		FCanvasBoxItem BoxItem(Start / Canvas->GetDPIScale(), (Curr - Start) / Canvas->GetDPIScale());
		BoxItem.SetColor(FLinearColor::White);
		Canvas->DrawItem(BoxItem);
	}
}

#undef LOCTEXT_NAMESPACE

