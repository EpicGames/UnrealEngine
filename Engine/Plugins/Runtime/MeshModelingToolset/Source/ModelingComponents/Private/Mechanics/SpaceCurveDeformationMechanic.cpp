// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/SpaceCurveDeformationMechanic.h"

#include "BaseBehaviors/SingleClickBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "Transforms/MultiTransformer.h"
#include "ToolDataVisualizer.h"
#include "IntBoxTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpaceCurveDeformationMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USpaceCurveDeformationMechanic"

const FText SpaceCurvePointDeselectionTransactionText = LOCTEXT("PointDeselection", "Point Deselection");
const FText SpaceCurvePointSelectionTransactionText = LOCTEXT("PointSelection", "Point Selection");
const FText SpaceCurvePointMovementTransactionText = LOCTEXT("PointMovement", "Point Movement");

USpaceCurveDeformationMechanic::~USpaceCurveDeformationMechanic()
{
	checkf(PreviewGeometryActor == nullptr, TEXT("Shutdown() should be called before USpaceCurveDeformationMechanic is destroyed."));
}

void USpaceCurveDeformationMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	ClickBehavior = NewObject<USingleClickInputBehavior>();
	ClickBehavior->Initialize(this);
	ClickBehavior->Modifiers.RegisterModifier(ShiftModifierId, FInputDeviceState::IsShiftKeyDown);
	ClickBehavior->Modifiers.RegisterModifier(CtrlModifierId, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(ClickBehavior);

	HoverBehavior = NewObject<UMouseHoverBehavior>();
	HoverBehavior->Initialize(this);
	HoverBehavior->Modifiers.RegisterModifier(ShiftModifierId, FInputDeviceState::IsShiftKeyDown);
	HoverBehavior->Modifiers.RegisterModifier(CtrlModifierId, FInputDeviceState::IsCtrlKeyDown);
	ParentTool->AddInputBehavior(HoverBehavior);

	// We use custom materials that are visible through other objects.
	// TODO: This probably should be configurable. For instance, we may want the segments to be dashed, or not visible at all.
	RenderPoints = NewObject<UPointSetComponent>();
	RenderPoints->SetPointMaterial(
		ToolSetupUtil::GetDefaultPointComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ false));
	RenderSegments = NewObject<ULineSetComponent>();
	RenderSegments->SetLineMaterial(
		ToolSetupUtil::GetDefaultLineComponentMaterial(GetParentTool()->GetToolManager(), /*bDepthTested*/ false));

	NormalCurveColor = FColor::Red;
	CurrentSegmentsColor = FColor::Orange;
	SegmentsThickness = 4.0f;
	CurrentPointsColor = FColor::Black;
	PointsSize = 8.0f;
	HoverColor = FColor::Green;
	SelectedColor = FColor::Yellow;
	PreviewColor = HoverColor;
	DepthBias = 1.0f;

	GeometrySetToleranceTest = [this](const FVector3d& Position1, const FVector3d& Position2) {
		return ToolSceneQueriesUtil::PointSnapQuery(CameraState, Position1, Position2);
	};

	UInteractiveGizmoManager* GizmoManager = GetParentTool()->GetToolManager()->GetPairedGizmoManager();
	PointTransformProxy = NewObject<UTransformProxy>(this);
	PointTransformGizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, ETransformGizmoSubElements::StandardTranslateRotate, GetParentTool());
	PointTransformProxy->OnTransformChanged.AddUObject(this, &USpaceCurveDeformationMechanic::GizmoTransformChanged);
	PointTransformProxy->OnBeginTransformEdit.AddUObject(this, &USpaceCurveDeformationMechanic::GizmoTransformStarted);
	PointTransformProxy->OnEndTransformEdit.AddUObject(this, &USpaceCurveDeformationMechanic::GizmoTransformEnded);
	PointTransformGizmo->SetActiveTarget(PointTransformProxy);
	PointTransformGizmo->SetVisibility(false);

	PointTransformGizmo->bUseContextCoordinateSystem = false;
	PointTransformGizmo->CurrentCoordinateSystem = EToolContextCoordinateSystem::Local;


	// parent Tool has to register this...
	TransformProperties = NewObject<USpaceCurveDeformationMechanicPropertySet>(this);
	TransformProperties->WatchProperty(TransformProperties->TransformMode,
		[this](ESpaceCurveControlPointTransformMode NewMode) { UpdateGizmoLocation(); });
	TransformProperties->WatchProperty(TransformProperties->TransformOrigin,
		[this](ESpaceCurveControlPointOriginMode NewMode) { UpdateGizmoLocation(); });


	ClearCurveSource();
}

void USpaceCurveDeformationMechanic::SetWorld(UWorld* World)
{
	// It may be unreasonable to worry about SetWorld being called more than once, but let's be safe anyway
	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
	}

	// We need the world so we can create the geometry actor in the right place.
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	PreviewGeometryActor = World->SpawnActor<APreviewGeometryActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	// Attach the rendering components to the actor
	RenderPoints->Rename(nullptr, PreviewGeometryActor); // Changes the "outer"
	PreviewGeometryActor->SetRootComponent(RenderPoints);
	if (RenderPoints->IsRegistered())
	{
		RenderPoints->ReregisterComponent();
	}
	else
	{
		RenderPoints->RegisterComponent();
	}

	RenderSegments->Rename(nullptr, PreviewGeometryActor); // Changes the "outer"
	RenderSegments->AttachToComponent(RenderPoints, FAttachmentTransformRules::KeepWorldTransform);
	if (RenderSegments->IsRegistered())
	{
		RenderSegments->ReregisterComponent();
	}
	else
	{
		RenderSegments->RegisterComponent();
	}
	
}

void USpaceCurveDeformationMechanic::Shutdown()
{
	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
		PreviewGeometryActor = nullptr;
	}

	if (PointTransformGizmo)
	{
		GetParentTool()->GetToolManager()->GetPairedGizmoManager()->DestroyGizmo(PointTransformGizmo);
		PointTransformGizmo = nullptr;
	}
}

void USpaceCurveDeformationMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (HoveredPointID >= 0)
	{
		if (HoveredPointID < CurvePoints.Num())
		{
			FToolDataVisualizer Visualizer;
			Visualizer.BeginFrame(RenderAPI);
			Visualizer.DrawPoint(CurvePoints[HoveredPointID].Origin, FLinearColor::White, 2.0*PointsSize, false);
			Visualizer.EndFrame();
		}
	}
}


void USpaceCurveDeformationMechanic::Tick(float DeltaTime)
{
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);

	if (bSpatialValid == false)
	{
		UpdateSpatial();
	}

	if (bRenderGeometryValid == false)
	{
		UpdateRenderGeometry();
	}

	//TODO could filter visibility
	//if (PointTransformGizmo->IsVisible
	if (PointTransformGizmo)
	{
		bool bShowXRotate = (TransformProperties->TransformMode != ESpaceCurveControlPointTransformMode::PerVertex);
		PointTransformGizmo->GetGizmoActor()->RotateX->SetVisibility(bShowXRotate);
	}
}


void USpaceCurveDeformationMechanic::SetCurveSource(TSharedPtr<FSpaceCurveSource> CurveSourceIn)
{
	CurveSource = CurveSourceIn;

	TArray<int32> EmptySelection;
	UpdateSelection(EmptySelection);

	ClearHover();

	int32 NumPoints = CurveSource->GetPointCount();
	CurvePoints.SetNum(NumPoints);
	for (int32 k = 0; k < NumPoints; ++k)
	{
		CurvePoints[k] = CurveSource->GetPoint(k);
	}

	bSpatialValid = false;
	bRenderGeometryValid = false;
}


void USpaceCurveDeformationMechanic::ClearCurveSource()
{
	if (!EmptyCurveSource)
	{
		EmptyCurveSource = MakeShared<FSpaceCurveSource>();
		EmptyCurveSource->GetPointCount = []() { return 0; };
		EmptyCurveSource->GetPoint = [](int32) { return FFrame3d(); };
		EmptyCurveSource->IsLoop = []() { return false; };
	}

	SetCurveSource(EmptyCurveSource);
}



void USpaceCurveDeformationMechanic::GizmoTransformStarted(UTransformProxy* Proxy)
{
	//ParentTool->GetToolManager()->BeginUndoTransaction(SpaceCurvePointMovementTransactionText);

	GizmoStartPosition = FFrame3d(Proxy->GetTransform());

	CurveStartPositions = CurvePoints;

	bGizmoBeingDragged = true;
}



FQuaterniond SafeSlerp(FQuaterniond From, FQuaterniond To, double InterpT)
{
	From.Normalize();
	To.Normalize();

	double X, Y, Z, W;
	double cs = From.Dot(To);

	if (cs < -0.99)
	{
		From = -From;
	}

	double angle = (double)acos(cs);
	if (FMathd::Abs(angle) >= FMathd::ZeroTolerance)
	{
		double sn = (double)sin(angle);
		double invSn = 1 / sn;
		double tAngle = InterpT * angle;
		double coeff0 = (double)sin(angle - tAngle) * invSn;
		double coeff1 = (double)sin(tAngle) * invSn;
		X = coeff0 * From.X + coeff1 * To.X;
		Y = coeff0 * From.Y + coeff1 * To.Y;
		Z = coeff0 * From.Z + coeff1 * To.Z;
		W = coeff0 * From.W + coeff1 * To.W;
	}
	else
	{
		X = From.X;
		Y = From.Y;
		Z = From.Z;
		W = From.W;
	}
	return FQuaterniond(X, Y, Z, W).Normalized();
}





void USpaceCurveDeformationMechanic::GizmoTransformChanged(UTransformProxy* Proxy, FTransform NewGizmoTransform)
{
	if (SelectedPointIDs.Num() == 0 || !bGizmoBeingDragged) return;

	FFrame3d GizmoCurFrame(NewGizmoTransform);

	// compute relative rotation/translation by mapping current gizmo frame into initial gizmo frame
	FFrame3d GizmoRelTransform = GizmoStartPosition.ToFrame(GizmoCurFrame);
	// FQuaterniond LocalRotation = GizmoRelTransform.Rotation;
	// FVector3d LocalTranslation = GizmoRelTransform.Origin;

	// ugh could keep this sorted...
	TArray<int32> OrderedPoints = SelectedPointIDs;
	OrderedPoints.Sort();
	int32 NumPoints = OrderedPoints.Num();

	TArray<double> Distances;
	bool bApplyFalloff = (TransformProperties->Softness > 0)
		&& (TransformProperties->TransformOrigin != ESpaceCurveControlPointOriginMode::Shared || TransformProperties->TransformMode == ESpaceCurveControlPointTransformMode::PerVertex);
	if (bApplyFalloff)
	{
		int32 CenterID = SelectedPointIDs[0];
		if (TransformProperties->TransformOrigin == ESpaceCurveControlPointOriginMode::Last)
		{
			CenterID = SelectedPointIDs.Last();
		}

		int32 CenterIndex = 0;		// note: we skip k=0 below! so init to 0.
		Distances.SetNum(NumPoints);
		Distances[0] = 0;
		double AccumDist = 0;
		for (int32 k = 1; k < NumPoints; ++k)
		{
			double Dist = Distance( CurveStartPositions[OrderedPoints[k]].Origin, CurveStartPositions[OrderedPoints[k-1]].Origin );
			AccumDist += Dist;
			Distances[k] = AccumDist;
			if (OrderedPoints[k] == CenterID)
			{
				CenterIndex = k;
			}
		}

		double CenterDistance = Distances[CenterIndex];
		FInterval1d DistInterval(0, 0);
		for (int32 k = 0; k < NumPoints; ++k)
		{
			Distances[k] = Distances[k] - CenterDistance;
			DistInterval.Contain(Distances[k]);
		}
		for (int32 k = 0; k < NumPoints; ++k)
		{
			if (k != CenterIndex)
			{
				double Divisor = (Distances[k] < 0) ? DistInterval.Min : DistInterval.Max;
				check(Divisor != 0);
				Distances[k] /= Divisor;
			}
		}
	}

	for (int32 i = 0; i < NumPoints; ++i)
	{
		int32 Index = OrderedPoints[i];
		FFrame3d InitialFrame = CurveStartPositions[Index];

		FFrame3d NewFrame = InitialFrame;

		if (TransformProperties->TransformMode == ESpaceCurveControlPointTransformMode::PerVertex)
		{
			// apply gizmo transform locally in the frame of each point
			//InitialFrame.Origin += InitialFrame.FromFrameVector(LocalTranslation);
			//InitialFrame.Rotation = InitialFrame.Rotation * LocalRotation;
			NewFrame = InitialFrame.FromFrame(GizmoRelTransform);
		}
		else
		{
			// map each point from original gizmo frame to new gizmo frame
			FFrame3d GizmoInitRelFrame = GizmoStartPosition.ToFrame(InitialFrame);
			FFrame3d TransformedFrame = GizmoCurFrame.FromFrame(GizmoInitRelFrame);
			NewFrame = TransformedFrame;
		}

		if (bApplyFalloff)
		{
			double t = Distances[i];
			check(t >= 0 && t <= 1.0);
			t *= TransformProperties->Softness;
			
			if (TransformProperties->SoftFalloff == ESpaceCurveControlPointFalloffType::Smooth)
			{
				t = 1.0 - t * t;
				t = t * t * t;
			}
			else
			{
				t = 1.0 - t;		// linear
			}

			NewFrame.Origin = UE::Geometry::Lerp(InitialFrame.Origin, NewFrame.Origin, t);
			NewFrame.Rotation.SetToSlerp(InitialFrame.Rotation, NewFrame.Rotation, t);
			//NewFrame.Rotation = SafeSlerp(InitialFrame.Rotation, NewFrame.Rotation, t);
		}

		CurvePoints[OrderedPoints[i]] = NewFrame;
	}

	bRenderGeometryValid = false;

	OnPointsChanged.Broadcast();
}


void USpaceCurveDeformationMechanic::GizmoTransformEnded(UTransformProxy* Proxy)
{
	TUniquePtr<FSpaceCurveDeformationMechanicMovementChange> MoveChange =
		MakeUnique< FSpaceCurveDeformationMechanicMovementChange>(CurveStartPositions, CurvePoints);
	ParentTool->GetToolManager()->EmitObjectChange(this, MoveTemp(MoveChange), SpaceCurvePointMovementTransactionText);

	CurveStartPositions.Reset();

	bGizmoBeingDragged = false;
	bSpatialValid = false;
}



void USpaceCurveDeformationMechanic::UpdateCurve(const TArray<FFrame3d>& NewPositions)
{
	CurvePoints = NewPositions;

	bSpatialValid = false;
	bRenderGeometryValid = false;

	UpdateGizmoLocation();
	OnPointsChanged.Broadcast();
}




bool USpaceCurveDeformationMechanic::HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut)
{
	FGeometrySet3::FNearest Nearest;
	if (GeometrySet.FindNearestPointToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ResultOut = FInputRayHit(Nearest.RayParam);
		return true;
	}
	return false;
}

FInputRayHit USpaceCurveDeformationMechanic::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Result;
	HitTest(ClickPos, Result);
	return Result;
}

void USpaceCurveDeformationMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	FGeometrySet3::FNearest Nearest;

	if (GeometrySet.FindNearestPointToRay((FRay3d)ClickPos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		ParentTool->GetToolManager()->BeginUndoTransaction(SpaceCurvePointSelectionTransactionText);

		ChangeSelection(Nearest.ID, bAddToSelectionToggle);

		ParentTool->GetToolManager()->EndUndoTransaction();
	}
}

void USpaceCurveDeformationMechanic::ChangeSelection(int32 NewPointID, bool bAddToSelection)
{
	TArray<int32> InitialSelection = SelectedPointIDs;

	// If not adding to selection, clear it
	if (!bAddToSelection && SelectedPointIDs.Num() > 0)
	{
		for (int32 PointID : SelectedPointIDs)
		{
			// We check for validity here because we'd like to be able to use this function to deselect points after
			// deleting them.
			if (RenderPoints->IsPointValid(PointID))
			{
				RenderPoints->SetPointColor(PointID, CurrentPointsColor);
			}
		}

		SelectedPointIDs.Empty();
	}

	// We check for validity here because giving an invalid id (such as -1) with bAddToSelection == false
	// is an easy way to clear the selection.
	if (NewPointID >= 0)
	{
		if (bAddToSelection && DeselectPoint(NewPointID))
		{
		}
		else
		{
			SelectPoint(NewPointID);
		}
	}

	if (SelectedPointIDs != InitialSelection)
	{
		ParentTool->GetToolManager()->EmitObjectChange(this,
			MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(InitialSelection, SelectedPointIDs),
			SpaceCurvePointSelectionTransactionText);
	}

	UpdateGizmoLocation();
}



void USpaceCurveDeformationMechanic::UpdateSelection(const TArray<int32>& NewSelection)
{
	SelectedPointIDs = NewSelection;

	for (int32 k = 0; k < CurvePoints.Num(); ++k)
	{
		bool bSelected = SelectedPointIDs.Contains(k);
		RenderPoints->SetPointColor(k, (bSelected) ? SelectedColor : CurrentPointsColor);
	}

	UpdateGizmoLocation();
}



void USpaceCurveDeformationMechanic::UpdateGizmoLocation()
{
	if (!PointTransformGizmo)
	{
		return;
	}

	int32 NumSelectedPoints = SelectedPointIDs.Num();
	if (NumSelectedPoints == 0 || CurvePoints.Num() == 0)
	{
		PointTransformGizmo->SetVisibility(false);
	}
	else
	{
		FFrame3d NewGizmoFrame = CurvePoints[SelectedPointIDs[0]];
		if (TransformProperties->TransformOrigin == ESpaceCurveControlPointOriginMode::Last)
		{
			NewGizmoFrame = CurvePoints[SelectedPointIDs[NumSelectedPoints-1]];
		}
		else if (TransformProperties->TransformOrigin == ESpaceCurveControlPointOriginMode::Shared
			&& TransformProperties->TransformMode == ESpaceCurveControlPointTransformMode::Shared)
		{
			FVector3d SharedOrigin = FVector3d::Zero();
			for (int32 PointID : SelectedPointIDs)
			{
				SharedOrigin += CurvePoints[PointID].Origin;
			}
			NewGizmoFrame = FFrame3d(SharedOrigin / (double)SelectedPointIDs.Num());
		}

		//if (SelectedPointIDs.Num() > 1 && TransformProperties->TransformMode == ESpaceCurveControlPointTransformMode::Shared)
		//{
		//	FVector3d SharedOrigin = FVector3d::Zero();
		//	for (int32 PointID : SelectedPointIDs)
		//	{
		//		SharedOrigin += CurvePoints[PointID].Origin;
		//	}
		//	NewGizmoFrame.Origin = SharedOrigin / SelectedPointIDs.Num();
		//}
		
		PointTransformGizmo->ReinitializeGizmoTransform(NewGizmoFrame.ToFTransform());
		PointTransformGizmo->SetVisibility(true);
	}
}


bool USpaceCurveDeformationMechanic::DeselectPoint(int32 PointID)
{
	bool PointFound = false;
	int32 IndexInSelection;
	if (SelectedPointIDs.Find(PointID, IndexInSelection))
	{
		SelectedPointIDs.RemoveAt(IndexInSelection);
		RenderPoints->SetPointColor(PointID, CurrentPointsColor);

		PointFound = true;
	}
	
	return PointFound;
}

void USpaceCurveDeformationMechanic::SelectPoint(int32 PointID)
{
	SelectedPointIDs.Add(PointID);
	RenderPoints->SetPointColor(PointID, SelectedColor);
}

void USpaceCurveDeformationMechanic::ClearSelection()
{
	ChangeSelection(-1, false);
}


void USpaceCurveDeformationMechanic::SelectionClear()
{
	if (CurvePoints.Num() == 0 || SelectedPointIDs.Num() == 0 ) return;
	TArray<int32> NewSelection;
	ParentTool->GetToolManager()->EmitObjectChange(this,
		MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(SelectedPointIDs, NewSelection),
		SpaceCurvePointSelectionTransactionText);
	UpdateSelection(NewSelection);
}


void USpaceCurveDeformationMechanic::SelectionFill()
{
	if (CurvePoints.Num() == 0) return;

	TArray<int32> NewSelection = SelectedPointIDs;
	if (NewSelection.Num() <= 1)
	{
		SelectionGrowToEnd();
		return;
	}

	FInterval1i ValueRange = FInterval1i::Empty();
	for (int32 idx : NewSelection)
	{
		ValueRange.Contain(idx);
	}
	bool bAddAtStart = (TransformProperties->TransformOrigin == ESpaceCurveControlPointOriginMode::Last);
	for (int32 k = ValueRange.Min; k < ValueRange.Max; ++k)
	{
		if (bAddAtStart)
		{
			if (NewSelection.Contains(k) == false)
			{
				NewSelection.Insert(k, 0);
			}
		}
		else
		{
			NewSelection.AddUnique(k);
		}
	}

	if (NewSelection != SelectedPointIDs)
	{
		ParentTool->GetToolManager()->EmitObjectChange(this,
			MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(SelectedPointIDs, NewSelection),
			SpaceCurvePointSelectionTransactionText);
		UpdateSelection(NewSelection);
	}
}


void USpaceCurveDeformationMechanic::SelectionGrowToNext()
{
	if (CurvePoints.Num() == 0) return;
	TArray<int32> NewSelection = SelectedPointIDs;

	NewSelection.Sort();
	int32 Next = 0;
	if (NewSelection.Num() > 0)
	{
		Next = FMath::Min(NewSelection.Last() + 1, CurvePoints.Num() - 1);
		if (NewSelection.Contains(Next))
		{
			return;
		}
	}
	NewSelection.Add(Next);

	ParentTool->GetToolManager()->EmitObjectChange(this,
		MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(SelectedPointIDs, NewSelection),
		SpaceCurvePointSelectionTransactionText);
	UpdateSelection(NewSelection);
}


void USpaceCurveDeformationMechanic::SelectionGrowToPrev()
{
	if (CurvePoints.Num() == 0) return;
	TArray<int32> NewSelection = SelectedPointIDs;

	NewSelection.Sort();
	int32 Prev = CurvePoints.Num()-1;
	if (NewSelection.Num() > 0)
	{
		Prev = FMath::Max(NewSelection[0] - 1, 0);
		if (NewSelection.Contains(Prev))
		{
			return;
		}
	}
	NewSelection.Add(Prev);

	ParentTool->GetToolManager()->EmitObjectChange(this,
		MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(SelectedPointIDs, NewSelection),
		SpaceCurvePointSelectionTransactionText);
	UpdateSelection(NewSelection);
}


void USpaceCurveDeformationMechanic::SelectionGrowToEnd()
{
	if (CurvePoints.Num() == 0) return;
	TArray<int32> NewSelection = SelectedPointIDs;

	if (NewSelection.Num() == CurvePoints.Num())
	{
		// if entire curve is selected, we either ignore, or flip selection
		if (NewSelection[0] == 0)
		{
			return;
		}
		Algo::Reverse(NewSelection);
	}
	else
	{
		NewSelection.Sort();
		int32 Next = 0;
		if (NewSelection.Num() > 0)
		{
			Next = FMath::Min(NewSelection.Last() + 1, CurvePoints.Num() - 1);
		}

		for (int32 k = Next; k < CurvePoints.Num(); ++k)
		{
			if (NewSelection.Contains(k) == false)
			{
				NewSelection.Add(k);
			}
		}
	}

	if (NewSelection != SelectedPointIDs)
	{
		ParentTool->GetToolManager()->EmitObjectChange(this,
			MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(SelectedPointIDs, NewSelection),
			SpaceCurvePointSelectionTransactionText);
		UpdateSelection(NewSelection);
	}
}


void USpaceCurveDeformationMechanic::SelectionGrowToStart()
{
	if (CurvePoints.Num() == 0) return;
	TArray<int32> NewSelection = SelectedPointIDs;

	if (NewSelection.Num() == CurvePoints.Num())
	{
		// if entire curve is selected, we either ignore, or flip selection
		if (NewSelection[0] == CurvePoints.Num()-1)
		{
			return;
		}
		Algo::Reverse(NewSelection);
	}
	else
	{
		NewSelection.Sort();
		int32 Prev = CurvePoints.Num() - 1;
		if (NewSelection.Num() > 0)
		{
			Prev = FMath::Max(NewSelection[0] - 1, 0);
		}

		for (int32 k = Prev; k >= 0; --k)
		{
			if (NewSelection.Contains(k) == false)
			{
				NewSelection.Add(k);
			}
		}
	}

	if (NewSelection != SelectedPointIDs)
	{
		ParentTool->GetToolManager()->EmitObjectChange(this,
			MakeUnique<FSpaceCurveDeformationMechanicSelectionChange>(SelectedPointIDs, NewSelection),
			SpaceCurvePointSelectionTransactionText);
		UpdateSelection(NewSelection);
	}
}



FInputRayHit USpaceCurveDeformationMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Result;
	HitTest(PressPos, Result);
	Result.HitDepth = 0;
	return Result;
}

void USpaceCurveDeformationMechanic::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	OnUpdateHover(DevicePos);
}

void USpaceCurveDeformationMechanic::ClearHover()
{
	HoveredPointID = -1;
}


bool USpaceCurveDeformationMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	FGeometrySet3::FNearest Nearest;
	if (GeometrySet.FindNearestPointToRay((FRay3d)DevicePos.WorldRay, Nearest, GeometrySetToleranceTest))
	{
		HoveredPointID = Nearest.ID;
		return true;
	}
	else
	{
		return false;
	}
}

void USpaceCurveDeformationMechanic::OnEndHover()
{
	ClearHover();
}

// Detects Ctrl and Shift key states
void USpaceCurveDeformationMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	if (ModifierID == ShiftModifierId)
	{
		bAddToSelectionToggle = bIsOn;
	}
	else if (ModifierID == CtrlModifierId)
	{
		
	}
}





void USpaceCurveDeformationMechanic::UpdateSpatial()
{
	if (bSpatialValid) return;

	GeometrySet.Reset(true, true);

	int32 NumPoints = CurveSource->GetPointCount();
	for (int32 k = 0; k < NumPoints; ++k)
	{
		GeometrySet.AddPoint(k, CurveSource->GetPoint(k).Origin);
	}

	bSpatialValid = true;
}



void USpaceCurveDeformationMechanic::UpdateRenderGeometry()
{
	if (bRenderGeometryValid) return;

	int32 NumPoints = CurveSource->GetPointCount();

	RenderPoints->Clear();
	RenderSegments->Clear();

	if (NumPoints > 0)
	{
		for (int32 k = 0; k < NumPoints; ++k)
		{
			FFrame3d Point = CurveSource->GetPoint(k);
			bool bSelected = SelectedPointIDs.Contains(k);
			RenderPoints->AddPoint(FRenderablePoint((FVector)Point.Origin, 
				(bSelected) ? SelectedColor : CurrentPointsColor, PointsSize, DepthBias));
		}

		int32 StopIndex = CurveSource->IsLoop() ? NumPoints : (NumPoints-1);
		for (int32 k = 0; k < StopIndex; ++k)
		{
			FFrame3d PointA = CurveSource->GetPoint(k);
			FFrame3d PointB = CurveSource->GetPoint((k + 1) % NumPoints);
			RenderSegments->AddLine(
				FRenderableLine((FVector)PointA.Origin, (FVector)PointB.Origin, NormalCurveColor, SegmentsThickness, DepthBias));
		}
	}

	bRenderGeometryValid = true;
}



// ==================== Undo/redo object functions ====================

FSpaceCurveDeformationMechanicSelectionChange::FSpaceCurveDeformationMechanicSelectionChange(const TArray<int32>& FromIDs, const TArray<int32>& ToIDs)
{
	From = FromIDs;
	To = ToIDs;
}

void FSpaceCurveDeformationMechanicSelectionChange::Apply(UObject* Object)
{
	Cast<USpaceCurveDeformationMechanic>(Object)->UpdateSelection(To);
}

void FSpaceCurveDeformationMechanicSelectionChange::Revert(UObject* Object)
{
	Cast<USpaceCurveDeformationMechanic>(Object)->UpdateSelection(From);
}

FString FSpaceCurveDeformationMechanicSelectionChange::ToString() const
{
	return TEXT("FSpaceCurveDeformationMechanicSelectionChange");
}




FSpaceCurveDeformationMechanicMovementChange::FSpaceCurveDeformationMechanicMovementChange(const TArray<FFrame3d>& FromPositions, const TArray<FFrame3d>& ToPositions)
{
	From = FromPositions;
	To = ToPositions;
}

void FSpaceCurveDeformationMechanicMovementChange::Apply(UObject* Object)
{
	Cast<USpaceCurveDeformationMechanic>(Object)->UpdateCurve(To);
}

void FSpaceCurveDeformationMechanicMovementChange::Revert(UObject* Object)
{
	Cast<USpaceCurveDeformationMechanic>(Object)->UpdateCurve(From);
}

FString FSpaceCurveDeformationMechanicMovementChange::ToString() const
{
	return TEXT("FSpaceCurveDeformationMechanicMovementChange");
}




#undef LOCTEXT_NAMESPACE
