// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mechanics/SpatialCurveDistanceMechanic.h"
#include "ToolSceneQueriesUtil.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Distance/DistRay3Segment3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpatialCurveDistanceMechanic)

using namespace UE::Geometry;

void USpatialCurveDistanceMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);
}


void USpatialCurveDistanceMechanic::InitializePolyCurve(const TArray<FVector3d>& CurvePoints, const FTransform3d& TransformIn)
{
	Curve = CurvePoints;
	Transform = TransformIn;
	for (int32 k = 0; k < Curve.Num(); ++k)
	{
		Curve[k] = TransformIn.TransformPosition(Curve[k]);
	}
}


void USpatialCurveDistanceMechanic::InitializePolyLoop(const TArray<FVector3d>& CurvePoints, const FTransform3d& TransformIn)
{
	InitializePolyCurve(CurvePoints, TransformIn);
	Curve.Add(CurvePoints[0]);
}


void USpatialCurveDistanceMechanic::UpdateCurrentDistance(const FRay& WorldRay)
{
	FVector3d RayNearest = FVector3d::ZeroVector;
	FVector3d CurveNearest = FVector3d::ZeroVector;

	FRay3d Ray(WorldRay);
	int NumV = Curve.Num() - 1;
	double MinDistSqr = TNumericLimits<double>::Max();
	for (int k = 0; k < NumV; ++k)
	{
		FDistRay3Segment3d Distance(Ray, FSegment3d(Curve[k], Curve[k + 1]));
		double DistSqr = Distance.GetSquared();
		if (DistSqr < MinDistSqr)
		{
			MinDistSqr = DistSqr;
			RayNearest = Distance.RayClosestPoint;
			CurveNearest = Distance.SegmentClosestPoint;
		}
	}

	CurrentDistance = FMathd::Sqrt(MinDistSqr);
	CurrentCurvePoint = CurveNearest;
	CurrentSpacePoint = RayNearest;
}



void USpatialCurveDistanceMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	FViewCameraState RenderCameraState = RenderAPI->GetCameraState();
	float PDIScale = RenderCameraState.GetPDIScalingFactor();
	FPrimitiveDrawInterface* PDI = RenderAPI->GetPrimitiveDrawInterface();

	FColor AxisColor(128, 128, 0);
	PDI->DrawLine(
		(FVector)CurrentCurvePoint,  (FVector)CurrentSpacePoint,
		AxisColor, 1, 1.0f*PDIScale, 0.0f, true);
}
