// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshOpPreviewHelpers.h"
#include "Engine/World.h"

#if WITH_EDITOR
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshOpPreviewHelpers)


#define LOCTEXT_NAMESPACE "MeshOpPreviewHelpers"

namespace UE::Private::MeshOpPreviewLocal
{
	static void DisplayCriticalWarningMessage(const FText& InMessage, float ExpireDuration = 5.0f)
	{
#if WITH_EDITOR
		FNotificationInfo Info(InMessage);
		Info.ExpireDuration = ExpireDuration;
		FSlateNotificationManager::Get().AddNotification(Info);
#endif

		UE_LOGF(LogGeometry, Warning, "%ls", *InMessage.ToString());
	}

	static TAutoConsoleVariable<int32> CVarOverrideMaxBackgroundTasks(
		TEXT("modeling.MaxBackgroundTasksOverride"), 0,
		TEXT("Optional override for maximum allowed background tasks when generating preview results in tools. 0 to use default values. [def: 0]"));

	int32 MaxActiveBackgroundTasksWithOverride(int32 MaxWithoutOverride)
	{
		int32 Override = CVarOverrideMaxBackgroundTasks.GetValueOnAnyThread();
		return Override > 0 ? Override : MaxWithoutOverride;
	}
}


using namespace UE::Geometry;

void UMeshOpPreviewWithBackgroundCompute::Setup(UWorld* InWorld)
{
	PreviewMesh = NewObject<UPreviewMesh>(this, TEXT("PreviewMesh"));
	PreviewMesh->CreateInWorld(InWorld, FTransform::Identity);
	PreviewWorld = InWorld;
	bResultValid = false;
	bMeshInitialized = false;
}

void UMeshOpPreviewWithBackgroundCompute::Setup(UWorld* InWorld, IDynamicMeshOperatorFactory* OpGenerator)
{
	Setup(InWorld);
	BackgroundCompute = MakeUnique<FBackgroundDynamicMeshComputeSource>(OpGenerator);
	BackgroundCompute->MaxActiveTaskCount = UE::Private::MeshOpPreviewLocal::MaxActiveBackgroundTasksWithOverride(MaxActiveBackgroundTasks);
}

void UMeshOpPreviewWithBackgroundCompute::ChangeOpFactory(IDynamicMeshOperatorFactory* OpGenerator)
{
	CancelCompute();
	BackgroundCompute = MakeUnique<FBackgroundDynamicMeshComputeSource>(OpGenerator);
	BackgroundCompute->MaxActiveTaskCount = UE::Private::MeshOpPreviewLocal::MaxActiveBackgroundTasksWithOverride(MaxActiveBackgroundTasks);
	bResultValid = false;
	bMeshInitialized = false;
}

void UMeshOpPreviewWithBackgroundCompute::ClearOpFactory()
{
	CancelCompute();
	BackgroundCompute = nullptr;
	bResultValid = false;
	bMeshInitialized = false;
}


FDynamicMeshOpResult UMeshOpPreviewWithBackgroundCompute::Shutdown()
{
	CancelCompute();

	FDynamicMeshOpResult Result;
	Result.Mesh = PreviewMesh->ExtractPreviewMesh();
	Result.Transform = FTransformSRT3d(PreviewMesh->GetTransform());

	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	PreviewWorld = nullptr;

	return Result;
}


void UMeshOpPreviewWithBackgroundCompute::CancelCompute()
{
	if (BackgroundCompute)
	{
		BackgroundCompute->CancelActiveCompute();
	}
}

void UMeshOpPreviewWithBackgroundCompute::Cancel()
{
	CancelCompute();

	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;
}


void UMeshOpPreviewWithBackgroundCompute::Tick(float DeltaTime)
{
	if (BackgroundCompute)
	{
		BackgroundCompute->Tick(DeltaTime);
		UpdateResults();
	}

	if (!IsUsingWorkingMaterial())
	{
		if (OverrideMaterial != nullptr)
		{
			PreviewMesh->SetOverrideRenderMaterial(OverrideMaterial);
		}
		else
		{
			PreviewMesh->ClearOverrideRenderMaterial();
		}

		if (SecondaryMaterial != nullptr)
		{
			PreviewMesh->SetSecondaryRenderMaterial(SecondaryMaterial);
		}
		else
		{
			PreviewMesh->ClearSecondaryRenderMaterial();
		}
	}
	else
	{
		PreviewMesh->SetOverrideRenderMaterial(WorkingMaterial);
		PreviewMesh->ClearSecondaryRenderMaterial();
	}
}


void UMeshOpPreviewWithBackgroundCompute::SetMaxActiveBackgroundTasks(int32 InMaxActiveBackgroundTasks)
{
	MaxActiveBackgroundTasks = InMaxActiveBackgroundTasks;
	if (BackgroundCompute)
	{
		BackgroundCompute->MaxActiveTaskCount = UE::Private::MeshOpPreviewLocal::MaxActiveBackgroundTasksWithOverride(MaxActiveBackgroundTasks);
	}
}


void UMeshOpPreviewWithBackgroundCompute::UpdateResults()
{
	if (BackgroundCompute == nullptr)
	{
		LastComputeStatus = EBackgroundComputeTaskStatus::NotComputing;
		return;
	}

	FBackgroundDynamicMeshComputeSource::FStatus Status = BackgroundCompute->CheckStatus();	
	LastComputeStatus = Status.TaskStatus; 

	if (LastComputeStatus == EBackgroundComputeTaskStatus::ValidResultAvailable
		|| (bAllowDirtyResultUpdates && LastComputeStatus == EBackgroundComputeTaskStatus::DirtyResultAvailable))
	{
		TUniquePtr<FDynamicMeshOperator> MeshOp = BackgroundCompute->ExtractResult();
		OnOpCompleted.Broadcast(MeshOp.Get());

		TUniquePtr<FDynamicMesh3> ResultMesh = MeshOp->ExtractResult();
		PreviewMesh->SetTransform((FTransform)MeshOp->GetResultTransform());

		UPreviewMesh::ERenderUpdateMode UpdateType = (bMeshTopologyIsConstant && bMeshInitialized) ?
			UPreviewMesh::ERenderUpdateMode::FastUpdate
			: UPreviewMesh::ERenderUpdateMode::FullUpdate;
		
		PreviewMesh->UpdatePreview(MoveTemp(*ResultMesh), UpdateType, ChangingAttributeFlags);
		bMeshInitialized = true;

		PreviewMesh->SetVisible(bVisible);
		bResultValid = (LastComputeStatus == EBackgroundComputeTaskStatus::ValidResultAvailable);
		ValidResultComputeTimeSeconds = Status.ElapsedTime;

		OnMeshUpdated.Broadcast(this);

		bWaitingForBackgroundTasks = false;
	}
	else if (int WaitingTaskCount; BackgroundCompute->IsWaitingForBackgroundTasks(WaitingTaskCount))
	{
		if (!bWaitingForBackgroundTasks)
		{
			UE::Private::MeshOpPreviewLocal::DisplayCriticalWarningMessage(LOCTEXT("TooManyBackgroundTasks", "Too many background tasks: Cancelling earlier tasks before generating new preview."));
			bWaitingForBackgroundTasks = true;
		}
	}
	else
	{
		bWaitingForBackgroundTasks = false;
	}
}


void UMeshOpPreviewWithBackgroundCompute::InvalidateResult()
{
	if (BackgroundCompute)
	{
		BackgroundCompute->NotifyActiveComputeInvalidated();
	}
	bResultValid = false;
}


bool UMeshOpPreviewWithBackgroundCompute::GetCurrentResultCopy(FDynamicMesh3& MeshOut, bool bOnlyIfValid)
{
	if ( HaveValidResult() || bOnlyIfValid == false)
	{
		PreviewMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			MeshOut = ReadMesh;
		});
		return true;
	}
	return false;
}

bool UMeshOpPreviewWithBackgroundCompute::ProcessCurrentMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc, bool bOnlyIfValid)
{
	if (HaveValidResult() || bOnlyIfValid == false)
	{
		PreviewMesh->ProcessMesh(ProcessFunc);
		return true;
	}
	return false;
}


void UMeshOpPreviewWithBackgroundCompute::ConfigureMaterials(UMaterialInterface* StandardMaterialIn, UMaterialInterface* WorkingMaterialIn)
{
	TArray<UMaterialInterface*> Materials;
	Materials.Add(StandardMaterialIn);
	ConfigureMaterials(Materials, WorkingMaterialIn);
}


void UMeshOpPreviewWithBackgroundCompute::ConfigureMaterials(
	TArray<UMaterialInterface*> StandardMaterialsIn, 
	UMaterialInterface* WorkingMaterialIn,
	UMaterialInterface* SecondaryMaterialIn)
{
	this->StandardMaterials = StandardMaterialsIn;
	this->WorkingMaterial = WorkingMaterialIn;
	this->SecondaryMaterial = SecondaryMaterialIn;

	if (PreviewMesh != nullptr)
	{
		PreviewMesh->SetMaterials(this->StandardMaterials);
	}
}

void UMeshOpPreviewWithBackgroundCompute::ConfigurePreviewMaterials(
	UMaterialInterface* InProgressMaterialIn,
	UMaterialInterface* SecondaryMaterialIn)
{
	this->WorkingMaterial = InProgressMaterialIn;
	this->SecondaryMaterial = SecondaryMaterialIn;
}

void UMeshOpPreviewWithBackgroundCompute::DisablePreviewMaterials()
{
	this->WorkingMaterial = nullptr;
	this->SecondaryMaterial = nullptr;
}

void UMeshOpPreviewWithBackgroundCompute::SetVisibility(bool bVisibleIn)
{
	bVisible = bVisibleIn;
	PreviewMesh->SetVisible(bVisible);
}

void UMeshOpPreviewWithBackgroundCompute::SetIsMeshTopologyConstant(bool bOn, EMeshRenderAttributeFlags ChangingAttributesIn)
{
	bMeshTopologyIsConstant = bOn;
	ChangingAttributeFlags = ChangingAttributesIn;
}


bool UMeshOpPreviewWithBackgroundCompute::IsUsingWorkingMaterial()
{
	return WorkingMaterial && BackgroundCompute 
		&& LastComputeStatus == EBackgroundComputeTaskStatus::InProgress
		&& BackgroundCompute->GetElapsedComputeTime() > SecondsBeforeWorkingMaterial;
}

#undef LOCTEXT_NAMESPACE
