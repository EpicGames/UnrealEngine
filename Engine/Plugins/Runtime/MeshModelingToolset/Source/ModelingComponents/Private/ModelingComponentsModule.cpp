// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingComponentsModule.h"
#include "Components/BaseDynamicMeshComponent.h"
#include "Materials/Material.h"
#include "Misc/CoreDelegates.h"
#include "ShaderCore.h"

#define LOCTEXT_NAMESPACE "FModelingComponentsModule"

void FModelingComponentsModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FModelingComponentsModule::OnPostEngineInit);

	// Ensure that the GeometryFramework module is loaded so that UBaseDynamicMeshComponent materials are configured
	FModuleManager::Get().LoadModule(TEXT("GeometryFramework"));
}

void FModelingComponentsModule::OnPostEngineInit()
{
	if (AllowDebugViewmodes())
	{
		// Replace the standard UBaseDynamicMeshComponent vertex color material with something higher quality.
		// This is done in ModelingComponents module (ie part of MeshModelingToolset plugin) to avoid having to
		// make this Material a special "engine material", which has various undesirable implication
		UMaterial* VertexColorMaterial = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolset/Materials/M_DynamicMeshComponentVtxColor"));
		if (ensure(VertexColorMaterial))
		{
			UBaseDynamicMeshComponent::SetDefaultVertexColorMaterial(VertexColorMaterial);
		}
	}
}

void FModelingComponentsModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FModelingComponentsModule, ModelingComponents)