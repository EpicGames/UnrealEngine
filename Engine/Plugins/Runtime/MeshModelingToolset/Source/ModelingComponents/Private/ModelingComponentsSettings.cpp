// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingComponentsSettings.h"
#include "ModelingObjectsCreationAPI.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingComponentsSettings)


#define LOCTEXT_NAMESPACE "ModelingComponentsSettings"

#if WITH_EDITOR
FText UModelingComponentsSettings::GetSectionText() const 
{ 
	return LOCTEXT("ModelingComponentsProjectSettingsName", "Modeling Mode Tools"); 
}

FText UModelingComponentsSettings::GetSectionDescription() const
{
	return LOCTEXT("ModelingComponentsProjectSettingsDescription", "Configure Tool-level Settings for the Modeling Tools Editor Mode plugin");
}

FText UModelingComponentsEditorSettings::GetSectionText() const 
{ 
	return LOCTEXT("ModelingComponentsEditorSettingsName", "Modeling Mode Tools"); 
}

FText UModelingComponentsEditorSettings::GetSectionDescription() const
{
	return LOCTEXT("ModelingComponentsEditorSettingsDescription", "Configure Tool-level Settings for the Modeling Tools Editor Mode plugin");
}
#endif


void UModelingComponentsSettings::ApplyDefaultsToCreateMeshObjectParams(FCreateMeshObjectParams& Params)
{
	const UModelingComponentsSettings* Settings = GetDefault<UModelingComponentsSettings>();
	if (Settings)
	{
		Params.bEnableCollision = Settings->bEnableCollision;
		Params.CollisionMode = Settings->CollisionMode;
		Params.bGenerateLightmapUVs = Settings->bGenerateLightmapUVs;
		Params.bEnableRaytracingSupport = Settings->bEnableRayTracing;
	}
}

#undef LOCTEXT_NAMESPACE
