// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingObjectsCreationAPI.h"
#include "InteractiveToolManager.h"
#include "ContextObjectStore.h"

#include "Misc/Paths.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "ModelingComponentsSettings.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingObjectsCreationAPI)

// if set to 1, then we do not default-initialize our new mesh object parameters based on the Modeling Components Settings (the Modeling Tools' Project Settings)
static TAutoConsoleVariable<bool> CVarConstructMeshObjectsWithoutModelingComponentSettings(
	TEXT("modeling.CreateMesh.IgnoreProjectSettings"),
	false,
	TEXT("If enabled, do not use the preferences set in Modeling Tools' Project Settings when constructing new mesh objects"));

FCreateMeshObjectParams::FCreateMeshObjectParams(bool bConstructWithDefaultModelingComponentSettings)
{
	if (bConstructWithDefaultModelingComponentSettings && 
		!CVarConstructMeshObjectsWithoutModelingComponentSettings.GetValueOnGameThread())
	{
		UModelingComponentsSettings::ApplyDefaultsToCreateMeshObjectParams(*this);
	}
}

void FCreateMeshObjectParams::SetMesh(FMeshDescription&& MeshDescriptionIn)
{
	MeshDescription = MoveTemp(MeshDescriptionIn);
	MeshType = ECreateMeshObjectSourceMeshType::MeshDescription;
}

void FCreateMeshObjectParams::SetMesh(const UE::Geometry::FDynamicMesh3* DynamicMeshIn)
{
	DynamicMesh = *DynamicMeshIn;
	MeshType = ECreateMeshObjectSourceMeshType::DynamicMesh;
	if(DynamicMesh)
	{ 
		UE::Geometry::FNonManifoldMappingSupport::RemoveAllNonManifoldMappingData(*DynamicMesh);
	}
}

void FCreateMeshObjectParams::SetMesh(UE::Geometry::FDynamicMesh3&& DynamicMeshIn)
{
	DynamicMesh = MoveTemp(DynamicMeshIn);
	MeshType = ECreateMeshObjectSourceMeshType::DynamicMesh;
	if (DynamicMesh)
	{
		UE::Geometry::FNonManifoldMappingSupport::RemoveAllNonManifoldMappingData(*DynamicMesh);
	}
}







FCreateMeshObjectResult UE::Modeling::CreateMeshObject(UInteractiveToolManager* ToolManager, FCreateMeshObjectParams&& CreateMeshParams)
{
	if (ensure(ToolManager))
	{
		UModelingObjectsCreationAPI* UseAPI = ToolManager->GetContextObjectStore()->FindContext<UModelingObjectsCreationAPI>();
		if (UseAPI)
		{
			if (UseAPI->HasMoveVariants())
			{
				return UseAPI->CreateMeshObject(MoveTemp(CreateMeshParams));
			}
			else
			{
				return UseAPI->CreateMeshObject(CreateMeshParams);
			}
		}
	}
	return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_NoAPIFound };
}





FCreateTextureObjectResult UE::Modeling::CreateTextureObject(UInteractiveToolManager* ToolManager, FCreateTextureObjectParams&& CreateTexParams)
{
	if (ensure(ToolManager))
	{
		UModelingObjectsCreationAPI* UseAPI = ToolManager->GetContextObjectStore()->FindContext<UModelingObjectsCreationAPI>();
		if (UseAPI)
		{
			if (UseAPI->HasMoveVariants())
			{
				return UseAPI->CreateTextureObject(MoveTemp(CreateTexParams));
			}
			else
			{
				return UseAPI->CreateTextureObject(CreateTexParams);
			}
		}
	}
	return FCreateTextureObjectResult{ ECreateModelingObjectResult::Failed_NoAPIFound };
}





FCreateMaterialObjectResult UE::Modeling::CreateMaterialObject(UInteractiveToolManager* ToolManager, FCreateMaterialObjectParams&& CreateMaterialParams)
{
	if (ensure(ToolManager))
	{
		UModelingObjectsCreationAPI* UseAPI = ToolManager->GetContextObjectStore()->FindContext<UModelingObjectsCreationAPI>();
		if (UseAPI)
		{
			if (UseAPI->HasMoveVariants())
			{
				return UseAPI->CreateMaterialObject(MoveTemp(CreateMaterialParams));
			}
			else
			{
				return UseAPI->CreateMaterialObject(CreateMaterialParams);
			}
		}
	}
	return FCreateMaterialObjectResult{ ECreateModelingObjectResult::Failed_NoAPIFound };
}





FCreateActorResult UE::Modeling::CreateNewActor(UInteractiveToolManager* ToolManager, FCreateActorParams&& CreateActorParams)
{
	if (ensure(ToolManager))
	{
		UModelingObjectsCreationAPI* UseAPI = ToolManager->GetContextObjectStore()->FindContext<UModelingObjectsCreationAPI>();
		if (UseAPI)
		{
			if (UseAPI->HasMoveVariants())
			{
				return UseAPI->CreateNewActor(MoveTemp(CreateActorParams));
			}
			else
			{
				return UseAPI->CreateNewActor(CreateActorParams);
			}
		}
	}
	return FCreateActorResult{ ECreateModelingObjectResult::Failed_NoAPIFound };
}



FCreateComponentResult UE::Modeling::CreateNewComponentOnActor(UInteractiveToolManager* ToolManager, FCreateComponentParams&& CreateComponentParams)
{
	if (ensure(ToolManager))
	{
		UModelingObjectsCreationAPI* UseAPI = ToolManager->GetContextObjectStore()->FindContext<UModelingObjectsCreationAPI>();
		if (UseAPI)
		{
			if (UseAPI->HasMoveVariants())
			{
				return UseAPI->CreateNewComponentOnActor(MoveTemp(CreateComponentParams));
			}
			else
			{
				return UseAPI->CreateNewComponentOnActor(CreateComponentParams);
			}
		}
	}
	return FCreateComponentResult{ ECreateModelingObjectResult::Failed_NoAPIFound };
}




FString UE::Modeling::GetComponentAssetBaseName(UActorComponent* Component, bool bRemoveAutoGeneratedSuffixes)
{
	if (!ensure(Component != nullptr))
	{
		return TEXT("InvalidComponent");
	}

	// default to the Actor Name (or Label in Editor) if this is the unique root component, otherwise use component name
	FString ResultName = (Component->GetOwner()->GetRootComponent() == Component) ? 
		Component->GetOwner()->GetActorNameOrLabel() : Component->GetName();
	if (bRemoveAutoGeneratedSuffixes)
	{
		ResultName = UE::Modeling::StripGeneratedAssetSuffixFromName(ResultName);
	}

	// If this is a static mesh component, get the asset name instead.
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent != nullptr)
	{
		UStaticMesh* SourceMesh = StaticMeshComponent->GetStaticMesh();
		if (SourceMesh)
		{
			FString AssetName = FPaths::GetBaseFilename(SourceMesh->GetName());
			ResultName = (bRemoveAutoGeneratedSuffixes) ? UE::Modeling::StripGeneratedAssetSuffixFromName(AssetName) : AssetName;
		}
	}

	return ResultName;
}




FString UE::Modeling::StripGeneratedAssetSuffixFromName(FString InputName)
{
	// find final '_'
	int32 Index;
	if (!InputName.FindLastChar('_', Index))
	{
		return InputName;
	}
	// check that remaining characters are hex digits (from UUID)
	int32 Len = InputName.Len();
	int32 Count = 0, Letters = 0, Numbers = 0;
	for (int32 k = Index + 1; k < Len; ++k)
	{
		if (FChar::IsHexDigit(InputName[k]) == false)
		{
			return InputName;
		}
		Count++;
		if (FChar::IsDigit(InputName[k]))
		{
			Numbers++;
		}
		else
		{
			Letters++;
		}
	}
	// currently assuming appended UUID is at least 8 characters
	if (Numbers == 0 || Letters == 0 || Count < 8)
	{
		return InputName;
	}

	return InputName.Left(Index);
}





FString UE::Modeling::GenerateRandomShortHexString(int32 NumChars)
{
	int32 FailCount = 0;
	while (FailCount++ < 10)
	{
		FGuid Guid = FGuid::NewGuid();
		FString GuidString = Guid.ToString(EGuidFormats::UniqueObjectGuid).ToUpper();
		FString Result;
		int32 Digits = 0, Letters = 0;
		for (int32 k = 0; k < GuidString.Len(); ++k)
		{
			TCHAR Character = GuidString[k];
			if (FChar::IsHexDigit(Character))
			{
				Result.AppendChar(Character);
				if (FChar::IsDigit(Character))
				{
					Digits++;
				}
				else
				{
					Letters++;
				}
			}
			if (Result.Len() == NumChars)
			{
				if (Digits > 0 && Letters > 0)
				{
					return Result;
				}
				break;		// exit loop
			}
		}
	}
	return TEXT("BADGUID1");
}
