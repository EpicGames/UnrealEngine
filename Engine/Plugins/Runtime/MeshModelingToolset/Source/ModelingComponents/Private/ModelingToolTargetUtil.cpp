// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolTargetUtil.h"

#include "ToolTargets/ToolTarget.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/AssetBackedTarget.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "TargetInterfaces/PhysicsDataSource.h"

#include "ModelingObjectsCreationAPI.h"

#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "GameFramework/Volume.h"
#include "Components/DynamicMeshComponent.h"

#include "MeshConversionOptions.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "StaticMeshAttributes.h"
#if WITH_EDITOR
#include "StaticMeshCompiler.h"
#include "SkinnedAssetCompiler.h"

#include "Editor/UnrealEdEngine.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"
#endif
#include "Engine/World.h"

#include "DynamicMesh/NonManifoldMappingSupport.h"

#define LOCTEXT_NAMESPACE "ModelingToolTargetUtil"

using namespace UE::Geometry;

namespace UE::ToolTarget::Internal
{
	static TAutoConsoleVariable<bool> CVarCapturePostEditChangeInTransactions(
	TEXT("modeling.CapturePostEditChangeInTransactions"),
	true,
	TEXT("When true, PostEditChange will be included in tool-target based tool transactions."));

	// Scoped override toggled by FScopedSuppressPostEditChangeCapture.
	static bool bSuppressPostEditChangeCapture = false;
}

AActor* UE::ToolTarget::GetTargetActor(UToolTarget* Target)
{
	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		return TargetComponent->GetOwnerActor();
	}
	ensure(false);
	return nullptr;
}

void UE::ToolTarget::SafeDeleteActor(AActor* Actor)
{
	if (!Actor)
	{
		// no actor to delete
		return;
	}
	if (UWorld* ActorWorld = Actor->GetWorld())
	{
#if WITH_EDITOR
		// Attempt to delete in the standard editor way
		if (GIsEditor && GUnrealEd)
		{
			GUnrealEd->DeleteActors(TArray{ Actor }, ActorWorld, GUnrealEd->GetSelectedActors()->GetElementSelectionSet());
		}
		else
#endif
		// Alternate non-editor deletion path
		{
			ActorWorld->DestroyActor(Actor);
		}
	}
}

UPrimitiveComponent* UE::ToolTarget::GetTargetComponent(UToolTarget* Target)
{
	IPrimitiveComponentBackedTarget* TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		return TargetComponent->GetOwnerComponent();
	}
	return nullptr;
}

USceneComponent* UE::ToolTarget::GetTargetSceneComponent(UToolTarget* Target)
{
	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		return TargetComponent->GetOwnerSceneComponent();
	}
	return nullptr;
}

FString UE::ToolTarget::GetHumanReadableName(UToolTarget* Target)
{
	IAssetBackedTarget* TargetAsset = Cast<IAssetBackedTarget>(Target);
	if (TargetAsset)
	{
		return TargetAsset->GetSourceData()->GetName();
	}

	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		return TargetComponent->GetOwnerSceneComponent()->GetFullGroupName(false);
	}

	return FString("");
}

bool UE::ToolTarget::HideSourceObject(UToolTarget* Target)
{
	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		TargetComponent->SetOwnerVisibility(false);
		return true;
	}
	ensure(false);
	return false;
}

bool UE::ToolTarget::ShowSourceObject(UToolTarget* Target)
{
	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		TargetComponent->SetOwnerVisibility(true);
		return true;
	}
	ensure(false);
	return false;
}


bool UE::ToolTarget::SetSourceObjectVisible(UToolTarget* Target, bool bVisible)
{
	if (bVisible)
	{
		return ShowSourceObject(Target);
	}
	else
	{
		return HideSourceObject(Target);
	}
}


FTransform3d UE::ToolTarget::GetLocalToWorldTransform(UToolTarget* Target)
{
	ISceneComponentBackedTarget* TargetComponent = Cast<ISceneComponentBackedTarget>(Target);
	if (TargetComponent)
	{
		return (FTransform3d)TargetComponent->GetWorldTransform();
	}
	return FTransform3d();
}

FComponentMaterialSet UE::ToolTarget::GetMaterialSet(UToolTarget* Target, bool bPreferAssetMaterials)
{
	FComponentMaterialSet MaterialSet;
	IMaterialProvider* MaterialProvider = Cast<IMaterialProvider>(Target);
	if (ensure(MaterialProvider))
	{
		MaterialProvider->GetMaterialSet(MaterialSet, bPreferAssetMaterials);
	}
	return MaterialSet;
}


bool UE::ToolTarget::CommitMaterialSetUpdate(
	UToolTarget* Target,
	const FComponentMaterialSet& UpdatedMaterials,
	bool bApplyToAsset)
{
	IMaterialProvider* MaterialProvider = Cast<IMaterialProvider>(Target);
	if (MaterialProvider)
	{
		return MaterialProvider->CommitMaterialSetUpdate(UpdatedMaterials, bApplyToAsset);
	}
	return false;
}


TArray<EMeshLODIdentifier> UE::ToolTarget::GetMeshDescriptionLODs(UToolTarget* Target, bool& bOutTargetSupportsLODs, bool bOnlyReturnDefaultLOD, bool bExcludeAutoGeneratedLODs)
{
	if (!bOnlyReturnDefaultLOD)
	{
		if (IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target))
		{
			bOutTargetSupportsLODs = MeshDescriptionProvider->SupportsLODs();
			return MeshDescriptionProvider->GetAvailableLODs(bExcludeAutoGeneratedLODs);
		}
	}
	bOutTargetSupportsLODs = false;
	TArray<EMeshLODIdentifier> DefaultArray{ EMeshLODIdentifier::Default };
	return DefaultArray;
}


EMeshLODIdentifier UE::ToolTarget::GetTargetMeshDescriptionLOD(
	UToolTarget* Target,
	bool& bOutTargetSupportsLODs)
{
	if (IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target))
	{
		bOutTargetSupportsLODs = MeshDescriptionProvider->SupportsLODs();
		return MeshDescriptionProvider->GetMeshDescriptionLOD();
	}

	bOutTargetSupportsLODs = false;
	return EMeshLODIdentifier::Default;
}

const FMeshDescription* UE::ToolTarget::GetMeshDescription(UToolTarget* Target, const FGetMeshParameters& GetMeshParams)
{
	static FMeshDescription EmptyMeshDescription;

	IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target);
	if (MeshDescriptionProvider)
	{
		return MeshDescriptionProvider->GetMeshDescription(GetMeshParams);
	}
	ensure(false);
	return &EmptyMeshDescription;
}

FMeshDescription UE::ToolTarget::GetEmptyMeshDescription(UToolTarget* Target)
{
	static FMeshDescription EmptyMeshDescription;

	IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target);
	if (MeshDescriptionProvider)
	{
		return MeshDescriptionProvider->GetEmptyMeshDescription();
	}
	ensure(false);
	return EmptyMeshDescription;
}

FMeshDescription UE::ToolTarget::GetMeshDescriptionCopy(UToolTarget* Target, const FGetMeshParameters& GetMeshParams)
{
	IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target);
	if (MeshDescriptionProvider)
	{
		return MeshDescriptionProvider->GetMeshDescriptionCopy(GetMeshParams);
	}
	ensure(false);
	return FMeshDescription();
}

FDynamicMesh3 UE::ToolTarget::GetDynamicMeshCopy(UToolTarget* Target, bool bWantMeshTangents)
{
	FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = bWantMeshTangents;
	return GetDynamicMeshCopy(Target, GetMeshParams);
}

int32 UE::ToolTarget::GetTriangleCount(UToolTarget* Target)
{
	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource)
	{
		UDynamicMesh* DynamicMesh = DynamicMeshSource->GetDynamicMeshContainer();
		return DynamicMesh->GetTriangleCount();
	}

	IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target);
	if (MeshDescriptionProvider)
	{
		if (const FMeshDescription* MeshDescription = MeshDescriptionProvider->GetMeshDescription())
		{
			return MeshDescription->Triangles().Num();
		}
	}

	return 0;
}

FDynamicMesh3 UE::ToolTarget::GetDynamicMeshCopy(UToolTarget* Target, const FGetMeshParameters& InGetMeshParams)
{
	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource)
	{
		UDynamicMesh* DynamicMesh = DynamicMeshSource->GetDynamicMeshContainer();
		FDynamicMesh3 Mesh;
		DynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { Mesh = ReadMesh; });
		return Mesh;
	}

	IDynamicMeshProvider* DynamicMeshProvider = Cast<IDynamicMeshProvider>(Target);
	if (DynamicMeshProvider)
	{
		return DynamicMeshProvider->GetDynamicMesh(InGetMeshParams);
	}

	IMeshDescriptionProvider* MeshDescriptionProvider = Cast<IMeshDescriptionProvider>(Target);
	FDynamicMesh3 Mesh(EMeshComponents::FaceGroups);
	Mesh.EnableAttributes();
	if (MeshDescriptionProvider)
	{
		FMeshDescriptionToDynamicMesh Converter;
		Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
		Converter.bUseCompactedPolygonGroupIDValues = true;
		Converter.SetPolygonGroupToMaterialIndexMap(MeshDescriptionProvider->GetPolygonGroupToMaterialIndexMap());
		if (InGetMeshParams.bWantMeshTangents)
		{
			const FMeshDescription MeshDescriptionCopy = MeshDescriptionProvider->GetMeshDescriptionCopy(InGetMeshParams);
			Converter.Convert(&MeshDescriptionCopy, Mesh, InGetMeshParams.bWantMeshTangents);
		}
		else
		{
			Converter.Convert(MeshDescriptionProvider->GetMeshDescription(InGetMeshParams), Mesh);
		}

		return Mesh;
	}

	ensure(false);
	return Mesh;
}


UE::ToolTarget::EDynamicMeshUpdateResult UE::ToolTarget::CommitMeshDescriptionUpdate(UToolTarget* Target, const FMeshDescription* UpdatedMesh, const FComponentMaterialSet* UpdatedMaterials, const FCommitMeshParameters& CommitParams)
{
	if (!ensure(UpdatedMesh != nullptr))
	{
		return EDynamicMeshUpdateResult::Failed;
	}

	IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Target);
	if (!ensure(MeshDescriptionCommitter))
	{
		return EDynamicMeshUpdateResult::Failed;
	}

	if (UpdatedMaterials != nullptr)
	{
		CommitMaterialSetUpdate(Target, *UpdatedMaterials, true);
	}

	bool bOK = MeshDescriptionCommitter->CommitMeshDescription(*UpdatedMesh, CommitParams);
	return (bOK) ? EDynamicMeshUpdateResult::Ok : EDynamicMeshUpdateResult::Failed;
}


UE::ToolTarget::EDynamicMeshUpdateResult UE::ToolTarget::CommitMeshDescriptionUpdate(UToolTarget* Target, FMeshDescription&& UpdatedMesh, const FCommitMeshParameters& CommitParams)
{
	if (IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Target))
	{
		bool bOK = MeshDescriptionCommitter->CommitMeshDescription(MoveTemp(UpdatedMesh), CommitParams);
		return (bOK) ? EDynamicMeshUpdateResult::Ok : EDynamicMeshUpdateResult::Failed;
	}
	return EDynamicMeshUpdateResult::Failed;
}


UE::ToolTarget::EDynamicMeshUpdateResult UE::ToolTarget::CommitMeshDescriptionUpdateViaDynamicMesh(
	UToolTarget* Target, 
	const UE::Geometry::FDynamicMesh3& UpdatedMesh, 
	bool bHaveModifiedTopology,
	const FCommitMeshParameters& CommitParams)
{
	IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Target);
	if (!ensure(MeshDescriptionCommitter))
	{
		return EDynamicMeshUpdateResult::Failed;
	}

	FDynamicMeshToMeshDescription Converter;
	Converter.SetMaterialIDMapFromInverseMap(MeshDescriptionCommitter->GetPolygonGroupToMaterialIndexMap());
	FMeshDescription ConvertedMesh;
	if (bHaveModifiedTopology)
	{
		ConvertedMesh = UE::ToolTarget::GetEmptyMeshDescription(Target);		
		Converter.Convert(&UpdatedMesh, ConvertedMesh);
	}
	else
	{
		// FGetMeshParameters should never need to be initialized to anything other than the 
		// default because we are either (1) ignoring the tangents, in which case, we don't
		// want to update them anyway (2) replacing them, in which case, we don't need to
		// compute them

		ConvertedMesh = UE::ToolTarget::GetMeshDescriptionCopy(Target, FGetMeshParameters());
		Converter.Update(&UpdatedMesh, ConvertedMesh);
	}
	

	bool bOK = MeshDescriptionCommitter->CommitMeshDescription(MoveTemp(ConvertedMesh), CommitParams);
	return (bOK) ? EDynamicMeshUpdateResult::Ok : EDynamicMeshUpdateResult::Failed;
}



void UE::ToolTarget::Internal::CommitDynamicMeshViaIPersistentDynamicMeshSource(
	IPersistentDynamicMeshSource& DynamicMeshSource, 
	const FDynamicMesh3& UpdatedMesh, bool bHaveModifiedTopology)
{
	UDynamicMesh* DynamicMesh = DynamicMeshSource.GetDynamicMeshContainer();
	TUniquePtr<FDynamicMesh3> CurrentMesh = DynamicMesh->ExtractMesh();
	TSharedPtr<FDynamicMesh3> CurrentMeshShared(CurrentMesh.Release());

	DynamicMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh.CompactCopy(UpdatedMesh);
			FNonManifoldMappingSupport::RemoveAllNonManifoldMappingData(EditMesh);
		});

	TSharedPtr<FDynamicMesh3> NewMeshShared = MakeShared<FDynamicMesh3>();
	DynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { *NewMeshShared = ReadMesh; });

	TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(CurrentMeshShared, NewMeshShared);

	DynamicMeshSource.CommitDynamicMeshChange(MoveTemp(ReplaceChange), LOCTEXT("CommitDynamicMeshUpdate_MeshSource", "Update Mesh"));

	// todo support bModifiedTopology flag?
}

UE::ToolTarget::EDynamicMeshUpdateResult UE::ToolTarget::CommitDynamicMeshUpdate(
	UToolTarget* Target, const FDynamicMesh3& UpdatedMesh, 
	bool bHaveModifiedTopology,
	const FConversionToMeshDescriptionOptions& ConversionOptions,
	const FComponentMaterialSet* UpdatedMaterials)
{
	if (UpdatedMaterials != nullptr)
	{
		CommitMaterialSetUpdate(Target, *UpdatedMaterials, true);
	}

	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource)
	{
		Internal::CommitDynamicMeshViaIPersistentDynamicMeshSource(
			*DynamicMeshSource, UpdatedMesh, bHaveModifiedTopology);
		
		return EDynamicMeshUpdateResult::Ok;
	}

	IDynamicMeshCommitter* DynamicMeshCommitter = Cast<IDynamicMeshCommitter>(Target);
	if (DynamicMeshCommitter)
	{
		IDynamicMeshCommitter::FDynamicMeshCommitInfo CommitInfo;
		CommitInfo.bTopologyChanged = bHaveModifiedTopology;
		CommitInfo.bPolygroupsChanged = ConversionOptions.bSetPolyGroups;
		CommitInfo.bPositionsChanged = ConversionOptions.bUpdatePositions;
		CommitInfo.bNormalsChanged = ConversionOptions.bUpdateNormals;
		CommitInfo.bTangentsChanged = ConversionOptions.bUpdateTangents;
		CommitInfo.bUVsChanged = ConversionOptions.bUpdateUVs;
		CommitInfo.bVertexColorsChanged = ConversionOptions.bUpdateVtxColors;
		CommitInfo.bTransformVertexColorsSRGBToLinear = ConversionOptions.bTransformVtxColorsSRGBToLinear;

		DynamicMeshCommitter->CommitDynamicMesh(UpdatedMesh, CommitInfo);

		return EDynamicMeshUpdateResult::Ok;
	}

	IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Target);
	if (MeshDescriptionCommitter)
	{
		FMeshDescription ConvertedMesh;
		FDynamicMeshToMeshDescription Converter(ConversionOptions);
		Converter.SetMaterialIDMapFromInverseMap(MeshDescriptionCommitter->GetPolygonGroupToMaterialIndexMap());
		if (!bHaveModifiedTopology)
		{
			Converter.UpdateUsingConversionOptions(&UpdatedMesh, ConvertedMesh);
		}
		else
		{
			Converter.Convert(&UpdatedMesh, ConvertedMesh);
		}

		bool bOK = MeshDescriptionCommitter->CommitMeshDescription(MoveTemp(ConvertedMesh));
		return (bOK) ? EDynamicMeshUpdateResult::Ok : EDynamicMeshUpdateResult::Failed;
	}

	ensure(false);
	return EDynamicMeshUpdateResult::Failed;
}





UE::ToolTarget::EDynamicMeshUpdateResult UE::ToolTarget::CommitDynamicMeshUVUpdate(UToolTarget* Target, const UE::Geometry::FDynamicMesh3* UpdatedMesh)
{
	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource)
	{
		// just do a full mesh update for now
		// todo actually only update UVs? 
		return CommitDynamicMeshUpdate(Target, *UpdatedMesh, true);
	}

	IDynamicMeshCommitter* DynamicMeshCommitter = Cast<IDynamicMeshCommitter>(Target);
	if (DynamicMeshCommitter)
	{
		IDynamicMeshCommitter::FDynamicMeshCommitInfo CommitInfo(false);
		CommitInfo.bUVsChanged = true;

		DynamicMeshCommitter->CommitDynamicMesh(*UpdatedMesh, CommitInfo);

		return EDynamicMeshUpdateResult::Ok;
	}

	IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Target);
	if (!ensure(MeshDescriptionCommitter))
	{
		return EDynamicMeshUpdateResult::Failed;
	}

	FMeshDescription NewMeshDescription = UE::ToolTarget::GetMeshDescriptionCopy(Target);
	bool bVerticesOnly = false;
	bool bAttributesOnly = true;
	FDynamicMeshToMeshDescription Converter;
	Converter.SetMaterialIDMapFromInverseMap(MeshDescriptionCommitter->GetPolygonGroupToMaterialIndexMap());
	if (FDynamicMeshToMeshDescription::HaveMatchingElementCounts(UpdatedMesh, &NewMeshDescription, bVerticesOnly, bAttributesOnly))
	{
		Converter.UpdateAttributes(UpdatedMesh, NewMeshDescription, false, false, true/*update uvs*/);
	}
	else
	{
		// must have been duplicate tris in the mesh description; we can't count on 1-to-1 mapping of TriangleIDs.  Just convert 
		Converter.Convert(UpdatedMesh, NewMeshDescription);
	}

	bool bOK = MeshDescriptionCommitter->CommitMeshDescription(MoveTemp(NewMeshDescription));
	return (bOK) ? EDynamicMeshUpdateResult::Ok : EDynamicMeshUpdateResult::Failed;
}





UE::ToolTarget::EDynamicMeshUpdateResult UE::ToolTarget::CommitDynamicMeshNormalsUpdate(
	UToolTarget* Target, 
	const UE::Geometry::FDynamicMesh3* UpdatedMesh,
	bool bUpdateTangents)
{
	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource)
	{
		// just do a full mesh update for now
		return CommitDynamicMeshUpdate(Target, *UpdatedMesh, true);
	}

	IDynamicMeshCommitter* DynamicMeshCommitter = Cast<IDynamicMeshCommitter>(Target);
	if (DynamicMeshCommitter)
	{
		IDynamicMeshCommitter::FDynamicMeshCommitInfo CommitInfo(false);
		CommitInfo.bNormalsChanged = true;
		CommitInfo.bTangentsChanged = bUpdateTangents;
		DynamicMeshCommitter->CommitDynamicMesh(*UpdatedMesh, CommitInfo);
		return EDynamicMeshUpdateResult::Ok;
	}

	IMeshDescriptionCommitter* MeshDescriptionCommitter = Cast<IMeshDescriptionCommitter>(Target);
	if (!ensure(MeshDescriptionCommitter))
	{
		return EDynamicMeshUpdateResult::Failed;
	}

	FMeshDescription NewMeshDescription = UE::ToolTarget::GetMeshDescriptionCopy(Target);
	bool bVerticesOnly = false;
	bool bAttributesOnly = true;
	FDynamicMeshToMeshDescription Converter;
	Converter.SetMaterialIDMapFromInverseMap(MeshDescriptionCommitter->GetPolygonGroupToMaterialIndexMap());
	if (FDynamicMeshToMeshDescription::HaveMatchingElementCounts(UpdatedMesh, &NewMeshDescription, bVerticesOnly, bAttributesOnly))
	{
		Converter.UpdateAttributes(UpdatedMesh, NewMeshDescription, true, bUpdateTangents, false);
	}
	else
	{
		// must have been duplicate tris in the mesh description; we can't count on 1-to-1 mapping of TriangleIDs.  Just convert 
		Converter.Convert(UpdatedMesh, NewMeshDescription);
	}

	bool bOK = MeshDescriptionCommitter->CommitMeshDescription(MoveTemp(NewMeshDescription));
	return (bOK) ? EDynamicMeshUpdateResult::Ok : EDynamicMeshUpdateResult::Failed;
}



bool UE::ToolTarget::SupportsIncrementalMeshChanges(UToolTarget* Target)
{
	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource)
	{
		return DynamicMeshSource->HasDynamicMeshComponent();
	}
	return false;
}


bool UE::ToolTarget::ApplyIncrementalMeshEditChange(
	UToolTarget* Target,
	TFunctionRef<bool(FDynamicMesh3& EditMesh, UObject* TransactionTarget)> MeshEditChangeFunc )
{
	IPersistentDynamicMeshSource* DynamicMeshSource = Cast<IPersistentDynamicMeshSource>(Target);
	if (DynamicMeshSource && DynamicMeshSource->HasDynamicMeshComponent())
	{
		bool bOK;
		UDynamicMeshComponent* Component = DynamicMeshSource->GetDynamicMeshComponent();
		Component->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			bOK = MeshEditChangeFunc(EditMesh, Component);
		});
		return bOK;
	}
	return false;
}


bool UE::ToolTarget::ConfigureCreateMeshObjectParams(UToolTarget* SourceTarget, FCreateMeshObjectParams& DerivedParamsOut)
{
	ISceneComponentBackedTarget* ComponentTarget = Cast<ISceneComponentBackedTarget>(SourceTarget);
	if (ComponentTarget)
	{
		if (Cast<UStaticMeshComponent>(ComponentTarget->GetOwnerSceneComponent()) != nullptr)
		{
			DerivedParamsOut.TypeHint = ECreateObjectTypeHint::StaticMesh;
			return true;
		}

		if (Cast<UDynamicMeshComponent>(ComponentTarget->GetOwnerSceneComponent()) != nullptr)
		{
			DerivedParamsOut.TypeHint = ECreateObjectTypeHint::DynamicMeshActor;
			return true;
		}

		AVolume* VolumeActor = Cast<AVolume>(ComponentTarget->GetOwnerActor());
		if (VolumeActor != nullptr)
		{
			DerivedParamsOut.TypeHint = ECreateObjectTypeHint::Volume;
			DerivedParamsOut.TypeHintClass = VolumeActor->GetClass();
			return true;
		}
	}
	return false;
}


UBodySetup* UE::ToolTarget::GetPhysicsBodySetup(UToolTarget* Target)
{
	if (IPhysicsDataSource* PhysicsSource = Cast<IPhysicsDataSource>(Target))
	{
		return PhysicsSource->GetBodySetup();
	}
	return nullptr;
}

IInterface_CollisionDataProvider* UE::ToolTarget::GetPhysicsCollisionDataProvider(UToolTarget* Target)
{
	if (IPhysicsDataSource* PhysicsSource = Cast<IPhysicsDataSource>(Target))
	{
		return PhysicsSource->GetComplexCollisionProvider();
	}
	return nullptr;
}

UStaticMesh* UE::ToolTarget::GetStaticMeshFromTargetIfAvailable(UToolTarget* Target)
{
	IStaticMeshBackedTarget* TargetStaticMeshTarget = Cast<IStaticMeshBackedTarget>(Target);
	UStaticMesh* TargetStaticMesh = TargetStaticMeshTarget ? TargetStaticMeshTarget->GetStaticMesh() : nullptr;
	return TargetStaticMesh;
}

USkeletalMesh* UE::ToolTarget::GetSkeletalMeshFromTargetIfAvailable(UToolTarget* Target)
{
	ISkeletalMeshBackedTarget* TargetSkeletalMeshTarget = Cast<ISkeletalMeshBackedTarget>(Target);
	USkeletalMesh* TargetSkeletalMesh = TargetSkeletalMeshTarget ? TargetSkeletalMeshTarget->GetSkeletalMesh() : nullptr;
	return TargetSkeletalMesh;
}

#if WITH_EDITOR
void UE::ToolTarget::Internal::PostEditChangeWithConditionalUndo(UObject* Object)
{
	if (CVarCapturePostEditChangeInTransactions->GetBool() && !bSuppressPostEditChangeCapture)
	{
		Object->PostEditChange();
	}
	else
	{
		TGuardValue<ITransaction*> SuppressTransaction(GUndo, nullptr);
		Object->PostEditChange();

		// For StaticMesh & SkeletalMesh, we additionally block on the StaticMesh/SkeletalMesh build
		// to ensure it completes while transactions are disabled. This is necessary because
		// closing out the transaction scope will force the compile to complete outside of this scope
		// where transactions are still captured.
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(Object))
		{
			FStaticMeshCompilingManager::Get().FinishCompilation({StaticMesh});
		}
		else if (USkinnedAsset* SkinnedAsset = Cast<USkinnedAsset>(Object))
		{
			FSkinnedAssetCompilingManager::Get().FinishCompilation({SkinnedAsset});
		}
	}
}

UE::ToolTarget::Internal::FScopedSuppressPostEditChangeCapture::FScopedSuppressPostEditChangeCapture()
	: bSavedValue(bSuppressPostEditChangeCapture)
{
	bSuppressPostEditChangeCapture = true;
}

UE::ToolTarget::Internal::FScopedSuppressPostEditChangeCapture::~FScopedSuppressPostEditChangeCapture()
{
	bSuppressPostEditChangeCapture = bSavedValue;
}
#endif


#undef LOCTEXT_NAMESPACE