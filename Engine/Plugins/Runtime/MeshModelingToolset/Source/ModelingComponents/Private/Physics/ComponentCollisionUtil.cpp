// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Physics/ComponentCollisionUtil.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "Generators/SphereGenerator.h"
#include "Generators/BoxSphereGenerator.h"
#include "Generators/MarchingCubes.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Generators/CapsuleGenerator.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "VectorUtil.h"

#include "Physics/PhysicsDataCollection.h"
#include "Physics/CollisionGeometryConversion.h"

#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/ShapeElem.h"
#include "UObject/UObjectIterator.h"

using namespace UE::Geometry;

bool UE::Geometry::ComponentTypeSupportsCollision(
	const UPrimitiveComponent* Component, EComponentCollisionSupportLevel SupportLevel)
{
	// currently only supporting StaticMeshComponent and DynamicMeshComponent
	if (Cast<UStaticMeshComponent>(Component) != nullptr)
	{
		return true; // ReadOnly and ReadWrite both supported
	}
	if (Cast<UDynamicMeshComponent>(Component) != nullptr)
	{
		return true; // ReadOnly and ReadWrite both supported
	}
	if (Cast<UBrushComponent>(Component) != nullptr)
	{
		return SupportLevel == EComponentCollisionSupportLevel::ReadOnly; // do not support write on BrushComponent
	}
	return false;
}


FComponentCollisionSettings UE::Geometry::GetCollisionSettings(const UPrimitiveComponent* Component)
{
	FComponentCollisionSettings Settings;

	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component) )
	{
		if (const UBodySetup* BodySetup = GetBodySetup(Component))
		{
			Settings.CollisionTypeFlag = (int32)BodySetup->CollisionTraceFlag;
		}
	}
	else if (const UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		Settings.CollisionTypeFlag = DynamicMeshComponent->CollisionType;
	}
	else if (const UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		if (BrushComponent->BrushBodySetup)
		{
			Settings.CollisionTypeFlag = (int32)BrushComponent->BrushBodySetup->CollisionTraceFlag;
		}
	}

	return Settings;
}


bool UE::Geometry::GetCollisionShapes(const UPrimitiveComponent* Component, FKAggregateGeom& AggGeom)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		if (const UBodySetup* BodySetup = GetBodySetup(Component))
		{
			AggGeom = BodySetup->AggGeom;
			return true;
		}
	}
	else if (const UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		AggGeom = DynamicMeshComponent->GetSimpleCollisionShapes();
		return true;
	}
	else if (const UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		if (BrushComponent->BrushBodySetup)
		{
			AggGeom = BrushComponent->BrushBodySetup->AggGeom;
			return true;
		}
	}

	return false;
}

bool UE::Geometry::GetCollisionShapes(const UPrimitiveComponent* Component, FSimpleShapeSet3d& ShapeSet)
{
	FKAggregateGeom AggGeom;
	if (UE::Geometry::GetCollisionShapes(Component, AggGeom))
	{
		UE::Geometry::GetShapeSet(AggGeom, ShapeSet);
		return true;
	}

	return false;
}






void UE::Geometry::UpdateSimpleCollision(
	UBodySetup* BodySetup, 
	const FKAggregateGeom* NewGeometry,
	UStaticMesh* StaticMesh,
	FComponentCollisionSettings CollisionSettings)
{
	BodySetup->Modify();
	BodySetup->RemoveSimpleCollision();

	// set new collision geometry
	BodySetup->AggGeom = *NewGeometry;

	// update collision type
	BodySetup->CollisionTraceFlag = (ECollisionTraceFlag)CollisionSettings.CollisionTypeFlag;

	// rebuild physics meshes
	BodySetup->CreatePhysicsMeshes();

	if (StaticMesh != nullptr)
	{
		StaticMesh->RecreateNavCollision();

		// update physics state on all components using this StaticMesh
		for (FThreadSafeObjectIterator Iter(UStaticMeshComponent::StaticClass()); Iter; ++Iter)
		{
			UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(*Iter);
			if (SMComponent->GetStaticMesh() == StaticMesh)
			{
				if (SMComponent->IsPhysicsStateCreated())
				{
					SMComponent->RecreatePhysicsState();
				}
			}
		}

		// mark static mesh as dirty so it gets resaved?
		[[maybe_unused]] bool MarkedDirty = StaticMesh->MarkPackageDirty();

#if WITH_EDITORONLY_DATA
		// mark the static mesh as having customized collision so it is not regenerated on reimport
		StaticMesh->SetCustomizedCollision(CollisionSettings.bIsGeneratedCollision);
#endif // WITH_EDITORONLY_DATA
	}
}




const UBodySetup* UE::Geometry::GetBodySetup(const UPrimitiveComponent* Component)
{
	if (const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
		{
			return StaticMesh->GetBodySetup();
		}
	}
	else if (const UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		return BrushComponent->BrushBodySetup;
	}
	else if (const UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		return DynamicMeshComponent->GetBodySetup();
	}

	return nullptr;
}

UBodySetup* UE::Geometry::GetBodySetup(UPrimitiveComponent* Component)
{
	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
		{
			return StaticMesh->GetBodySetup();
		}
	}
	else if (UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		return BrushComponent->BrushBodySetup;
	}
	else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		return DynamicMeshComponent->GetBodySetup();
	}

	return nullptr;
}



bool UE::Geometry::SetSimpleCollision(
	UPrimitiveComponent* Component,
	const FSimpleShapeSet3d* ShapeSet,
	FComponentCollisionSettings CollisionSettings)
{
	FPhysicsDataCollection PhysicsData;
	PhysicsData.InitializeFromComponent(Component, false);
	if (ensure(PhysicsData.SourceComponent.IsValid()) == false || ensure(ShapeSet != nullptr) == false )
	{
		return false;
	}

	PhysicsData.Geometry = *ShapeSet;
	PhysicsData.CopyGeometryToAggregate();

	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
		{
			if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
			{
				UpdateSimpleCollision(BodySetup, &PhysicsData.AggGeom, StaticMesh, CollisionSettings);
				return true;
			}
		}
	}
	else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		DynamicMeshComponent->Modify();
		UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup();
		BodySetup->Modify();
		DynamicMeshComponent->CollisionType = (ECollisionTraceFlag)CollisionSettings.CollisionTypeFlag;
		DynamicMeshComponent->SetSimpleCollisionShapes(PhysicsData.AggGeom, true);
		return true;
	}

	return false;
}



bool UE::Geometry::TransformSimpleCollision(
	UPrimitiveComponent* Component,
	const FTransform3d& Transform)
{
	FPhysicsDataCollection PhysicsData;
	PhysicsData.InitializeFromComponent(Component, true);
	if ( ensure(PhysicsData.SourceComponent.IsValid()) == false)
	{
		return false;
	}

	PhysicsData.Geometry.ApplyTransform(Transform);
	PhysicsData.ClearAggregate();
	PhysicsData.CopyGeometryToAggregate();

	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
	{
		FComponentCollisionSettings Settings = GetCollisionSettings(Component);
		Settings.bIsGeneratedCollision = false;
		if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
		{
			if (UBodySetup* BodySetup = StaticMesh->GetBodySetup())
			{
				UpdateSimpleCollision(BodySetup, &PhysicsData.AggGeom, StaticMesh, Settings);
				return true;
			}
		}
	}
	else if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		DynamicMeshComponent->Modify();
		UBodySetup* BodySetup = DynamicMeshComponent->GetBodySetup();
		BodySetup->Modify();
		DynamicMeshComponent->SetSimpleCollisionShapes(PhysicsData.AggGeom, true);
		return true;
	}
	return false;
}




bool UE::Geometry::AppendSimpleCollision(
	const UPrimitiveComponent* Component,
	FSimpleShapeSet3d* ShapeSetOut,
	const FTransform3d& Transform)
{
	FPhysicsDataCollection PhysicsData;
	PhysicsData.InitializeFromComponent(Component, true);
	if (ensure(PhysicsData.SourceComponent.IsValid()) == false || ensure(ShapeSetOut != nullptr) == false)
	{
		return false;
	}

	ShapeSetOut->Append(PhysicsData.Geometry, Transform);
	return true;
}


bool UE::Geometry::AppendSimpleCollision(
	const UPrimitiveComponent* Component,
	FSimpleShapeSet3d* ShapeSetOut,
	const TArray<FTransform3d>& TransformSeqeuence)
{
	FPhysicsDataCollection PhysicsData;
	PhysicsData.InitializeFromComponent(Component, true);
	if (ensure(PhysicsData.SourceComponent.IsValid()) == false || ensure(ShapeSetOut != nullptr) == false)
	{
		return false;
	}

	ShapeSetOut->Append(PhysicsData.Geometry, TransformSeqeuence);

	return true;
}

namespace UE::Private::ConvertCollisionInternal
{
	static void ConvertSimpleCollisionToMeshesHelper(
		const FKAggregateGeom& AggGeom, FVector ExternalScale,
		const FSimpleCollisionTriangulationSettings& TriangulationSettings,
		const FSimpleCollisionToMeshAttributeSettings& MeshAttributeSettings,
		TFunctionRef<void(int32 Index, const FKShapeElem& ShapeElem, FDynamicMesh3&)> DynamicMeshCallback,
		TFunctionRef<bool(const FKShapeElem&)> IncludeElement)
	{
		// Clear other attribute related settings if we're not enabling attributes
		bool bInitializeConvexAndLevelSetUVs = MeshAttributeSettings.bEnableAttributes && MeshAttributeSettings.bInitializeConvexAndLevelSetUVs;
		bool bSetToPerTriangleNormals = MeshAttributeSettings.bEnableAttributes && MeshAttributeSettings.bSetToPerTriangleNormals;

		// Helpers to transform vertex and normal arrays / invert triangle buffers
		auto TransformVerts = [](FTransform Transform, TArrayView<FVector3d> Vertices, TArrayView<FVector3f> Normals = TArrayView<FVector3f>())
		{
			for (FVector3d& V : Vertices)
			{
				V = Transform.TransformPosition(V);
			}
			for (FVector3f& N : Normals)
			{
				N = (FVector3f)VectorUtil::TransformNormal<double>(Transform, (FVector3d)N);
			}
		};
		auto TranslateVerts = [](FVector3d Offset, TArrayView<FVector3d> Vertices)
		{
			for (FVector3d& V : Vertices)
			{
				V += Offset;
			}
		};
		auto InvertTris = [](TArrayView<FIndex3i> Tris)
		{
			for (FIndex3i& Tri : Tris)
			{
				Swap(Tri.B, Tri.C);
			}
		};

		auto RunCallbacks = 
			[&MeshAttributeSettings, &DynamicMeshCallback, bSetToPerTriangleNormals]
			(int32 Index, const FKShapeElem& ShapeElem, FMeshShapeGenerator& Gen)
		{
			if (!MeshAttributeSettings.bEnableAttributes)
			{
				Gen.ResetAttributes(true);
			}
			FDynamicMesh3 ShapeMesh(&Gen);
			if (bSetToPerTriangleNormals)
			{
				FMeshNormals::InitializeMeshToPerTriangleNormals(&ShapeMesh);
			} // otherwise allow the generator's normals to pass through
			DynamicMeshCallback(Index, ShapeElem, ShapeMesh);
		};
			
		int32 ShapeIndex = 0;
		for (const FKSphereElem& Sphere : AggGeom.SphereElems)
		{
			if (!IncludeElement(Sphere))
			{
				continue;
			}
			FVector Center = Sphere.Center * ExternalScale;
			double Radius = FMath::Max(FMathf::ZeroTolerance, Sphere.Radius) * ExternalScale.GetAbsMin();
			if (TriangulationSettings.bUseBoxSphere)
			{
				FBoxSphereGenerator BoxSphereGen;
				BoxSphereGen.Box.Frame.Origin = Center;
				BoxSphereGen.Radius = Radius;
				int32 StepsPerSide = FMath::Max(1, TriangulationSettings.BoxSphereStepsPerSide);
				BoxSphereGen.EdgeVertices = FIndex3i(StepsPerSide, StepsPerSide, StepsPerSide);
				BoxSphereGen.Generate();
				RunCallbacks(ShapeIndex++, Sphere, BoxSphereGen);
			}
			else
			{
				FSphereGenerator LatLongSphereGen;
				LatLongSphereGen.Radius = Radius;
				LatLongSphereGen.NumPhi = LatLongSphereGen.NumTheta = FMath::Max(1, TriangulationSettings.LatLongSphereSteps);
				LatLongSphereGen.bPolygroupPerQuad = false;
				LatLongSphereGen.Generate();
				TranslateVerts(FVector3d(Center), LatLongSphereGen.Vertices);
				RunCallbacks(ShapeIndex++, Sphere, LatLongSphereGen);
			}
		}

		for (const FKBoxElem& Box : AggGeom.BoxElems)
		{
			if (!IncludeElement(Box))
			{
				continue;
			}
			FMinimalBoxMeshGenerator BoxGen;
			BoxGen.Box = UE::Geometry::FOrientedBox3d(
				FFrame3d(FVector3d(Box.Center * ExternalScale), FQuaterniond(Box.Rotation.Quaternion())),
				0.5 * FVector3d(Box.X, Box.Y, Box.Z) * ExternalScale);
			BoxGen.Generate();
			RunCallbacks(ShapeIndex++, Box, BoxGen);
		}

		for (const FKSphylElem& Capsule : AggGeom.SphylElems)
		{
			if (!IncludeElement(Capsule))
			{
				continue;
			}
			FCapsuleGenerator CapsuleGen;
			CapsuleGen.Radius = (double)Capsule.GetScaledRadius(ExternalScale);
			CapsuleGen.SegmentLength = (double)Capsule.GetScaledCylinderLength(ExternalScale);
			CapsuleGen.NumHemisphereArcSteps = TriangulationSettings.CapsuleHemisphereSteps;
			CapsuleGen.NumCircleSteps = TriangulationSettings.CapsuleCircleSteps;
			CapsuleGen.bPolygroupPerQuad = false;
			CapsuleGen.Generate();
			TranslateVerts(FVector3d(0, 0, -0.5 * CapsuleGen.SegmentLength), CapsuleGen.Vertices);
			FTransform Transform = Capsule.GetTransform();
			Transform.ScaleTranslation(ExternalScale);
			// Note: Capsule transform cannot invert; it is generated from a rotation and translation
			TransformVerts(Transform, CapsuleGen.Vertices, CapsuleGen.Normals);
			RunCallbacks(ShapeIndex++, Capsule, CapsuleGen);
		}

		for (const FKConvexElem& Convex : AggGeom.ConvexElems)
		{
			if (!IncludeElement(Convex))
			{
				continue;
			}
			FTransform ElemTransform = Convex.GetTransform();
			ElemTransform.ScaleTranslation(ExternalScale);
			ElemTransform.MultiplyScale3D(ExternalScale);
			const int32 NumVertices = Convex.VertexData.Num();
			const int32 NumTris = Convex.IndexData.Num() / 3;
			bool bInvert = ElemTransform.GetDeterminant() < 0;

			// Create the convex hull mesh from the FKConvexElem vertex/index buffer data

			FDynamicMesh3 ConvexMesh(EMeshComponents::None);
			for (int32 k = 0; k < NumVertices; ++k)
			{
				ConvexMesh.AppendVertex(ElemTransform.TransformPosition(FVector3d(Convex.VertexData[k])));
			}
			int32 NumTriangles = Convex.IndexData.Num() / 3;
			for (int32 k = 0; k < NumTriangles; ++k)
			{
				// Note: Invert tri indices because chaos stores them inverted
				ConvexMesh.AppendTriangle(Convex.IndexData[3 * k], Convex.IndexData[3 * k + 2], Convex.IndexData[3 * k + 1]);
			}
			if (bInvert)
			{
				ConvexMesh.ReverseOrientation(false /*bFlipNormals*/);
			}

			if (MeshAttributeSettings.bEnableAttributes)
			{
				ConvexMesh.EnableTriangleGroups(0);
				ConvexMesh.EnableAttributes();
				if (bSetToPerTriangleNormals)
				{
					FMeshNormals::InitializeMeshToPerTriangleNormals(&ConvexMesh);
				}
				else
				{
					FMeshNormals::InitializeOverlayToPerVertexNormals(ConvexMesh.Attributes()->PrimaryNormals(), false);
				}
				if (bInitializeConvexAndLevelSetUVs)
				{
					FDynamicMeshUVEditor UVEditor(&ConvexMesh, 0, true);
					// TODO: Consider adding more options to control the generated UVs (e.g., scaling, box projection?) here and for level sets below
					UVEditor.SetPerTriangleUVs();
				}
			}
			DynamicMeshCallback(ShapeIndex, Convex, ConvexMesh);

			ShapeIndex++;
		}

		TArray<FVector3f> LevelSetVerts;
		TArray<FIntVector> LevelSetTris;
		for (const FKLevelSetElem& LevelSet : AggGeom.LevelSetElems)
		{
			if (!IncludeElement(LevelSet))
			{
				continue;
			}
			FTransform ElemTransform = LevelSet.GetTransform();
			ElemTransform.ScaleTranslation(ExternalScale);
			ElemTransform.MultiplyScale3D(ExternalScale);
			bool bInvert = ElemTransform.GetDeterminant() < 0;

			FDynamicMesh3 LevelSetMesh(EMeshComponents::None);

			if (!TriangulationSettings.bApproximateLevelSetWithCubes)
			{
				FMarchingCubes MarchingCubes;
				MarchingCubes.CubeSize = LevelSet.GetLevelSet()->GetGrid().Dx().GetMin() * .5;
				MarchingCubes.IsoValue = 0.0f;
				Chaos::FAABB3 Box = LevelSet.GetLevelSet()->BoundingBox();
				MarchingCubes.Bounds = FAxisAlignedBox3d(Box.Min(), Box.Max());
				MarchingCubes.Bounds.Expand(UE_DOUBLE_KINDA_SMALL_NUMBER);
				MarchingCubes.RootMode = ERootfindingModes::SingleLerp;

				MarchingCubes.Implicit = [&LevelSet](const FVector3d& Pt) { return -LevelSet.GetLevelSet()->SignedDistance(Pt); };
				MarchingCubes.Generate();

				if (!MeshAttributeSettings.bEnableAttributes)
				{
					MarchingCubes.ResetAttributes(true);
				}
				TransformVerts(ElemTransform, MarchingCubes.Vertices, MarchingCubes.Normals);

				LevelSetMesh.Copy(&MarchingCubes);
			}
			else // bApproximateLevelSetWithCubes
			{
				LevelSetVerts.Reset();
				LevelSetTris.Reset();
				LevelSet.GetZeroIsosurfaceGridCellFaces(LevelSetVerts, LevelSetTris);

				for (int32 Idx = 0; Idx < LevelSetVerts.Num(); ++Idx)
				{
					LevelSetMesh.AppendVertex(ElemTransform.TransformPosition(FVector3d(LevelSetVerts[Idx])));
				}
				for (int32 Idx = 0; Idx < LevelSetTris.Num(); ++Idx)
				{
					FIntVector Tri = LevelSetTris[Idx];
					LevelSetMesh.AppendTriangle(Tri.X, Tri.Y, Tri.Z);
				}
			}

			if (bInvert)
			{
				LevelSetMesh.ReverseOrientation(false /*bFlipNormals*/);
			}

			if (MeshAttributeSettings.bEnableAttributes)
			{
				LevelSetMesh.EnableTriangleGroups(0);
				LevelSetMesh.EnableAttributes();
				if (bSetToPerTriangleNormals)
				{
					FMeshNormals::InitializeMeshToPerTriangleNormals(&LevelSetMesh);
				}
				else
				{
					FMeshNormals::InitializeOverlayToPerVertexNormals(LevelSetMesh.Attributes()->PrimaryNormals(), false);
				}
				if (bInitializeConvexAndLevelSetUVs)
				{
					FDynamicMeshUVEditor UVEditor(&LevelSetMesh, 0, true);
					UVEditor.SetPerTriangleUVs();
				}
			}
			DynamicMeshCallback(ShapeIndex, LevelSet, LevelSetMesh);

			ShapeIndex++;
		}
	}
}

void UE::Geometry::ConvertSimpleCollisionToDynamicMeshes(
	const FKAggregateGeom& AggGeom, FVector ExternalScale,
	TFunctionRef<void(int32, const FKShapeElem&, FDynamicMesh3&)> PerElementMeshCallback,
	const FSimpleCollisionTriangulationSettings& TriangulationSettings,
	const FSimpleCollisionToMeshAttributeSettings& MeshAttributeSettings)
{
	UE::Private::ConvertCollisionInternal::ConvertSimpleCollisionToMeshesHelper(
		AggGeom, ExternalScale,
		TriangulationSettings,
		MeshAttributeSettings,
		PerElementMeshCallback,
		[](const FKShapeElem&)->bool { return true; });
}

void UE::Geometry::ConvertSimpleCollisionToDynamicMeshes(
	const FKAggregateGeom& AggGeom, FVector ExternalScale,
	TFunctionRef<void(int32, const FKShapeElem&, FDynamicMesh3&)> PerElementMeshCallback,
	TFunctionRef<bool(const FKShapeElem&)> IncludeElement,
	const FSimpleCollisionTriangulationSettings& TriangulationSettings,
	const FSimpleCollisionToMeshAttributeSettings& MeshAttributeSettings)
{
	UE::Private::ConvertCollisionInternal::ConvertSimpleCollisionToMeshesHelper(
		AggGeom, ExternalScale,
		TriangulationSettings,
		MeshAttributeSettings,
		PerElementMeshCallback,
		IncludeElement);
}


void UE::Geometry::ConvertSimpleCollisionToMeshes(
	const FKAggregateGeom& AggGeom,
	FDynamicMesh3& MeshOut,
	const FTransformSequence3d& TransformSequence,
	int32 SphereResolution,
	bool bSetToPerTriangleNormals,
	bool bInitializeConvexAndLevelSetUVs,
	TFunction<void(int, const FDynamicMesh3&)> PerElementMeshCallback,
	bool bApproximateLevelSetWithCubes,
	FVector ExternalScale)
{
	FDynamicMeshEditor Editor(&MeshOut);

	bool bTransformInverts = TransformSequence.WillInvert();

	FSimpleCollisionTriangulationSettings TriangulationSettings;
	TriangulationSettings.InitFromSphereResolution(SphereResolution);
	TriangulationSettings.bApproximateLevelSetWithCubes = bApproximateLevelSetWithCubes;

	FSimpleCollisionToMeshAttributeSettings MeshAttributeSettings;
	MeshAttributeSettings.bEnableAttributes = true;
	MeshAttributeSettings.bSetToPerTriangleNormals = bSetToPerTriangleNormals;
	MeshAttributeSettings.bInitializeConvexAndLevelSetUVs = bInitializeConvexAndLevelSetUVs;

	FMeshIndexMappings Mappings;
	UE::Private::ConvertCollisionInternal::ConvertSimpleCollisionToMeshesHelper(
		AggGeom, ExternalScale,
		TriangulationSettings,
		MeshAttributeSettings,
		[&](int32 Index, const FKShapeElem& Elem, const UE::Geometry::FDynamicMesh3& Mesh)
		{
			if (PerElementMeshCallback)
			{
				PerElementMeshCallback((int)Elem.GetShapeType(), Mesh);
			}
			Mappings.Reset();
			Editor.AppendMesh(&Mesh, Mappings,
				[&TransformSequence](int32 vid, const FVector3d& P) { return TransformSequence.TransformPosition(P); },
				[&TransformSequence](int32 vid, const FVector3d& N) { return TransformSequence.TransformNormal(N); });
		},
		[](const FKShapeElem& Elem) { return true; }
	);
	if (bTransformInverts)
	{
		MeshOut.ReverseOrientation(false);
	}
}



bool UE::Geometry::ConvertComplexCollisionToMeshes(
	IInterface_CollisionDataProvider* CollisionProvider,
	UE::Geometry::FDynamicMesh3& MeshOut,
	const FTransformSequence3d& TransformSeqeuence,
	bool& bFoundMeshErrorsOut,
	bool bWeldEdges,
	bool bSetToPerTriangleNormals)
{
	bFoundMeshErrorsOut = false;
	if (CollisionProvider && CollisionProvider->ContainsPhysicsTriMeshData(true))
	{
		FTriMeshCollisionData CollisionData;
		if (CollisionProvider->GetPhysicsTriMeshData(&CollisionData, true))
		{
			bool bWillInvert = TransformSeqeuence.WillInvert();
			TArray<int32> VertexIDMap;
			for (int32 k = 0; k < CollisionData.Vertices.Num(); ++k)
			{
				int32 VertexID = MeshOut.AppendVertex(TransformSeqeuence.TransformPosition((FVector)CollisionData.Vertices[k]));
				VertexIDMap.Add(VertexID);
			}
			for (const FTriIndices& TriIndices : CollisionData.Indices)
			{
				FIndex3i Triangle(TriIndices.v0, TriIndices.v1, TriIndices.v2);
				int32 TriangleID = MeshOut.AppendTriangle(Triangle);
				if (TriangleID < 0)
				{
					bFoundMeshErrorsOut = true;
					// create new vertices for this triangle
					int32 A = MeshOut.AppendVertex(TransformSeqeuence.TransformPosition(MeshOut.GetVertex(TriIndices.v0)));
					int32 B = MeshOut.AppendVertex(TransformSeqeuence.TransformPosition(MeshOut.GetVertex(TriIndices.v1)));
					int32 C = MeshOut.AppendVertex(TransformSeqeuence.TransformPosition(MeshOut.GetVertex(TriIndices.v2)));
					MeshOut.AppendTriangle(FIndex3i(A,B,C));
				}
			}

			if (bWeldEdges)
			{
				FMergeCoincidentMeshEdges Weld(&MeshOut);
				Weld.OnlyUniquePairs = true;
				Weld.Apply();
				Weld.OnlyUniquePairs = false;
				Weld.Apply();
			}

			if (!CollisionData.bFlipNormals != bWillInvert)		// collision mesh has reversed orientation
			{
				MeshOut.ReverseOrientation(false);
			}
		}
	}

	if (MeshOut.HasAttributes() && MeshOut.Attributes()->PrimaryNormals() != nullptr)
	{
		if (bSetToPerTriangleNormals)
		{
			FMeshNormals::InitializeMeshToPerTriangleNormals(&MeshOut);
		}
		else
		{
			FMeshNormals::InitializeOverlayToPerVertexNormals(MeshOut.Attributes()->PrimaryNormals(), false);
		}
	}

	return (MeshOut.TriangleCount() > 0);
}
