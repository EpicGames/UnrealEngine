// Copyright Epic Games, Inc. All Rights Reserved.

#include "Physics/PhysicsDataCollection.h"
#include "Physics/CollisionGeometryConversion.h"

#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "PhysicsEngine/BodySetup.h"

using namespace UE::Geometry;

void FPhysicsDataCollection::InitializeFromComponent(const UActorComponent* Component, bool bInitializeAggGeom)
{
	SourceComponent = Component;
	BodySetup = nullptr;

	if ( const UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component) )
	{
		if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
		{
			SourceStaticMesh = StaticMesh;
			BodySetup = StaticMesh->GetBodySetup();
		}
	}
	else if (const UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		BodySetup = BrushComponent->BrushBodySetup;
	}
	else if (const UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		BodySetup = DynamicMeshComponent->GetBodySetup();
	}

	ExternalScale3D = FVector(1.f, 1.f, 1.f);

	if (bInitializeAggGeom && BodySetup != nullptr)
	{
		AggGeom = BodySetup->AggGeom;
		UE::Geometry::GetShapeSet(AggGeom, Geometry);
	}
}


void FPhysicsDataCollection::InitializeFromStaticMesh(const UStaticMesh* StaticMesh, bool bInitializeAggGeom)
{
	if (ensure(StaticMesh))
	{
		SourceStaticMesh = StaticMesh;
		BodySetup = StaticMesh->GetBodySetup();

		ExternalScale3D = FVector(1.f, 1.f, 1.f);

		if (bInitializeAggGeom)
		{
			AggGeom = BodySetup->AggGeom;
			UE::Geometry::GetShapeSet(AggGeom, Geometry);
		}
	}
}


void FPhysicsDataCollection::InitializeFromExisting(const FPhysicsDataCollection& Other)
{
	SourceComponent = Other.SourceComponent;
	SourceStaticMesh = Other.SourceStaticMesh;
	BodySetup = Other.BodySetup;

	ExternalScale3D = Other.ExternalScale3D;
}



void FPhysicsDataCollection::CopyGeometryFromExisting(const FPhysicsDataCollection& Other)
{
	Geometry = Other.Geometry;
	AggGeom = Other.AggGeom;
}


void FPhysicsDataCollection::ClearAggregate()
{
	AggGeom = FKAggregateGeom();
}

void FPhysicsDataCollection::CopyGeometryToAggregate()
{
	ClearAggregate();

	for (FBoxShape3d& BoxGeom : Geometry.Boxes)
	{
		FKBoxElem Element;
		UE::Geometry::GetFKElement(BoxGeom.Box, Element);
		AggGeom.BoxElems.Add(Element);
	}

	for (FSphereShape3d& SphereGeom : Geometry.Spheres)
	{
		FKSphereElem Element;
		UE::Geometry::GetFKElement(SphereGeom.Sphere, Element);
		AggGeom.SphereElems.Add(Element);
	}

	for (UE::Geometry::FCapsuleShape3d& CapsuleGeom : Geometry.Capsules)
	{
		FKSphylElem Element;
		UE::Geometry::GetFKElement(CapsuleGeom.Capsule, Element);
		AggGeom.SphylElems.Add(Element);
	}

	for (FConvexShape3d& ConvexGeom : Geometry.Convexes)
	{
		FKConvexElem Element;
		UE::Geometry::GetFKElement(ConvexGeom.Mesh, Element);

		AggGeom.ConvexElems.Add(Element);
	}

	for (UE::Geometry::FLevelSetShape3d& LevelSetGeom : Geometry.LevelSets)
	{
		FKLevelSetElem Element;
		UE::Geometry::GetFKElement(LevelSetGeom.GridTransform, LevelSetGeom.Grid, LevelSetGeom.CellSize, Element);
		AggGeom.LevelSetElems.Emplace(MoveTemp(Element));
	}

}



