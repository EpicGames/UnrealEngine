// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreviewMesh.h"

#include "TargetInterfaces/MaterialProvider.h" //FComponentMaterialSet

#include "DynamicMeshToMeshDescription.h"
#include "Engine/World.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "SceneInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PreviewMesh)

using namespace UE::Geometry;

APreviewMeshActor::APreviewMeshActor()
{
#if WITH_EDITORONLY_DATA
	// hide this actor in the scene outliner
	bListedInSceneOutliner = false;
#endif
}


UPreviewMesh::UPreviewMesh()
{
	bBuildSpatialDataStructure = false;
}

UPreviewMesh::~UPreviewMesh()
{
	checkf(DynamicMeshComponent == nullptr, TEXT("You must explicitly Disconnect() PreviewMesh before it is GCd"));
	checkf(TemporaryParentActor == nullptr, TEXT("You must explicitly Disconnect() PreviewMesh before it is GCd"));
}


void UPreviewMesh::CreateInWorld(UWorld* World, const FTransform& WithTransform)
{
	FRotator Rotation(0.0f, 0.0f, 0.0f);
	FActorSpawnParameters SpawnInfo;
	TemporaryParentActor = World->SpawnActor<APreviewMeshActor>(FVector::ZeroVector, Rotation, SpawnInfo);

	DynamicMeshComponent = NewObject<UDynamicMeshComponent>(TemporaryParentActor);

	// Disable VerifyUsedMaterials on the DynamicMeshSceneProxy. Material verification is prone
	// to data races when materials are subject to change at a high frequency. Since the
	// preview mesh material usage (override render materials) is particularly prone to these
	// races and we are certain the component materials are updated appropriately, we disable
	// used material verification.
	DynamicMeshComponent->SetSceneProxyVerifyUsedMaterials(false);
	
	TemporaryParentActor->SetRootComponent(DynamicMeshComponent);
	//DynamicMeshComponent->SetupAttachment(TemporaryParentActor->GetRootComponent());
	DynamicMeshComponent->RegisterComponent();

	TemporaryParentActor->SetActorTransform(WithTransform);
	//Builder.NewMeshComponent->SetWorldTransform(PlaneFrame.ToFTransform());

	// Disable View Mode Overrides because we don't want editor mode visualizations 
	// to be removed/replaced by View Mode Overrides
	DynamicMeshComponent->SetViewModeOverridesEnabled(false);
}


void UPreviewMesh::Disconnect()
{
	if (DynamicMeshComponent != nullptr)
	{
		DynamicMeshComponent->UnregisterComponent();
		DynamicMeshComponent->DestroyComponent();
		DynamicMeshComponent = nullptr;
	}

	if (TemporaryParentActor != nullptr)
	{
		TemporaryParentActor->Destroy();
		TemporaryParentActor = nullptr;
	}
}


void UPreviewMesh::SetMaterial(UMaterialInterface* Material)
{
	SetMaterial(0, Material);
}

void UPreviewMesh::SetMaterial(int MaterialIndex, UMaterialInterface* Material)
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->SetMaterial(MaterialIndex, Material);

	// force rebuild because we can't change materials yet - surprisingly complicated
	DynamicMeshComponent->NotifyMeshUpdated();

	// if we change materials we have to force a decomposition update because it decomposes by material
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}
}

void UPreviewMesh::SetMaterials(const TArray<UMaterialInterface*>& Materials)
{
	check(DynamicMeshComponent);
	for (int k = 0; k < Materials.Num(); ++k)
	{
		DynamicMeshComponent->SetMaterial(k, Materials[k]);
	}

	// force rebuild because we can't change materials yet - surprisingly complicated
	DynamicMeshComponent->NotifyMeshUpdated();

	// if we change materials we have to force a decomposition update because it decomposes by material
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}
}

int32 UPreviewMesh::GetNumMaterials() const
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->GetNumMaterials();
}

UMaterialInterface* UPreviewMesh::GetMaterial(int MaterialIndex) const
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->GetMaterial(MaterialIndex);
}

void UPreviewMesh::GetMaterials(TArray<UMaterialInterface*>& OutMaterials) const
{
	check(DynamicMeshComponent);
	for (int32 i = 0; i < DynamicMeshComponent->GetNumMaterials(); ++i)
	{
		OutMaterials.Add(DynamicMeshComponent->GetMaterial(i));
	}
}

void UPreviewMesh::SetOverrideRenderMaterial(UMaterialInterface* Material)
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->SetOverrideRenderMaterial(Material);
}

void UPreviewMesh::ClearOverrideRenderMaterial()
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->ClearOverrideRenderMaterial();
}


UMaterialInterface* UPreviewMesh::GetActiveMaterial(int MaterialIndex) const
{
	return DynamicMeshComponent->HasOverrideRenderMaterial(MaterialIndex) ?
		DynamicMeshComponent->GetOverrideRenderMaterial(MaterialIndex) : GetMaterial(MaterialIndex);
}



void UPreviewMesh::SetSecondaryRenderMaterial(UMaterialInterface* Material)
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->SetSecondaryRenderMaterial(Material);
}

void UPreviewMesh::ClearSecondaryRenderMaterial()
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->ClearSecondaryRenderMaterial();
}

void UPreviewMesh::SetOverrideWireframeRenderMaterial(UMaterialInterface* Material)
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->SetOverrideWireframeRenderMaterial(Material);
}

void UPreviewMesh::ClearOverrideWireframeRenderMaterial()
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->ClearOverrideWireframeRenderMaterial();
}

void UPreviewMesh::SetOverrideSecondaryWireframeRenderMaterial(UMaterialInterface* Material)
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->SetOverrideSecondaryWireframeRenderMaterial(Material);
}

void UPreviewMesh::ClearOverrideSecondaryWireframeRenderMaterial()
{
	check(DynamicMeshComponent);
	return DynamicMeshComponent->ClearOverrideSecondaryWireframeRenderMaterial();
}

void UPreviewMesh::EnableSecondaryTriangleBuffers(TUniqueFunction<bool(const FDynamicMesh3*, int32)> TriangleFilterFunc)
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->EnableSecondaryTriangleBuffers(MoveTemp(TriangleFilterFunc));
}

void UPreviewMesh::DisableSecondaryTriangleBuffers()
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->DisableSecondaryTriangleBuffers();
}

void UPreviewMesh::SetSecondaryBuffersVisibility(bool bSecondaryVisibility)
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->SetSecondaryBuffersVisibility(bSecondaryVisibility);
}

void UPreviewMesh::FastNotifySecondaryTrianglesChanged()
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->FastNotifySecondaryTrianglesChanged();
}

void UPreviewMesh::SetTangentsMode(EDynamicMeshComponentTangentsMode TangentsType)
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->SetTangentsType(TangentsType);
}

bool UPreviewMesh::CalculateTangents()
{
	check(DynamicMeshComponent);

	UDynamicMesh *const DynamicMesh = DynamicMeshComponent->GetDynamicMesh();
	FDynamicMesh3 *const Mesh = DynamicMesh ? DynamicMesh->GetMeshPtr() : nullptr;

	if (Mesh)
	{
		// Holds temporary tangents in case we don't have access to existing tangents and need to compute them within this function.
		FMeshTangentsf TempTangents;

		const FMeshTangentsf* Tangents = [this, Mesh, &TempTangents]() -> const FMeshTangentsf*
		{
			if (DynamicMeshComponent->GetTangentsType() == EDynamicMeshComponentTangentsMode::AutoCalculated)
			{
				if (const FMeshTangentsf* AutoCalculatedTangents = DynamicMeshComponent->GetAutoCalculatedTangents())
				{
					return AutoCalculatedTangents;
				}
			}

			const FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
			const FDynamicMeshUVOverlay* UVOverlay = nullptr;

			if (const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
			{
				if (Attributes->NumNormalLayers() > 0)
				{
					NormalOverlay = Attributes->GetNormalLayer(0);
				}

				if (Attributes->NumUVLayers() > 0)
				{
					UVOverlay = Attributes->GetUVLayer(0);
				}
			}

			if (NormalOverlay && UVOverlay)
			{
				TempTangents.ComputeTriVertexTangents(NormalOverlay, UVOverlay, {});

				return &TempTangents;
			}

			return nullptr;
		}();

		if (Tangents)
		{
			Mesh->Attributes()->EnableTangents();
			if (Tangents->CopyToOverlays(*Mesh))
			{
				DynamicMeshComponent->FastNotifyVertexAttributesUpdated(true, false, false);
				NotifyWorldPathTracedOutputInvalidated();

				return true;
			}
		}
	}

	return false;
}

void UPreviewMesh::EnableWireframe(bool bEnable)
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->bExplicitShowWireframe = bEnable;
}

void UPreviewMesh::SetShadowsEnabled(bool bEnable)
{
	check(DynamicMeshComponent);
	DynamicMeshComponent->SetShadowsEnabled(bEnable);
}


FTransform UPreviewMesh::GetTransform() const
{
	if (TemporaryParentActor != nullptr)
	{
		return TemporaryParentActor->GetTransform();
	}
	return FTransform();
}

void UPreviewMesh::SetTransform(const FTransform& UseTransform)
{
	if (TemporaryParentActor != nullptr)
	{
		if (!TemporaryParentActor->GetActorTransform().Equals(UseTransform))
		{
			TemporaryParentActor->SetActorTransform(UseTransform);
			NotifyWorldPathTracedOutputInvalidated();
		}
	}
}

void UPreviewMesh::SetVisible(bool bVisible)
{
	if (DynamicMeshComponent != nullptr && IsVisible() != bVisible )
	{
		DynamicMeshComponent->SetVisibility(bVisible, true);
		NotifyWorldPathTracedOutputInvalidated();
	}
}


bool UPreviewMesh::IsVisible() const
{
	if (DynamicMeshComponent != nullptr)
	{
		return DynamicMeshComponent->IsVisible();
	}
	return false;
}



void UPreviewMesh::ClearPreview() 
{
	FDynamicMesh3 Empty;
	UpdatePreview(&Empty);
	
	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(&Empty, true);
	}
}


void UPreviewMesh::UpdatePreview(const FDynamicMesh3* Mesh, ERenderUpdateMode UpdateMode,
	EMeshRenderAttributeFlags ModifiedAttribs)
{
	DynamicMeshComponent->GetMesh()->Copy(*Mesh);

	NotifyDeferredEditCompleted(UpdateMode, ModifiedAttribs, bBuildSpatialDataStructure);
}

void UPreviewMesh::UpdatePreview(FDynamicMesh3&& Mesh, ERenderUpdateMode UpdateMode, 
	EMeshRenderAttributeFlags ModifiedAttribs)
{
	*(DynamicMeshComponent->GetMesh()) = MoveTemp(Mesh);

	NotifyDeferredEditCompleted(UpdateMode, ModifiedAttribs, bBuildSpatialDataStructure);
}


const FDynamicMesh3* UPreviewMesh::GetMesh() const
{
	if (DynamicMeshComponent != nullptr)
	{
		return DynamicMeshComponent->GetMesh();
	}
	return nullptr;
}


void UPreviewMesh::ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const
{
	if (ensure(DynamicMeshComponent))
	{
		DynamicMeshComponent->ProcessMesh(ProcessFunc);
	}
}



FDynamicMeshAABBTree3* UPreviewMesh::GetSpatial()
{
	if (DynamicMeshComponent != nullptr && bBuildSpatialDataStructure)
	{
		if (MeshAABBTree.IsValid(false))
		{
			return &MeshAABBTree;
		}
	}
	return nullptr;
}




TUniquePtr<FDynamicMesh3> UPreviewMesh::ExtractPreviewMesh() const
{
	if (DynamicMeshComponent != nullptr)
	{
		return DynamicMeshComponent->GetDynamicMesh()->ExtractMesh();
	}
	return nullptr;
}



bool UPreviewMesh::TestRayIntersection(const FRay3d& WorldRay, TFunction<bool(int32 Tid)> TriangleFilterF)
{
	if (IsVisible() && TemporaryParentActor != nullptr && bBuildSpatialDataStructure)
	{
		FFrame3d TransformFrame(TemporaryParentActor->GetActorTransform());
		FRay3d LocalRay = TransformFrame.ToFrame(WorldRay);

		UE::Geometry::IMeshSpatial::FQueryOptions QueryOptions;
		QueryOptions.TriangleFilterF = TriangleFilterF;

		int HitTriID = MeshAABBTree.FindNearestHitTriangle(LocalRay, QueryOptions);
		if (HitTriID != FDynamicMesh3::InvalidID)
		{
			return true;
		}
	}
	return false;
}



bool UPreviewMesh::FindRayIntersection(const FRay3d& WorldRay, FHitResult& HitOut, TFunction<bool(int32)>TriangleFilterF)
{
	if (IsVisible() && TemporaryParentActor != nullptr && bBuildSpatialDataStructure)
	{
		FTransformSRT3d Transform(TemporaryParentActor->GetActorTransform());
		FRay3d LocalRay(Transform.InverseTransformPosition(WorldRay.Origin),
			Transform.InverseTransformVector(WorldRay.Direction));
		UE::Geometry::Normalize(LocalRay.Direction);

		UE::Geometry::IMeshSpatial::FQueryOptions QueryOptions;
		QueryOptions.TriangleFilterF = TriangleFilterF;

		int HitTriID = MeshAABBTree.FindNearestHitTriangle(LocalRay, QueryOptions);
		if (HitTriID != FDynamicMesh3::InvalidID)
		{
			const FDynamicMesh3* UseMesh = GetPreviewDynamicMesh();
			FTriangle3d Triangle;
			UseMesh->GetTriVertices(HitTriID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
			FIntrRay3Triangle3d Query(LocalRay, Triangle);
			Query.Find();

			HitOut.FaceIndex = HitTriID;
			HitOut.Distance = Query.RayParameter;
			HitOut.Normal = (FVector)Transform.TransformNormal(Triangle.Normal());
			HitOut.ImpactNormal = HitOut.Normal;
			HitOut.ImpactPoint = (FVector)Transform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
			return true;
		}
	}
	return false;
}


FVector3d UPreviewMesh::FindNearestPoint(const FVector3d& WorldPoint, bool bLinearSearch)
{
	const FDynamicMesh3* UseMesh = GetMesh();
	if (bLinearSearch)
	{
		return TMeshQueries<FDynamicMesh3>::FindNearestPoint_LinearSearch(*UseMesh, WorldPoint);
	}
	if (bBuildSpatialDataStructure)
	{
		return MeshAABBTree.FindNearestPoint(WorldPoint);
	}
	return WorldPoint;
}



void UPreviewMesh::ReplaceMesh(const FDynamicMesh3& NewMesh)
{
	ReplaceMesh(FDynamicMesh3(NewMesh));
}

void UPreviewMesh::ReplaceMesh(FDynamicMesh3&& NewMesh)
{
	DynamicMeshComponent->SetMesh(MoveTemp(NewMesh));

	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}

	NotifyWorldPathTracedOutputInvalidated();
}


void UPreviewMesh::EditMesh(TFunctionRef<void(FDynamicMesh3&)> EditFunc)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	EditFunc(*Mesh);

	DynamicMeshComponent->NotifyMeshUpdated();

	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}

	NotifyWorldPathTracedOutputInvalidated();
}


void UPreviewMesh::DeferredEditMesh(TFunctionRef<void(FDynamicMesh3&)> EditFunc, bool bRebuildSpatial)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	EditFunc(*Mesh);

	if (bBuildSpatialDataStructure && bRebuildSpatial)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
}


void UPreviewMesh::ForceRebuildSpatial()
{
	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
}


void UPreviewMesh::NotifyDeferredEditCompleted(ERenderUpdateMode UpdateMode, EMeshRenderAttributeFlags ModifiedAttribs, bool bRebuildSpatial)
{
	if (bBuildSpatialDataStructure && bRebuildSpatial)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}

	if (UpdateMode == ERenderUpdateMode::FullUpdate)
	{
		DynamicMeshComponent->NotifyMeshUpdated();
		if (bDecompositionEnabled)
		{
			UpdateRenderMeshDecomposition();
		}
	}
	else if (UpdateMode == ERenderUpdateMode::FastUpdate)
	{
		bool bPositions = (ModifiedAttribs & EMeshRenderAttributeFlags::Positions) != EMeshRenderAttributeFlags::None;
		bool bNormals = (ModifiedAttribs & EMeshRenderAttributeFlags::VertexNormals) != EMeshRenderAttributeFlags::None;
		bool bColors = (ModifiedAttribs & EMeshRenderAttributeFlags::VertexColors) != EMeshRenderAttributeFlags::None;
		bool bUVs = (ModifiedAttribs & EMeshRenderAttributeFlags::VertexUVs) != EMeshRenderAttributeFlags::None;
		if (bPositions)
		{
			DynamicMeshComponent->FastNotifyPositionsUpdated(bNormals, bColors, bUVs);
		}
		else
		{
			DynamicMeshComponent->FastNotifyVertexAttributesUpdated(bNormals, bColors, bUVs);
		}
	}

	NotifyWorldPathTracedOutputInvalidated();
}


TUniquePtr<FMeshChange> UPreviewMesh::TrackedEditMesh(TFunctionRef<void(FDynamicMesh3&, FDynamicMeshChangeTracker&)> EditFunc)
{
	FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	EditFunc(*Mesh, ChangeTracker);
	TUniquePtr<FMeshChange> Change = MakeUnique<FMeshChange>(ChangeTracker.EndChange());

	DynamicMeshComponent->NotifyMeshUpdated();
	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}

	NotifyWorldPathTracedOutputInvalidated();

	return MoveTemp(Change);
}


void UPreviewMesh::ApplyChange(const FMeshVertexChange* Change, bool bRevert)
{
	check(DynamicMeshComponent != nullptr);
	DynamicMeshComponent->ApplyChange(Change, bRevert);
	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
	// should not need to update render mesh decomposition here...
}
void UPreviewMesh::ApplyChange(const FMeshChange* Change, bool bRevert)
{
	check(DynamicMeshComponent != nullptr);
	DynamicMeshComponent->ApplyChange(Change, bRevert);
	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}

	NotifyWorldPathTracedOutputInvalidated();
}
void UPreviewMesh::ApplyChange(const FMeshReplacementChange* Change, bool bRevert)
{
	check(DynamicMeshComponent != nullptr);
	DynamicMeshComponent->ApplyChange(Change, bRevert);
	if (bBuildSpatialDataStructure)
	{
		MeshAABBTree.SetMesh(DynamicMeshComponent->GetMesh(), true);
	}
	if (bDecompositionEnabled)
	{
		UpdateRenderMeshDecomposition();
	}

	NotifyWorldPathTracedOutputInvalidated();
}

FSimpleMulticastDelegate& UPreviewMesh::GetOnMeshChanged()
{
	check(DynamicMeshComponent != nullptr);
	return DynamicMeshComponent->OnMeshChanged;
}



void UPreviewMesh::SetTriangleColorFunction(TFunction<FColor(const FDynamicMesh3*, int)> TriangleColorFunc, ERenderUpdateMode UpdateMode)
{
	DynamicMeshComponent->SetTriangleColorFunction(TriangleColorFunc,  (EDynamicMeshComponentRenderUpdateMode)(int32)UpdateMode );
}

void UPreviewMesh::ClearTriangleColorFunction(ERenderUpdateMode UpdateMode)
{
	DynamicMeshComponent->ClearTriangleColorFunction((EDynamicMeshComponentRenderUpdateMode)(int32)UpdateMode);
}


void UPreviewMesh::SetEnableRenderMeshDecomposition(bool bEnable)
{
	if (bDecompositionEnabled != bEnable)
	{
		bDecompositionEnabled = bEnable;

		if (bDecompositionEnabled)
		{
			UpdateRenderMeshDecomposition();
		}
		else
		{
			DynamicMeshComponent->SetExternalDecomposition(nullptr);
		}
	}
}


void UPreviewMesh::UpdateRenderMeshDecomposition()
{
	check(bDecompositionEnabled);

	const FDynamicMesh3* Mesh = DynamicMeshComponent->GetMesh();
	FComponentMaterialSet MaterialSet;
	GetMaterials(MaterialSet.Materials);

	TUniquePtr<FMeshRenderDecomposition> Decomp = MakeUnique<FMeshRenderDecomposition>();
	FMeshRenderDecomposition::BuildChunkedDecomposition(Mesh, &MaterialSet, *Decomp);
	Decomp->BuildAssociations(Mesh);
	DynamicMeshComponent->SetExternalDecomposition(MoveTemp(Decomp));
}


void UPreviewMesh::NotifyRegionDeferredEditCompleted(const TArray<int32>& Triangles, EMeshRenderAttributeFlags ModifiedAttribs)
{
	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(Triangles, ModifiedAttribs);
	NotifyWorldPathTracedOutputInvalidated();
}

void UPreviewMesh::NotifyRegionDeferredEditCompleted(const TSet<int32>& Triangles, EMeshRenderAttributeFlags ModifiedAttribs)
{
	DynamicMeshComponent->FastNotifyTriangleVerticesUpdated(Triangles, ModifiedAttribs);
	NotifyWorldPathTracedOutputInvalidated();
}

void UPreviewMesh::NotifyVertexRegionDeferredEditCompleted(const TArray<int32>& Vertices, EMeshRenderAttributeFlags ModifiedAttribs)
{
	DynamicMeshComponent->FastNotifyVerticesUpdated(Vertices, ModifiedAttribs);
	NotifyWorldPathTracedOutputInvalidated();
}


void UPreviewMesh::NotifyWorldPathTracedOutputInvalidated()
{
	if (TemporaryParentActor != nullptr)
	{
		UWorld* World = TemporaryParentActor->GetWorld();
		if (World && World->Scene && FApp::CanEverRender())
		{
			World->Scene->InvalidatePathTracedOutput();
		}
	}
}

