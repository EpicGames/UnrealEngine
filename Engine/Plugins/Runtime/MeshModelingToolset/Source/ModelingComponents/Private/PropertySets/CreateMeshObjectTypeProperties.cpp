// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "ModelingObjectsCreationAPI.h"
#include "ModelingComponentsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CreateMeshObjectTypeProperties)


const FString UCreateMeshObjectTypeProperties::StaticMeshIdentifier = TEXT("Static Mesh");
const FString UCreateMeshObjectTypeProperties::VolumeIdentifier = TEXT("Volume");
const FString UCreateMeshObjectTypeProperties::DynamicMeshActorIdentifier = TEXT("Dynamic Mesh");
const FString UCreateMeshObjectTypeProperties::AutoIdentifier = TEXT("From Input");

bool UCreateMeshObjectTypeProperties::bEnableDynamicMeshActorSupport = false;
FString UCreateMeshObjectTypeProperties::DefaultObjectTypeIdentifier = UCreateMeshObjectTypeProperties::StaticMeshIdentifier;


void UCreateMeshObjectTypeProperties::InitializeDefault()
{
	bool bStaticMeshes = true;
	bool bDynamicMeshes = true;

#if WITH_EDITOR
	bool bVolumes = true;
#else
	bool bVolumes = false;
#endif

	Initialize(bStaticMeshes, bVolumes, bDynamicMeshes);
}

void UCreateMeshObjectTypeProperties::InitializeDefaultWithAuto()
{
	InitializeDefault();
	OutputTypeNamesList.Add(AutoIdentifier);
}

void UCreateMeshObjectTypeProperties::Initialize(bool bEnableStaticMeshes, bool bEnableVolumes, bool bEnableDynamicMeshActor)
{
	if (bEnableStaticMeshes)
	{
		OutputTypeNamesList.Add(StaticMeshIdentifier);
	}
	if (bEnableVolumes)
	{
		OutputTypeNamesList.Add(VolumeIdentifier);
	}
	if (bEnableDynamicMeshActor && bEnableDynamicMeshActorSupport)
	{
		OutputTypeNamesList.Add(DynamicMeshActorIdentifier);
	}

	if (OutputTypeNamesList.Num() == 0)
	{
		return;
	}

	if ((OutputType.Len() == 0) || (OutputType.Len() > 0 && OutputTypeNamesList.Contains(OutputType) == false))
	{
		if (OutputTypeNamesList.Contains(DefaultObjectTypeIdentifier))
		{
			OutputType = DefaultObjectTypeIdentifier;
		}
		else
		{
			OutputType = OutputTypeNamesList[0];
		}
	}
}

const TArray<FString>& UCreateMeshObjectTypeProperties::GetOutputTypeNamesFunc()
{
	return OutputTypeNamesList;
}



bool UCreateMeshObjectTypeProperties::ShouldShowPropertySet() const
{
	return (OutputTypeNamesList.Num() > 1)
		|| OutputTypeNamesList.Contains(VolumeIdentifier);
}

ECreateObjectTypeHint UCreateMeshObjectTypeProperties::GetCurrentCreateMeshType() const
{
	if (OutputType == StaticMeshIdentifier)
	{
		return ECreateObjectTypeHint::StaticMesh;
	}
	else if (OutputType == VolumeIdentifier)
	{
		return ECreateObjectTypeHint::Volume;
	}
	else if (OutputType == DynamicMeshActorIdentifier)
	{
		return ECreateObjectTypeHint::DynamicMeshActor;
	}
	return ECreateObjectTypeHint::Undefined;
}

void UCreateMeshObjectTypeProperties::UpdatePropertyVisibility()
{
	bShowVolumeList = (OutputType == VolumeIdentifier);
}


bool UCreateMeshObjectTypeProperties::ConfigureCreateMeshObjectParams(FCreateMeshObjectParams& ParamsOut) const
{
	// client has to handle this case
	ensure(OutputType != AutoIdentifier);

	if (OutputType == StaticMeshIdentifier)
	{
		ParamsOut.TypeHint = ECreateObjectTypeHint::StaticMesh;
		return true;
	}
	else if (OutputType == VolumeIdentifier)
	{
		ParamsOut.TypeHint = ECreateObjectTypeHint::Volume;
		ParamsOut.TypeHintClass = VolumeType.Get();
		return true;
	}
	else if (OutputType == DynamicMeshActorIdentifier)
	{
		ParamsOut.TypeHint = ECreateObjectTypeHint::DynamicMeshActor;
		return true;
	}
	return false;
}

