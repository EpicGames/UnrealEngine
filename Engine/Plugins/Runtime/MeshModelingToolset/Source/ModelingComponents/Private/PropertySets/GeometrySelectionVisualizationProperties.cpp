// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "Engine/Engine.h"
#include "ToolSetupUtil.h"
#include "Materials/MaterialRenderProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometrySelectionVisualizationProperties)

void UGeometrySelectionVisualizationProperties::Initialize(UInteractiveTool* Tool)
{
	WatchProperty(bShowSelection, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(bShowTriangleROIBorder, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(bShowHidden, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(bShowEdgeSelectionVertices, [this](bool bNewValue) { bVisualizationDirty = true; });
	WatchProperty(LineThickness, [this](float NewValue) { bVisualizationDirty = true; });
	WatchProperty(PointSize, [this](float NewValue) { bVisualizationDirty = true; });
	WatchProperty(DepthBias, [this](float NewValue) { bVisualizationDirty = true; });
	WatchProperty(FaceColor, [this](FColor NewValue) { bVisualizationDirty = true; });
	WatchProperty(LineColor, [this](FColor NewValue) { bVisualizationDirty = true; });
	WatchProperty(PointColor, [this](FColor NewValue) { bVisualizationDirty = true; });
	WatchProperty(TriangleROIBorderColor, [this](FColor NewValue) { bVisualizationDirty = true; });

	const float Opacity = .3f;
	const float PercentDepthOffset = .05f;

	// TODO Point/Line selection rendering in the viewport also includes a checkboard effect which is missing here

	PointMaterial = ToolSetupUtil::GetDefaultPointComponentMaterial(Tool->GetToolManager(), true);
	PointMaterialShowingHidden = ToolSetupUtil::GetDefaultPointComponentMaterial(Tool->GetToolManager(), false);

	// TODO Remove or fix the dashed line rendering for hidden lines here
	LineMaterial = ToolSetupUtil::GetDefaultLineComponentMaterial(Tool->GetToolManager(), true);
	LineMaterialShowingHidden = ToolSetupUtil::GetDefaultLineComponentMaterial(Tool->GetToolManager(), false);

	// Both of these materials are transparent (Blend Mode is Translucent), TriangleMaterialShowingHidden also has the
	// "Disable Depth Test" enabled. Note the Opacity value for TriangleMaterial is chosen so that the material matches
	// the TriangleMaterialShowingHidden
	TriangleMaterial = ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(Tool->GetToolManager(), FaceColor, PercentDepthOffset, Opacity);
	TriangleMaterialShowingHidden = GEngine->ConstraintLimitMaterialX->GetRenderProxy()->GetMaterialInterface();
}




UMaterialInterface* UGeometrySelectionVisualizationProperties::GetPointMaterial() const
{
	ensure(PointMaterial);
	ensure(PointMaterialShowingHidden);
	return bShowHidden ? PointMaterialShowingHidden : PointMaterial;
}

UMaterialInterface* UGeometrySelectionVisualizationProperties::GetLineMaterial() const
{
	ensure(LineMaterial);
	ensure(LineMaterialShowingHidden);
	return bShowHidden ? LineMaterialShowingHidden : LineMaterial;
}

UMaterialInterface* UGeometrySelectionVisualizationProperties::GetFaceMaterial() const
{
	ensure(TriangleMaterial);
	ensure(TriangleMaterialShowingHidden);
	return bShowHidden ? TriangleMaterialShowingHidden : TriangleMaterial;
}




bool UGeometrySelectionVisualizationProperties::ShowVertexSelectionPointSet() const
{
	if (bShowSelection == false)
	{
		return false;
	}

	if (SelectionElementType != EGeometrySelectionElementType::Vertex)
	{
		return false;
	}

	return true;
}

bool UGeometrySelectionVisualizationProperties::ShowEdgeSelectionLineSet() const
{
	if (bShowSelection == false)
	{
		return false;
	}

	if (SelectionElementType != EGeometrySelectionElementType::Edge)
	{
		return false;
	}

	if (bEnableShowEdgeSelectionVertices)
	{
		return !bShowEdgeSelectionVertices;
	}

	return true;
}

bool UGeometrySelectionVisualizationProperties::ShowFaceSelectionTriangleSet() const
{
	if (bShowSelection == false)
	{
		return false;
	}

	if (SelectionElementType != EGeometrySelectionElementType::Face)
	{
		return false;
	}

	return true;
}

bool UGeometrySelectionVisualizationProperties::ShowEdgeSelectionVerticesPointSet() const
{
	if (bShowSelection == false)
	{
		return false;
	}

	if (SelectionElementType != EGeometrySelectionElementType::Edge)
	{
		return false;
	}

	if (bEnableShowEdgeSelectionVertices)
	{
		return bShowEdgeSelectionVertices;
	}

	return true;
}

bool UGeometrySelectionVisualizationProperties::ShowTriangleROIBorderLineSet() const
{
	// Ignore bShowSelection here
	//if (bShowSelection == false)
	//{
	//	return false;
	//}

	if (bEnableShowTriangleROIBorder == false)
	{
		return false;
	}

	return bShowTriangleROIBorder;
}
