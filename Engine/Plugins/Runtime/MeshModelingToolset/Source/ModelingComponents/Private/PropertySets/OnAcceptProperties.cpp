// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySets/OnAcceptProperties.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectGlobals.h"
#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OnAcceptProperties)

#define LOCTEXT_NAMESPACE "UOnAcceptHandleSourcesProperties"


void UOnAcceptHandleSourcesPropertiesBase::ApplyMethod(const TArray<AActor*>& Actors, UInteractiveToolManager* ToolManager, const AActor* MustKeepActor)
{
	const EHandleSourcesMethod HandleInputs = GetHandleInputs();

	// Hide or destroy the sources
	bool bKeepSources = HandleInputs == EHandleSourcesMethod::KeepSources;
	if (Actors.Num() == 1 && (HandleInputs == EHandleSourcesMethod::KeepFirstSource || HandleInputs == EHandleSourcesMethod::KeepLastSource))
	{
		// if there's only one actor, keeping any source == keeping all sources
		bKeepSources = true;
	}
	if (!bKeepSources)
	{
		bool bDelete = HandleInputs == EHandleSourcesMethod::DeleteSources
					|| HandleInputs == EHandleSourcesMethod::KeepFirstSource
					|| HandleInputs == EHandleSourcesMethod::KeepLastSource;
		if (bDelete)
		{
			ToolManager->BeginUndoTransaction(LOCTEXT("RemoveSources", "Remove Inputs"));
		}
		else
		{
#if WITH_EDITOR
			ToolManager->BeginUndoTransaction(LOCTEXT("HideSources", "Hide Inputs"));
#endif
		}

		const int32 ActorIdxBegin = HandleInputs == EHandleSourcesMethod::KeepFirstSource ? 1 : 0;
		const int32 ActorIdxEnd = HandleInputs == EHandleSourcesMethod::KeepLastSource ? Actors.Num() - 1 : Actors.Num();

		for (int32 ActorIdx = ActorIdxBegin; ActorIdx < ActorIdxEnd; ActorIdx++)
		{
			AActor* Actor = Actors[ActorIdx];
			if (Actor == MustKeepActor)
			{
				continue;
			}

			if (bDelete)
			{
				UE::ToolTarget::SafeDeleteActor(Actor);
			}
			else
			{
#if WITH_EDITOR
				// Save the actor to the transaction buffer to support undo/redo, but do
				// not call Modify, as we do not want to dirty the actor's package and
				// we're only editing temporary, transient values
				SaveToTransactionBuffer(Actor, false);
				Actor->SetIsTemporarilyHiddenInEditor(true);
#endif
			}
		}
		if (bDelete)
		{
			ToolManager->EndUndoTransaction();
		}
		else
		{
#if WITH_EDITOR
			ToolManager->EndUndoTransaction();
#endif
		}
	}
}

#undef LOCTEXT_NAMESPACE

