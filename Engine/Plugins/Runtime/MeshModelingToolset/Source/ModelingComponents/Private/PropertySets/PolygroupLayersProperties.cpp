// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySets/PolygroupLayersProperties.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Polygroups/PolygroupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolygroupLayersProperties)

using namespace UE::Geometry;

void UPolygroupLayersProperties::InitializeGroupLayers(const FDynamicMesh3* Mesh)
{
	GroupLayersList.Reset();
	GroupLayersList.Add(TEXT("Default"));		// always have standard group
	
	if (const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
	{
		for (int32 k = 0; k < Attributes->NumPolygroupLayers(); k++)
		{
			FName Name = Attributes->GetPolygroupLayer(k)->GetName();
			GroupLayersList.Add(Name.ToString());
		}
		
		if (bSupportLabels)
		{
			for (const auto& [AttrName, TriangleLabelAttr]: Attributes->GetTriangleLabelAttributes())
			{
				GroupLayersList.Add(AttrName.ToString());
			}
		}
	}

	if (GroupLayersList.Contains(ActiveGroupLayer.ToString()) == false)		// discard restored value if it doesn't apply
	{
		ActiveGroupLayer = FName(GroupLayersList[0]);
	}
}

void UPolygroupLayersProperties::InitializeGroupLayers(const TSet<FName>& LayerNames)
{
	GroupLayersList.Reset();
	GroupLayersList.Add(TEXT("Default"));		// always have standard group
	for (const FName& Name : LayerNames)
	{
		GroupLayersList.Add(Name.ToString());
	}

	if (GroupLayersList.Contains(ActiveGroupLayer.ToString()) == false)		// discard restored value if it doesn't apply
	{
		ActiveGroupLayer = FName(GroupLayersList[0]);
	}
}


bool UPolygroupLayersProperties::HasSelectedPolygroup() const
{
	return ActiveGroupLayer != FName(GroupLayersList[0]);
}


void UPolygroupLayersProperties::SetSelectedFromPolygroupIndex(int32 Index)
{
	if (Index < 0)
	{
		ActiveGroupLayer = FName(GroupLayersList[0]);
	}
	else
	{
		ActiveGroupLayer = FName(GroupLayersList[Index+1]);
	}
}


UE::Geometry::FPolygroupLayer UPolygroupLayersProperties::GetSelectedLayer(const FDynamicMesh3& FromMesh)
{
	if (HasSelectedPolygroup() == false)
	{
		return FPolygroupLayer::Default();
	}
	else
	{
		int32 Index = UE::Geometry::FindPolygroupLayerIndexByName(FromMesh, ActiveGroupLayer);
		return (Index >= 0) ? FPolygroupLayer::Layer(Index) : FPolygroupLayer::Default();
	}
}
