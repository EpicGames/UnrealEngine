// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertySets/WeightMapSetProperties.h"
#include "WeightMapUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeightMapSetProperties)


void UWeightMapSetProperties::InitializeWeightMaps(const TArray<FName>& WeightMapNames)
{
	// populate weight maps list
	WeightMapsList.Add(TEXT("None"));
	for (FName Name : WeightMapNames)
	{
		WeightMapsList.Add(Name.ToString());
	}
	if (WeightMapNames.Contains(WeightMap) == false)		// discard restored value if it doesn't apply
	{
		WeightMap = FName(WeightMapsList[0]);
	}
}


void UWeightMapSetProperties::InitializeFromMesh(const FMeshDescription* Mesh)
{
	TArray<FName> TargetWeightMaps;
	UE::WeightMaps::FindVertexWeightMaps(Mesh, TargetWeightMaps);
	InitializeWeightMaps(TargetWeightMaps);
}

bool UWeightMapSetProperties::HasSelectedWeightMap() const
{
	return WeightMap != FName(WeightMapsList[0]);
}


TArray<FString> UWeightMapSetProperties::GetWeightMapsFunc()
{
	return WeightMapsList;
}

void UWeightMapSetProperties::SetSelectedFromWeightMapIndex(int32 Index)
{
	if (Index < 0)
	{
		WeightMap = FName(WeightMapsList[0]);
	}
	else
	{
		WeightMap = FName(WeightMapsList[Index+1]);
	}
}
