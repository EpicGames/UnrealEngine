// Copyright Epic Games, Inc. All Rights Reserved.


#include "Scene/MeshSceneAdapter.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/ColliderMesh.h"
#include "Spatial/MeshAABBTree3.h"
#include "Spatial/FastWinding.h"
#include "MeshDescriptionAdapter.h"
#include "StaticMeshLODResourcesAdapter.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMesh/MeshTransforms.h"
#include "BoxTypes.h"
#include "FrameTypes.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "Operations/OffsetMeshRegion.h"
#include "DynamicMeshEditor.h"
#include "CompGeom/ConvexHull2.h"
#include "Generators/PlanarPolygonMeshGenerator.h"
#include "Spatial/SparseDynamicOctree3.h"

#include "GameFramework/Actor.h"
#include "Components/SplineMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"

#include "Async/ParallelFor.h"
#include "UObject/Package.h"

using namespace UE::Geometry;


static TAutoConsoleVariable<int32> CVarMeshSceneAdapterDisableMultiThreading(
	TEXT("geometry.MeshSceneAdapter.SingleThreaded"),
	0,
	TEXT("Determines whether or not to use multi-threading in MeshSceneAdapter.\n"));


namespace UE
{
namespace Geometry
{


/** 
 * Compute the bounds of the vertices of Mesh, under 3D transformation TransformFunc
 * @return computed bounding box
 */
template<typename MeshType>
FAxisAlignedBox3d GetTransformedVertexBounds(const MeshType& Mesh, TFunctionRef<FVector3d(const FVector3d&)> TransformFunc)
{
	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	int32 NumVertices = Mesh.VertexCount();
	for (int32 k = 0; k < NumVertices; ++k)
	{
		if (Mesh.IsVertex(k))
		{
			Bounds.Contain(TransformFunc(Mesh.GetVertex(k)));
		}
	}
	return Bounds;
}

/**
 * Collect a subset of vertices of the mesh as "seed points" for algorithms like marching-cubes/etc.
 * Generally every vertex does not need to be used. This function will return at most 5000 point
 * @param TransformFunc transformation applied to points, eg local-to-world mapping
 * @param AccumPointsInOut points are added here
 * @param MaxPoints at most this many vertices will be returned
 */
template<typename MeshType>
void CollectSeedPointsFromMeshVertices(
	const MeshType& Mesh, 
	TFunctionRef<FVector3d(const FVector3d&)> TransformFunc, TArray<FVector3d>& AccumPointsInOut,
	int32 MaxPoints = 500)
{
	int32 NumVertices = Mesh.VertexCount();
	int32 LogNumVertices = FMath::Max(1, (int32)FMathd::Ceil(FMathd::Log(NumVertices)));
	int32 SeedPointCount = (int)(10 * LogNumVertices);
	SeedPointCount = FMath::Min(SeedPointCount, MaxPoints);
	int32 Skip = FMath::Max(NumVertices / SeedPointCount, 2);
	for (int32 k = 0; k < NumVertices; k += Skip)
	{
		AccumPointsInOut.Add(TransformFunc(Mesh.GetVertex(k)));
	}
}
void CollectSeedPointsFromMeshVertices(
	const FDynamicMesh3& Mesh,
	TFunctionRef<FVector3d(const FVector3d&)> TransformFunc, TArray<FVector3d>& AccumPointsInOut,
	double NormalOffset = 0,
	int32 MaxPoints = 500)
{
	int32 NumVertices = Mesh.VertexCount();
	int32 LogNumVertices = FMath::Max(1, (int32)FMathd::Ceil(FMathd::Log(NumVertices)));
	int32 SeedPointCount = (int)(10 * LogNumVertices);
	SeedPointCount = FMath::Min(SeedPointCount, MaxPoints);
	int32 Skip = FMath::Max(NumVertices / SeedPointCount, 2);
	for (int32 k = 0; k < NumVertices; k += Skip)
	{
		FVector3d Pos = Mesh.GetVertex(k);
		if (NormalOffset > 0)
		{
			int32 tid = *Mesh.VtxTrianglesItr(k).begin();
			FVector3d TriNormal = Mesh.GetTriNormal(tid);
			Pos += NormalOffset * TriNormal;
		}
		AccumPointsInOut.Add(TransformFunc(Pos));
	}
}

/**
 * Try to check if a Mesh is "thin", ie basically a planar patch (open or closed), relative to a given plane
 * 
 */
template<typename MeshType>
double MeasureThickness(const MeshType& Mesh, const FFrame3d& Plane)
{
	FAxisAlignedBox3d PlaneExtents = FAxisAlignedBox3d::Empty();
	int32 VertexCount = Mesh.VertexCount();
	for (int32 k = 0; k < VertexCount; ++k)
	{
		if (Mesh.IsVertex(k))
		{
			PlaneExtents.Contain(Plane.ToFramePoint(Mesh.GetVertex(k)));
		}
	}

	return PlaneExtents.Depth();
}



/**
 * Try to check if the subset of Triangles of Mesh represent a "thin" region, ie basically a planar patch (open or closed).
 * The Normal of the largest-area triangle is taken as the plane normal, and then the "thickness" is measured relative to this plane
 * @param ThinTolerance identify as Thin if the thickness extents is within this size
 * @param ThinPlaneOut thin plane normal will be returned via this frame
 * @return true if submesh identified as thin
 */
template<typename MeshType>
bool IsThinPlanarSubMesh(
	const MeshType& Mesh, const TArray<int32>& Triangles, 
	FTransformSequence3d& Transform, 
	double ThinTolerance, 
	FFrame3d& ThinPlaneOut)
{
	FVector3d Scale = Transform.GetAccumulatedScale();
	int32 TriCount = Triangles.Num();
	FAxisAlignedBox3d MeshBounds = FAxisAlignedBox3d::Empty();

	// Find triangle with largest area and use it's normal as the plane normal
	// (this is not ideal and we should probably do a normals histogram
	double MaxArea = 0;
	FVector3d MaxAreaNormal;
	FVector3d MaxAreaPoint;
	for (int32 i = 0; i < TriCount; ++i)
	{
		int32 tid = Triangles[i];
		if (Mesh.IsTriangle(tid))
		{
			FVector3d A, B, C;
			Mesh.GetTriVertices(tid, A, B, C);
			A *= Scale; B *= Scale; C *= Scale;
			MeshBounds.Contain(A);
			MeshBounds.Contain(B);
			MeshBounds.Contain(C);
			double TriArea;
			FVector3d TriNormal = VectorUtil::NormalArea(A, B, C, TriArea);
			if (TriArea > MaxArea)
			{
				MaxArea = TriArea;
				MaxAreaNormal = TriNormal;
				MaxAreaPoint = A;
			}
		}
	}

	// if one of the AABB dimensions is below the thin tolerance, just use it
	FVector3d BoundsDimensions = MeshBounds.Diagonal();
	if (BoundsDimensions.GetAbsMin() < ThinTolerance)
	{
		int32 Index = UE::Geometry::MinAbsElementIndex(BoundsDimensions);
		FVector3d BoxNormal = FVector3d::Zero();
		BoxNormal[Index] = 1.0;
		ThinPlaneOut = FFrame3d(MeshBounds.Center(), BoxNormal);
		return true;
	}

	// Now compute the bounding box in the local space of this plane
	ThinPlaneOut = FFrame3d(MaxAreaPoint, MaxAreaNormal);
	FAxisAlignedBox3d PlaneExtents = FAxisAlignedBox3d::Empty();
	for (int32 i = 0; i < TriCount; ++i)
	{
		int32 tid = Triangles[i];
		if (Mesh.IsTriangle(tid))
		{
			FVector3d TriVerts[3];
			Mesh.GetTriVertices(tid, TriVerts[0], TriVerts[1], TriVerts[2]);
			for (int32 j = 0; j < 3; ++j)
			{
				TriVerts[j] = ThinPlaneOut.ToFramePoint(Scale * TriVerts[j]);
				PlaneExtents.Contain(TriVerts[j]);
			}
		}

		// early-out if we exceed tolerance
		if (PlaneExtents.Depth() > ThinTolerance)
		{
			return false;
		}
	}

	// shift plane to center
	FVector3d Center = PlaneExtents.Center();
	ThinPlaneOut.Origin += Center.X*ThinPlaneOut.X() + Center.Y*ThinPlaneOut.Y() + Center.Z*ThinPlaneOut.Z();

	return true;
}




/**
 * @return false if any of Triangles in Mesh have open boundary edges
 */
static bool IsClosedRegion(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles)
{
	for (int32 tid : Triangles)
	{
		FIndex3i TriEdges = Mesh.GetTriEdges(tid);
		if (Mesh.IsBoundaryEdge(TriEdges.A) ||
			Mesh.IsBoundaryEdge(TriEdges.B) ||
			Mesh.IsBoundaryEdge(TriEdges.C))
		{
			return false;
		}
	}
	return true;
}



/**
 * Base type for various spatial wrappers, to provide common config variables
 */
class FBaseMeshSpatialWrapper : public IMeshSpatialWrapper
{
public:
	// if true, Mesh is in world space (whatever that means)
	bool bHasBakedTransform = false;
	// if true, Mesh is only translated and rotated (allows some assumptions to be made)
	bool bHasBakedScale = false;
	// if true, use unsigned distance to determine inside/outside instead of winding number
	bool bUseDistanceShellForWinding = false;
	// unsigned distance isovalue that defines inside
	double WindingShellThickness = 0.0;
	// if true, do winding query before testing shell thickness. This is necessary in cases where the
	// shell thickness is very small (eg slightly thickening a relatively thick object). Then the thickness
	// query would only create a shell instead of a full solid
	bool bRequiresWindingQueryFallback = false;

};



/**
 * IMeshSpatialWrapper for a FDynamicMesh3, with optional AABBTree and FWNTree.
 * 
 */
class FDynamicMeshSpatialWrapper : public FBaseMeshSpatialWrapper
{
public:
	FDynamicMesh3 Mesh;
	TUniquePtr<TMeshAABBTree3<FDynamicMesh3>> AABBTree;
	TUniquePtr<TFastWindingTree<FDynamicMesh3>> FWNTree;

	FDynamicMeshSpatialWrapper()
	{
	}

	FDynamicMeshSpatialWrapper(FDynamicMesh3&& SourceMesh)
	{
		Mesh = MoveTemp(SourceMesh);
	}

	virtual bool Build(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
		ensure(Mesh.TriangleCount() > 0);
		if (BuildOptions.bBuildSpatialDataStructures)
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_DMesh_AABBTree);
				AABBTree = MakeUnique<TMeshAABBTree3<FDynamicMesh3>>(&Mesh, true);
			}
			if (bUseDistanceShellForWinding == false || bRequiresWindingQueryFallback)
			{
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_DMesh_FWNTree);
					FWNTree = MakeUnique<TFastWindingTree<FDynamicMesh3>>(AABBTree.Get(), true);
				}
			}
		}
		return true;
	}

	virtual int32 GetTriangleCount() const override
	{
		return Mesh.TriangleCount();
	}

	virtual bool IsTriangle(int32 TriId) const override
	{
		return Mesh.IsTriangle(TriId);
	}

	virtual FIndex3i GetTriangle(int32 TriId) const override
	{
		return Mesh.GetTriangle(TriId);
	}

	virtual bool HasNormals() const override
	{
		return Mesh.HasAttributes() && Mesh.Attributes()->PrimaryNormals();
	}

	virtual bool HasUVs(const int UVLayer = 0) const override
	{
		return Mesh.HasAttributes() && Mesh.Attributes()->GetUVLayer(UVLayer);
	}

	virtual FVector3d TriBaryInterpolatePoint(int32 TriId, const FVector3d& BaryCoords) const override
	{
		return Mesh.GetTriBaryPoint(TriId, BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
	}

	virtual bool TriBaryInterpolateNormal(int32 TriId, const FVector3d& BaryCoords, FVector3f& NormalOut) const override
	{
		if (Mesh.HasAttributes())
		{
			const FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
			if (NormalOverlay)
			{
				FVector3d Normal;
				NormalOverlay->GetTriBaryInterpolate(TriId, &BaryCoords.X, &Normal.X);
				NormalOut = FVector3f(Normal);
				return true;
			}
		}
		return false;
	}

	virtual bool TriBaryInterpolateUV(const int32 TriId, const FVector3d& BaryCoords, const int UVLayer, FVector2f& UVOut) const override
	{
		if (Mesh.HasAttributes())
		{
			const FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(UVLayer);
			if (UVOverlay)
			{
				FVector2d UV;
				UVOverlay->GetTriBaryInterpolate(TriId, &BaryCoords.X, &UV.X);
				UVOut = FVector2f(UV);
				return true;
			}
		}
		return false;
	}

	virtual FAxisAlignedBox3d GetWorldBounds(TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		FAxisAlignedBox3d Bounds = bHasBakedTransform ?
			GetTransformedVertexBounds<FDynamicMesh3>(Mesh, [&](const FVector3d& P) {return P;}) :
			GetTransformedVertexBounds<FDynamicMesh3>(Mesh, LocalToWorldFunc);
		if (bUseDistanceShellForWinding)
		{
			Bounds.Expand(WindingShellThickness);
		}
		return Bounds;
	}

	virtual void CollectSeedPoints(TArray<FVector3d>& WorldPoints, TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		return bHasBakedTransform ?
			CollectSeedPointsFromMeshVertices(Mesh, [&](const FVector3d& P) {return P;}, WorldPoints, WindingShellThickness) :
			CollectSeedPointsFromMeshVertices(Mesh, LocalToWorldFunc, WorldPoints, WindingShellThickness);
	}

	virtual double FastWindingNumber(const FVector3d& P, const FTransformSequence3d& LocalToWorldTransform) override
	{
		if (bUseDistanceShellForWinding)
		{
			if (bRequiresWindingQueryFallback)
			{
				double WindingNumber = bHasBakedTransform ?
					FWNTree->FastWindingNumber(P) :
					FWNTree->FastWindingNumber(LocalToWorldTransform.InverseTransformPosition(P));
				if (WindingNumber > 0.99)
				{
					return 1.0;
				}
			}

			if (bHasBakedTransform || bHasBakedScale)
			{
				FVector3d UseP = (bHasBakedTransform) ? P : LocalToWorldTransform.InverseTransformPosition(P);
				double NearestDistSqr;
				int32 NearTriID = AABBTree->FindNearestTriangle(UseP, NearestDistSqr, IMeshSpatial::FQueryOptions(WindingShellThickness));
				if (NearTriID != IndexConstants::InvalidID)
				{
					// Do we even need to do this? won't we return InvalidID if we don't find point within distance?
					// (also technically we can early-out as soon as we find any point, not the nearest point - might be worth a custom query)
					FDistPoint3Triangle3d Query = TMeshQueries<FDynamicMesh3>::TriangleDistance(Mesh, NearTriID, UseP);
					if (Query.GetSquared() < WindingShellThickness * WindingShellThickness)
					{
						return 1.0;
					}
				}
			}
			else
			{
				ensure(false);		// not supported yet
			}
			return 0.0;
		}
		else
		{
			return bHasBakedTransform ?
				FWNTree->FastWindingNumber(P) :
				FWNTree->FastWindingNumber(LocalToWorldTransform.InverseTransformPosition(P));
		}
	}


	virtual bool RayIntersection(const FRay3d& WorldRay, const FTransformSequence3d& LocalToWorldTransform, FMeshSceneRayHit& WorldHitResultOut) override
	{
		FRay3d UseRay = WorldRay;
		if (bHasBakedTransform == false)
		{
			FVector3d LocalOrigin = LocalToWorldTransform.InverseTransformPosition(WorldRay.Origin);
			FVector3d LocalDir = Normalized(LocalToWorldTransform.InverseTransformPosition(WorldRay.PointAt(1.0)) - LocalOrigin);
			UseRay = FRay3d(LocalOrigin, LocalDir);
		}

		double RayHitT; int32 HitTID; FVector3d HitBaryCoords;
		if (AABBTree->FindNearestHitTriangle(UseRay, RayHitT, HitTID, HitBaryCoords))
		{
			WorldHitResultOut.HitMeshTriIndex = HitTID;
			WorldHitResultOut.HitMeshSpatialWrapper = this;
			if (bHasBakedTransform)
			{
				WorldHitResultOut.RayDistance = RayHitT;
			}
			else
			{
				FVector3d WorldPos = LocalToWorldTransform.TransformPosition(UseRay.PointAt(RayHitT));
				WorldHitResultOut.RayDistance = WorldRay.GetParameter(WorldPos);
			}
			WorldHitResultOut.HitMeshBaryCoords = HitBaryCoords;
			return true;
		}
		return false;
	}


	virtual bool ProcessVerticesInWorld(TFunctionRef<bool(const FVector3d&)> ProcessFunc, const FTransformSequence3d& LocalToWorldTransform) override
	{
		bool bContinue = true;
		if (bHasBakedTransform)
		{
			for (FVector3d P : Mesh.VerticesItr())
			{
				bContinue = ProcessFunc(P);
				if (!bContinue)
				{
					break;
				}
			}
		}
		else
		{
			for (FVector3d P : Mesh.VerticesItr())
			{
				bContinue = ProcessFunc(LocalToWorldTransform.TransformPosition(P));
				if (!bContinue)
				{
					break;
				}
			}
		}
		return bContinue;
	}


	virtual void AppendMesh(FDynamicMesh3& AppendTo, const FTransformSequence3d& TransformSeq) override
	{
		FDynamicMeshEditor Editor(&AppendTo);
		FMeshIndexMappings Mappings;
		if (bHasBakedTransform)
		{
			Editor.AppendMesh(&Mesh, Mappings,
				[&](int, const FVector3d& Pos) { return Pos; },
				[&](int, const FVector3d& Normal) { return Normal; });
		}
		else
		{
			Editor.AppendMesh(&Mesh, Mappings,
				[&](int, const FVector3d& Pos) { return TransformSeq.TransformPosition(Pos); },
				[&](int, const FVector3d& Normal) { return TransformSeq.TransformNormal(Normal); });
		}
	}

};





/**
* IMeshSpatialWrapper that stores a minimal ("compressed") version of an input mesh,
* basically just a vertex array and indexed triangle list. This reduces the memory
* footprint of large MeshDescription/FDynamicMeshes by dropping expensive attribute 
* overlays and connectivity storage. In addition, for very small meshes, FDynamicMesh3
* is not an efficient representation because each internal buffer has a minimum size,
* so using this format can significantly reduce the overhead of all those tiny meshes.
* 
* Generally supports the same options and behavior as FDynamicMeshSpatialWrapper. 
* It might be useful to try and combine the various queries in a template base class?
* 
* Note that this wrapper currently does not support the barycentric-interpolation queries 
* of IMeshSpatialWrapper, and cannot be used if they are requested in the build options.
*/
class FCompressedMeshSpatialWrapper : public FBaseMeshSpatialWrapper
{
public:
	FDynamicMesh3 BuildInputMesh;		// source mesh data - only used in build, then discarded

	// compressed representation
	FColliderMesh ColliderMesh;
	TUniquePtr<TFastWindingTree<FColliderMesh>> FWNTree;

	FCompressedMeshSpatialWrapper()
	{
	}

	FCompressedMeshSpatialWrapper(FDynamicMesh3&& SourceMesh)
	{
		BuildInputMesh = MoveTemp(SourceMesh);
	}


	virtual bool Build(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
		ensure(BuildOptions.bEnableUVQueries == false && BuildOptions.bEnableNormalsQueries == false);
		ensure(BuildInputMesh.TriangleCount() > 0);

		FColliderMesh::FBuildOptions ColliderBuildOptions;
		ColliderBuildOptions.bBuildAABBTree = BuildOptions.bBuildSpatialDataStructures;
		ColliderMesh.Initialize(BuildInputMesh);
		BuildInputMesh = FDynamicMesh3();		// discard input mesh data

		if (BuildOptions.bBuildSpatialDataStructures)
		{
			if (bUseDistanceShellForWinding == false || bRequiresWindingQueryFallback)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_DMesh_FWNTree);
				check( ColliderMesh.GetRawAABBTreeUnsafe() != nullptr );
				FWNTree = MakeUnique<TFastWindingTree<FColliderMesh>>(ColliderMesh.GetRawAABBTreeUnsafe(), true);
			}
		}
		return true;
	}

	virtual int32 GetTriangleCount() const override
	{
		return ColliderMesh.TriangleCount();
	}

	virtual bool IsTriangle(int32 TriId) const override
	{
		return ColliderMesh.IsTriangle(TriId);
	}

	virtual FIndex3i GetTriangle(int32 TriId) const override
	{
		return ColliderMesh.GetTriangle(TriId);
	}

	virtual bool HasNormals() const override
	{
		check(false);
		return false;
	}
	virtual bool HasUVs(const int UVLayer = 0) const override
	{
		check(false);
		return false;
	}

	virtual FVector3d TriBaryInterpolatePoint(int32 TriId, const FVector3d& BaryCoords) const override
	{
		check(false);
		return FVector3d::Zero();
	}
	virtual bool TriBaryInterpolateNormal(int32 TriId, const FVector3d& BaryCoords, FVector3f& NormalOut) const override
	{
		check(false);
		return false;
	}
	virtual bool TriBaryInterpolateUV(const int32 TriId, const FVector3d& BaryCoords, const int UVLayer, FVector2f& UVOut) const override
	{
		check(false);
		return false;
	}

	virtual FAxisAlignedBox3d GetWorldBounds(TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		FAxisAlignedBox3d Bounds = bHasBakedTransform ?
			GetTransformedVertexBounds<FColliderMesh>(ColliderMesh, [](const FVector3d& P) { return P;}) :
			GetTransformedVertexBounds<FColliderMesh>(ColliderMesh, LocalToWorldFunc);
		if (bUseDistanceShellForWinding)
		{
			Bounds.Expand(WindingShellThickness);
		}
		return Bounds;
	}

	virtual void CollectSeedPoints(TArray<FVector3d>& WorldPoints, TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		return bHasBakedTransform ?
			CollectSeedPointsFromMeshVertices<FColliderMesh>(ColliderMesh, [](const FVector3d& P) {return P;}, WorldPoints) :
			CollectSeedPointsFromMeshVertices<FColliderMesh>(ColliderMesh, LocalToWorldFunc, WorldPoints);
	}

	virtual double FastWindingNumber(const FVector3d& P, const FTransformSequence3d& LocalToWorldTransform) override
	{
		if (bUseDistanceShellForWinding)
		{
			if (bRequiresWindingQueryFallback)
			{
				double WindingNumber = bHasBakedTransform ?
					FWNTree->FastWindingNumber(P) :
					FWNTree->FastWindingNumber(LocalToWorldTransform.InverseTransformPosition(P));
				if (WindingNumber > 0.99)
				{
					return 1.0;
				}
			}

			if (bHasBakedTransform || bHasBakedScale)
			{
				FVector3d UseP = (bHasBakedTransform) ? P : LocalToWorldTransform.InverseTransformPosition(P);
				double NearestDistSqr;
				int32 NearTriID = ColliderMesh.FindNearestTriangle(UseP, NearestDistSqr, IMeshSpatial::FQueryOptions(WindingShellThickness));
				if (NearTriID != IndexConstants::InvalidID)
				{
					// Do we even need to do this? won't we return InvalidID if we don't find point within distance?
					// (also technically we can early-out as soon as we find any point, not the nearest point - might be worth a custom query)
					FDistPoint3Triangle3d Query = TMeshQueries<FColliderMesh>::TriangleDistance(ColliderMesh, NearTriID, UseP);
					if (Query.GetSquared() < WindingShellThickness * WindingShellThickness)
					{
						return 1.0;
					}
				}
			}
			else
			{
				ensure(false);		// not supported yet
			}
			return 0.0;
		}
		else
		{
			return bHasBakedTransform ?
				FWNTree->FastWindingNumber(P) :
				FWNTree->FastWindingNumber(LocalToWorldTransform.InverseTransformPosition(P));
		}
	}


	virtual bool RayIntersection(const FRay3d& WorldRay, const FTransformSequence3d& LocalToWorldTransform, FMeshSceneRayHit& WorldHitResultOut) override
	{
		FRay3d UseRay = WorldRay;
		if (bHasBakedTransform == false)
		{
			FVector3d LocalOrigin = LocalToWorldTransform.InverseTransformPosition(WorldRay.Origin);
			FVector3d LocalDir = Normalized(LocalToWorldTransform.InverseTransformPosition(WorldRay.PointAt(1.0)) - LocalOrigin);
			UseRay = FRay3d(LocalOrigin, LocalDir);
		}

		double RayHitT; int32 HitTID; FVector3d HitBaryCoords;
		if (ColliderMesh.FindNearestHitTriangle(UseRay, RayHitT, HitTID, HitBaryCoords))
		{
			WorldHitResultOut.HitMeshTriIndex = HitTID;
			WorldHitResultOut.HitMeshSpatialWrapper = this;
			if (bHasBakedTransform)
			{
				WorldHitResultOut.RayDistance = RayHitT;
			}
			else
			{
				FVector3d WorldPos = LocalToWorldTransform.TransformPosition(UseRay.PointAt(RayHitT));
				WorldHitResultOut.RayDistance = WorldRay.GetParameter(WorldPos);
			}
			WorldHitResultOut.HitMeshBaryCoords = HitBaryCoords;
			return true;
		}
		return false;
	}


	virtual bool ProcessVerticesInWorld(TFunctionRef<bool(const FVector3d&)> ProcessFunc, const FTransformSequence3d& LocalToWorldTransform) override
	{
		bool bContinue = true;
		if (bHasBakedTransform)
		{
			for ( int32 k = 0; k < ColliderMesh.VertexCount(); ++k)
			{
				bContinue = ProcessFunc(ColliderMesh.GetVertex(k));
				if (!bContinue)
				{
					break;
				}
			}
		}
		else
		{
			for ( int32 k = 0; k < ColliderMesh.VertexCount(); ++k )
			{
				bContinue = ProcessFunc( LocalToWorldTransform.TransformPosition(ColliderMesh.GetVertex(k)) );
				if (!bContinue)
				{
					break;
				}
			}
		}
		return bContinue;
	}


	virtual void AppendMesh(FDynamicMesh3& AppendTo, const FTransformSequence3d& TransformSeq) override
	{
		// this is necessary because the Editor.AppendMesh function is not a template and explicitly takes a FTriangleMeshAdapterd
		TMeshWrapperAdapterd<FColliderMesh> AdapterWrapper(&ColliderMesh);

		FDynamicMeshEditor Editor(&AppendTo);
		FMeshIndexMappings Mappings;
		if (bHasBakedTransform)
		{
			Editor.AppendMesh(&AdapterWrapper, Mappings,
				[](int, const FVector3d& Pos) { return Pos; });
		}
		else
		{
			Editor.AppendMesh(&AdapterWrapper, Mappings,
				[&](int, const FVector3d& Pos) { return TransformSeq.TransformPosition(Pos); });
		}
	}

};






/**
 * Utility function for handling static mesh material queries
 */
static int32 GetStaticMeshMaterialIndexFromSlotName(UStaticMesh* StaticMesh, FName MaterialSlotName, bool bFallbackToSlot0OnFailure)
{
	const TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();
	int32 NumMaterials = StaticMaterials.Num();

	// search primary slot names first
	for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
	{
		const FStaticMaterial& StaticMaterial = StaticMaterials[MaterialIndex];
		if (StaticMaterial.MaterialSlotName == MaterialSlotName)
		{
			return MaterialIndex;
		}
	}

	// sometimes we are given the ImportedSlotName
	for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
	{
		const FStaticMaterial& StaticMaterial = StaticMaterials[MaterialIndex];
		if (StaticMaterial.ImportedMaterialSlotName == MaterialSlotName)
		{
			return MaterialIndex;
		}
	}

	return (bFallbackToSlot0OnFailure && NumMaterials > 0) ? 0 : INDEX_NONE;
}



static UMaterialInterface* GetStaticMeshMaterialFromSlotName(UStaticMesh* StaticMesh, FName MaterialSlotName, bool bFallbackToSlot0OnFailure)
{
	int32 MaterialIndex = GetStaticMeshMaterialIndexFromSlotName(StaticMesh, MaterialSlotName, bFallbackToSlot0OnFailure);
	if (MaterialIndex > INDEX_NONE)
	{
		UMaterialInterface* MaterialInterface = StaticMesh->GetMaterial(MaterialIndex);
		return (MaterialInterface != nullptr) ? MaterialInterface : UMaterial::GetDefaultMaterial(MD_Surface);
	}
	else
	{
		return UMaterial::GetDefaultMaterial(MD_Surface);
	}
}




/**
 * Mesh adapter to filter out all mesh sections that are using materials for which the material domain isn't "Surface"
 * This excludes decals for example.
 */
struct FMeshDescriptionTriangleMeshSurfaceAdapter : public FMeshDescriptionTriangleMeshAdapter
{
public:
	FMeshDescriptionTriangleMeshSurfaceAdapter(const FMeshDescription* MeshIn, UStaticMesh* StaticMeshIn, bool bOnlySurfaceMaterials)
		: FMeshDescriptionTriangleMeshAdapter(MeshIn)
		, NumValidTriangles(0)
		, bHasInvalidPolyGroup(false)
	{
		FStaticMeshConstAttributes MeshDescriptionAttributes(*MeshIn);
		TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();

		if (bOnlySurfaceMaterials)
		{
			for (FPolygonGroupID PolygonGroupID : MeshIn->PolygonGroups().GetElementIDs())
			{
				// Try to find correct material index for the slot name in the MeshDescription. 
				// Sometimes this data is wrong, in that case fall back to slot 0
				FName MaterialSlotName = MaterialSlotNames[PolygonGroupID];
				int32 MaterialIndex = GetStaticMeshMaterialIndexFromSlotName(StaticMeshIn, MaterialSlotName, true);

				UMaterialInterface* MaterialInterface = GetStaticMeshMaterialFromSlotName(StaticMeshIn, MaterialSlotName, true);
				const UMaterial* Material = MaterialInterface != nullptr ? MaterialInterface->GetMaterial_Concurrent() : nullptr;

				bool bValidPolyGroup = Material != nullptr && Material->MaterialDomain == EMaterialDomain::MD_Surface;
				if (bValidPolyGroup)
				{
					ValidPolyGroups.Add(PolygonGroupID);
					NumValidTriangles += MeshIn->GetNumPolygonGroupTriangles(PolygonGroupID);
				}
				else
				{
					bHasInvalidPolyGroup = true;
				}
			}
		}
	}

	bool IsTriangle(int32 TID) const
	{
		return bHasInvalidPolyGroup ? ValidPolyGroups.Contains(Mesh->GetTrianglePolygonGroup(TID)) : FMeshDescriptionTriangleMeshAdapter::IsTriangle(TID);
	}

	int32 TriangleCount() const
	{
		return bHasInvalidPolyGroup ? NumValidTriangles : FMeshDescriptionTriangleMeshAdapter::TriangleCount();
	}

private:
	TSet<FPolygonGroupID> ValidPolyGroups;
	int32 NumValidTriangles;
	bool bHasInvalidPolyGroup;
};

/**
* Extension of Mesh adapter for StaticMesh LODResources (ie LOD0 rendering mesh) to filter out all mesh sections that 
* are using materials for which the material domain isn't "Surface". This excludes decals for example.
*/
struct FStaticMeshLODResourcesMeshSurfaceAdapter : public FStaticMeshLODResourcesMeshAdapter
{
	FStaticMeshLODResourcesMeshSurfaceAdapter(const FStaticMeshLODResources* MeshIn, UStaticMesh* StaticMeshIn, bool bOnlySurfaceMaterials)
	{
		this->Mesh = MeshIn;
		this->NumTriangles = 0;

		TriangleOffsetArray.Reserve(MeshIn->Sections.Num() + 1);
		ValidSections.Reserve(MeshIn->Sections.Num());

		for (const FStaticMeshSection& Section : MeshIn->Sections)
		{
			auto IsValidMaterial = [StaticMeshIn, Section]()
			{
				UMaterialInterface* MaterialInterface = StaticMeshIn->GetMaterial(Section.MaterialIndex);
				const UMaterial* Material = MaterialInterface != nullptr ? MaterialInterface->GetMaterial_Concurrent() : nullptr;

				return Material != nullptr && Material->MaterialDomain == EMaterialDomain::MD_Surface;
			};

			bool bIsValidMaterial = bOnlySurfaceMaterials ? IsValidMaterial() : true;
			if (bIsValidMaterial)
			{
				TriangleOffsetArray.Add(NumTriangles);
				NumTriangles += Section.NumTriangles;
				ValidSections.Add(&Section);
			}
		}

		TriangleOffsetArray.Add(NumTriangles);
	}
};



/**
 * TStaticMeshSpatialWrapperBase is an IMeshSpatialWrapper implementation that can be used
 * with various different mesh adapter types, depending on whether the "source" mesh or built/render
 * mesh should be used. This is an abstract base class, calling code should use FStaticMeshSourceDataSpatialWrapper 
 * or FStaticMeshRenderDataSpatialWrapper as needed
 */
template <class MeshAdapterType>
class TStaticMeshSpatialWrapperBase : public IMeshSpatialWrapper
{
public:
	UStaticMesh* StaticMesh = nullptr;
	int32 LODIndex = 0;

	TUniquePtr<MeshAdapterType> Adapter;

	TUniquePtr<TMeshAABBTree3<MeshAdapterType>> AABBTree;
	TUniquePtr<TFastWindingTree<MeshAdapterType>> FWNTree;

	/**
	 * Create the MeshAdapterType instance based on the build options and source mesh.
	 * Subclasses must implement this function.
	 */
	virtual TUniquePtr<MeshAdapterType> MakeMeshAdapter(const FMeshSceneAdapterBuildOptions& BuildOptions) = 0;

	virtual bool Build(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
		check(StaticMesh);

		Adapter = MakeMeshAdapter(BuildOptions);
		if (!Adapter)
		{
			return false;
		}

		if (BuildOptions.bBuildSpatialDataStructures)
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_StaticMesh_AABBTree);
				AABBTree = MakeUnique<TMeshAABBTree3<MeshAdapterType>>(Adapter.Get(), true);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_StaticMesh_FWNTree);
				FWNTree = MakeUnique<TFastWindingTree<MeshAdapterType>>(AABBTree.Get(), true);
			}
		}

		return true;
	}

	virtual int32 GetTriangleCount() const override
	{
		return Adapter ? Adapter->TriangleCount() : 0;
	}

	virtual bool IsTriangle(int32 TriId) const override
	{
		return Adapter ? Adapter->IsTriangle(TriId) : false;
	}

	virtual FIndex3i GetTriangle(int32 TriId) const override
	{
		return Adapter ? Adapter->GetTriangle(TriId) : FIndex3i::Invalid();
	}

	virtual bool HasNormals() const override
	{
		return Adapter ? Adapter->HasNormals() : false;
	}

	virtual bool HasUVs(const int UVLayer = 0) const override
	{
		return Adapter ? Adapter->HasUVs(UVLayer) : false;
	}

	virtual FVector3d TriBaryInterpolatePoint(int32 TriId, const FVector3d& BaryCoords) const override
	{
		if (Adapter)
		{
			FVector3d P0, P1, P2;
			Adapter->GetTriVertices(TriId, P0, P1, P2);
			return BaryCoords[0] * P0 + BaryCoords[1] * P1 + BaryCoords[2] * P2;
		}
		return FVector3d::Zero();
	}

	virtual bool TriBaryInterpolateNormal(int32 TriId, const FVector3d& BaryCoords, FVector3f& NormalOut) const override
	{
		if (Adapter && Adapter->HasNormals())
		{
			FIndex3i VertIds = Adapter->GetTriangle(TriId);
			
			FVector3f N0, N1, N2;
			Adapter->GetTriNormals(TriId, N0, N1, N2);
			NormalOut = (float) BaryCoords[0] * N0 + (float) BaryCoords[1] * N1 + (float) BaryCoords[2] * N2;
			return true;
		}
		return false;
	}

	virtual bool TriBaryInterpolateUV(const int32 TriId, const FVector3d& BaryCoords, const int UVLayer, FVector2f& UVOut) const override
	{
		if (Adapter && Adapter->HasUVs(UVLayer))
		{
			FVector2f UV0, UV1, UV2;
			Adapter->GetTriUVs(TriId, UVLayer, UV0, UV1, UV2);
			UVOut = (float) BaryCoords[0] * UV0 + (float) BaryCoords[1] * UV1 + (float) BaryCoords[2] * UV2;
			return true;
		}
		return false;
	}

	virtual FAxisAlignedBox3d GetWorldBounds(TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		if (!Adapter) return FAxisAlignedBox3d::Empty();
		return GetTransformedVertexBounds<MeshAdapterType>(*Adapter, LocalToWorldFunc);
	}

	virtual void CollectSeedPoints(TArray<FVector3d>& WorldPoints, TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		if (!Adapter) return;
		CollectSeedPointsFromMeshVertices<MeshAdapterType>(*Adapter, LocalToWorldFunc, WorldPoints);
	}

	virtual double FastWindingNumber(const FVector3d& P, const FTransformSequence3d& LocalToWorldTransform) override
	{
		return (Adapter) ? FWNTree->FastWindingNumber(LocalToWorldTransform.InverseTransformPosition(P)) : 0.0;
	}

	virtual bool RayIntersection(const FRay3d& WorldRay, const FTransformSequence3d& LocalToWorldTransform, FMeshSceneRayHit& WorldHitResultOut) override
	{
		FVector3d LocalOrigin = LocalToWorldTransform.InverseTransformPosition(WorldRay.Origin);
		FVector3d LocalDir = Normalized(LocalToWorldTransform.InverseTransformPosition(WorldRay.PointAt(1.0)) - LocalOrigin);
		FRay3d LocalRay(LocalOrigin, LocalDir);

		double LocalHitT; int32 HitTID; FVector3d HitBaryCoords;
		if (AABBTree->FindNearestHitTriangle(LocalRay, LocalHitT, HitTID, HitBaryCoords))
		{
			WorldHitResultOut.HitMeshTriIndex = HitTID;
			WorldHitResultOut.HitMeshSpatialWrapper = this;
			FVector3d WorldPos = LocalToWorldTransform.TransformPosition(LocalRay.PointAt(LocalHitT));
			WorldHitResultOut.RayDistance = WorldRay.GetParameter(WorldPos);
			WorldHitResultOut.HitMeshBaryCoords = HitBaryCoords;
			return true;
		}
		return false;
	}


	virtual bool ProcessVerticesInWorld(TFunctionRef<bool(const FVector3d&)> ProcessFunc, const FTransformSequence3d& LocalToWorldTransform) override
	{
		bool bContinue = true;
		int32 NumVertices = (Adapter) ? Adapter->VertexCount() : 0;
		for (int32 vi = 0; vi < NumVertices && bContinue; ++vi)
		{
			if (Adapter->IsVertex(vi))
			{
				bContinue = ProcessFunc(LocalToWorldTransform.TransformPosition(Adapter->GetVertex(vi)));
			}
		}
		return bContinue;
	}

	virtual void AppendMesh(FDynamicMesh3& AppendTo, const FTransformSequence3d& TransformSeq) override
	{
		if (!Adapter) return;

		// TODO: need to move this to FStaticMeshSourceDataSpatialWrapper?
//#if WITH_EDITOR
//		// Fast path only works on non-filtered meshes
//		// as it relies on the mesh description directly rather than the adapter
//		bool bFilteredMesh = SourceMesh->Triangles().Num() != Adapter->TriangleCount();
//		if (!bFilteredMesh && AppendTo.TriangleCount() == 0 && TransformSeq.Num() == 0)
//		{
//			// this is somewhat faster in profiling
//			FMeshDescription* UseMeshDescription = StaticMesh->GetMeshDescription(LODIndex);
//			FMeshDescriptionToDynamicMesh Converter;
//			Converter.bEnableOutputGroups = false; Converter.bCalculateMaps = false;
//			Converter.bDisableAttributes = true;
//			Converter.Convert(UseMeshDescription, AppendTo);
//			MeshTransforms::Scale(AppendTo, BuildScale, FVector3d::Zero());
//			return;
//		}
//#endif

		FDynamicMeshEditor Editor(&AppendTo);
		FMeshIndexMappings Mappings;
		// this is necessary because the Editor.AppendMesh function is not a template and explicitly takes a FTriangleMeshAdapterd
		TMeshWrapperAdapterd<MeshAdapterType> AdapterWrapper(Adapter.Get());
		Editor.AppendMesh(&AdapterWrapper, Mappings,
			[&](int, const FVector3d& Pos) { return TransformSeq.TransformPosition(Pos); });
	}

};



class FStaticMeshSourceDataSpatialWrapper : public TStaticMeshSpatialWrapperBase<FMeshDescriptionTriangleMeshSurfaceAdapter>
{
public:
	FMeshDescription* CachedSourceMeshDescription = nullptr;

	virtual ~FStaticMeshSourceDataSpatialWrapper()
	{
#if WITH_EDITOR
		// Release any MeshDescriptions that were loaded. This is potentially not ideal and should probably be controlled by the build options...
		if (CachedSourceMeshDescription != nullptr)
		{
			CachedSourceMeshDescription = nullptr;
			StaticMesh->ClearMeshDescriptions();
		}
#endif
	}


	virtual TUniquePtr<FMeshDescriptionTriangleMeshSurfaceAdapter> MakeMeshAdapter(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
#if WITH_EDITOR
		CachedSourceMeshDescription = StaticMesh->GetMeshDescription(LODIndex);
		TUniquePtr<FMeshDescriptionTriangleMeshSurfaceAdapter> SurfaceAdapter = MakeUnique<FMeshDescriptionTriangleMeshSurfaceAdapter>(CachedSourceMeshDescription, StaticMesh, BuildOptions.bOnlySurfaceMaterials);
		const FMeshBuildSettings& LODBuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;
		SurfaceAdapter->SetBuildScale(LODBuildSettings.BuildScale3D, false);
		return SurfaceAdapter;
#else
		// cannot use this path in non-editor builds, but this should have been handled higher up by instantiating 
		// the runtime subclass
		check(false);
		return nullptr;
#endif
	}

};


class FStaticMeshRenderDataSpatialWrapper : public TStaticMeshSpatialWrapperBase<FStaticMeshLODResourcesMeshSurfaceAdapter>
{
public:
	virtual TUniquePtr<FStaticMeshLODResourcesMeshSurfaceAdapter> MakeMeshAdapter(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
		return MakeUnique<FStaticMeshLODResourcesMeshSurfaceAdapter>(StaticMesh->GetRenderData()->GetCurrentFirstLOD(0), StaticMesh, BuildOptions.bOnlySurfaceMaterials);
	}
};



/**
 * FCompressedStaticMeshSpatialWrapper mirrors the behavior of FStaticMeshSourceDataSpatialWrapper, but instead of
 * using the underlying MeshDescription directly, it copies the mesh into a minimal indexed mesh (vert list and indexed tris)
 * and then releases the MeshDescription memory. This is useful to limit the total memory usage of the MeshSceneAdapter
 * in cases where only certain queries are needed. In particular, the UV and Normal queries
 * are not available with this MeshSpatialWrapper. 
 */
class FCompressedStaticMeshSpatialWrapper : public IMeshSpatialWrapper
{
public:
	UStaticMesh* StaticMesh = nullptr;

	FColliderMesh ColliderMesh;
	TUniquePtr<TMeshAABBTree3<FColliderMesh>> AABBTree;
	TUniquePtr<TFastWindingTree<FColliderMesh>> FWNTree;

	virtual TUniquePtr<TStaticMeshSpatialWrapperBase<FMeshDescriptionTriangleMeshSurfaceAdapter>> GetIntermediateWrapper()
	{
		TUniquePtr<FStaticMeshSourceDataSpatialWrapper> TempWrapper = MakeUnique<FStaticMeshSourceDataSpatialWrapper>();
		TempWrapper->SourceContainer = SourceContainer;
		TempWrapper->StaticMesh = StaticMesh;
		return TempWrapper;
	}

	virtual bool Build(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
		// this is because Build() will be called twice
		if (ColliderMesh.TriangleCount() == 0)
		{
			TUniquePtr<TStaticMeshSpatialWrapperBase<FMeshDescriptionTriangleMeshSurfaceAdapter>> TempWrapper = GetIntermediateWrapper();

			FMeshSceneAdapterBuildOptions TempBuildOptions;
			TempBuildOptions.bBuildSpatialDataStructures = false;	// we are just copying from this so do not build spatials
			TempWrapper->Build(TempBuildOptions);

			FDynamicMesh3 TempMesh(EMeshComponents::None);
			TempWrapper->AppendMesh(TempMesh, FTransformSequence3d());

#if WITH_EDITOR
			// dump mesh description memory
			StaticMesh->ClearMeshDescriptions();
#endif

			FColliderMesh::FBuildOptions ColliderBuildOptions;
			ColliderBuildOptions.bBuildAABBTree = false;		// will use our own AABBTree so that we can defer build
			ColliderMesh.Initialize(TempMesh, ColliderBuildOptions);
		}

		if (BuildOptions.bBuildSpatialDataStructures)
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_StaticMeshCompressed_AABBTree);
				AABBTree = MakeUnique<TMeshAABBTree3<FColliderMesh>>(&ColliderMesh, true);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_WrapperBuild_StaticMeshCompressed_FWNTree);
				FWNTree = MakeUnique<TFastWindingTree<FColliderMesh>>(AABBTree.Get(), true);
			}
		}

		return true;
	}

	virtual int32 GetTriangleCount() const override
	{
		return ColliderMesh.TriangleCount();
	}

	virtual bool IsTriangle(int32 TriId) const override
	{
		return ColliderMesh.IsTriangle(TriId);
	}

	virtual FIndex3i GetTriangle(int32 TriId) const override
	{
		return ColliderMesh.GetTriangle(TriId);
	}

	virtual bool HasNormals() const override { return false; }
	virtual bool HasUVs(const int UVLayer = 0) const override { return false; }
	virtual FVector3d TriBaryInterpolatePoint(int32 TriId, const FVector3d& BaryCoords) const override { check(false); return FVector3d::Zero(); }
	virtual bool TriBaryInterpolateNormal(int32 TriId, const FVector3d& BaryCoords, FVector3f& NormalOut) const override { check(false); return false; }
	virtual bool TriBaryInterpolateUV(const int32 TriId, const FVector3d& BaryCoords, const int UVLayer, FVector2f& UVOut) const override { check(false); return false; }

	virtual FAxisAlignedBox3d GetWorldBounds(TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		return GetTransformedVertexBounds<FColliderMesh>(ColliderMesh, LocalToWorldFunc);
	}

	virtual void CollectSeedPoints(TArray<FVector3d>& WorldPoints, TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) override
	{
		CollectSeedPointsFromMeshVertices<FColliderMesh>(ColliderMesh, LocalToWorldFunc, WorldPoints);
	}

	virtual double FastWindingNumber(const FVector3d& P, const FTransformSequence3d& LocalToWorldTransform) override
	{
		return FWNTree->FastWindingNumber(LocalToWorldTransform.InverseTransformPosition(P));
	}

	virtual bool RayIntersection(const FRay3d& WorldRay, const FTransformSequence3d& LocalToWorldTransform, FMeshSceneRayHit& WorldHitResultOut) override
	{
		FVector3d LocalOrigin = LocalToWorldTransform.InverseTransformPosition(WorldRay.Origin);
		FVector3d LocalDir = Normalized(LocalToWorldTransform.InverseTransformPosition(WorldRay.PointAt(1.0)) - LocalOrigin);
		FRay3d LocalRay(LocalOrigin, LocalDir);

		double LocalHitT; int32 HitTID; FVector3d HitBaryCoords;
		if (AABBTree->FindNearestHitTriangle(LocalRay, LocalHitT, HitTID, HitBaryCoords))
		{
			WorldHitResultOut.HitMeshTriIndex = HitTID;
			WorldHitResultOut.HitMeshSpatialWrapper = this;
			FVector3d WorldPos = LocalToWorldTransform.TransformPosition(LocalRay.PointAt(LocalHitT));
			WorldHitResultOut.RayDistance = WorldRay.GetParameter(WorldPos);
			WorldHitResultOut.HitMeshBaryCoords = HitBaryCoords;
			return true;
		}
		return false;
	}


	virtual bool ProcessVerticesInWorld(TFunctionRef<bool(const FVector3d&)> ProcessFunc, const FTransformSequence3d& LocalToWorldTransform) override
	{
		bool bContinue = true;
		int32 NumVertices = ColliderMesh.VertexCount();
		for (int32 vi = 0; vi < NumVertices && bContinue; ++vi)
		{
			if (ColliderMesh.IsVertex(vi))
			{
				bContinue = ProcessFunc(LocalToWorldTransform.TransformPosition(ColliderMesh.GetVertex(vi)));
			}
		}
		return bContinue;
	}

	virtual void AppendMesh(FDynamicMesh3& AppendTo, const FTransformSequence3d& TransformSeq) override
	{
		TArray<int32> NewVertIDs;
		NewVertIDs.SetNum(ColliderMesh.VertexCount());
		for (int32 k = 0; k < ColliderMesh.VertexCount(); ++k)
		{
			NewVertIDs[k] = AppendTo.AppendVertex(ColliderMesh.GetVertex(k));
		}
		for (int32 k = 0; k < ColliderMesh.TriangleCount(); ++k)
		{
			FIndex3i Tri = ColliderMesh.GetTriangle(k);
			AppendTo.AppendTriangle(NewVertIDs[Tri.A], NewVertIDs[Tri.B], NewVertIDs[Tri.C]);
		}
	}

};


class FSplineStaticMeshSpatialWrapperBase : public TStaticMeshSpatialWrapperBase<FMeshDescriptionTriangleMeshSurfaceAdapter>
{
public:
	FMeshDescription DeformedSourceMeshDescription;
	USplineMeshComponent* SplineMeshComponent = nullptr;

	virtual void InitializeSourceMeshDescription() = 0;

	virtual TUniquePtr<FMeshDescriptionTriangleMeshSurfaceAdapter> MakeMeshAdapter(const FMeshSceneAdapterBuildOptions& BuildOptions) override
	{
#if WITH_EDITOR
		InitializeSourceMeshDescription();
		SplineMeshComponent->PropagateSplineDeformationToMesh(DeformedSourceMeshDescription);

		TUniquePtr<FMeshDescriptionTriangleMeshSurfaceAdapter> SurfaceAdapter = MakeUnique<FMeshDescriptionTriangleMeshSurfaceAdapter>(&DeformedSourceMeshDescription, StaticMesh, BuildOptions.bOnlySurfaceMaterials);
		const FMeshBuildSettings& LODBuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;
		SurfaceAdapter->SetBuildScale(LODBuildSettings.BuildScale3D, false);
		return SurfaceAdapter;
#else
		// cannot use this path in non-editor builds, but this should have been handled higher up by instantiating 
		// the runtime subclass
		check(false);
		return nullptr;
#endif
	}
};


class FSplineStaticMeshSourceDataSpatialWrapper : public FSplineStaticMeshSpatialWrapperBase
{
	virtual void InitializeSourceMeshDescription() override
	{
#if WITH_EDITOR
		StaticMesh->CloneMeshDescription(LODIndex, DeformedSourceMeshDescription);
#endif
	}
};


class FSplineStaticMeshRenderDataSpatialWrapper : public FSplineStaticMeshSpatialWrapperBase
{
	virtual void InitializeSourceMeshDescription() override
	{
#if WITH_EDITOR
		if (const FStaticMeshLODResources* LODResource = StaticMesh->GetRenderData()->GetCurrentFirstLOD(0))
		{
			StaticMesh->ExportStaticMeshLOD(*LODResource, DeformedSourceMeshDescription);
		}
#endif
	}
};


class FCompressedSplineStaticMeshSpatialWrapper : public FCompressedStaticMeshSpatialWrapper
{
public:
	USplineMeshComponent* SplineMeshComponent = nullptr;

	virtual TUniquePtr<TStaticMeshSpatialWrapperBase<FMeshDescriptionTriangleMeshSurfaceAdapter>> GetIntermediateWrapper() override
	{
		TUniquePtr<FSplineStaticMeshSourceDataSpatialWrapper> TempWrapper = MakeUnique<FSplineStaticMeshSourceDataSpatialWrapper>();
		TempWrapper->SourceContainer = SourceContainer;
		TempWrapper->StaticMesh = StaticMesh;
		TempWrapper->SplineMeshComponent = SplineMeshComponent;
		return TempWrapper;
	}
};


static TUniquePtr<IMeshSpatialWrapper> SpatialWrapperFactory( const FMeshTypeContainer& MeshContainer, const TArray<FActorChildMesh*>& ParentMeshes, const FMeshSceneAdapterBuildOptions& BuildOptions )
{
	if (MeshContainer.MeshType == ESceneMeshType::StaticMeshAsset)
	{
		UStaticMesh* StaticMesh = MeshContainer.GetStaticMesh();
		if (ensure(StaticMesh != nullptr))
		{
#if WITH_EDITOR
			bool bUseRenderMeshes = BuildOptions.bIgnoreStaticMeshSourceData || StaticMesh->GetOutermost()->bIsCookedForEditor;
			bUseRenderMeshes &= StaticMesh->GetRenderData() && StaticMesh->GetRenderData()->GetCurrentFirstLOD(0);
#else
			bool bUseRenderMeshes = true;
#endif
			const bool bIsSplineMesh = (ParentMeshes.Num() == 1 && ParentMeshes[0]->ComponentType == EActorMeshComponentType::SplineStaticMesh);
			if (bIsSplineMesh)
			{
				if (bUseRenderMeshes)
				{
					TUniquePtr<FSplineStaticMeshRenderDataSpatialWrapper> SplineSMWrapper = MakeUnique<FSplineStaticMeshRenderDataSpatialWrapper>();
					SplineSMWrapper->SourceContainer = MeshContainer;
					SplineSMWrapper->StaticMesh = StaticMesh;
					SplineSMWrapper->SplineMeshComponent = Cast<USplineMeshComponent>(ParentMeshes[0]->SourceComponent);
					return SplineSMWrapper;
				}
				else if (BuildOptions.bEnableUVQueries == false && BuildOptions.bEnableNormalsQueries == false)
				{
					TUniquePtr<FCompressedSplineStaticMeshSpatialWrapper> SplineSMWrapper = MakeUnique<FCompressedSplineStaticMeshSpatialWrapper>();
					SplineSMWrapper->SourceContainer = MeshContainer;
					SplineSMWrapper->StaticMesh = StaticMesh;
					SplineSMWrapper->SplineMeshComponent = Cast<USplineMeshComponent>(ParentMeshes[0]->SourceComponent);
					return SplineSMWrapper;
				}
				else
				{
					TUniquePtr<FSplineStaticMeshSourceDataSpatialWrapper> SplineSMWrapper = MakeUnique<FSplineStaticMeshSourceDataSpatialWrapper>();
					SplineSMWrapper->SourceContainer = MeshContainer;
					SplineSMWrapper->StaticMesh = StaticMesh;
					SplineSMWrapper->SplineMeshComponent = Cast<USplineMeshComponent>(ParentMeshes[0]->SourceComponent);
					return SplineSMWrapper;
				}
			}
			else
			{
				if (bUseRenderMeshes)
				{
					TUniquePtr<FStaticMeshRenderDataSpatialWrapper> SMWrapper = MakeUnique<FStaticMeshRenderDataSpatialWrapper>();
					SMWrapper->SourceContainer = MeshContainer;
					SMWrapper->StaticMesh = StaticMesh;
					return SMWrapper;
				}
				else if (BuildOptions.bEnableUVQueries == false && BuildOptions.bEnableNormalsQueries == false)
				{
					TUniquePtr<FCompressedStaticMeshSpatialWrapper> SMWrapper = MakeUnique<FCompressedStaticMeshSpatialWrapper>();
					SMWrapper->SourceContainer = MeshContainer;
					SMWrapper->StaticMesh = StaticMesh;
					return SMWrapper;
				}
				else
				{
					TUniquePtr<FStaticMeshSourceDataSpatialWrapper> SMWrapper = MakeUnique<FStaticMeshSourceDataSpatialWrapper>();
					SMWrapper->SourceContainer = MeshContainer;
					SMWrapper->StaticMesh = StaticMesh;
					return SMWrapper;
				}
			}
		}
	}

	return TUniquePtr<IMeshSpatialWrapper>();
}




static void CollectComponentChildMeshes(UActorComponent* Component, FActorAdapter& Adapter)
{
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent != nullptr)
	{
		UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh();
		if (Mesh != nullptr)
		{
			TUniquePtr<FActorChildMesh> ChildMesh = MakeUnique<FActorChildMesh>();
			ChildMesh->SourceComponent = Component;
			ChildMesh->MeshContainer = FMeshTypeContainer{ Mesh, ESceneMeshType::StaticMeshAsset };

			UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent);
			if (ISMComponent != nullptr)
			{
				// does anything additional need to happen here for HISMC?

				ChildMesh->ComponentType = EActorMeshComponentType::InstancedStaticMesh;

				int32 NumInstances = ISMComponent->GetInstanceCount();
				for (int32 i = 0; i < NumInstances; ++i)
				{
					if (ISMComponent->IsValidInstance(i))
					{
						FTransform InstanceTransform;
						if (ensure(ISMComponent->GetInstanceTransform(i, InstanceTransform, true)))
						{
							TUniquePtr<FActorChildMesh> InstanceChild = MakeUnique<FActorChildMesh>();
							InstanceChild->SourceComponent = ChildMesh->SourceComponent;
							InstanceChild->MeshContainer = ChildMesh->MeshContainer;
							InstanceChild->ComponentType = ChildMesh->ComponentType;
							InstanceChild->ComponentIndex = i;
							InstanceChild->WorldTransform.Append(InstanceTransform);
							InstanceChild->bIsNonUniformScaled = InstanceChild->WorldTransform.HasNonUniformScale();
							Adapter.ChildMeshes.Add(MoveTemp(InstanceChild));
						}
					}
				}

			}
			else
			{
				// base StaticMeshComponent
				if (StaticMeshComponent->IsA<USplineMeshComponent>())
				{
					ChildMesh->ComponentType = EActorMeshComponentType::SplineStaticMesh;
				}
				else
				{
					ChildMesh->ComponentType = EActorMeshComponentType::StaticMesh;
				}
				ChildMesh->ComponentIndex = 0;
				ChildMesh->WorldTransform.Append(StaticMeshComponent->GetComponentTransform());
				ChildMesh->bIsNonUniformScaled = ChildMesh->WorldTransform.HasNonUniformScale();
				Adapter.ChildMeshes.Add(MoveTemp(ChildMesh));
			}
		}
		
	}

}


}	// end namespace UE
}	// end namespace Geometry



void FMeshSceneAdapter::AddActors(const TArray<AActor*>& ActorsSetIn)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_AddActors);

	// build an FActorAdapter for each Actor, that contains all mesh Components we know 
	// how to process, including those contained in ChildActorComponents
	TArray<AActor*> ChildActors;
	TArray<FActorAdapter*> NewActorAdapters;
	for (AActor* Actor : ActorsSetIn)
	{
		TUniquePtr<FActorAdapter> Adapter = MakeUnique<FActorAdapter>();
		Adapter->SourceActor = Actor;

		for (UActorComponent* Component : Actor->GetComponents())
		{
			CollectComponentChildMeshes(Component, *Adapter);
		}

		ChildActors.Reset();
		Actor->GetAllChildActors(ChildActors, true);
		for (AActor* ChildActor : ChildActors)
		{
			for (UActorComponent* Component : ChildActor->GetComponents())
			{
				CollectComponentChildMeshes(Component, *Adapter);
			}
		}

		NewActorAdapters.Add(Adapter.Get());
		SceneActors.Add(MoveTemp(Adapter));
	}

	InitializeSpatialWrappers(NewActorAdapters);
}



void FMeshSceneAdapter::AddComponents(const TArray<UActorComponent*>& ComponentSetIn)
{
	// Build an FActorAdapter for each Component. This may result in multiple FActorAdapters
	// for a single AActor, this is currently not a problem
	TArray<FActorAdapter*> NewActorAdapters;
	for (UActorComponent* Component : ComponentSetIn)
	{
		if (Component == nullptr)
		{
			continue;
		}
		
		TUniquePtr<FActorAdapter> Adapter = MakeUnique<FActorAdapter>();
		Adapter->SourceActor = Component->GetOwner();
		CollectComponentChildMeshes(Component, *Adapter);
		if (Adapter->ChildMeshes.Num() > 0)
		{
			NewActorAdapters.Add(Adapter.Get());
			SceneActors.Add(MoveTemp(Adapter));
		}
	}

	InitializeSpatialWrappers(NewActorAdapters);
}



void FMeshSceneAdapter::InitializeSpatialWrappers(const TArray<FActorAdapter*>& NewItemsToProcess)
{
	auto CreateNewWrapperInfo = [](TUniquePtr<FActorChildMesh>& ChildMesh)
	{
		TSharedPtr<FSpatialWrapperInfo> NewWrapperInfo = MakeShared<FSpatialWrapperInfo>();
		NewWrapperInfo->SourceContainer = ChildMesh->MeshContainer;
		NewWrapperInfo->SpatialWrapper = nullptr; // these are now initialized at beginning of Build() function
		return NewWrapperInfo;
	};

	// Find IMeshSpatialWrapper for each child mesh component. If one does not exist
	// and we have not seen the underlying unique mesh (eg StaticMesh Asset, etc, construct a new one
	for (FActorAdapter* Actor : NewItemsToProcess)
	{
		for (TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			FSpatialWrapperInfo* MeshInfo;
			if (ChildMesh->ComponentType == EActorMeshComponentType::SplineStaticMesh)
			{
				TSharedPtr<FSpatialWrapperInfo> NewWrapperInfo = CreateNewWrapperInfo(ChildMesh);
				SplineMeshSpatialAdapters.Add(NewWrapperInfo);
				MeshInfo = NewWrapperInfo.Get();
			}
			else
			{
				void* MeshKey = ChildMesh->MeshContainer.GetMeshKey();
				TSharedPtr<FSpatialWrapperInfo>* Found = SpatialAdapters.Find(MeshKey);
				if (Found == nullptr)
				{
					TSharedPtr<FSpatialWrapperInfo> NewWrapperInfo = CreateNewWrapperInfo(ChildMesh);
					SpatialAdapters.Add(MeshKey, NewWrapperInfo);
					MeshInfo = NewWrapperInfo.Get();
				}
				else
				{
					MeshInfo = (*Found).Get();
				}
			}

			MeshInfo->ParentMeshes.Add(ChildMesh.Get());
			if (ChildMesh->bIsNonUniformScaled)
			{
				MeshInfo->NonUniformScaleCount++;
			}
			ChildMesh->MeshSpatial = nullptr;	// these are now initialized at beginning of Build() function
		}
	}
}

void FMeshSceneAdapter::ForEachSpatialWrapper(TFunctionRef<void(TSharedPtr<FSpatialWrapperInfo>)> Func) const
{
	for (TPair<void*, TSharedPtr<FSpatialWrapperInfo>> Pair : SpatialAdapters)
	{
		Func(Pair.Value);
	}

	for (TSharedPtr<FSpatialWrapperInfo> Adapter : SplineMeshSpatialAdapters)
	{
		Func(Adapter);
	}
}

bool FMeshSceneAdapter::IsValid() const
{
	return !SpatialAdapters.IsEmpty() || !SplineMeshSpatialAdapters.IsEmpty();
}

void FMeshSceneAdapter::Build(const FMeshSceneAdapterBuildOptions& BuildOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build);

	// Initialize all Spatial Wrappers using the default factory
	ForEachSpatialWrapper([&BuildOptions](TSharedPtr<FSpatialWrapperInfo> Adapter)
	{
		Adapter->SpatialWrapper = SpatialWrapperFactory(Adapter->SourceContainer, Adapter->ParentMeshes, BuildOptions);
		// populate the MeshSpatial members of the FActorChildMeshes
		for (FActorChildMesh* ParentMesh : Adapter->ParentMeshes)
		{
			ParentMesh->MeshSpatial = Adapter->SpatialWrapper.Get();
		}
	});

	if (BuildOptions.bThickenThinMeshes)
	{
		Build_FullDecompose(BuildOptions);
	}
	else
	{
		TArray<FSpatialWrapperInfo*> ToBuild;
		ForEachSpatialWrapper([&ToBuild](TSharedPtr<FSpatialWrapperInfo> Adapter)
		{
			ToBuild.Add(Adapter.Get());
		});

		FCriticalSection ListsLock;

		std::atomic<int32> DecomposedSourceMeshCount;
		DecomposedSourceMeshCount = 0;
		std::atomic<int32> DecomposedMeshesCount;
		DecomposedMeshesCount = 0;
		int32 AddedTrisCount = 0;

		// parallel build of all the spatial data structures
		ParallelFor(ToBuild.Num(), [&](int32 i)
		{
			FSpatialWrapperInfo* WrapperInfo = ToBuild[i];
			TUniquePtr<IMeshSpatialWrapper>& Wrapper = WrapperInfo->SpatialWrapper;
			bool bOK = Wrapper->Build(BuildOptions);
			ensure(bOK);	// assumption is that the wrapper will handle failure gracefully
		});

		if (BuildOptions.bPrintDebugMessages)
		{
			UE_LOGF(LogGeometry, Warning, "[FMeshSceneAdapter] decomposed %d source meshes into %d unique meshes containing %d triangles", DecomposedSourceMeshCount.load(), DecomposedMeshesCount.load(), AddedTrisCount)
		}

	}


	// update bounding boxes
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build_ActorBounds);
		ParallelFor(SceneActors.Num(), [&](int32 i)
		{
			UpdateActorBounds(*SceneActors[i]);
		});
	}
}


void FMeshSceneAdapter::UpdateActorBounds(FActorAdapter& Actor)
{
	int32 NumChildren = Actor.ChildMeshes.Num();
	TArray<FAxisAlignedBox3d> ChildBounds;
	ChildBounds.Init(FAxisAlignedBox3d::Empty(), NumChildren);
	ParallelFor(NumChildren, [&](int32 k)
	{
		const TUniquePtr<FActorChildMesh>& ChildMesh = Actor.ChildMeshes[k];
		if (ChildMesh->MeshSpatial != nullptr)
		{
			ChildBounds[k] = ChildMesh->MeshSpatial->GetWorldBounds(
				[&](const FVector3d& P) { return ChildMesh->WorldTransform.TransformPosition(P); });
		}
	});

	Actor.WorldBounds = FAxisAlignedBox3d::Empty();
	for (FAxisAlignedBox3d ChildBound : ChildBounds)
	{
		Actor.WorldBounds.Contain(ChildBound);
	}
}



/**
 * This function is used to group the input set of transforms into subsets
 * that have the same scale. Each of those subsets can be represented by
 * a single scaled mesh with different rotate/translate-only transforms.
 * We use this to reduce the number of times a mesh has to be duplicated when
 * breaking it up into parts that require further processing that is incompatible
 * with (nonuniform) scaling.
 * TODO: Currently cannot differentiate between uniform and nonuniform scaling
 */
void ConstructUniqueScalesMapping(
	const TArray<FTransformSequence3d>& TransformSet, 
	TArray<TArray<int32>>& UniqueScaleSetsOut,
	TArray<FTransformSequence3d>& UniqueScaleTransformsOut,
	double ScaleComponentTolerance = 0.01)
{
	// two transforms are "the same up to scaling" if this returns true
	auto CompareScales = [ScaleComponentTolerance](const FTransformSRT3d& T1, const FTransformSRT3d& T2)
	{
		return (T1.GetScale() - T2.GetScale()).GetAbsMax() < ScaleComponentTolerance;
	};

	UniqueScaleTransformsOut.Reset();
	int32 N = TransformSet.Num();
	TArray<int32> UniqueScaleMap;
	UniqueScaleMap.SetNum(N);
	for (int32 k = 0; k < N; ++k)
	{
		FTransformSequence3d CurTransform = TransformSet[k];
		int32 FoundIndex = -1;
		for (int32 j = 0; j < UniqueScaleTransformsOut.Num(); ++j)
		{
			if (CurTransform.IsEquivalent(UniqueScaleTransformsOut[j], CompareScales))
			{
				FoundIndex = j;
				break;
			}
		}
		if (FoundIndex >= 0)
		{
			UniqueScaleMap[k] = FoundIndex;
		}
		else
		{
			UniqueScaleMap[k] = UniqueScaleTransformsOut.Num();
			UniqueScaleTransformsOut.Add(CurTransform);
		}
	}

	// build clusters
	int32 NumUniqueScales = UniqueScaleTransformsOut.Num();
	UniqueScaleSetsOut.SetNum(NumUniqueScales);
	for (int32 k = 0; k < N; ++k)
	{
		UniqueScaleSetsOut[UniqueScaleMap[k]].Add(k);
	}

}


void FMeshSceneAdapter::Build_FullDecompose(const FMeshSceneAdapterBuildOptions& BuildOptions)
{
	EParallelForFlags ParallelFlags = (CVarMeshSceneAdapterDisableMultiThreading.GetValueOnAnyThread() != 0) ?
		EParallelForFlags::ForceSingleThread : EParallelForFlags::Unbalanced;

	bool bCanUseCompressedMeshes = (BuildOptions.bEnableUVQueries == false && BuildOptions.bEnableNormalsQueries == false);

	// initial list of spatial wrappers that need to be built
	TArray<FSpatialWrapperInfo*> ToBuild;
	ForEachSpatialWrapper([&ToBuild](TSharedPtr<FSpatialWrapperInfo> Adapter)
	{
		ToBuild.Add(Adapter.Get());
	});

	// Initialize the initial set of wrappers. Must do this here so that meshes are loaded and TriangleCount() below is valid
	FMeshSceneAdapterBuildOptions TempBuildOptions = BuildOptions;
	TempBuildOptions.bBuildSpatialDataStructures = false;
	ParallelFor(ToBuild.Num(), [&](int32 i)
	{
		FSpatialWrapperInfo* WrapperInfo = ToBuild[i];
		WrapperInfo->SpatialWrapper->Build(TempBuildOptions);
	}, ParallelFlags);

	// sort build list by increasing triangle count
	ToBuild.Sort([&](const FSpatialWrapperInfo& A, const FSpatialWrapperInfo& B)
	{
		return const_cast<FSpatialWrapperInfo&>(A).SpatialWrapper->GetTriangleCount() < const_cast<FSpatialWrapperInfo&>(B).SpatialWrapper->GetTriangleCount();
	});

	// stats we will collect during execution
	int32 NumInitialSources = ToBuild.Num();
	int32 NumSourceUniqueTris = 0;
	std::atomic<int32> DecomposedSourceMeshCount = 0;
	std::atomic<int32> SourceInstancesCount = 0;
	std::atomic<int32> NewInstancesCount = 0;
	std::atomic<int32> NewUniqueMeshesCount = 0;
	std::atomic<int32> SkippedDecompositionCount = 0;
	std::atomic<int32> SingleTriangleMeshes = 0;
	std::atomic<int32> NumFinalUniqueTris = 0;
	int32 AddedUniqueTrisCount = 0;
	int32 InstancedTrisCount = 0;

	struct FProcessedSourceMeshStats
	{
		FString AssetName;
		int32 SourceTriangles = 0;
		int32 SourceInstances = 0;
		int32 SourceComponents = 0;
		int32 UniqueInstanceScales = 0;
		int32 NumOpen = 0;
		int32 NumThin = 0;
		int32 InstancedSubmeshComponents = 0;
		int32 InstancedSubmeshTris = 0;		// number of source tris used directly, ie copied to a single mesh for original instances
		TArray<FVector2i> SubmeshSizeCount;
		int32 NewUniqueTris = 0;
		int32 NewInstances = 0;
		int32 TotalNumUniqueTris = 0;
	};
	TArray<FProcessedSourceMeshStats> Stats;
	Stats.SetNum(NumInitialSources);

	// these locks are used below to control access
	FCriticalSection ToBuildQueueLock;
	FCriticalSection InternalListsLock;

	// The loop below will emit new IMeshSpatialWrapper's that need to have Build() called.
	// Since larger meshes take longer, it is a better strategy to collect up these jobs and
	// then call Build() in decreasing-size order
	struct FBuildJob
	{
		int TriangleCount;
		IMeshSpatialWrapper* BuildWrapper;
	};
	TArray<FBuildJob> PendingBuildJobs;
	FCriticalSection PendingBuildsLock;
	// this lambda is used below to append to the PendingBuildJobs list above
	auto AddBuildJob = [&PendingBuildJobs, &PendingBuildsLock](IMeshSpatialWrapper* ToBuild, int TriangleCount)
	{
		PendingBuildsLock.Lock();
		PendingBuildJobs.Add(FBuildJob{ TriangleCount, ToBuild });
		PendingBuildsLock.Unlock();
	};

	std::atomic<int32> NumTinyComponents = 0;
	std::atomic<int32> NumTinyInstances = 0;
	std::atomic<int32> TinyInstanceTotalTriangles = 0;

	// Parallel-process all the ToBuild spatial wrappers. If the mesh is closed and all the pieces are good,
	// this will just emit a Build job. Otherwise it will pull the mesh apart into pieces, move all the closed non-thin
	// pieces into a new instance to be referenced by the original FActorChildMesh, and then make new meshes/wrappers
	// for anything that needs geometric changes (eg to bake in scale, thicken mesh, etc), and in those cases, generate
	// new instances as FActorAdapter/FActorChildMesh's. And emit BuildJob's for those different spatial wrappers.
	ParallelFor(ToBuild.Num(), [&](int32 i)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build_ProcessMesh);
		
		// ParallelFor will not respect the sorting by triangle-count we did above, so we have to treat the list as a queue and pop from the back
		ToBuildQueueLock.Lock();
		check(ToBuild.Num() > 0);
		FProcessedSourceMeshStats& ItemStats = Stats[ToBuild.Num() - 1];
		FSpatialWrapperInfo* WrapperInfo = ToBuild.Pop(EAllowShrinking::No);
		NumSourceUniqueTris += WrapperInfo->SpatialWrapper->GetTriangleCount();
		ToBuildQueueLock.Unlock();

		// get name for debugging purposes
		ItemStats.AssetName = WrapperInfo->SourceContainer.GetStaticMesh() ? WrapperInfo->SourceContainer.GetStaticMesh()->GetName() : TEXT("Unknown");

		// convert this mesh to a dynamicmesh for processing
		FDynamicMesh3 LocalMesh;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build_ProcessMesh_1Copy);
			WrapperInfo->SpatialWrapper->AppendMesh(LocalMesh, FTransformSequence3d());
		}
		ItemStats.SourceTriangles = LocalMesh.TriangleCount();

		// construct list of per-instance transforms that reference this mesh
		TArray<FActorChildMesh*> MeshesToDecompose = WrapperInfo->ParentMeshes;
		TArray<FTransformSequence3d> ParentTransforms;
		for (FActorChildMesh* MeshInstance : MeshesToDecompose)
		{
			ParentTransforms.Add(MeshInstance->WorldTransform);
			SourceInstancesCount++;
		}
		ItemStats.SourceInstances = SourceInstancesCount;

		// Decompose the per-instance transforms into subsets that share the same total scaling ("unique scale").
		// If we apply these different scales to copies of the mesh, we can generate new instances for the copies,
		// which can avoid uniquing a lot of geometry
		TArray<TArray<int32>> UniqueScaleTransformSets;
		TArray<FTransformSequence3d> UniqueScaleTransforms;
		ConstructUniqueScalesMapping(ParentTransforms, UniqueScaleTransformSets, UniqueScaleTransforms);
		int32 NumUniqueScales = UniqueScaleTransformSets.Num();
		ItemStats.UniqueInstanceScales = NumUniqueScales;

		// if mesh is not too huge, try to weld edges
		if (LocalMesh.TriangleCount() < 100000)
		{
			FMergeCoincidentMeshEdges WeldEdges(&LocalMesh);
			WeldEdges.Apply();
		}

		// find separate submeshes of the mesh
		FMeshConnectedComponents Components(&LocalMesh);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build_ProcessMesh_2Components);
			Components.FindConnectedTriangles();
		}
		int32 NumComponents = Components.Num();
		ItemStats.SourceComponents = NumComponents;

		// for each submesh/component, determine if it is closed, and if it is 'thin'
		TArray<bool> IsClosed, IsThin;
		IsClosed.Init(false, NumComponents);
		IsThin.Init(false, NumComponents);
		std::atomic<int32> NumNonClosed = 0;
		TArray<TArray<FFrame3d>> BestFitPlanes;
		BestFitPlanes.SetNum(NumComponents);
		std::atomic<int32> NumThin = 0;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build_ProcessMesh_3Closed);
			ParallelFor(NumComponents, [&](int32 ci)
			{
				const FMeshConnectedComponents::FComponent& Component = Components[ci];
				const TArray<int32>& Triangles = Component.Indices;
				IsClosed[ci] = IsClosedRegion(LocalMesh, Triangles);
				if (IsClosed[ci] == false)
				{
					NumNonClosed++;
				}

				BestFitPlanes[ci].SetNum(NumUniqueScales);
				for (int32 k = 0; k < NumUniqueScales; ++k)
				{
					bool bIsThinUnderTransform = IsThinPlanarSubMesh<FDynamicMesh3>(LocalMesh, Triangles, UniqueScaleTransforms[k], BuildOptions.DesiredMinThickness, BestFitPlanes[ci][k]);
					IsThin[ci] = IsThin[ci] || bIsThinUnderTransform;
				}
				if (IsThin[ci])
				{
					NumThin++;
				}
			}, ParallelFlags);
		}
		ItemStats.NumOpen = NumNonClosed.load();
		ItemStats.NumThin = NumThin.load();

		// if we have no open meshes and no thin meshes, we can just use the SpatialWrapper we already have, 
		// but we have to rebuild it because we did not do a full build above
		// note: possibly some other cases where we can do this, if the StaticMesh wrapper supported unsigned/offset mode
		if (NumNonClosed == 0 && NumThin == 0)
		{
			AddBuildJob(WrapperInfo->SpatialWrapper.Get(), LocalMesh.TriangleCount());
			return;
		}

		// Accumulate submesh/components that do *not* need further processing here, that accumulated mesh
		// (if non-empty) can be shared among all the original FActorChildMesh instances
		FDynamicMesh3 LocalSpaceParts;
		FDynamicMeshEditor LocalSpaceAccumulator(&LocalSpaceParts);

		// a new copy of one of the submeshes that has been scaled/processed such that it can only be
		// represented with some of the original instance transforms (NewTransforms). 
		struct FInstancedSubmesh
		{
			TSharedPtr<FDynamicMesh3> Mesh;
			TArray<FTransformSequence3d> NewTransforms;
			double ComputedThickness = 0;
		};
		TArray<FInstancedSubmesh> NewSubmeshes;

		// Split all the submeshes/components into the LocalSpaceParts mesh (for closed and non-thin) and
		// a set of new FInstancedSubmesh's
		{
			FMeshIndexMappings Mappings;		// these are re-used between calls
			FDynamicMeshEditResult EditResult;

			TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_Build_ProcessMesh_4Accumulate);
			for (int32 ci = 0; ci < NumComponents; ++ci)
			{
				const TArray<int32>& Triangles = Components[ci].Indices;

				int32 NewUniqueTriangles = Triangles.Num() * NumUniqueScales;

				// We will make unscaled copies of a mesh if (1) it is "thin" and (2) it has a moderate number of triangles *or* a single usage
				// TODO: should we always unique a mesh with a single usage? We can just make it unsigned...
				bool bIsClosed = IsClosed[ci];
				if (IsThin[ci] == false || (NewUniqueTriangles > 1000000) )
				{
					Mappings.Reset(); EditResult.Reset();
					LocalSpaceAccumulator.AppendTriangles(&LocalMesh, Triangles, Mappings, EditResult, false);
					ItemStats.InstancedSubmeshComponents++;
					continue;
				}

				// if we go this far, we need to unique this mesh once for each "unique scale", and then
				// make a new set of instance transforms for it
				for (int32 k = 0; k < NumUniqueScales; ++k)
				{
					FInstancedSubmesh NewSubmesh;
					const TArray<int32>& InstanceIndices = UniqueScaleTransformSets[k];
					// make unique copy of submesh
					NewSubmesh.Mesh = MakeShared<FDynamicMesh3>();
					FDynamicMeshEditor Editor(NewSubmesh.Mesh.Get());
					Mappings.Reset(); EditResult.Reset();
					Editor.AppendTriangles(&LocalMesh, Triangles, Mappings, EditResult, false);
					// bake in the scaling
					FVector3d Scale = ParentTransforms[InstanceIndices[0]].GetAccumulatedScale();
					FAxisAlignedBox3d ScaledBounds = FAxisAlignedBox3d::Empty();
					for (int32 vid : EditResult.NewVertices)
					{
						FVector3d LocalPos = NewSubmesh.Mesh->GetVertex(vid);
						LocalPos *= Scale;
						NewSubmesh.Mesh->SetVertex(vid, LocalPos);
						ScaledBounds.Contain(LocalPos);
					}

					// if this is a tiny submesh we will skip it
					if (BuildOptions.bFilterTinyObjects && ScaledBounds.MaxDim() < BuildOptions.TinyObjectBoxMaxDimension)
					{
						NumTinyComponents++;
						NumTinyInstances += InstanceIndices.Num();
						TinyInstanceTotalTriangles += InstanceIndices.Num() * Triangles.Num();
						continue;
					}

					// Recompute thickness of scaled mesh and store it. Note that after scaling we might fail to 
					// be considered "thin" anymore, in that case we will fall back to using winding number
					// for this mesh (So, it was a waste to do this separation, but messy to turn back now)
					if (bIsClosed == false)
					{
						NewSubmesh.ComputedThickness = 0;
					}
					else
					{
						double NewThickness = MeasureThickness<FDynamicMesh3>(*NewSubmesh.Mesh, BestFitPlanes[ci][k]);
						NewSubmesh.ComputedThickness = FMathd::Min(NewThickness, BuildOptions.DesiredMinThickness);
					}

					// make new set of instances
					for (int32 j : InstanceIndices)
					{
						FTransformSequence3d InstanceTransform = ParentTransforms[j];
						InstanceTransform.ClearScales();
						NewSubmesh.NewTransforms.Add(InstanceTransform);
					}
					NewSubmeshes.Add(MoveTemp(NewSubmesh));
				}
			}
		}
		// At this point we have processed all the Submeshes/Components. Now we generate new MeshSpatialWrapper's
		// and any necessary new FActorAdapter's/FActorChildMesh's

		ItemStats.InstancedSubmeshTris = LocalSpaceParts.TriangleCount();
		ItemStats.TotalNumUniqueTris += ItemStats.InstancedSubmeshTris;

		// First handle the LocalSpaceParts mesh, which can still be shared between the original FActorChildMesh instances
		int32 LocalSpacePartsTriangleCount = LocalSpaceParts.TriangleCount();
		if (LocalSpacePartsTriangleCount > 0)
		{
			if (bCanUseCompressedMeshes)
			{
				WrapperInfo->SpatialWrapper = MakeUnique<FCompressedMeshSpatialWrapper>(MoveTemp(LocalSpaceParts));
			}
			else
			{
				WrapperInfo->SpatialWrapper = MakeUnique<FDynamicMeshSpatialWrapper>(MoveTemp(LocalSpaceParts));
			}

			AddBuildJob(WrapperInfo->SpatialWrapper.Get(), LocalSpacePartsTriangleCount);
			for (FActorChildMesh* MeshInstance : MeshesToDecompose)
			{
				MeshInstance->MeshSpatial = WrapperInfo->SpatialWrapper.Get();
			}
			InternalListsLock.Lock();
			InstancedTrisCount += LocalSpacePartsTriangleCount * ParentTransforms.Num();
			InternalListsLock.Unlock();
		}
		else
		{
			// disconnect parent meshes from this spatialwrapper as it is now invalid
			for (FActorChildMesh* MeshInstance : MeshesToDecompose)
			{
				MeshInstance->MeshSpatial = nullptr;
			}
			WrapperInfo->ParentMeshes.Reset();
		}

		// Exit if we don't have any more work to do. This happens if we ended up skipping all the possible decompositions
		// Note: in this case we could just re-use the existing actor and skip the LocalSpaceParts mesh entirely?
		if (NewSubmeshes.Num() == 0)
		{
			SkippedDecompositionCount++;
			return;
		}

		// definitely decomposing this mesh
		DecomposedSourceMeshCount++;

		// Now we create a new FActorAdapter for each new InstancedSubmesh, and then an FActorChildMesh
		// for each Instance (ie rotate/translate transform of that instance). This is somewhat arbitrary,
		// eg it could all be done in a single Actor, or split up further. At evaluation time we will have
		// pulled these back out of the Actor so it doesn't really matter.
		for (FInstancedSubmesh& Submesh : NewSubmeshes)
		{
			NewUniqueMeshesCount++;
			TUniquePtr<FActorAdapter> NewActor = MakeUnique<FActorAdapter>();
			NewActor->SourceActor = nullptr;		// not a "real" actor

			int32 SubmeshTriangleCount = Submesh.Mesh->TriangleCount();
			ensure(SubmeshTriangleCount > 0);

			// make new spatialwrapper for this instanced mesh
			TUniquePtr<FBaseMeshSpatialWrapper> NewSpatialWrapper;
			if (bCanUseCompressedMeshes)
			{
				NewSpatialWrapper = MakeUnique<FCompressedMeshSpatialWrapper>(MoveTemp(*Submesh.Mesh));
			}
			else
			{
				NewSpatialWrapper = MakeUnique<FDynamicMeshSpatialWrapper>(MoveTemp(*Submesh.Mesh));
			}

			// configure flags
			NewSpatialWrapper->bHasBakedTransform = false;
			NewSpatialWrapper->bHasBakedScale = true;
			// if mesh is too thin, configure the extra shell offset based on 'missing' thickness
			if (Submesh.ComputedThickness < BuildOptions.DesiredMinThickness)
			{
				NewSpatialWrapper->bUseDistanceShellForWinding = true;
				NewSpatialWrapper->WindingShellThickness = 0.5 * (BuildOptions.DesiredMinThickness - Submesh.ComputedThickness);
				NewSpatialWrapper->bRequiresWindingQueryFallback = (NewSpatialWrapper->WindingShellThickness < 0.6*BuildOptions.DesiredMinThickness);
			}

			TSharedPtr<FSpatialWrapperInfo> NewWrapperInfo = MakeShared<FSpatialWrapperInfo>();
			NewWrapperInfo->SpatialWrapper = MoveTemp(NewSpatialWrapper);

			// queue up build job
			AddBuildJob(NewWrapperInfo->SpatialWrapper.Get(), SubmeshTriangleCount);

			// add to internal lists
			InternalListsLock.Lock();
			AddedUniqueTrisCount += SubmeshTriangleCount;
			InstancedTrisCount += SubmeshTriangleCount * Submesh.NewTransforms.Num();
			void* UseKey = (void*)NewWrapperInfo->SpatialWrapper.Get();
			SpatialAdapters.Add(UseKey, NewWrapperInfo);
			InternalListsLock.Unlock();

			// create the new transform instances
			for (FTransformSequence3d InstanceTransform : Submesh.NewTransforms)
			{
				TUniquePtr<FActorChildMesh> ChildMesh = MakeUnique<FActorChildMesh>();
				ChildMesh->SourceComponent = nullptr;
				ChildMesh->ComponentType = EActorMeshComponentType::InternallyGeneratedComponent;
				ChildMesh->ComponentIndex = 0;
				ChildMesh->WorldTransform = InstanceTransform;
				ChildMesh->bIsNonUniformScaled = false;

				NewWrapperInfo->ParentMeshes.Add(ChildMesh.Get());
				ChildMesh->MeshSpatial = NewWrapperInfo->SpatialWrapper.Get();

				NewActor->ChildMeshes.Add(MoveTemp(ChildMesh));
				NewInstancesCount++;
			}

			ItemStats.SubmeshSizeCount.Add(FVector2i(SubmeshTriangleCount, Submesh.NewTransforms.Num()));
			ItemStats.NewInstances += Submesh.NewTransforms.Num();
			ItemStats.NewUniqueTris += SubmeshTriangleCount;
			ItemStats.TotalNumUniqueTris += SubmeshTriangleCount;

			// add actor our actor set
			InternalListsLock.Lock();
			SceneActors.Add(MoveTemp(NewActor));
			InternalListsLock.Unlock();
		}

		NumFinalUniqueTris += ItemStats.TotalNumUniqueTris;

	}, ParallelFlags);		// end outer ParallelFor over ToBuild meshes

	check(ToBuild.Num() == 0);

	// Now all that is left is to actually Build() all the different spatial wrappers that exist at this point

	// sort by increasing triangle size. 
	PendingBuildJobs.Sort([](const FBuildJob& A, const FBuildJob& B)
	{
		return A.TriangleCount < B.TriangleCount;
	});
	ParallelFor(PendingBuildJobs.Num(), [&](int32 i)
	{
		// ParallelFor will not respect our sort order if we just use the index directly (because it splits into chunks internally), so
		// we have to treat the list like a queue to get it to be processed in our desired order
		ToBuildQueueLock.Lock();
		check(PendingBuildJobs.Num() > 0);
		FBuildJob BuildJob = PendingBuildJobs.Pop(EAllowShrinking::No);
		ToBuildQueueLock.Unlock();
		BuildJob.BuildWrapper->Build(BuildOptions);
		if (BuildJob.BuildWrapper->GetTriangleCount() == 1)
		{
			SingleTriangleMeshes++;
		}
	}, ParallelFlags);
	check(PendingBuildJobs.Num() == 0);

	// currently true with the methods used above?
	bSceneIsAllSolids = true;

	if (BuildOptions.bPrintDebugMessages)
	{
		UE_LOGF(LogGeometry, Warning, "[FMeshSceneAdapter] decomposed %d source meshes used in %d instances (of %d total source meshes with %ld unique triangles), into %d new part meshes used in %d new instances containing %ld unique triangles. Scene has %ld total unique and %ld total instanced. Skipped %d decompositions. Skipped %d tiny components (%d instances, %d total triangles). %d 1-triangle meshes.", 
			   DecomposedSourceMeshCount.load(), SourceInstancesCount.load(), NumInitialSources, NumSourceUniqueTris, 
			   NewUniqueMeshesCount.load(), NewInstancesCount.load(), AddedUniqueTrisCount, 
			   NumFinalUniqueTris.load(), InstancedTrisCount,
			   SkippedDecompositionCount.load(), 
			   NumTinyComponents.load(), NumTinyInstances.load(), TinyInstanceTotalTriangles.load(),
			   SingleTriangleMeshes.load())


		Stats.Sort([](const FProcessedSourceMeshStats& A, const FProcessedSourceMeshStats& B)
		{
			return A.NewUniqueTris > B.NewUniqueTris;
		});
		for (int32 k = 0; k < FMath::Min(Stats.Num(), 20); ++k)
		{
			const FProcessedSourceMeshStats& Stat = Stats[k];
			FString NewSubmeshesStats;
			for (FVector2i V : Stat.SubmeshSizeCount)
			{
				NewSubmeshesStats += FString::Printf(TEXT(" (%d,%d)"), V.X, V.Y);
			}
			UE_LOGF(LogTemp, Warning, "  %ls : SourceTris %d Inst %d ResultTris %d Inst %d | UniqueScales %d Components %d | KeptTris %d NewUniqueTris %d | (Tri,Inst) %ls",
				   *Stat.AssetName, Stat.SourceTriangles, Stat.SourceInstances, Stat.TotalNumUniqueTris, (Stat.NewInstances + Stat.SourceInstances),
				   Stat.UniqueInstanceScales, Stat.SourceComponents,
				   Stat.InstancedSubmeshTris, Stat.NewUniqueTris, *NewSubmeshesStats);
		}
	}
}



void FMeshSceneAdapter::GetGeometryStatistics(FStatistics& StatsOut)
{
	StatsOut.UniqueMeshCount = 0;
	StatsOut.UniqueMeshTriangleCount = 0;
	ForEachSpatialWrapper([&StatsOut](TSharedPtr<FSpatialWrapperInfo> Adapter)
	{
		StatsOut.UniqueMeshCount++;
		StatsOut.UniqueMeshTriangleCount += (int64)Adapter->SpatialWrapper->GetTriangleCount();
	});

	StatsOut.InstanceMeshCount = 0;
	StatsOut.InstanceMeshTriangleCount = 0;
	for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			StatsOut.InstanceMeshCount++;
			if (ChildMesh->MeshSpatial != nullptr)
			{
				StatsOut.InstanceMeshTriangleCount += (int64)ChildMesh->MeshSpatial->GetTriangleCount();
			}
		}
	}
}


FAxisAlignedBox3d FMeshSceneAdapter::GetBoundingBox()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_GetBoundingBox);

	if (bHaveSpatialEvaluationCache)
	{
		return CachedWorldBounds;
	}

	// this could be done in parallel...
	FAxisAlignedBox3d SceneBounds = FAxisAlignedBox3d::Empty();
	for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			if (ChildMesh->MeshSpatial != nullptr)
			{
				FAxisAlignedBox3d ChildBounds = ChildMesh->MeshSpatial->GetWorldBounds(
					[&](const FVector3d& P) { return ChildMesh->WorldTransform.TransformPosition(P); });
				SceneBounds.Contain(ChildBounds);
			}
		}
	}
	return SceneBounds;
}


void FMeshSceneAdapter::CollectMeshSeedPoints(TArray<FVector3d>& Points)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_CollectMeshSeedPoints);

	for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			if (ChildMesh->MeshSpatial != nullptr)
			{
				ChildMesh->MeshSpatial->CollectSeedPoints(Points,
					[&](const FVector3d& P) { return ChildMesh->WorldTransform.TransformPosition(P); } );
			}
		}
	}
}

double FMeshSceneAdapter::FastWindingNumber(const FVector3d& P, bool bFastEarlyOutIfPossible)
{
	check(bHaveSpatialEvaluationCache);		// must call BuildSpatialEvaluationCache() to build Octree

	double SumWinding = 0.0;

	// if all objects in scene are solids, then all winding queries will return integers so if any value
	// is > 0, we are "inside"
	if (bSceneIsAllSolids)
	{
		if (bFastEarlyOutIfPossible)
		{
			Octree->ContainmentQueryCancellable(P, [&](int32 k)
			{
				double WindingNumber = SortedSpatials[k].Spatial->FastWindingNumber(P, SortedSpatials[k].ChildMesh->WorldTransform);
				SumWinding += WindingNumber;
				return (FMath::Abs(WindingNumber) < 0.99);		// if we see an "inside" winding number we can just exit
			});
		}
		else
		{
			Octree->ContainmentQuery(P, [&](int32 k)
			{
				double WindingNumber = SortedSpatials[k].Spatial->FastWindingNumber(P, SortedSpatials[k].ChildMesh->WorldTransform);
				SumWinding += WindingNumber;
			});
		}
	}
	else
	{
		for (const FSpatialCacheInfo& SpatialInfo : SortedSpatials)
		{
			double WindingNumber = SpatialInfo.Spatial->FastWindingNumber(P, SpatialInfo.ChildMesh->WorldTransform);
			SumWinding += WindingNumber;
		}
	}

	return SumWinding;
}



void FMeshSceneAdapter::BuildSpatialEvaluationCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_BuildSpatialEvaluationCache);

	// validate that internals are correctly configured
	ForEachSpatialWrapper([](TSharedPtr<FSpatialWrapperInfo> Adapter)
	{
		FSpatialWrapperInfo& WrapperInfo = *Adapter;
		IMeshSpatialWrapper* SpatialWrapper = WrapperInfo.SpatialWrapper.Get();
		for (FActorChildMesh* MeshInstance : WrapperInfo.ParentMeshes)
		{
			if (ensure(MeshInstance->MeshSpatial == SpatialWrapper) == false)
			{
				UE_LOGF(LogGeometry, Warning, "FMeshSceneAdapter::BuildSpatialEvaluationCache: broken MeshSpatial link found!");
			}
		}
	});

	// build list of unique meshes we need to evaluate for spatial queries
	SortedSpatials.Reset();
	for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			if (ChildMesh->MeshSpatial != nullptr)
			{
				FSpatialCacheInfo Cache;
				Cache.Actor = Actor.Get();
				Cache.ChildMesh = ChildMesh.Get();
				Cache.Spatial = ChildMesh->MeshSpatial;
				SortedSpatials.Add(Cache);
			}
		}
	}

	// sort the list (not really necessary but might improve cache coherency during linear queries)
	SortedSpatials.Sort([&](const FSpatialCacheInfo& A, const FSpatialCacheInfo& B)
	{
		return A.Spatial < B.Spatial;
	});

	int32 NumSpatials = SortedSpatials.Num();
	CachedWorldBounds = FAxisAlignedBox3d::Empty();
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_BuildSpatialEvaluationCache_Bounds);
		ParallelFor(SortedSpatials.Num(), [&](int32 k)
		{
			SortedSpatials[k].Bounds = SortedSpatials[k].Spatial->GetWorldBounds(
				[&](const FVector3d& P) { return SortedSpatials[k].ChildMesh->WorldTransform.TransformPosition(P); });
		});

		for (const FSpatialCacheInfo& Cache : SortedSpatials)
		{
			CachedWorldBounds.Contain(Cache.Bounds);
		}
	}


	// build an octree of the mesh objects
	Octree = MakeShared<FSparseDynamicOctree3>();
	Octree->RootDimension = CachedWorldBounds.MaxDim() / 4.0;
	Octree->SetMaxTreeDepth(5);
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_BuildSpatialEvaluationCache_OctreeInserts);
		for (int32 k = 0; k < NumSpatials; ++k)
		{
			Octree->InsertObject(k, SortedSpatials[k].Bounds);
		}
	}

	bHaveSpatialEvaluationCache = true;
}



bool FMeshSceneAdapter::FindNearestRayIntersection(const FRay3d& WorldRay, FMeshSceneRayHit& HitResultOut)
{
	check(bHaveSpatialEvaluationCache);		// must call BuildSpatialEvaluationCache() to build Octree

	int32 HitObjectID = Octree->FindNearestHitObject(WorldRay,
		[&](int32 idx) { return SortedSpatials[idx].Bounds; },
		[&](int idx, const FRay3d& Ray) {
			FMeshSceneRayHit LocalHitResult;
			FSpatialCacheInfo& CacheInfo = SortedSpatials[idx];
			if (CacheInfo.Spatial->RayIntersection(Ray, CacheInfo.ChildMesh->WorldTransform, LocalHitResult))
			{
				return LocalHitResult.RayDistance;
			}
			return TNumericLimits<double>::Max();
		});

	if (HitObjectID >= 0)
	{
		FSpatialCacheInfo& CacheInfo = SortedSpatials[HitObjectID];
		bool bHit = CacheInfo.Spatial->RayIntersection(WorldRay, CacheInfo.ChildMesh->WorldTransform, HitResultOut);
		if (ensure(bHit))
		{
			HitResultOut.HitActor = CacheInfo.Actor->SourceActor;
			HitResultOut.HitComponent = CacheInfo.ChildMesh->SourceComponent;
			HitResultOut.HitComponentElementIndex = CacheInfo.ChildMesh->ComponentIndex;
			HitResultOut.Ray = WorldRay;
			HitResultOut.LocalToWorld = CacheInfo.ChildMesh->WorldTransform;
			return true;
		}
	}

	return false;
}



void FMeshSceneAdapter::FastUpdateTransforms(bool bRebuildSpatialCache)
{
	for (TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			ChildMesh->WorldTransform = FTransformSequence3d();

			if ((ChildMesh->ComponentType == EActorMeshComponentType::InstancedStaticMesh) ||
				(ChildMesh->ComponentType == EActorMeshComponentType::HierarchicalInstancedStaticMesh))
			{
				UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(ChildMesh->SourceComponent);
				if (ISMComponent->IsValidInstance(ChildMesh->ComponentIndex))
				{
					FTransform InstanceTransform;
					ISMComponent->GetInstanceTransform(ChildMesh->ComponentIndex, InstanceTransform, true);
					ChildMesh->WorldTransform.Append(InstanceTransform);
					ChildMesh->bIsNonUniformScaled = ChildMesh->WorldTransform.HasNonUniformScale();
				}
			}
			else if ((ChildMesh->ComponentType == EActorMeshComponentType::StaticMesh) ||
				(ChildMesh->ComponentType == EActorMeshComponentType::SplineStaticMesh))
			{
				UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ChildMesh->SourceComponent);
				ChildMesh->WorldTransform.Append(StaticMeshComponent->GetComponentTransform());
				ChildMesh->bIsNonUniformScaled = ChildMesh->WorldTransform.HasNonUniformScale();
			}
		}
	}

	// this cache is invalid now
	bHaveSpatialEvaluationCache = false;

	// update bounding boxes for actors
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshScene_FastUpdateTransforms_ActorBounds);
		ParallelFor(SceneActors.Num(), [&](int32 i)
		{
			UpdateActorBounds(*SceneActors[i]);
		});
	}

	if (bRebuildSpatialCache)
	{
		BuildSpatialEvaluationCache();
	}

}



void FMeshSceneAdapter::GetMeshBoundingBoxes(TArray<FAxisAlignedBox3d>& Bounds)
{
	if (bHaveSpatialEvaluationCache)
	{
		for (FSpatialCacheInfo& CacheInfo : SortedSpatials)
		{
			Bounds.Add(CacheInfo.Bounds);
		}
	}
	else
	{
		for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
		{
			for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
			{
				if (ChildMesh->MeshSpatial != nullptr)
				{
					FAxisAlignedBox3d ChildBounds = ChildMesh->MeshSpatial->GetWorldBounds(
						[&](const FVector3d& P) { return ChildMesh->WorldTransform.TransformPosition(P); });
					Bounds.Add(ChildBounds);
				}
			}
		}
	}
}


FAxisAlignedBox3d FMeshSceneAdapter::GetMeshBoundingBox(UActorComponent* Component, int32 ComponentIndex)
{
	// implementation below depends on the spatial cache
	check(bHaveSpatialEvaluationCache);

	for (FSpatialCacheInfo& CacheInfo : SortedSpatials)
	{
		if (CacheInfo.ChildMesh->SourceComponent == Component)
		{
			if (ComponentIndex == -1 || ComponentIndex == CacheInfo.ChildMesh->ComponentIndex )
			{
				return CacheInfo.Bounds;
			}
		}
	}

	return FAxisAlignedBox3d(FVector3d::Zero(), 1.0);
}



void FMeshSceneAdapter::GetAccumulatedMesh(FDynamicMesh3& AccumMesh)
{
	for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			if (ChildMesh->MeshSpatial != nullptr)
			{
				ChildMesh->MeshSpatial->AppendMesh(AccumMesh, ChildMesh->WorldTransform);
			}
		}
	}
}




void FMeshSceneAdapter::GenerateBaseClosingMesh(double BaseHeight, double ExtrudeHeight)
{
	FAxisAlignedBox3d WorldBounds = GetBoundingBox();
	FInterval1d ZRange(WorldBounds.Min.Z, WorldBounds.Min.Z + BaseHeight);

	TArray<FActorChildMesh*> AllChildMeshes;
	for (TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		for (TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
		{
			if (ChildMesh->MeshSpatial != nullptr)
			{
				AllChildMeshes.Add(ChildMesh.Get());
			}
		}
	}

	TArray<FVector2d> WorldHullPoints;
	FCriticalSection WorldHullPointsLock;

	int32 NumChildren = AllChildMeshes.Num();
	ParallelFor(NumChildren, [&](int32 ci)
	{
		FActorChildMesh* ChildMesh = AllChildMeshes[ci];
		TArray<FVector2d> LocalHullPoints;
		ChildMesh->MeshSpatial->ProcessVerticesInWorld([&](const FVector3d& WorldPos)
		{
			if (ZRange.Contains(WorldPos.Z))
			{
				LocalHullPoints.Add(FVector2d(WorldPos.X, WorldPos.Y));
			}
			return true;
		}, ChildMesh->WorldTransform);

		if (LocalHullPoints.Num() > 0)
		{
			FConvexHull2d HullSolver;
			if (HullSolver.Solve(LocalHullPoints))
			{
				WorldHullPointsLock.Lock();
				for (int32 idx : HullSolver.GetPolygonIndices())
				{
					WorldHullPoints.Add(LocalHullPoints[idx]);
				}
				WorldHullPointsLock.Unlock();
			}
		}
	});

	FConvexHull2d FinalHullSolver;
	bool bOK = FinalHullSolver.Solve(WorldHullPoints);
	if (bOK == false)
	{
		UE_LOGF(LogTemp, Warning, "[FMeshSceneAdapter::GenerateBaseClosingMesh] failed to solve convex hull");
		return;
	}
	FPolygon2d ConvexHullPoly;
	for (int32 idx : FinalHullSolver.GetPolygonIndices())
	{
		ConvexHullPoly.AppendVertex(WorldHullPoints[idx]);
	}
	if (ConvexHullPoly.VertexCount() < 2)
	{
		UE_LOGF(LogTemp, Warning, "[FMeshSceneAdapter::GenerateBaseClosingMesh] convex hull is degenerate");
		return;
	}

	FPlanarPolygonMeshGenerator MeshGen;
	MeshGen.Polygon = ConvexHullPoly;
	FDynamicMesh3 BasePolygonMesh(&MeshGen.Generate());
	MeshTransforms::Translate(BasePolygonMesh, ZRange.Min * FVector3d::UnitZ());

	if (ExtrudeHeight == 0)
	{
		BasePolygonMesh.ReverseOrientation();		// flip so it points down
		bSceneIsAllSolids = false;					// if scene was solids, it's not anymore
	}
	else
	{
		FOffsetMeshRegion Offset(&BasePolygonMesh);
		for (int32 tid : BasePolygonMesh.TriangleIndicesItr())
		{
			Offset.Triangles.Add(tid);
		}
		Offset.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage;
		Offset.DefaultOffsetDistance = ExtrudeHeight;
		Offset.bIsPositiveOffset = (ExtrudeHeight > 0);
		Offset.Apply();
	}

	//
	// append a fake actor/mesh
	//

	TUniquePtr<FActorAdapter> ActorAdapter = MakeUnique<FActorAdapter>();
	ActorAdapter->SourceActor = nullptr;

	TUniquePtr<FActorChildMesh> ChildMesh = MakeUnique<FActorChildMesh>();
	ChildMesh->SourceComponent = nullptr;
	//InstanceChild->MeshContainer = ;
	ChildMesh->ComponentType = EActorMeshComponentType::InternallyGeneratedComponent;
	ChildMesh->ComponentIndex = 0;
	//ChildMesh->WorldTransform.Append(InstanceTransform);
	ChildMesh->bIsNonUniformScaled = false;

	TSharedPtr<FSpatialWrapperInfo> NewWrapperInfo = MakeShared<FSpatialWrapperInfo>();
	SpatialAdapters.Add(ChildMesh.Get(), NewWrapperInfo);
	TUniquePtr<FDynamicMeshSpatialWrapper> DynamicMeshWrapper = MakeUnique<FDynamicMeshSpatialWrapper>();
	//DynamicMeshWrapper->SourceContainer = NewWrapperInfo->SourceContainer;
	DynamicMeshWrapper->Mesh = MoveTemp(BasePolygonMesh);
	DynamicMeshWrapper->bHasBakedTransform = DynamicMeshWrapper->bHasBakedScale = true;
	FMeshSceneAdapterBuildOptions UseBuildOptions;
	DynamicMeshWrapper->Build(UseBuildOptions);

	//NewWrapperInfo->SourceContainer = ChildMesh->MeshContainer;
	NewWrapperInfo->SpatialWrapper = MoveTemp(DynamicMeshWrapper);
	NewWrapperInfo->ParentMeshes.Add(ChildMesh.Get());
	ChildMesh->MeshSpatial = NewWrapperInfo->SpatialWrapper.Get();
	ActorAdapter->ChildMeshes.Add(MoveTemp(ChildMesh));
	UpdateActorBounds(*ActorAdapter);
	SceneActors.Add(MoveTemp(ActorAdapter));

}



void FMeshSceneAdapter::ParallelMeshVertexEnumeration(
	TFunctionRef<bool(int32 NumMeshes)> InitializeFunc,
	TFunctionRef<bool(int32 MeshIndex, AActor* SourceActor, const FActorChildMesh* ChildMeshInfo, const FAxisAlignedBox3d& WorldBounds)> MeshFilterFunc,
	TFunctionRef<bool(int32 MeshIndex, AActor* SourceActor, const FActorChildMesh* ChildMeshInfo, const FVector3d& WorldPos)> PerVertexFunc,
	bool bForceSingleThreaded )
{
	int32 N = SortedSpatials.Num();
	if (InitializeFunc(N) == false)
	{
		return;
	}

	bool bSingleThread = bForceSingleThreaded || (CVarMeshSceneAdapterDisableMultiThreading.GetValueOnAnyThread() != 0);

	ParallelFor(N, [&](int32 Index)
	{
		const FSpatialCacheInfo& CacheInfo = SortedSpatials[Index];

		bool bContinue = MeshFilterFunc(Index, CacheInfo.Actor->SourceActor, CacheInfo.ChildMesh, CacheInfo.Bounds);
		if (!bContinue)
		{
			return;
		}

		FTransformSequence3d WorldTransform = CacheInfo.ChildMesh->WorldTransform;
		CacheInfo.Spatial->ProcessVerticesInWorld([&](const FVector3d& WorldPos)
		{
			bContinue = PerVertexFunc(Index, CacheInfo.Actor->SourceActor, CacheInfo.ChildMesh, WorldPos);
			return bContinue;
		}, WorldTransform);

	}, bSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::Unbalanced );


}

void FMeshSceneAdapter::ProcessActorChildMeshes(TFunctionRef<void(const FActorAdapter* ActorAdapter, const FActorChildMesh* ChildMesh)> ProcessFunc)
{
	for (const TUniquePtr<FActorAdapter>& Actor : SceneActors)
	{
		if (Actor)
		{
			for (const TUniquePtr<FActorChildMesh>& ChildMesh : Actor->ChildMeshes)
			{
				if (ChildMesh)
				{
					ProcessFunc(Actor.Get(), ChildMesh.Get());
				}
			}
		}
	}
}

