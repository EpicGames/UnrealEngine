// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/SceneCapturePhotoSet.h"
#include "EngineUtils.h"
#include "Misc/ScopedSlowTask.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "SceneCapture"

FSceneCapturePhotoSet::FSceneCapturePhotoSet()
{
	// These defaults are chosen so an unfamiliar user doesn't need to call the configuration functions to get a result
	ForEachCaptureType([this](ERenderCaptureType CaptureType)
	{
		bool bIsDisabled = (
			CaptureType == ERenderCaptureType::DeviceDepth ||
			CaptureType == ERenderCaptureType::Roughness ||
			CaptureType == ERenderCaptureType::Specular ||
			CaptureType == ERenderCaptureType::Metallic);
		PhotoSetStatus[CaptureType] = bIsDisabled ? ECaptureTypeStatus::Disabled : ECaptureTypeStatus::Pending;

		RenderCaptureConfig[CaptureType] = GetDefaultRenderCaptureConfig(CaptureType);
	});
}

void FSceneCapturePhotoSet::SetCaptureSceneActors(UWorld* World, const TArray<AActor*>& Actors)
{
	SetCaptureSceneActorsAndComponents(World, Actors, {});
}

void FSceneCapturePhotoSet::SetCaptureSceneComponents(UWorld* World, const TArray<UActorComponent*>& Components)
{
	SetCaptureSceneActorsAndComponents(World, {}, Components);
}

void FSceneCapturePhotoSet::SetCaptureSceneActorsAndComponents(UWorld* World, const TArray<AActor*>& Actors, const TArray<UActorComponent*>& Components)
{
	if (this->TargetWorld != World || this->VisibleActors != Actors || this->VisibleComponents != Components)
	{
		ForEachCaptureType([this](ERenderCaptureType CaptureType)
		{
			if (PhotoSetStatus[CaptureType] != ECaptureTypeStatus::Disabled)
			{
				PhotoSetStatus[CaptureType] = ECaptureTypeStatus::Pending;
			}
		});

		// Empty the photo sets because they rendered different actors
		EmptyAllPhotoSets();

		// Empty the spatial photo parameters because these are computed from the bounding box of the actors
		PhotoSetParams.Empty();
	}
	this->TargetWorld = World;
	this->VisibleActors = Actors;
	this->VisibleComponents = Components;
}

UWorld* FSceneCapturePhotoSet::GetCaptureTargetWorld()
{
	return TargetWorld;
}

void FSceneCapturePhotoSet::SetSpatialPhotoParams(const TArray<FSpatialPhotoParams>& SpatialParams)
{
	// TODO Discard/reset on a per array element level rather than discarding everything when any viewpoint changed
	if (PhotoSetParams != SpatialParams)
	{
		ForEachCaptureType([this](ERenderCaptureType CaptureType)
		{
			if (PhotoSetStatus[CaptureType] != ECaptureTypeStatus::Disabled)
			{
				PhotoSetStatus[CaptureType] = ECaptureTypeStatus::Pending;
			}
		});

		EmptyAllPhotoSets();
		PhotoSetParams = SpatialParams;
	}
}

const TArray<FSpatialPhotoParams>& FSceneCapturePhotoSet::GetSpatialPhotoParams() const
{
	return PhotoSetParams;
}


void FSceneCapturePhotoSet::SetCaptureConfig(ERenderCaptureType CaptureType, const FRenderCaptureConfig& NewConfig)
{
	if (RenderCaptureConfig[CaptureType] != NewConfig)
	{
		if (PhotoSetStatus[CaptureType] != ECaptureTypeStatus::Disabled)
		{
			PhotoSetStatus[CaptureType] = ECaptureTypeStatus::Pending;
		}

		EmptyPhotoSet(CaptureType);
		RenderCaptureConfig[CaptureType] = NewConfig;
	}
}

FRenderCaptureConfig FSceneCapturePhotoSet::GetCaptureConfig(ERenderCaptureType CaptureType) const
{
	return RenderCaptureConfig[CaptureType];
}

void FSceneCapturePhotoSet::DisableAllCaptureTypes()
{
	ForEachCaptureType([this](ERenderCaptureType CaptureType)
	{
		SetCaptureTypeEnabled(CaptureType, false);
	});
}


void FSceneCapturePhotoSet::SetCaptureTypeEnabled(ERenderCaptureType CaptureType, bool bEnable)
{
	if (bEnable)
	{
		if (PhotoSetStatus[CaptureType] == ECaptureTypeStatus::Disabled)
		{
			PhotoSetStatus[CaptureType] = ECaptureTypeStatus::Pending;
		}
	}
	else
	{
		PhotoSetStatus[CaptureType] = ECaptureTypeStatus::Disabled;
		EmptyPhotoSet(CaptureType);
	}
}

FSceneCapturePhotoSet::ECaptureTypeStatus FSceneCapturePhotoSet::GetCaptureTypeStatus(ERenderCaptureType CaptureType) const
{
	return PhotoSetStatus[CaptureType];
}

FSceneCapturePhotoSet::FStatus FSceneCapturePhotoSet::GetSceneCaptureStatus() const
{
	return PhotoSetStatus;
}

void FSceneCapturePhotoSet::Compute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SceneCapturePhotoSet::Compute);

	bWasCancelled = false;

	int NumPending = 0;
	ForEachCaptureType([this, &NumPending](ERenderCaptureType CaptureType)
	{
		NumPending += static_cast<int>(PhotoSetStatus[CaptureType] == ECaptureTypeStatus::Pending);
	});

	if (NumPending == 0)
	{
		return;
	}

	FScopedSlowTask Progress(static_cast<float>(NumPending), LOCTEXT("CapturingScene", "Capturing Scene..."));
	Progress.MakeDialog(bAllowCancel);

	check(this->TargetWorld != nullptr);

	// Unregister all components to remove unwanted proxies from the scene. This was previously the only way to "hide" nanite meshes, now optional.
	TSet<AActor*> VisibleActorsSet(VisibleActors);
	TArray<AActor*> ActorsToRegister;
	if (bEnforceVisibilityViaUnregister)
	{
		for (TActorIterator<AActor> Actor(TargetWorld); Actor; ++Actor)
		{
			if (!VisibleActorsSet.Contains(*Actor))
			{
				Actor->UnregisterAllComponents();
				ActorsToRegister.Add(*Actor);
			}
		}
	}

	ON_SCOPE_EXIT
	{
		// Workaround for Nanite scene proxies visibility
		// Reregister all components we previously unregistered
		for (AActor* Actor : ActorsToRegister)
		{
			Actor->RegisterAllComponents();
		}
	};

	FWorldRenderCapture RenderCapture;
	RenderCapture.SetWorld(TargetWorld);
	RenderCapture.SetVisibleActorsAndComponents(VisibleActors, VisibleComponents);
	RenderCapture.SetEnableWriteDebugImage(bWriteDebugImages, 0, DebugImagesFolderName);

	auto CapturePhoto3f = [this, &RenderCapture](ERenderCaptureType CaptureType, const FSpatialPhotoParams& Params)
	{
		FSpatialPhoto3f NewPhoto;
		NewPhoto.Frame = Params.Frame;
		NewPhoto.NearPlaneDist = Params.NearPlaneDist;
		NewPhoto.HorzFOVDegrees = Params.HorzFOVDegrees;
		NewPhoto.Dimensions = Params.Dimensions;

		// TODO Do something with the success boolean returned by RenderCapture.CaptureFromPosition
		FImageAdapter Image(&NewPhoto.Image);
		FRenderCaptureConfig Config = GetCaptureConfig(CaptureType);
		RenderCapture.SetDimensions(Params.Dimensions);
		RenderCapture.CaptureFromPosition(CaptureType, NewPhoto.Frame, NewPhoto.HorzFOVDegrees, NewPhoto.NearPlaneDist, Image, Config);
		GetPhotoSet3f(CaptureType).Add(MoveTemp(NewPhoto));
	};

	auto CapturePhoto1f = [this, &RenderCapture](ERenderCaptureType CaptureType, const FSpatialPhotoParams& Params)
	{
		FSpatialPhoto1f NewPhoto;
		NewPhoto.Frame = Params.Frame;
		NewPhoto.NearPlaneDist = Params.NearPlaneDist;
		NewPhoto.HorzFOVDegrees = Params.HorzFOVDegrees;
		NewPhoto.Dimensions = Params.Dimensions;

		// TODO Do something with the success boolean returned by RenderCapture.CaptureFromPosition
		FImageAdapter Image(&NewPhoto.Image);
		FRenderCaptureConfig Config = GetCaptureConfig(CaptureType);
		RenderCapture.SetDimensions(Params.Dimensions);
		RenderCapture.CaptureFromPosition(CaptureType, NewPhoto.Frame, NewPhoto.HorzFOVDegrees, NewPhoto.NearPlaneDist, Image, Config);
		GetPhotoSet1f(CaptureType).Add(MoveTemp(NewPhoto));
	};

	// Iterate by channel computing all the photos rather than by photo/viewpoint computing all the channels.
	// Note: This ordering means that some captures may have computed even if the computation is cancelled.
	auto CapturePhotoSet = [this, &Progress](
		TFunctionRef<void(ERenderCaptureType, const FSpatialPhotoParams&)> CapturePhoto,
		ERenderCaptureType CaptureType,
		const FText& TaskMessage)
	{
		if (PhotoSetStatus[CaptureType] == ECaptureTypeStatus::Pending)
		{
			Progress.EnterProgressFrame(1.f);
			FScopedSlowTask PhotoSetProgress(PhotoSetParams.Num(), TaskMessage);

			EmptyPhotoSet(CaptureType);
			for (const FSpatialPhotoParams& Params : PhotoSetParams)
			{
				if (Progress.ShouldCancel())
				{
					UE_LOGF(LogGeometry, Display, "FSceneCapturePhotoSet: The pending '%ls' step was cancelled", *TaskMessage.ToString());
					bWasCancelled = true;
					return;
				}

				PhotoSetProgress.EnterProgressFrame(1.f);
				CapturePhoto(CaptureType, Params);
				Progress.TickProgress();
			}

			PhotoSetStatus[CaptureType] = ECaptureTypeStatus::Computed;
		}
	};

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_DeviceDepth);
		CapturePhotoSet(CapturePhoto1f, ERenderCaptureType::DeviceDepth, LOCTEXT("CapturingScene_DeviceDepth", "Capturing Device Depth"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_BaseColor);
		CapturePhotoSet(CapturePhoto3f, ERenderCaptureType::BaseColor, LOCTEXT("CapturingScene_BaseColor", "Capturing Base Color"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_WorldNormal);
		CapturePhotoSet(CapturePhoto3f, ERenderCaptureType::WorldNormal, LOCTEXT("CapturingScene_WorldNormal", "Capturing World Normal"));
	}
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_CombinedMRS);
		CapturePhotoSet(CapturePhoto3f, ERenderCaptureType::CombinedMRS, LOCTEXT("CapturingScene_CombinedMRS", "Capturing Packed MRS"));
	}
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_Metallic);
		CapturePhotoSet(CapturePhoto1f, ERenderCaptureType::Metallic, LOCTEXT("CapturingScene_Metallic", "Capturing Metallic"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_Roughness);
		CapturePhotoSet(CapturePhoto1f, ERenderCaptureType::Roughness, LOCTEXT("CapturingScene_Roughness", "Capturing Roughness"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_Specular);
		CapturePhotoSet(CapturePhoto1f, ERenderCaptureType::Specular, LOCTEXT("CapturingScene_Specular", "Capturing Specular"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_Emissive);
		CapturePhotoSet(CapturePhoto3f, ERenderCaptureType::Emissive, LOCTEXT("CapturingScene_Emissive", "Capturing Emissive"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_Opacity);
		CapturePhotoSet(CapturePhoto1f, ERenderCaptureType::Opacity, LOCTEXT("CapturingScene_Opacity", "Capturing Opacity"));
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FSceneCapturePhotoSet::Compute_SubsurfaceColor);
		CapturePhotoSet(CapturePhoto3f, ERenderCaptureType::SubsurfaceColor, LOCTEXT("CapturingScene_SubsurfaceColor", "Capturing Subsurface Color"));
	}
}



void FSceneCapturePhotoSet::OptimizePhotoSets()
{
	// todo:
	//  1) crop photos to regions with actual pixels
	//  2) pack into fewer photos  (eg pack spec/rough/metallic)
	//  3) RLE encoding or other compression
}



FSceneCapturePhotoSet::FSceneSample::FSceneSample()
{
	HaveValues = FRenderCaptureTypeFlags::None();
	BaseColor = FVector3f(0, 0, 0);
	Roughness = 0.0f;
	Specular = 0.0f;
	Metallic = 0.0f;
	Emissive = FVector3f(0, 0, 0);
	Opacity = 0.0f;
	SubsurfaceColor = FVector3f(0, 0, 0);
	WorldNormal = FVector3f(0, 0, 1);
	DeviceDepth = 0.0f; // Since we use reverse z perspective projection the far plane corresponds to DeviceDepth == 0
}

FVector3f FSceneCapturePhotoSet::FSceneSample::GetValue3f(ERenderCaptureType CaptureType) const
{
	switch (CaptureType)
	{
	case ERenderCaptureType::BaseColor:
		return BaseColor;
	case ERenderCaptureType::WorldNormal:
		return WorldNormal;
	case ERenderCaptureType::Roughness:
		return Roughness * FVector3f::One();
	case ERenderCaptureType::Metallic:
		return Metallic * FVector3f::One();
	case ERenderCaptureType::Specular:
		return Specular * FVector3f::One();
	case ERenderCaptureType::Emissive:
		return Emissive;
	case ERenderCaptureType::Opacity:
		return Opacity * FVector3f::One();
	case ERenderCaptureType::SubsurfaceColor:
		return SubsurfaceColor;
	case ERenderCaptureType::DeviceDepth:
		return DeviceDepth * FVector3f::One();
	default:
		ensure(false);
	}
	return FVector3f::Zero();
}

FVector4f FSceneCapturePhotoSet::FSceneSample::GetValue4f(ERenderCaptureType CaptureType) const
{
	switch (CaptureType)
	{
	case ERenderCaptureType::BaseColor:
		return FVector4f(BaseColor.X, BaseColor.Y, BaseColor.Z, 1.0f);
	case ERenderCaptureType::WorldNormal:
		return FVector4f(WorldNormal.X, WorldNormal.Y, WorldNormal.Z, 1.0f);
	case ERenderCaptureType::Roughness:
		return FVector4f(Roughness, Roughness, Roughness, 1.0f);
	case ERenderCaptureType::Metallic:
		return FVector4f(Metallic, Metallic, Metallic, 1.0f);
	case ERenderCaptureType::Specular:
		return FVector4f(Specular, Specular, Specular, 1.0f);
	case ERenderCaptureType::CombinedMRS:
		return FVector4f(Metallic, Roughness, Specular, 1.0f);
	case ERenderCaptureType::Emissive:
		return FVector4f(Emissive.X, Emissive.Y, Emissive.Z, 1.0f);
	case ERenderCaptureType::Opacity:
		return FVector4f(Opacity, Opacity, Opacity, 1.0f);
	case ERenderCaptureType::SubsurfaceColor:
		return FVector4f(SubsurfaceColor.X, SubsurfaceColor.Y, SubsurfaceColor.Z, 1.0f);
	case ERenderCaptureType::DeviceDepth:
		return FVector4f(DeviceDepth, DeviceDepth, DeviceDepth, 1.0f);
	default:
		ensure(false);
	}
	return FVector4f::Zero();
}

bool FSceneCapturePhotoSet::ComputeSampleLocation(
	const FVector3d& Position,
	const FVector3d& Normal,
	const float ValidSampleDepthThreshold,
	TFunctionRef<bool(const FVector3d&, const FVector3d&)> VisibilityFunction,
	int& PhotoIndex,
	FVector2d& PhotoCoords) const
{
	double DotTolerance = -0.1;		// dot should be negative for normal pointing towards photo

	if (ValidSampleDepthThreshold > 0)
	{
		check(DeviceDepthPhotoSet.Num() == PhotoSetParams.Num());
	}

	PhotoIndex = IndexConstants::InvalidID;
	PhotoCoords = FVector2d(0., 0.);

	double MinDot = 1.0;

	int32 NumPhotos = PhotoSetParams.Num();
	for (int32 Index = 0; Index < NumPhotos; ++Index)
	{
		const FSpatialPhotoParams& Params = PhotoSetParams[Index];
		check(Params.Dimensions.IsSquare());

		const FViewMatrices ViewMatrices = GetRenderCaptureViewMatrices(Params.Frame, Params.HorzFOVDegrees, Params.NearPlaneDist, Params.Dimensions);

		FFrame3d RenderFrame(Params.Frame.Origin, Params.Frame.Y(), Params.Frame.Z(), Params.Frame.X());

		FVector3d ViewDirection = RenderFrame.Z();
		double ViewDot = ViewDirection.Dot(Normal);
		if (ViewDot > DotTolerance || ViewDot > MinDot)
		{
			// The sample is facing away from the photo, or we found a photo more aligned with this sample
			continue;
		}

		FFrame3d ViewPlane = RenderFrame;
		ViewPlane.Origin += Params.NearPlaneDist * ViewDirection;

		double ViewPlaneWidthWorld = Params.NearPlaneDist * FMathd::Tan(Params.HorzFOVDegrees * 0.5 * FMathd::DegToRad);
		double ViewPlaneHeightWorld = ViewPlaneWidthWorld;

		// Shoot a ray from the camera position toward the sample position and find the hit point on photo plane
		constexpr int NormalAxisIndex = 2;
		FVector3d RayOrigin = RenderFrame.Origin;
		FVector3d RayDir = Normalized(Position - RayOrigin);
		FVector3d HitPoint;
		bool bHit = ViewPlane.RayPlaneIntersection(RayOrigin, RayDir, NormalAxisIndex, HitPoint);
		if (bHit)
		{
			FVector2d DeviceXY;
			DeviceXY.X = (HitPoint - ViewPlane.Origin).Dot(ViewPlane.X()) / ViewPlaneWidthWorld;
			DeviceXY.Y = (HitPoint - ViewPlane.Origin).Dot(ViewPlane.Y()) / ViewPlaneHeightWorld;
			if (FMathd::Abs(DeviceXY.X) < 1 && FMathd::Abs(DeviceXY.Y) < 1)
			{
				// Shoot a ray from the sample position toward the camera position checking occlusion
				bool bVisible = VisibilityFunction(Position, RayOrigin);
				if (bVisible)
				{
					FVector2d UVCoords = FRenderCaptureCoordinateConverter2D::DeviceToUV(DeviceXY);
					if (ValidSampleDepthThreshold > 0)
					{
						// Look up the device depth from the depth photo set, these values are from 0 (far plane) to
						// 1 (near plane). We skip points which would unproject to the far plane, which is positioned at
						// infinity, we also do not interpolate the depth values, doing so is not a good approximation
						// of the underlying scene
						float DeviceZ = DeviceDepthPhotoSet.Get(Index).Image.NearestSampleUV(UVCoords);
						if (DeviceZ > 0)
						{
							// Compute the pixel position in world space to use it to compute a depth according to the render
							FVector3d PixelPositionDevice{DeviceXY, DeviceZ};
							FVector4d PixelPositionWorld = ViewMatrices.GetClipToWorld().TransformPosition(PixelPositionDevice);
							PixelPositionWorld /= PixelPositionWorld.W;

							// Compare the depth of the sample with the depth of the pixel and consider the sample invalid
							// if these do not match closely enough. This fixes artefacts which occur when sample ray just
							// misses an obstruction and hits a pixel where the color was set by a slightly different ray,
							// through the pixel center, which does hit the obstruction. This problem occurs because the
							// depth capture was obtained by renderering the the source meshes but the visiblity function
							// works on the target mesh
							float PixelDepth = (RayOrigin - FVector3d(PixelPositionWorld)).Length();
							float SampleDepth = (RayOrigin - Position).Length();
							if (FMath::Abs(PixelDepth - SampleDepth) < ValidSampleDepthThreshold)
							{
								PhotoCoords.X = UVCoords.X * (double)Params.Dimensions.GetWidth();
								PhotoCoords.Y = UVCoords.Y * (double)Params.Dimensions.GetHeight();
								PhotoIndex = Index;
								MinDot = ViewDot;
							}
						} // Test DeviceZ > 0
					} 
					else
					{
						PhotoCoords.X = UVCoords.X * (double)Params.Dimensions.GetWidth();
						PhotoCoords.Y = UVCoords.Y * (double)Params.Dimensions.GetHeight();
						PhotoIndex = Index;
						MinDot = ViewDot;
					}
				} // Test bVisible
			} // Test UVCoords in (-1,1)x(-1,1)
		} // Hit photo plane
	} // Photo loop

	return PhotoIndex != IndexConstants::InvalidID;
}

bool FSceneCapturePhotoSet::ComputeSample(
	const FRenderCaptureTypeFlags& SampleChannels,
	const FVector3d& Position,
	const FVector3d& Normal,
	TFunctionRef<bool(const FVector3d&, const FVector3d&)> VisibilityFunction,
	FSceneSample& DefaultsInResultsOut) const
{
	// This could be much more efficient if (eg) we knew that all the photo sets have
	// the same captures, then the query only has to be done once and can be used to sample each specific photo
	// This is implemented in the other ComputeSample overload

	if (SampleChannels.bBaseColor)
	{
		DefaultsInResultsOut.BaseColor =
			BaseColorPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.BaseColor);
		DefaultsInResultsOut.HaveValues.bBaseColor = true;
	}
	if (SampleChannels.bRoughness)
	{
		DefaultsInResultsOut.Roughness =
			RoughnessPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.Roughness);
		DefaultsInResultsOut.HaveValues.bRoughness = true;
	}
	if (SampleChannels.bSpecular)
	{
		DefaultsInResultsOut.Specular =
			SpecularPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.Specular);
		DefaultsInResultsOut.HaveValues.bSpecular = true;
	}
	if (SampleChannels.bMetallic)
	{
		DefaultsInResultsOut.Metallic =
			MetallicPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.Metallic);
		DefaultsInResultsOut.HaveValues.bMetallic = true;
	}
	if (SampleChannels.bCombinedMRS)
	{
		FVector3f MRSValue(DefaultsInResultsOut.Metallic, DefaultsInResultsOut.Roughness, DefaultsInResultsOut.Specular);
		MRSValue = PackedMRSPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, MRSValue);
		DefaultsInResultsOut.Metallic = MRSValue.X;
		DefaultsInResultsOut.Roughness = MRSValue.Y;
		DefaultsInResultsOut.Specular = MRSValue.Z;
		DefaultsInResultsOut.HaveValues.bMetallic = true;
		DefaultsInResultsOut.HaveValues.bRoughness = true;
		DefaultsInResultsOut.HaveValues.bSpecular = true;
	}
	if (SampleChannels.bEmissive)
	{
		DefaultsInResultsOut.Emissive =
			EmissivePhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.Emissive);
		DefaultsInResultsOut.HaveValues.bEmissive = true;
	}
	if (SampleChannels.bOpacity)
	{
		DefaultsInResultsOut.Opacity =
			OpacityPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.Opacity);
		DefaultsInResultsOut.HaveValues.bOpacity = true;
	}
	if (SampleChannels.bSubsurfaceColor)
	{
		DefaultsInResultsOut.SubsurfaceColor =
			SubsurfaceColorPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.SubsurfaceColor);
		DefaultsInResultsOut.HaveValues.bSubsurfaceColor = true;
	}
	if (SampleChannels.bWorldNormal)
	{
		DefaultsInResultsOut.WorldNormal =
			WorldNormalPhotoSet.ComputeSample(Position, Normal, VisibilityFunction, DefaultsInResultsOut.WorldNormal);
		DefaultsInResultsOut.HaveValues.bWorldNormal = true;
	}

	return true;
}

void FSceneCapturePhotoSet::GetSceneSamples(FSceneSamples& OutSamples)
{
	if (!ensure(PhotoSetStatus.DeviceDepth == ECaptureTypeStatus::Computed))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(SceneCapturePhotoSet::GetSceneSamples);

	for (int32 PhotoIndex = 0; PhotoIndex < GetSpatialPhotoParams().Num(); ++PhotoIndex)
	{
		const FSpatialPhotoParams& Params = GetSpatialPhotoParams()[PhotoIndex];

		const FViewMatrices ViewMatrices = GetRenderCaptureViewMatrices(Params.Frame, Params.HorzFOVDegrees, Params.NearPlaneDist, Params.Dimensions);

		for (int64 PixelLinearIndex = 0; PixelLinearIndex < Params.Dimensions.Num(); ++PixelLinearIndex)
		{
			const FSpatialPhoto1f& DeviceDepthPhoto = GetDeviceDepthPhotoSet().Get(PhotoIndex);

			const float DeviceZ = DeviceDepthPhoto.Image.GetPixel(PixelLinearIndex);
			const bool bValidDeviceZ = (DeviceZ > 0) && (DeviceZ < 1);
			if (bValidDeviceZ) // Skip samples on the near/far plane
			{
				bool bComputePointWorld  = OutSamples.WorldPoint != nullptr;
				bool bComputeNormalWorld = false;
				if (PhotoSetStatus.WorldNormal == ECaptureTypeStatus::Computed)
				{
					bComputePointWorld  =  bComputePointWorld || (OutSamples.WorldOrientedPoints != nullptr);
					bComputeNormalWorld = (OutSamples.WorldNormal != nullptr) || (OutSamples.WorldOrientedPoints != nullptr);
				}

				FVector3f PointWorld;
				if (bComputePointWorld)
				{
					// Map from pixel space to normalized device coordinates
					FVector2d DeviceXY = FRenderCaptureCoordinateConverter2D::PixelToDevice(
						Params.Dimensions.GetCoords(PixelLinearIndex),
						Params.Dimensions.GetWidth(),
						Params.Dimensions.GetHeight());

					// Map from normalized device coordinates to world coordinates
					FVector3d PointDevice(DeviceXY, DeviceZ);
					const FVector4d PointWorld4 = ViewMatrices.GetClipToWorld().TransformPosition(PointDevice);

					// Convert from Homogenous to Cartesian coordinates
					PointWorld.X = static_cast<float>(PointWorld4.X / PointWorld4.W);
					PointWorld.Y = static_cast<float>(PointWorld4.Y / PointWorld4.W);
					PointWorld.Z = static_cast<float>(PointWorld4.Z / PointWorld4.W);
				}

				FVector3f NormalWorld;
				if (bComputeNormalWorld)
				{
					// Map normal from color components [0,1] to world components [-1,1]
					const FSpatialPhoto3f& NormalPhoto = GetWorldNormalPhotoSet().Get(PhotoIndex);
					const FVector4f NormalColor = NormalPhoto.Image.GetPixel(PixelLinearIndex);
					NormalWorld = 2.f * (FVector3f(NormalColor) - FVector3f(.5f));
				}

				if (OutSamples.WorldPoint)
				{
					OutSamples.WorldPoint->Add(PointWorld);
				}

				if (PhotoSetStatus.WorldNormal == ECaptureTypeStatus::Computed)
				{
					if (OutSamples.WorldNormal)
					{
						OutSamples.WorldNormal->Add(NormalWorld);
					}

					if (OutSamples.WorldOrientedPoints)
					{
						OutSamples.WorldOrientedPoints->Add(FFrame3f(PointWorld, NormalWorld));
					}
				}

				if (PhotoSetStatus.Metallic == ECaptureTypeStatus::Computed && OutSamples.Metallic)
				{
					OutSamples.Metallic->Add(GetMetallicPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.Roughness == ECaptureTypeStatus::Computed && OutSamples.Roughness)
				{
					OutSamples.Roughness->Add(GetRoughnessPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.Specular == ECaptureTypeStatus::Computed && OutSamples.Specular)
				{
					OutSamples.Specular->Add(GetSpecularPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.CombinedMRS == ECaptureTypeStatus::Computed && OutSamples.PackedMRS)
				{
					OutSamples.PackedMRS->Add(GetPackedMRSPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.Emissive == ECaptureTypeStatus::Computed && OutSamples.Emissive)
				{
					OutSamples.Emissive->Add(GetEmissivePhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.BaseColor == ECaptureTypeStatus::Computed && OutSamples.BaseColor)
				{
					OutSamples.BaseColor->Add(GetBaseColorPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.SubsurfaceColor == ECaptureTypeStatus::Computed && OutSamples.SubsurfaceColor)
				{
					OutSamples.SubsurfaceColor->Add(GetSubsurfaceColorPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}

				if (PhotoSetStatus.Opacity == ECaptureTypeStatus::Computed && OutSamples.Opacity)
				{
					OutSamples.Opacity->Add(GetOpacityPhotoSet().Get(PhotoIndex).Image.GetPixel(PixelLinearIndex));
				}
			} // bValidDeviceZ
		} // PixelLinearIndex
	} // PhotoIndex
}


void FSceneCapturePhotoSet::SetEnableVisibilityByUnregisterMode(bool bEnable)
{
	bEnforceVisibilityViaUnregister = bEnable;
}

void FSceneCapturePhotoSet::EmptyAllPhotoSets()
{
	ForEachCaptureType([this](ERenderCaptureType CaptureType)
	{
		EmptyPhotoSet(CaptureType);
	});
}

void FSceneCapturePhotoSet::SetEnableWriteDebugImages(bool bEnable, FString FolderName)
{
	bWriteDebugImages = bEnable;
	if (FolderName.Len() > 0)
	{
		DebugImagesFolderName = FolderName;
	}
}

FSpatialPhotoSet1f& FSceneCapturePhotoSet::GetPhotoSet1f(ERenderCaptureType CaptureType)
{
	switch (CaptureType)
	{
	case ERenderCaptureType::Roughness:
		return RoughnessPhotoSet;
	case ERenderCaptureType::Metallic:
		return MetallicPhotoSet;
	case ERenderCaptureType::Specular:
		return SpecularPhotoSet;
	case ERenderCaptureType::Opacity:
		return OpacityPhotoSet;
	case ERenderCaptureType::DeviceDepth:
		return DeviceDepthPhotoSet;
	default:
		ensure(false);
	}
	return RoughnessPhotoSet;
}

FSpatialPhotoSet3f& FSceneCapturePhotoSet::GetPhotoSet3f(ERenderCaptureType CaptureType)
{
	switch (CaptureType)
	{
	case ERenderCaptureType::BaseColor:
		return BaseColorPhotoSet;
	case ERenderCaptureType::CombinedMRS:
		return PackedMRSPhotoSet;
	case ERenderCaptureType::WorldNormal:
		return WorldNormalPhotoSet;
	case ERenderCaptureType::Emissive:
		return EmissivePhotoSet;
	case ERenderCaptureType::SubsurfaceColor:
		return SubsurfaceColorPhotoSet;
	default:
		ensure(false);
	}
	return BaseColorPhotoSet;
}


void FSceneCapturePhotoSet::EmptyPhotoSet(ERenderCaptureType CaptureType)
{
	// Set the functions to empty the photo sets
	switch (CaptureType)
	{
	case ERenderCaptureType::BaseColor:
		BaseColorPhotoSet.Empty();
		break;
	case ERenderCaptureType::WorldNormal:
		WorldNormalPhotoSet.Empty();
		break;
	case ERenderCaptureType::Roughness:
		RoughnessPhotoSet.Empty();
		break;
	case ERenderCaptureType::Metallic:
		MetallicPhotoSet.Empty();
		break;
	case ERenderCaptureType::Specular:
		SpecularPhotoSet.Empty();
		break;
	case ERenderCaptureType::Emissive:
		EmissivePhotoSet.Empty();
		break;
	case ERenderCaptureType::CombinedMRS:
		PackedMRSPhotoSet.Empty();
		break;
	case ERenderCaptureType::Opacity:
		OpacityPhotoSet.Empty();
		break;
	case ERenderCaptureType::SubsurfaceColor:
		SubsurfaceColorPhotoSet.Empty();
		break;
	case ERenderCaptureType::DeviceDepth:
		DeviceDepthPhotoSet.Empty();
		break;
	default:
		ensure(false);
	}
}

// TODO Deprecate this function and replace it with ComputeRenderCaptureCamerasForBox
TArray<FSpatialPhotoParams> UE::Geometry::ComputeStandardExteriorSpatialPhotoParameters(
	UWorld* Unused,
	const TArray<AActor*>& Actors,
	const TArray<UActorComponent*>& Components,
	FImageDimensions PhotoDimensions,
	double HorizontalFOVDegrees,
	double NearPlaneDist,
	bool bFaces,
	bool bUpperCorners,
	bool bLowerCorners,
	bool bUpperEdges,
	bool bSideEdges)
{
	if (Actors.IsEmpty() && Components.IsEmpty())
	{
		return {};
	}

	TArray<FVector3d> Directions;
	if (bFaces)
	{
		Directions.Add(FVector3d::UnitX());
		Directions.Add(-FVector3d::UnitX());
		Directions.Add(FVector3d::UnitY());
		Directions.Add(-FVector3d::UnitY());
		Directions.Add(FVector3d::UnitZ());
		Directions.Add(-FVector3d::UnitZ());
	}
	if (bUpperCorners)
	{
		Directions.Add(Normalized(FVector3d(1, 1, -1)));
		Directions.Add(Normalized(FVector3d(-1, 1, -1)));
		Directions.Add(Normalized(FVector3d(1, -1, -1)));
		Directions.Add(Normalized(FVector3d(-1, -1, -1)));
	}
	if (bLowerCorners)
	{
		Directions.Add(Normalized(FVector3d(1, 1, 1)));
		Directions.Add(Normalized(FVector3d(-1, 1, 1)));
		Directions.Add(Normalized(FVector3d(1, -1, 1)));
		Directions.Add(Normalized(FVector3d(-1, -1, 1)));
	}
	if (bUpperEdges)
	{
		Directions.Add(Normalized(FVector3d(-1, 0, -1)));
		Directions.Add(Normalized(FVector3d(1, 0, -1)));
		Directions.Add(Normalized(FVector3d(0, -1, -1)));
		Directions.Add(Normalized(FVector3d(0, 1, -1)));
	}
	// TODO We are missing bLowerEdges!
	if (bSideEdges)
	{
		Directions.Add(Normalized(FVector3d(1, 1, 0)));
		Directions.Add(Normalized(FVector3d(-1, 1, 0)));
		Directions.Add(Normalized(FVector3d(1, -1, 0)));
		Directions.Add(Normalized(FVector3d(-1, -1, 0)));
	}

	// Compute a sphere bounding the give actors so we can use it to position the render capture viewpoints
	// Note: We use FWorldRenderCapture to do this but we are not going to render anything in this function
	FSphere RenderSphere;
	{
		FWorldRenderCapture RenderCapture;
		RenderCapture.SetVisibleActorsAndComponents(Actors, Components);
		RenderSphere = RenderCapture.ComputeContainingRenderSphere(HorizontalFOVDegrees);
	}

	TArray<FSpatialPhotoParams> Result;

	int32 NumDirections = Directions.Num();
	for (int32 di = 0; di < NumDirections; ++di)
	{
		FVector3d ViewDirection = Directions[di];
		ViewDirection.Normalize();

		FSpatialPhotoParams Params;
		Params.NearPlaneDist = NearPlaneDist;
		Params.HorzFOVDegrees = HorizontalFOVDegrees;
		Params.Dimensions = PhotoDimensions;
		// TODO Align the frame with the renderer coordinate system then remove the axis swapping in WorldRenderCapture.cpp
		Params.Frame.AlignAxis(0, ViewDirection);
		Params.Frame.ConstrainedAlignAxis(2, FVector3d::UnitZ(), Params.Frame.X());
		Params.Frame.Origin = RenderSphere.Center;
		Params.Frame.Origin -= RenderSphere.W * Params.Frame.X();

		Result.Add(Params);
	}

	return Result;
}


#undef LOCTEXT_NAMESPACE