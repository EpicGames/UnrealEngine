// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/SceneGeometrySpatialCache.h"

#include "Engine/Brush.h"
#include "GameFramework/Volume.h"
#include "Components/BrushComponent.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "TransformTypes.h"

using namespace UE::Geometry;


// This CVar controls whether scene hit queries will hit volume surfaces.
// When enabled, Volumes with a valid CollisionProfile will be included as snap targets
TAutoConsoleVariable<bool> CVarEnableModelingVolumeSnapping(
	TEXT("modeling.EnableVolumeSnapping"),
	false,
	TEXT("Enable snapping to volumes"));

namespace UE
{
namespace Local
{
	
static bool IsHiddenComponent(UPrimitiveComponent* Component)
{
#if WITH_EDITOR
	return (Component != nullptr) && ( (Component->IsVisible() == false) || (Component->IsVisibleInEditor() == false) );
#else
	return (Component != nullptr) && (Component->IsVisible() == false);
#endif
}

}
}


std::atomic<int32> ISceneGeometrySpatial::UniqueIDGenerator = 1;

ISceneGeometrySpatial::ISceneGeometrySpatial()
{
	SpatialID = UniqueIDGenerator++;
}




class FSceneVolumeSpatial : public ISceneGeometrySpatial
{
public:
	virtual void Initialize(UBrushComponent* Component);

	virtual bool IsValid() override;
	virtual bool IsVisible() override;

	virtual UPrimitiveComponent* GetComponent() override;
	virtual FAxisAlignedBox3d GetWorldBounds() override;

	virtual bool FindNearestHit(
		const FRay3d& WorldRay, 
		FSceneGeometryPoint& HitResultOut,
		double MaxDistance = TNumericLimits<double>::Max()) override;
	virtual bool GetGeometry(EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C) override;

	virtual void OnGeometryModified(bool bDeferSpatialRebuild) override;
	virtual void OnTransformModified() override;

public:
	TWeakObjectPtr<UBrushComponent> Component;
	FAxisAlignedBox3d LocalBounds;
	FTransformSRT3d Transform;
	FColliderMesh ColliderMesh;
	bool bDeferredRebuildPending;
};





void FSceneVolumeSpatial::Initialize(UBrushComponent* ComponentIn)
{
	this->Component = ComponentIn;
	bDeferredRebuildPending = true;
}

bool FSceneVolumeSpatial::IsValid()
{
	return Component.IsValid(false);
}

bool FSceneVolumeSpatial::IsVisible()
{
	return IsValid() && UE::Local::IsHiddenComponent(Component.Get()) == false;
}

UPrimitiveComponent* FSceneVolumeSpatial::GetComponent() 
{ 
	return Component.Get(); 
}

FAxisAlignedBox3d FSceneVolumeSpatial::GetWorldBounds()
{
	return FAxisAlignedBox3d(LocalBounds, Transform);
}

bool FSceneVolumeSpatial::FindNearestHit(
	const FRay3d& WorldRay, 
	FSceneGeometryPoint& HitResultOut,
	double MaxDistance)
{
	if (IsValid() == false)
	{
		return false;
	}

	if (bDeferredRebuildPending)
	{
		OnGeometryModified(false);
		ensure(bDeferredRebuildPending == false);
	}

	FRay3d LocalRay = Transform.InverseTransformRay(WorldRay);
	double RayHitT; int32 HitTID; FVector3d HitBaryCoords;
	if ( ColliderMesh.FindNearestHitTriangle(LocalRay, RayHitT, HitTID, HitBaryCoords) )
	{
		HitResultOut.Component = Component.Get();
		HitResultOut.Actor = HitResultOut.Component->GetOwner();
		HitResultOut.GeometryIndex = HitTID;
		HitResultOut.GeometryType = EGeometryPointType::Triangle;
		HitResultOut.PointBaryCoords = HitBaryCoords;
		HitResultOut.LocalPoint = LocalRay.PointAt(RayHitT);
		HitResultOut.WorldPoint = Transform.TransformPosition(HitResultOut.LocalPoint);
		HitResultOut.Ray = WorldRay;
		HitResultOut.RayDistance = WorldRay.GetParameter(HitResultOut.WorldPoint);
		return true;
	}
	return false;
}


bool FSceneVolumeSpatial::GetGeometry(EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C)
{
	if (GeometryType == EGeometryPointType::Triangle)
	{
		if (ColliderMesh.IsTriangle(GeometryIndex))
		{
			ColliderMesh.GetTriVertices(GeometryIndex, A, B, C);
			if (bWorldSpace)
			{
				A = Transform.TransformPosition(A);
				B = Transform.TransformPosition(B);
				C = Transform.TransformPosition(C);
			}
			return true;
		}
	}
	return false;
}


void FSceneVolumeSpatial::OnGeometryModified(bool bDeferSpatialRebuild)
{
	ColliderMesh.Reset(EAllowShrinking::No);
	if (IsValid() == false)
	{
		return;
	}

	Transform = (FTransformSRT3d)Component->GetComponentTransform();
	LocalBounds = FAxisAlignedBox3d( Component->CalcBounds(FTransform::Identity).GetBox() );

	if (bDeferSpatialRebuild)
	{
		bDeferredRebuildPending = true;
		return;
	}

	AVolume* Volume = Cast<AVolume>(Component->GetOwner());
	if (Volume)
	{
		UE::Conversion::FVolumeToMeshOptions Options;
		Options.bSetGroups = false;
		Options.bMergeVertices = false;
		Options.bAutoRepairMesh = false;
		Options.bOptimizeMesh = false;
		FDynamicMesh3 TmpMesh;
		UE::Conversion::VolumeToDynamicMesh(Volume, TmpMesh, Options);
		ColliderMesh.Initialize(TmpMesh);
	}

	bDeferredRebuildPending = false;
}


void FSceneVolumeSpatial::OnTransformModified()
{
	if (IsValid())
	{
		Transform = (FTransformSRT3d)Component->GetComponentTransform();
	}
}






class FSceneDynamicMeshSpatial : public ISceneGeometrySpatial
{
public:
	virtual void Initialize(UDynamicMeshComponent* Component);

	virtual bool IsValid() override;
	virtual bool IsVisible() override;

	virtual UPrimitiveComponent* GetComponent() override;
	virtual FAxisAlignedBox3d GetWorldBounds() override;

	virtual bool FindNearestHit(
		const FRay3d& WorldRay, 
		FSceneGeometryPoint& HitResultOut,
		double MaxDistance = TNumericLimits<double>::Max()) override;
	virtual bool GetGeometry(EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C) override;

	virtual void OnGeometryModified(bool bDeferSpatialRebuild) override;
	virtual void OnTransformModified() override;

public:
	TWeakObjectPtr<UDynamicMeshComponent> Component;
	FAxisAlignedBox3d LocalBounds;
	FTransformSRT3d Transform;
	FDynamicMeshAABBTree3 AABBTree;
	bool bDeferredRebuildPending;
};



void FSceneDynamicMeshSpatial::Initialize(UDynamicMeshComponent* ComponentIn)
{
	this->Component = ComponentIn;
	bDeferredRebuildPending = true;
}

bool FSceneDynamicMeshSpatial::IsValid()
{
	return Component.IsValid(false);
}

bool FSceneDynamicMeshSpatial::IsVisible()
{
	return IsValid() && UE::Local::IsHiddenComponent(Component.Get()) == false;
}

UPrimitiveComponent* FSceneDynamicMeshSpatial::GetComponent() 
{ 
	return Component.Get(); 
}

FAxisAlignedBox3d FSceneDynamicMeshSpatial::GetWorldBounds()
{
	return FAxisAlignedBox3d(LocalBounds, Transform);
}

bool FSceneDynamicMeshSpatial::FindNearestHit(
	const FRay3d& WorldRay, 
	FSceneGeometryPoint& HitResultOut,
	double MaxDistance)
{
	if (IsValid() == false)
	{
		return false;
	}

	if (bDeferredRebuildPending)
	{
		OnGeometryModified(false);
		ensure(bDeferredRebuildPending == false);
	}

	FRay3d LocalRay = Transform.InverseTransformRay(WorldRay);
	double RayHitT; int32 HitTID; FVector3d HitBaryCoords;
	if (AABBTree.FindNearestHitTriangle(LocalRay, RayHitT, HitTID, HitBaryCoords))
	{
		HitResultOut.Component = Component.Get();
		HitResultOut.Actor = HitResultOut.Component->GetOwner();
		HitResultOut.GeometryIndex = HitTID;
		HitResultOut.GeometryType = EGeometryPointType::Triangle;
		HitResultOut.PointBaryCoords = HitBaryCoords;
		HitResultOut.LocalPoint = LocalRay.PointAt(RayHitT);
		HitResultOut.WorldPoint = Transform.TransformPosition(HitResultOut.LocalPoint);
		HitResultOut.Ray = WorldRay;
		HitResultOut.RayDistance = WorldRay.GetParameter(HitResultOut.WorldPoint);
		return true;
	}
	return false;
}


bool FSceneDynamicMeshSpatial::GetGeometry(EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C)
{
	bool bFound = false;
	if (GeometryType == EGeometryPointType::Triangle)
	{
		UDynamicMeshComponent* DynamicMeshComponent = Component.Get();
		if (DynamicMeshComponent)
		{
			DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& Mesh)
			{
				if (Mesh.IsTriangle(GeometryIndex))
				{
					Mesh.GetTriVertices(GeometryIndex, A, B, C);
					if (bWorldSpace)
					{
						A = Transform.TransformPosition(A);
						B = Transform.TransformPosition(B);
						C = Transform.TransformPosition(C);
					}
					bFound = true;
				}
			});
		}
	}
	return bFound;
}


void FSceneDynamicMeshSpatial::OnGeometryModified(bool bDeferSpatialRebuild)
{
	if (IsValid() == false)
	{
		return;
	}
	UDynamicMeshComponent* DynamicMeshComponent = Component.Get();
	if (!DynamicMeshComponent)
	{
		return;
	}

	Transform = (FTransformSRT3d)DynamicMeshComponent->GetComponentTransform();
	// may be reasonable to use USceneComponent::GetLocalBounds() here. Unclear if it will have already been updated on mesh changes...
	DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& Mesh)
	{
		LocalBounds = Mesh.GetBounds(true);
	});

	if (bDeferSpatialRebuild)
	{
		bDeferredRebuildPending = true;
		return;
	}

	DynamicMeshComponent->ProcessMesh([&](const FDynamicMesh3& Mesh)
	{
		AABBTree.SetMesh(&Mesh, true);
	});

	bDeferredRebuildPending = false;
}


void FSceneDynamicMeshSpatial::OnTransformModified()
{
	if (IsValid())
	{
		Transform = (FTransformSRT3d)Component->GetComponentTransform();
	}
}






FSceneGeometrySpatialCache::FSceneGeometrySpatialCache()
{
	BoundsOctree.RootDimension = 100000;		// 1000m, this can do until we come up w/ something better?
	BoundsOctree.SetMaxTreeDepth(7);
}

FSceneGeometrySpatialCache::~FSceneGeometrySpatialCache()
{
}


bool FSceneGeometrySpatialCache::EnableComponentTracking(UPrimitiveComponent* Component, FSceneGeometryID& IdentifierOut)
{
	if (ComponentMap.Contains(Component))
	{
		IdentifierOut = ComponentMap[Component];
		return true;
	}

	bool bEnableVolumes = CVarEnableModelingVolumeSnapping.GetValueOnAnyThread();

	TSharedPtr<ISceneGeometrySpatial> NewSpatial;
	if (bEnableVolumes && Cast<UBrushComponent>(Component) != nullptr)
	{
		IdentifierOut = FSceneGeometryID{ UniqueIDGenerator++ };

		TSharedPtr<FSceneVolumeSpatial> Spatial = MakeShared<FSceneVolumeSpatial>();
		Spatial->Initialize(Cast<UBrushComponent>(Component));
		NewSpatial = Spatial;
	}
	else if (Cast<UDynamicMeshComponent>(Component) != nullptr)
	{
		IdentifierOut = FSceneGeometryID{ UniqueIDGenerator++ };

		TSharedPtr<FSceneDynamicMeshSpatial> Spatial = MakeShared<FSceneDynamicMeshSpatial>();
		Spatial->Initialize(Cast<UDynamicMeshComponent>(Component));
		NewSpatial = Spatial;
	}

	if (NewSpatial != nullptr)
	{
		ComponentMap.Add(Component, IdentifierOut);
		Spatials.Add(IdentifierOut, NewSpatial);

		// do elsewhere?
		NewSpatial->OnGeometryModified(true);

		BoundsOctree.InsertObject(IdentifierOut.UniqueID, NewSpatial->GetWorldBounds());

		return true;
	}

	return false;
}



void FSceneGeometrySpatialCache::DisableComponentTracking(UPrimitiveComponent* Component)
{
	if (ComponentMap.Contains(Component))
	{
		FSceneGeometryID Identifier = ComponentMap[Component];
		ComponentMap.Remove(Component);
		DisableGeometryTracking(Identifier);
	}
}


void FSceneGeometrySpatialCache::DisableGeometryTracking(FSceneGeometryID& Identifier)
{
	if (Spatials.Contains(Identifier))
	{
		BoundsOctree.RemoveObject(Identifier.UniqueID);

		Spatials.Remove(Identifier);
	}
}



bool FSceneGeometrySpatialCache::FindNearestHit(
	const FRay3d& WorldRay,
	FSceneGeometryPoint& HitResultOut,
	FSceneGeometryID& HitIdentifier,
	const FSceneQueryVisibilityFilter* VisibilityFilter,
	double MaxDistance)
{

	HitResultOut = FSceneGeometryPoint();
	bool bFoundAnyHit = false;

	// TODO: this current approach will force any hit objects to run any deferred AABBTree builds.
	// What we probably should do is have a FindAllHitObjects query that only tests the boxes, and
	// then for those hits, parallel-build any pending AABBTrees before testing them.

	// cast ray into octree
	int32 HitSpatialID = BoundsOctree.FindNearestHitObject(WorldRay,
		[&](int32 Identifier) { 
			return Spatials[FSceneGeometryID(Identifier)]->GetWorldBounds(); 
		},
		[&](int Identifier, const FRay3d& Ray) {
			TSharedPtr<ISceneGeometrySpatial> Spatial = Spatials[FSceneGeometryID(Identifier)];
			if (VisibilityFilter != nullptr && VisibilityFilter->IsVisible(Spatial->GetComponent()) == false)
			{
				return TNumericLimits<double>::Max();
			}		
			FSceneGeometryPoint Hit;
			if (Spatial->FindNearestHit(WorldRay, Hit, MaxDistance))
			{
				return Hit.RayDistance;
			}
			return TNumericLimits<double>::Max();
		});

	// if we hit something above, repeat it's hit-query here (dumb...)
	if (HitSpatialID >= 0)
	{
		TSharedPtr<ISceneGeometrySpatial> Spatial = Spatials[FSceneGeometryID(HitSpatialID)];
		FSceneGeometryPoint Hit;
		bool bHit = Spatial->FindNearestHit(WorldRay, Hit, MaxDistance);
		if (ensure(bHit))
		{
			HitResultOut = Hit;
			HitIdentifier = FSceneGeometryID(HitSpatialID);
			return true;
		}
	}

	return false;
}


bool FSceneGeometrySpatialCache::GetGeometry(FSceneGeometryID Identifier, EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C)
{
	TSharedPtr<ISceneGeometrySpatial>* Found = Spatials.Find(Identifier);
	if (Found != nullptr)
	{
		return (*Found)->GetGeometry(GeometryType, GeometryIndex, bWorldSpace, A, B, C);
	}
	return false;
}


bool FSceneGeometrySpatialCache::HaveCacheForComponent(UPrimitiveComponent* Component)
{
	const FSceneGeometryID* FoundID = ComponentMap.Find(Component);
	if (FoundID != nullptr)
	{
		TSharedPtr<ISceneGeometrySpatial> Spatial = Spatials[*FoundID];
		return Spatial->IsValid();
	}
	return false;
}


bool FSceneGeometrySpatialCache::NotifyTransformUpdate(UPrimitiveComponent* Component)
{
	const FSceneGeometryID* FoundID = ComponentMap.Find(Component);
	if (FoundID != nullptr)
	{
		TSharedPtr<ISceneGeometrySpatial> Spatial = Spatials[*FoundID];
		Spatial->OnTransformModified();

		BoundsOctree.ReinsertObject(FoundID->UniqueID, Spatial->GetWorldBounds());

		return true;
	}
	return false;	
}

bool FSceneGeometrySpatialCache::NotifyGeometryUpdate(UPrimitiveComponent* Component, bool bDeferRebuild)
{
	const FSceneGeometryID* FoundID = ComponentMap.Find(Component);
	if (FoundID != nullptr)
	{
		TSharedPtr<ISceneGeometrySpatial> Spatial = Spatials[*FoundID];
		Spatial->OnGeometryModified(bDeferRebuild);

		BoundsOctree.ReinsertObject(FoundID->UniqueID, Spatial->GetWorldBounds());

		return true;
	}
	return false;	
}