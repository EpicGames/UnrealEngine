// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/WorldRenderCapture.h"
#include "AssetUtils/Texture2DUtil.h"
#include "BufferVisualizationData.h"

#include "Components/PrimitiveComponent.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/Material.h"
#include "BufferVisualizationData.h"
#include "GameFramework/WorldSettings.h"
#include "LegacyScreenPercentageDriver.h"
#include "EngineModule.h"
#include "Rendering/Texture2DResource.h"
#include "ImagePixelData.h"
#include "ImageWriteStream.h"
#include "ImageCore.h"
#include "Async/ParallelFor.h"
#include "RendererInterface.h"

#include "PreviewScene.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/ChildActorComponent.h"

#include "SceneViewExtension.h"
#include "RenderCaptureInterface.h" // For debugging with RenderCaptureInterface::FScopedCapture
#include "AssetCompilingManager.h"

using namespace UE::Geometry;

static TAutoConsoleVariable<int32> CVarModelingWorldRenderCaptureWarmupFrames(
	TEXT("modeling.WorldRenderCapture.VTWarmupFrames"),
	5,
	TEXT("Number of frames to render before each capture in order to warmup the renderer."));

FRenderCaptureTypeFlags FRenderCaptureTypeFlags::All(bool bCombinedMRS)
{
	FRenderCaptureTypeFlags Result;
	ForEachCaptureType([&Result](ERenderCaptureType CaptureType)
	{
		Result[CaptureType] = true;
	});
	Result.bCombinedMRS = bCombinedMRS;
	return Result;
}

FRenderCaptureTypeFlags FRenderCaptureTypeFlags::None()
{
	return FRenderCaptureTypeFlags{};
}

FRenderCaptureTypeFlags FRenderCaptureTypeFlags::BaseColor()
{
	return Single(ERenderCaptureType::BaseColor);
}

FRenderCaptureTypeFlags FRenderCaptureTypeFlags::WorldNormal()
{
	return Single(ERenderCaptureType::WorldNormal);
}

FRenderCaptureTypeFlags FRenderCaptureTypeFlags::Single(ERenderCaptureType CaptureType)
{
	FRenderCaptureTypeFlags Flags = None();
	Flags.SetEnabled(CaptureType, true);
	return Flags;
}

void FRenderCaptureTypeFlags::SetEnabled(ERenderCaptureType CaptureType, bool bEnabled)
{
	this->operator[](CaptureType) = bEnabled;
}

/** @return mutable flag corresponding to the given CaptureType */
bool& FRenderCaptureTypeFlags::operator[](ERenderCaptureType CaptureType)
{
	switch (CaptureType)
	{
	case ERenderCaptureType::BaseColor:
		return bBaseColor;
	case ERenderCaptureType::WorldNormal:
		return bWorldNormal;
	case ERenderCaptureType::Roughness:
		return bRoughness;
	case ERenderCaptureType::Metallic:
		return bMetallic;
	case ERenderCaptureType::Specular:
		return bSpecular;
	case ERenderCaptureType::Emissive:
		return bEmissive;
	case ERenderCaptureType::CombinedMRS:
		return bCombinedMRS;
	case ERenderCaptureType::Opacity:
		return bOpacity;
	case ERenderCaptureType::SubsurfaceColor:
		return bSubsurfaceColor;
	case ERenderCaptureType::DeviceDepth:
		return bDeviceDepth;
	default:
		ensure(false);
	}
	return bBaseColor;
}

/** @return constant flag corresponding to the given CaptureType */
const bool& FRenderCaptureTypeFlags::operator[](ERenderCaptureType CaptureType) const
{
	return const_cast<FRenderCaptureTypeFlags*>(this)->operator[](CaptureType);
}

/** @return true if the flags for this object match the Other object and false otherwise */
bool FRenderCaptureTypeFlags::operator==(const FRenderCaptureTypeFlags& Other) const
{
	bool bEqual = true;
	ForEachCaptureType([this, &Other, &bEqual](ERenderCaptureType CaptureType)
	{
		bEqual = bEqual && (this->operator[](CaptureType) == Other[CaptureType]);
	});
	return bEqual;
}

bool FRenderCaptureTypeFlags::operator!=(const FRenderCaptureTypeFlags& Other) const
{
	return !this->operator==(Other);
}


namespace UE
{
namespace Geometry
{

FRenderCaptureConfig GetDefaultRenderCaptureConfig(ERenderCaptureType CaptureType)
{
	FRenderCaptureConfig Config;

	// The DeviceDepth channel ignores this option, but since its hard-coded to false we set it to false
	Config.bAntiAliasing = (CaptureType != ERenderCaptureType::DeviceDepth);

	return Config;
}

} // end namespace Geometry

namespace Internal
{

void FlushRHIThreadToUpdateTextureRenderTargetReference()
{
	// Flush RHI thread after creating texture render target to make sure that RHIUpdateTextureReference is executed
	// before doing any rendering with it This makes sure that
	//    Value->TextureReference.TextureReferenceRHI->GetReferencedTexture()
	// is valid so that
	//    UniformExpressionSet::FillUniformBuffer
	// properly uses the texture for rendering, instead of using a fallback texture
	ENQUEUE_RENDER_COMMAND(FlushRHIThreadToUpdateTextureRenderTargetReference)(
	[](FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
	});
}

} // end namespace Internal
} // end namespace UE





FWorldRenderCapture::~FWorldRenderCapture()
{
	RestoreComponentsMaxDrawDistance();
}

void FWorldRenderCapture::SetWorld(UWorld* WorldIn)
{
	this->World = WorldIn;
}

void FWorldRenderCapture::SetVisibleActors(const TArray<AActor*>& Actors)
{
	SetVisibleActorsAndComponents(Actors, {});
}

void FWorldRenderCapture::SetVisibleComponents(const TArray<UActorComponent*>& Components)
{
	SetVisibleActorsAndComponents({}, Components);
}

void FWorldRenderCapture::RestoreComponentsMaxDrawDistance()
{
	for (UPrimitiveComponent* PrimitiveComponent : ComponentsRestoreMaxDrawDistance)
	{
		PrimitiveComponent->SetCachedMaxDrawDistance(PrimitiveComponent->LDMaxDrawDistance);
	}

	ComponentsRestoreMaxDrawDistance.Reset();
}

void FWorldRenderCapture::SetVisibleActorsAndComponents(const TArray<AActor*>& Actors, const TArray<UActorComponent*>& Components)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldRenderCapture::SetVisibleActorsAndComponents);

	VisiblePrimitives.Reset();

	RestoreComponentsMaxDrawDistance();

	FBoxSphereBounds::Builder BoundsBuilder;

	// Find all components that need to be included in rendering.
	// This also descends into any ChildActorComponents
		
	auto AddComponent = [&](UActorComponent* Component)
	{
		auto AddComponentImpl = [&](UActorComponent* Component, auto& AddComponentRef) -> void
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(Component))
			{				
				VisiblePrimitives.Add(PrimitiveComponent->GetPrimitiveSceneId());

				if (!PrimitiveComponent->bNeverDistanceCull)
				{
					ComponentsRestoreMaxDrawDistance.Add(PrimitiveComponent);
					PrimitiveComponent->SetCachedMaxDrawDistance(0.0f);
				}
				
				// Append bounds of visible components only
				BoundsBuilder += PrimitiveComponent->Bounds;
			}
			else if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(Component))
			{
				if (AActor* ChildActor = ChildActorComponent->GetChildActor())
				{
					for (UActorComponent* SubComponent : ChildActor->GetComponents())
					{
						AddComponentRef(SubComponent, AddComponentRef);
					}
				}
			}			
		};

		AddComponentImpl(Component, AddComponentImpl);
	};

	// Add actors
	for (AActor* Actor : Actors)
	{
		for (UActorComponent* Component : Actor->GetComponents())
		{
			AddComponent(Component);
		}
	}

	// Add components
	for (UActorComponent* Component : Components)
	{
		AddComponent(Component);
	}

	VisibleBounds = BoundsBuilder;
}


void FWorldRenderCapture::SetDimensions(const FImageDimensions& DimensionsIn)
{
	this->Dimensions = DimensionsIn;
}


FSphere FWorldRenderCapture::ComputeContainingRenderSphere(float HorzFOVDegrees, float SafetyBoundsScale) const
{
	if (VisiblePrimitives.Num() == 0)
	{
		ensure(false);		// unclear what we should do here - get bounds of all actors?
		return FSphere(FVector(0,0,0), 1000.0f);
	}

	// todo: I think this maybe needs to be based on the box corners? 

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(HorzFOVDegrees) * 0.5f;
	const float HalfMeshSize = VisibleBounds.SphereRadius * SafetyBoundsScale;
	const float TargetDistance = HalfMeshSize / FMath::Tan(HalfFOVRadians);
	return FSphere(VisibleBounds.Origin, TargetDistance);
}

UTextureRenderTarget2D* FWorldRenderCapture::GetRenderTexture()
{
	if (RenderTextureDimensions != this->Dimensions)
	{
		LinearRenderTexture = nullptr;
	}

	if (LinearRenderTexture != nullptr)
	{
		LinearRenderTexture->UpdateResourceImmediate(true);

		UE::Internal::FlushRHIThreadToUpdateTextureRenderTargetReference();

		return LinearRenderTexture;
	}

	RenderTextureDimensions = Dimensions;
	int32 Width = Dimensions.GetWidth();
	int32 Height = Dimensions.GetHeight();

	LinearRenderTexture = NewObject<UTextureRenderTarget2D>();
	if (ensure(LinearRenderTexture))
	{
		LinearRenderTexture->ClearColor = FLinearColor::Transparent;
		LinearRenderTexture->TargetGamma = 1.0f;
		LinearRenderTexture->InitCustomFormat(Width, Height, PF_FloatRGBA, false);
		LinearRenderTexture->UpdateResourceImmediate(true);

		UE::Internal::FlushRHIThreadToUpdateTextureRenderTargetReference();
	}

	return LinearRenderTexture;
}


UTextureRenderTarget2D* FWorldRenderCapture::GetDepthRenderTexture()
{
	if (RenderTextureDimensions != this->Dimensions)
	{
		DepthRenderTexture = nullptr;
	}

	if (DepthRenderTexture != nullptr)
	{
		return DepthRenderTexture;
	}

	RenderTextureDimensions = Dimensions;
	int32 Width = Dimensions.GetWidth();
	int32 Height = Dimensions.GetHeight();

	DepthRenderTexture = NewObject<UTextureRenderTarget2D>();
	if (ensure(DepthRenderTexture))
	{
		DepthRenderTexture->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA32f;
		DepthRenderTexture->ClearColor = FLinearColor::Black;
		DepthRenderTexture->InitAutoFormat(Width, Height);
		DepthRenderTexture->UpdateResourceImmediate(true);

		UE::Internal::FlushRHIThreadToUpdateTextureRenderTargetReference();
	}

	return DepthRenderTexture;
}


namespace UE 
{
namespace Internal
{

/**
 * Render the scene to the provided canvas. Will potentially perform the render multiple times, depending on the value of
 * the warmup frames CVar. This is needed to ensure the VT/Nanite are primed properly before capturing the scene.
 */
static void PerformSceneRender(FCanvas* Canvas, FSceneViewFamily* ViewFamily)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Internal::PerformSceneRender);

	bool bCompiledAssets = false;

	do
	{
		int32 NumRender = 1 + CVarModelingWorldRenderCaptureWarmupFrames.GetValueOnGameThread();
		for (int32 i = 0; i < NumRender; i++)
		{
			GetRendererModule().BeginRenderingViewFamily(Canvas, ViewFamily);
		}

		// Flush rendering commands, may queue assets compilation (shaders)
		FlushRenderingCommands();

		// If there are assets to be compiled, we need to wait for compilation to finish and start render over
		bCompiledAssets = FAssetCompilingManager::Get().GetNumRemainingAssets() > 0;
		if (bCompiledAssets)
		{
			FAssetCompilingManager::Get().FinishAllCompilation();

			// Ensure all deferred construction scripts are executed
			FAssetCompilingManager::Get().ProcessAsyncTasks();
		}
	} while (bCompiledAssets);
}

// Copied from ImageCore.cpp
template <typename Lambda>
static void ParallelLoop(const TCHAR* DebugName, int32 NumJobs, int64 TexelsPerJob, int64 NumTexels, const Lambda& Func)
{
	ParallelFor(DebugName, NumJobs, 1, [=](int64 JobIndex)
	{
		const int64 StartIndex = JobIndex * TexelsPerJob;
		const int64 EndIndex = FMath::Min(StartIndex + TexelsPerJob, NumTexels);
		for (int64 TexelIndex = StartIndex; TexelIndex < EndIndex; ++TexelIndex)
		{
			Func(TexelIndex);
		}
	}, EParallelForFlags::Unbalanced);
}

// This function is needed because we need to convert pixels from the GPU read back format to the format we prefer in the calling code
// e.g., we may readback linear colors but prefer to store only 3 components to save memory
static void CopyCaptureDataToOutputImageFormat(
	const FImageView& CaptureData,
	FImageAdapter& OutputImage,
	const TFunctionRef<FLinearColor(const FLinearColor&)>& Transform = [](const FLinearColor Color) { return Color; })
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CopyCaptureDataToOutputImageFormat);

	const FImageDimensions Dims(CaptureData.GetWidth(), CaptureData.GetHeight());
	OutputImage.SetDimensions(Dims);

	const int64 NumTexels = CaptureData.GetNumPixels();
	int64 TexelsPerJob;
	const int32 NumJobs = ImageParallelForComputeNumJobsForPixels(TexelsPerJob, NumTexels);
	ParallelLoop(TEXT("PF.CopyCaptureDataToOutputImageFormat"), NumJobs, TexelsPerJob, NumTexels,
		[&CaptureData, &OutputImage, &Transform, Dims](int64 TexelIndex)
	{
		const FVector2i Coords = Dims.GetCoords(TexelIndex);
		const FLinearColor Color = CaptureData.GetOnePixelLinear(Coords.X, Coords.Y);

		// This call will store a subset of the components of the (transformed) color, depending on OutputImage.ImageType
		OutputImage.SetPixel(Coords, Transform(Color));
	});
}

// The .ViewFamily member should be set on the return value later
// VisiblePrimitives: If non-empty, only these primitives are shown
// HiddenPrimitives: These will be hidden
FSceneViewInitOptions MakeSceneViewInitOptions(
	const FFrame3d& Frame,
	const FImageDimensions& Dimensions,
	double HorzFOVDegrees,
	double NearPlaneDist,
	const TSet<FPrimitiveComponentId>& VisiblePrimitives,
	const TSet<FPrimitiveComponentId>& HiddenPrimitives = {}
	)
{
	int32 Width = Dimensions.GetWidth();
	int32 Height = Dimensions.GetHeight();

	// TODO The input frame should have been the render frame the whole time
	FFrame3d RenderFrame(Frame.Origin, Frame.Y(), Frame.Z(), Frame.X());

	FVector ViewOrigin = RenderFrame.Origin;
	FMatrix ViewRotationMatrix = FRotationMatrix::Make((FQuat)RenderFrame.Rotation.Inverse());

	const float HalfFOVRadians = FMath::DegreesToRadians<float>(HorzFOVDegrees) * 0.5f;
	FMatrix ProjectionMatrix = FReversedZPerspectiveMatrix(HalfFOVRadians, 1.0f, 1.0f, (float)NearPlaneDist);

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.SetViewRectangle(FIntRect(0, 0, Width, Height));
	if (VisiblePrimitives.Num() > 0)
	{
		ViewInitOptions.ShowOnlyPrimitives = VisiblePrimitives;
	}
	ViewInitOptions.HiddenPrimitives = HiddenPrimitives;
	ViewInitOptions.ViewOrigin = ViewOrigin;
	ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
	ViewInitOptions.ProjectionMatrix = ProjectionMatrix;

	return ViewInitOptions;
}

void SetCommonShowFlags(FEngineShowFlags& ShowFlags, bool bAntiAliasing)
{
	ShowFlags.SetAntiAliasing(bAntiAliasing);

	// Disable advanced features
	ShowFlags.DisableAdvancedFeatures();

	// Disable some more features. These are sorted alphabetically.
	ShowFlags.SetAtmosphere(false);
	ShowFlags.SetBloom(false);
	ShowFlags.SetCapsuleShadows(false);
	ShowFlags.SetContactShadows(false);
	ShowFlags.SetDiffuse(false);
	ShowFlags.SetDirectionalLights(false);
	ShowFlags.SetDynamicShadows(false);
	ShowFlags.SetFog(false);
	ShowFlags.SetGlobalIllumination(false);
	ShowFlags.SetLighting(false);
	ShowFlags.SetMotionBlur(false);
	ShowFlags.SetPointLights(false);
	ShowFlags.SetPostProcessing(false);
	ShowFlags.SetRectLights(false);
	ShowFlags.SetReflectionEnvironment(false);
	ShowFlags.SetRefraction(false);
	ShowFlags.SetSceneColorFringe(false);
	ShowFlags.SetSkyLighting(false);
	ShowFlags.SetSpecular(false);
	ShowFlags.SetSpotLights(false);
	ShowFlags.SetTonemapper(false);
	ShowFlags.SetToneCurve(false);
	ShowFlags.SetTranslucency(false);
}

FName GetBufferVisualizationModeName(ERenderCaptureType CaptureType)
{
	switch (CaptureType)
	{
	case ERenderCaptureType::BaseColor:
		return FName("BaseColor");
	case ERenderCaptureType::Roughness:
		return FName("Roughness");
	case ERenderCaptureType::Metallic:
		return FName("Metallic");
	case ERenderCaptureType::Specular:
		return FName("Specular");
	case ERenderCaptureType::WorldNormal:
		return FName("WorldNormal");
	case ERenderCaptureType::DeviceDepth:
		return FName("DeviceDepth");
	case ERenderCaptureType::Opacity:
		return FName("Opacity");
	case ERenderCaptureType::SubsurfaceColor:
		return FName("SubsurfaceColor");
	case ERenderCaptureType::Emissive:
		return FName("PreTonemapHDRColor");
	default:
		ensure(false);
	}
	return FName("BaseColor");
}

} // end namespace Internal
} // end namespace UE


static const TCHAR* MRSPostProcessMaterialAssetPath = TEXT("/MeshModelingToolsetExp/Materials/PostProcess_PackedMRSA.PostProcess_PackedMRSA");

bool FWorldRenderCapture::CaptureUsingPostProcessMaterialPipe(
	UMaterialInterface* PostProcessMaterial,
	const FFrame3d& ViewFrame,
	double HorzFOVDegrees,
	double NearPlaneDist,
	FImageAdapter& ResultImageOut,
	const FRenderCaptureConfig& Config,
	bool bEnableLighting,
	const TCHAR* DebugImageName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("FWorldRenderCapture_%s"), DebugImageName));

	if (!ensure(PostProcessMaterial))
	{
		return false;
	}

	check(this->World);
	FSceneInterface* Scene = this->World->Scene;

	UTextureRenderTarget2D* RenderTargetTexture = GetRenderTexture();
	if (!ensure(RenderTargetTexture))
	{
		return false;
	}
	FRenderTarget* RenderTargetResource = RenderTargetTexture->GameThread_GetRenderTargetResource();

	FEngineShowFlags ShowFlags(ESFIM_Game);
	ApplyViewMode(VMI_Unlit, true, ShowFlags);
	Internal::SetCommonShowFlags(ShowFlags, Config.bAntiAliasing);
	ShowFlags.SetLighting(bEnableLighting);
	if (bEnableLighting)
	{
		// Emissive capture requires the post-processing pipeline to be active.
		ShowFlags.SetPostProcessing(true);
		ShowFlags.SetPostProcessMaterial(true);
	}

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		RenderTargetResource, Scene, ShowFlags)
		.SetTime(FGameTime())
		.SetRealtimeUpdate(false)
	);

	ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(ViewFamily, 1.f));

	FSceneViewInitOptions ViewInitOptions = Internal::MakeSceneViewInitOptions(
		ViewFrame,
		Dimensions,
		HorzFOVDegrees,
		NearPlaneDist,
		VisiblePrimitives);
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.FOV = HorzFOVDegrees; // Probably unecessary (FOV is already accounted for in projection matrix)

	FSceneView* NewView = new FSceneView(ViewInitOptions);
	ViewFamily.Views.Add(NewView);

	NewView->StartFinalPostprocessSettings(ViewInitOptions.ViewOrigin);

	//
	// Add custom PostProcessMaterial. This will be configured to write to a FImagePixelPipe,
	// which will be filled by the renderer and available after we force the render
	//
	TUniquePtr<FImagePixelData> PostProcessPassPixelData;

	// clear any existing PostProcess materials and add our own
	NewView->FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Empty();
	NewView->FinalPostProcessSettings.BufferVisualizationOverviewMaterials.Add(PostProcessMaterial);

	// create new FImagePixelPipe and add output lambda that takes ownership of the data
	NewView->FinalPostProcessSettings.BufferVisualizationPipes.Empty();
	TSharedPtr<FImagePixelPipe, ESPMode::ThreadSafe> ImagePixelPipe = MakeShared<FImagePixelPipe, ESPMode::ThreadSafe>();
	ImagePixelPipe->bIsExpecting32BitPixelData = true;
	ImagePixelPipe->AddEndpoint([&PostProcessPassPixelData](TUniquePtr<FImagePixelData>&& Data) {
		PostProcessPassPixelData = MoveTemp(Data);
	});
	NewView->FinalPostProcessSettings.BufferVisualizationPipes.Add(PostProcessMaterial->GetFName(), ImagePixelPipe);

	// enable buffer visualization writing
	NewView->FinalPostProcessSettings.bBufferVisualizationDumpRequired = true;
	NewView->EndFinalPostprocessSettings(ViewInitOptions);

	// can we fully disable auto-exposure?
	NewView->FinalPostProcessSettings.AutoExposureMethod = EAutoExposureMethod::AEM_Manual;

	// do we actually need for force SM5 here? 
	FCanvas Canvas = FCanvas(RenderTargetResource, nullptr, this->World, ERHIFeatureLevel::SM5, FCanvas::CDM_DeferDrawing, 1.0f);
	Canvas.Clear(FLinearColor::Transparent);

	UE::Internal::PerformSceneRender(&Canvas, &ViewFamily);

	// Cache the view/projection matricies we used to render the scene
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LastCaptureViewMatrices = NewView->ViewMatrices;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (ensure(PostProcessPassPixelData.IsValid()) == false)
	{
		return false;
	}

	UE::Internal::CopyCaptureDataToOutputImageFormat(PostProcessPassPixelData->GetImageView(), ResultImageOut);

	if (bWriteDebugImage)
	{
		WriteDebugImage(ResultImageOut, DebugImageName);
	}

	return true;
}


bool FWorldRenderCapture::CaptureMRSFromPosition(
	const FFrame3d& ViewFrame,
	double HorzFOVDegrees,
	double NearPlaneDist,
	FImageAdapter& ResultImageOut,
	const FRenderCaptureConfig& Config)
{
	TSoftObjectPtr<UMaterialInterface> PostProcessMaterialPtr = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(MRSPostProcessMaterialAssetPath));
	UMaterialInterface* PostProcessMaterial = PostProcessMaterialPtr.LoadSynchronous();
	if (!ensure(PostProcessMaterial))
	{
		return false;
	}

	const bool bEnableLighting = false;
	return CaptureUsingPostProcessMaterialPipe(PostProcessMaterial, ViewFrame, HorzFOVDegrees, NearPlaneDist, ResultImageOut, Config, bEnableLighting, TEXT("CombinedMRS"));
}


bool FWorldRenderCapture::CaptureDeviceDepthFromPosition(
	const FFrame3d& Frame,
	double HorzFOVDegrees,
	double NearPlaneDist,
	FImageAdapter& ResultImageOut,
	const FRenderCaptureConfig& Config)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldRenderCapture_DeviceDepth);

	DepthRenderTexture = GetDepthRenderTexture();
	if (ensure(DepthRenderTexture) == false)
	{
		return false;
	}
	FTextureRenderTargetResource* RenderTargetResource = DepthRenderTexture->GameThread_GetRenderTargetResource();

	FEngineShowFlags ShowFlags(ESFIM_Game);
	Internal::SetCommonShowFlags(ShowFlags, false); // Never use AntiAliasing so we dont blend pixels at different depths

	FSceneViewFamilyContext ViewFamily(
		FSceneViewFamily::ConstructionValues(
			RenderTargetResource,
			World->Scene,
			ShowFlags)
		.SetTime(FGameTime())
		.SetRealtimeUpdate(false)
		.SetResolveScene(false)
	);

	// Request a scene depth render
	ViewFamily.SceneCaptureSource = ESceneCaptureSource::SCS_DeviceDepth;

	FSceneViewInitOptions ViewInitOptions = Internal::MakeSceneViewInitOptions(
		Frame,
		Dimensions,
		HorzFOVDegrees,
		NearPlaneDist,
		VisiblePrimitives);
	ViewInitOptions.ViewFamily = &ViewFamily;
	ViewInitOptions.OverrideFarClippingPlaneDistance = -1.f;
	if (ViewFamily.Scene->GetWorld() != nullptr && ViewFamily.Scene->GetWorld()->GetWorldSettings() != nullptr)
	{
		ViewInitOptions.WorldToMetersScale = ViewFamily.Scene->GetWorld()->GetWorldSettings()->WorldToMeters;
	}

	FSceneView* NewView = new FSceneView(ViewInitOptions);
	ViewFamily.Views.Add(NewView);

	NewView->AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
	NewView->SetupAntiAliasingMethod();

	NewView->StartFinalPostprocessSettings(ViewInitOptions.ViewOrigin);
	NewView->EndFinalPostprocessSettings(ViewInitOptions);
	ViewFamily.SetScreenPercentageInterface(new FLegacyScreenPercentageDriver(ViewFamily, 1.f));
	
	ViewFamily.ViewExtensions = GEngine->ViewExtensions->GatherActiveExtensions(FSceneViewExtensionContext(World->Scene));
	for (const FSceneViewExtensionRef& Extension : ViewFamily.ViewExtensions)
	{
		Extension->SetupViewFamily(ViewFamily);
		Extension->SetupView(ViewFamily, *NewView);
	}

	FCanvas Canvas(RenderTargetResource, nullptr, FGameTime(), NewView->GetFeatureLevel());
	Canvas.Clear(FLinearColor::Transparent);

	// Unlike other capture types, we don't need to cache any shaders before we capture the device depth render
	UE::Internal::PerformSceneRender(&Canvas, &ViewFamily);

	// Cache the view/projection matricies we used to render the scene
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LastCaptureViewMatrices = NewView->ViewMatrices;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Copy the contents of the remote texture to system memory
	ReadImageBuffer.Reset();
	ReadImageBuffer.SetNumUninitialized(Dimensions.Num());
	FReadSurfaceDataFlags ReadSurfaceDataFlags(RCM_MinMax);
	ReadSurfaceDataFlags.SetLinearToGamma(false);
	RenderTargetResource->ReadLinearColorPixels(ReadImageBuffer, ReadSurfaceDataFlags, FIntRect(0, 0, Dimensions.GetWidth(), Dimensions.GetHeight()));

	FImageView CaptureData(ReadImageBuffer.GetData(), Dimensions.GetWidth(), Dimensions.GetHeight());
	UE::Internal::CopyCaptureDataToOutputImageFormat(CaptureData, ResultImageOut, [](const FLinearColor& Color)
	{
		// Reverse the float encoding used for the Depth value in SceneCapturePixelShader.usf (one minus the device
		// z is encoded in the RGB components). See DecodeFloatRGB in WaterInfoMerge.usf and, for a reference on the
		// method, see https://aras-p.info/blog/2009/07/30/encoding-floats-to-rgba-the-final/
		FVector3f EncodedDepth(Color.R, Color.G, Color.B);
		float Depth = FVector3f::DotProduct(EncodedDepth, FVector3f(1.0, 1 / 255.0f, 1 / 65025.0f));

		// Reverse the expression used to compute the Depth to recover the DeviceZ aka normalized device coordinate z
		float DeviceZ = -(Depth - 1);
		ensure(DeviceZ >= 0. && DeviceZ <= 1.); // Points on the near plane have Z=1, points on the far plane have Z=0

		return FLinearColor(DeviceZ, 0., 0., 0.);
	});

	if (bWriteDebugImage)
	{
		// It is also helpful to save a world point cloud .obj file. See FSceneCapturePhotoSet::GetSceneSamples which uses
		// GetRenderCaptureViewMatrices to turn the depth texture into world positions
		WriteDebugImage(ResultImageOut, TEXT("DeviceDepth"));
	}

	return true;
}



bool FWorldRenderCapture::CaptureBufferVisualizationFromPosition(
	const FName& VisualizationMode,
	const FFrame3d& Frame,
	double HorzFOVDegrees,
	double NearPlaneDist,
	FImageAdapter& ResultImageOut,
	const FRenderCaptureConfig& Config)
{
	if (!GetBufferVisualizationData().IsInitialized())
	{
		return false;
	}

	UMaterialInterface* VisMaterial = GetBufferVisualizationData().GetMaterial(VisualizationMode);
	if (!ensure(VisMaterial))
	{
		return false;
	}

	const bool bEnableLighting = (VisualizationMode == Internal::GetBufferVisualizationModeName(ERenderCaptureType::Emissive));
	return CaptureUsingPostProcessMaterialPipe(VisMaterial, Frame, HorzFOVDegrees, NearPlaneDist, ResultImageOut, Config, bEnableLighting, *VisualizationMode.ToString());
}


// This is useful to debug the GPU state of the render captures in RenderDoc. Usage: set `r.RenderCaptureDraw 1` in the
// and open the capture in RenderDoc (which may have opened automatically)
static int32 RenderCaptureDraws = 0;
static FAutoConsoleVariableRef CVarRenderCaptureDraws(
	TEXT("r.RenderCaptureDraws"),
	RenderCaptureDraws,
	TEXT("Enable capturing of render capture texture for the next N draws"));

bool FWorldRenderCapture::CaptureFromPosition(
	ERenderCaptureType CaptureType,
	const FFrame3d& ViewFrame,
	double HorzFOVDegrees,
	double NearPlaneDist,
	FImageAdapter& ResultImageOut,
	const FRenderCaptureConfig& Config)
{
	RenderCaptureInterface::FScopedCapture RenderCapture(RenderCaptureDraws > 0, TEXT("RenderCaptureFromPosition"));
	RenderCaptureDraws--;

	if (CaptureType == ERenderCaptureType::DeviceDepth)
	{
		return CaptureDeviceDepthFromPosition(ViewFrame, HorzFOVDegrees, NearPlaneDist, ResultImageOut, Config);
	}

	UMaterialInterface* Material = nullptr;
	FString DebugName;

	if (CaptureType == ERenderCaptureType::CombinedMRS)
	{
		TSoftObjectPtr<UMaterialInterface> PostProcessMaterialPtr = TSoftObjectPtr<UMaterialInterface>(FSoftObjectPath(MRSPostProcessMaterialAssetPath));
		Material = PostProcessMaterialPtr.LoadSynchronous();
		DebugName = TEXT("CombinedMRS");
	}
	else
	{
		const FName BufferVisualizationMode = UE::Internal::GetBufferVisualizationModeName(CaptureType);
		if (GetBufferVisualizationData().IsInitialized())
		{
			Material = GetBufferVisualizationData().GetMaterial(BufferVisualizationMode);
		}
		DebugName = BufferVisualizationMode.ToString();
	}

	if (!ensure(Material))
	{
		return false;
	}

	const bool bEnableLighting = (CaptureType == ERenderCaptureType::Emissive);
	return CaptureUsingPostProcessMaterialPipe(Material, ViewFrame, HorzFOVDegrees, NearPlaneDist, ResultImageOut, Config, bEnableLighting, *DebugName);
}


void FWorldRenderCapture::SetEnableWriteDebugImage(bool bEnable, int32 ImageCounter, FString FolderName)
{
	bWriteDebugImage = bEnable;
	if (ImageCounter > 0)
	{
		DebugImageCounter = ImageCounter;
	}
	if (FolderName.Len() > 0)
	{
		DebugImageFolderName = FolderName;
	}
}

void FWorldRenderCapture::WriteDebugImage(const FImageAdapter& ResultImageOut, const FString& ImageTypeName)
{
	static int32 CaptureIndex = 0;
	int32 UseCounter = (DebugImageCounter >= 0) ? DebugImageCounter : CaptureIndex++;
	UE::AssetUtils::SaveDebugImage(ResultImageOut, false, DebugImageFolderName, *ImageTypeName, UseCounter);
}


FViewMatrices UE::Geometry::GetRenderCaptureViewMatrices(const FFrame3d& ViewFrame, double HorzFOVDegrees, double NearPlaneDist, FImageDimensions ViewRect)
{
	const FSceneViewInitOptions ViewInitOptions = UE::Internal::MakeSceneViewInitOptions(
		ViewFrame,
		ViewRect,
		HorzFOVDegrees,
		NearPlaneDist,
		{});

	return FViewMatrices(ViewInitOptions);
}

void FWorldRenderCapture::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(ComponentsRestoreMaxDrawDistance);
	Collector.AddReferencedObject(LinearRenderTexture);
	Collector.AddReferencedObject(DepthRenderTexture);
}

FString FWorldRenderCapture::GetReferencerName() const
{
	return TEXT("FWorldRenderCapture");
}
