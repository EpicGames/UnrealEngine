// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/BoundaryEdgeSpanSelector.h"
#include "MeshBoundaryLoops.h"
#include "ToolDataVisualizer.h"
#include "MeshBoundaryLoops.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FBoundaryEdgeSpanSelector"

namespace EdgeSpanSelectorHelpers
{

	class FEdgeSpanTopologyProvider : public FTopologyProvider
	{
	public:

		FEdgeSpanTopologyProvider(const FMeshBoundaryLoops* BoundaryLoops);

		virtual ~FEdgeSpanTopologyProvider() = default;

		// FTopologyProvider
		virtual int GetNumEdges() const override;
		virtual void GetEdgePolyline(int EdgeID, FPolyline3d& OutPolyline) const override;
		virtual int FindGroupEdgeID(int MeshEdgeID) const override;
		virtual const TArray<int>& GetGroupEdgeEdges(int GroupEdgeID) const override;
		virtual const TArray<int>& GetGroupEdgeVertices(int GroupEdgeID) const override;
		virtual void ForGroupSetEdges(const TSet<int32>& GroupIDs, const TFunction<void(int EdgeID)>& EdgeFunc) const override;
		virtual FAxisAlignedBox3d GetSelectionBounds(const FGroupTopologySelection& Selection, TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const override;
		virtual FFrame3d GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame) const override;

		virtual int GetNumCorners() const override;
		virtual int GetCornerVertexID(int CornerID) const override;
		virtual int GetNumGroups() const override;
		virtual int GetGroupIDAt(int GroupIndex) const override;
		virtual int GetGroupIDForTriangle(int TriangleID) const override;

	private:

		bool GetSpanTangent(int SpanID, FVector3d& TangentOut) const;
		FVector3d GetSpanMidpoint(int32 SpanID, double* ArcLengthOut = nullptr, TArray<double>* PerVertexLengthsOut = nullptr) const;
		double GetSpanArcLength(int32 SpanID, TArray<double>* PerVertexLengthsOut = nullptr) const;

		const FMeshBoundaryLoops* BoundaryLoops;

		TArray<int32> CornerVertexIndices;
	};

	FEdgeSpanTopologyProvider::FEdgeSpanTopologyProvider(const FMeshBoundaryLoops* BoundaryLoops)
		: BoundaryLoops(BoundaryLoops)
	{
		// Build corners, don't duplicate vertices for connected spans
		TSet<int32> Corners;
		for (const FEdgeSpan& Span : BoundaryLoops->Spans) 
		{ 
			Corners.Add(Span.Vertices[0]); 
			Corners.Add(Span.Vertices.Last()); 
		}
		CornerVertexIndices = Corners.Array();
	}


	int FEdgeSpanTopologyProvider::GetNumEdges() const
	{
		return BoundaryLoops->Spans.Num();
	}

	void FEdgeSpanTopologyProvider::GetEdgePolyline(int EdgeID, FPolyline3d& OutPolyline) const
	{
		OutPolyline.Clear();
		const TArray<int>& Vertices = BoundaryLoops->Spans[EdgeID].Vertices;
		for (int i = 0; i < Vertices.Num(); ++i)
		{
			OutPolyline.AppendVertex(BoundaryLoops->Mesh->GetVertex(Vertices[i]));
		}
	}

	int FEdgeSpanTopologyProvider::FindGroupEdgeID(int MeshEdgeID) const
	{
		return BoundaryLoops->FindSpanContainingEdge(MeshEdgeID);
	}

	const TArray<int>& FEdgeSpanTopologyProvider::GetGroupEdgeEdges(int GroupEdgeID) const
	{
		return BoundaryLoops->Spans[GroupEdgeID].Edges;
	}

	const TArray<int>& FEdgeSpanTopologyProvider::GetGroupEdgeVertices(int GroupEdgeID) const
	{
		return BoundaryLoops->Spans[GroupEdgeID].Vertices;
	}

	void FEdgeSpanTopologyProvider::ForGroupSetEdges(const TSet<int32>& GroupIDs, const TFunction<void(int EdgeID)>& EdgeFunc) const
	{
		// We should only have at most one "group" for MeshBoundaryLoops
		check(GroupIDs.Num() < 2);
		for (int32 GroupID : GroupIDs)
		{
			for (const FEdgeSpan& Span : BoundaryLoops->Spans)
			{
				for (const int32 EdgeID : Span.Edges)
				{
					EdgeFunc(EdgeID);
				}
			}
		}
	}

	int FEdgeSpanTopologyProvider::GetNumCorners() const
	{
		return CornerVertexIndices.Num();
	}

	int FEdgeSpanTopologyProvider::GetCornerVertexID(int CornerID) const
	{
		return CornerVertexIndices[CornerID];
	}

	int FEdgeSpanTopologyProvider::GetNumGroups() const
	{
		return 1;
	}

	int FEdgeSpanTopologyProvider::GetGroupIDAt(int GroupIndex) const
	{
		return 0;
	}

	int FEdgeSpanTopologyProvider::GetGroupIDForTriangle(int TriangleID) const
	{
		return 0;
	}


	bool FEdgeSpanTopologyProvider::GetSpanTangent(int SpanID, FVector3d& TangentOut) const
	{
		check(SpanID >= 0 && SpanID < BoundaryLoops->Spans.Num());

		const FEdgeSpan& Span = BoundaryLoops->Spans[SpanID];
		const FVector3d StartPos = BoundaryLoops->Mesh->GetVertex(Span.Vertices[0]);
		const FVector3d EndPos = BoundaryLoops->Mesh->GetVertex(Span.Vertices.Last());

		TangentOut = EndPos - StartPos;
		const bool bNormalized = TangentOut.Normalize(100 * FMathd::ZeroTolerance);

		if (bNormalized)
		{
			return true;
		}
		else
		{
			TangentOut = FVector3d::UnitX();
			return false;
		}
	}

	double FEdgeSpanTopologyProvider::GetSpanArcLength(int32 SpanID, TArray<double>* PerVertexLengthsOut) const
	{
		check(BoundaryLoops);
		check(SpanID >= 0 && SpanID < BoundaryLoops->Spans.Num());
		const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
		check(Mesh);

		const TArray<int32>& Vertices = GetGroupEdgeVertices(SpanID);
		const int32 NumV = Vertices.Num();
		if (PerVertexLengthsOut != nullptr)
		{
			PerVertexLengthsOut->SetNum(NumV);
			(*PerVertexLengthsOut)[0] = 0.0;
		}
		double AccumLength = 0;
		for (int32 VertexIndex = 1; VertexIndex < NumV; ++VertexIndex)
		{
			AccumLength += Distance(Mesh->GetVertex(Vertices[VertexIndex]), Mesh->GetVertex(Vertices[VertexIndex - 1]));
			if (PerVertexLengthsOut != nullptr)
			{
				(*PerVertexLengthsOut)[VertexIndex] = AccumLength;
			}
		}
		return AccumLength;
	}


	FVector3d FEdgeSpanTopologyProvider::GetSpanMidpoint(int32 SpanID, double* ArcLengthOut, TArray<double>* PerVertexLengthsOut) const
	{
		check(BoundaryLoops);
		check(SpanID >= 0 && SpanID < BoundaryLoops->Spans.Num());
		const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
		check(Mesh);

		const TArray<int32>& Vertices = GetGroupEdgeVertices(SpanID);
		const int32 NumV = Vertices.Num();

		// trivial case
		if (NumV == 2)
		{
			const FVector3d A(Mesh->GetVertex(Vertices[0]));
			const FVector3d B(Mesh->GetVertex(Vertices[1]));
			if (ArcLengthOut)
			{
				*ArcLengthOut = Distance(A, B);
			}
			if (PerVertexLengthsOut)
			{
				(*PerVertexLengthsOut).SetNum(2);
				(*PerVertexLengthsOut)[0] = 0;
				(*PerVertexLengthsOut)[1] = Distance(A, B);
			}
			return (A + B) * 0.5;
		}

		// if we want lengths anyway we can avoid second loop
		if (PerVertexLengthsOut)
		{
			const double TotalArcLength = GetSpanArcLength(SpanID, PerVertexLengthsOut);
			if (ArcLengthOut)
			{
				*ArcLengthOut = TotalArcLength;
			}

			int32 VertexIndex = 0;
			while ((*PerVertexLengthsOut)[VertexIndex] < TotalArcLength)
			{
				VertexIndex++;
			}
			const int32 PreviousVertexIndex = VertexIndex - 1;
			const double ArcLengthA = (*PerVertexLengthsOut)[PreviousVertexIndex];
			const double ArcLengthB = (*PerVertexLengthsOut)[VertexIndex];
			double InterpT = (0.5 * TotalArcLength - ArcLengthA) / (ArcLengthB - ArcLengthA);
			const FVector3d VertexA(Mesh->GetVertex(Vertices[PreviousVertexIndex]));
			const FVector3d VertexB(Mesh->GetVertex(Vertices[VertexIndex]));
			return Lerp(VertexA, VertexB, InterpT);
		}

		// compute arclen and then walk forward until we get halfway
		const double TotalArcLength = GetSpanArcLength(SpanID);
		if (ArcLengthOut)
		{
			*ArcLengthOut = TotalArcLength;
		}

		double AccumLength = 0;
		for (int32 VertexIndex = 1; VertexIndex < NumV; ++VertexIndex)
		{
			double NewLen = AccumLength + Distance(Mesh->GetVertex(Vertices[VertexIndex]), Mesh->GetVertex(Vertices[VertexIndex - 1]));
			if (NewLen > 0.5 * TotalArcLength)
			{
				const double InterpT = (0.5 * TotalArcLength - AccumLength) / (NewLen - AccumLength);
				const FVector3d VertexA(Mesh->GetVertex(Vertices[VertexIndex - 1]));
				const FVector3d VertexB(Mesh->GetVertex(Vertices[VertexIndex]));
				return Lerp(VertexA, VertexB, InterpT);
			}
			AccumLength = NewLen;
		}

		// somehow failed?
		return (Mesh->GetVertex(Vertices[0]) + Mesh->GetVertex(Vertices[NumV - 1])) * 0.5;
	}



	FFrame3d FEdgeSpanTopologyProvider::GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame) const
	{
		check(BoundaryLoops);
		const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
		check(Mesh);


		int32 NumSpans = Selection.SelectedEdgeIDs.Num();
		FFrame3d StartFrame = (InitialLocalFrame) ? (*InitialLocalFrame) : FFrame3d();

		if (NumSpans == 1)
		{
			const int32 SpanID = Selection.GetASelectedEdgeID();
			const int32 MeshEdgeID = BoundaryLoops->Spans[SpanID].Edges[0];

			// align Z axis of frame to face normal of one of the connected faces. 
			const FIndex2i EdgeTris = Mesh->GetEdgeT(MeshEdgeID);
			const int32 UseFace = (EdgeTris.B != IndexConstants::InvalidID) ? FMath::Min(EdgeTris.A, EdgeTris.B) : EdgeTris.A;
			const FVector3d FaceNormal = Mesh->GetTriNormal(UseFace);
			if (FaceNormal.Length() > 0.1)
			{
				StartFrame.AlignAxis(2, FaceNormal);
			}

			// align X axis along the edge, around the aligned Z axis
			FVector3d Tangent;
			if (GetSpanTangent(SpanID, Tangent))
			{
				StartFrame.ConstrainedAlignAxis(0, Tangent, StartFrame.Z());
			}

			StartFrame.Origin = GetSpanMidpoint(SpanID);
			return StartFrame;
		}

		// If we have multiple spans, just align the frame with the world axes

		const int NumSelectedSpans = Selection.SelectedEdgeIDs.Num();

		FVector3d AccumulatedOrigin = FVector3d::Zero();
		for (int32 SpanID : Selection.SelectedEdgeIDs)
		{
			const FEdgeSpan& Span = BoundaryLoops->Spans[SpanID];
			const FVector3d StartPos = Mesh->GetVertex(Span.Vertices[0]);
			const FVector3d EndPos = Mesh->GetVertex(Span.Vertices[Span.Vertices.Num() - 1]);
			AccumulatedOrigin += 0.5 * (StartPos + EndPos);
		}

		check(NumSelectedSpans > 1);

		FFrame3d AccumulatedFrame;
		AccumulatedOrigin /= (double)NumSelectedSpans;
		AccumulatedFrame = FFrame3d(AccumulatedOrigin, FQuaterniond::Identity());

		return AccumulatedFrame;
	}

	FAxisAlignedBox3d FEdgeSpanTopologyProvider::GetSelectionBounds(
		const FGroupTopologySelection& Selection,
		TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const
	{
		check(BoundaryLoops);
		const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
		check(Mesh);

		if (ensure(!Selection.IsEmpty()) == false)
		{
			return Mesh->GetBounds();
		}

		FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
		for (int32 EdgeID : Selection.SelectedEdgeIDs)
		{
			const FEdgeSpan& Span = BoundaryLoops->Spans[EdgeID];
			for (int32 SpanVertexID : Span.Vertices)
			{
				Bounds.Contain(TransformFunc(Mesh->GetVertex(SpanVertexID)));
			}
		}

		return Bounds;
	}
}


FBoundaryEdgeSpanSelector::FBoundaryEdgeSpanSelector(const FDynamicMesh3* MeshIn, const FMeshBoundaryLoops* BoundaryLoopsIn) :
	FMeshTopologySelector()
{
	Mesh = MeshIn;
	TopologyProvider = MakeUnique<EdgeSpanSelectorHelpers::FEdgeSpanTopologyProvider>(BoundaryLoopsIn);
	bGeometryInitialized = false;
	bGeometryUpToDate = false;
}

void FBoundaryEdgeSpanSelector::DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle)
{
	const FLinearColor UseColor = Renderer->LineColor;
	const float LineWidth = Renderer->LineThickness;

	for (const int EdgeID : Selection.SelectedEdgeIDs)
	{
		const TArray<int>& Vertices = TopologyProvider->GetGroupEdgeVertices(EdgeID);
		const int NV = Vertices.Num() - 1;

		// Draw the edge, but also draw the endpoints in ortho mode (to make projected edges visible)
		FVector VertexA = (FVector)Mesh->GetVertex(Vertices[0]);
		if (CameraState->bIsOrthographic)
		{
			Renderer->DrawPoint(VertexA, UseColor, 10, false);
		}

		for (int VertexIndex = 0; VertexIndex < NV; ++VertexIndex)
		{
			const FVector VertexB = (FVector)Mesh->GetVertex(Vertices[VertexIndex + 1]);
			Renderer->DrawLine(VertexA, VertexB, UseColor, LineWidth, false);
			VertexA = VertexB;
		}

		if (CameraState->bIsOrthographic)
		{
			Renderer->DrawPoint(VertexA, UseColor, LineWidth, false);
		}
	}
}


#undef LOCTEXT_NAMESPACE
