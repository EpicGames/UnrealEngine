// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/BoundarySelectionMechanic.h"
#include "MeshBoundaryLoops.h"
#include "Selection/BoundarySelector.h"
#include "Selection/BoundaryEdgeSpanSelector.h"
#include "ToolSceneQueriesUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoundarySelectionMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UBoundarySelectionMechanic"

void UBoundarySelectionMechanic::Initialize(
	const FDynamicMesh3* MeshIn,
	FTransform3d TargetTransformIn,
	UWorld* WorldIn,
	const UE::Geometry::FMeshBoundaryLoops* BoundaryLoopsIn,
	TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFuncIn,
	EBoundarySelectionType SelectionType)
{
	if (SelectionType == EBoundarySelectionType::Loops)
	{
		TopoSelector = MakeShared<FBoundarySelector>(MeshIn, BoundaryLoopsIn);
	}
	else
	{
		TopoSelector = MakeShared<FBoundaryEdgeSpanSelector>(MeshIn, BoundaryLoopsIn);
	}
	
	UMeshTopologySelectionMechanic::Initialize(MeshIn, TargetTransformIn, WorldIn, GetSpatialSourceFuncIn);
}

bool UBoundarySelectionMechanic::UpdateHighlight(const FRay& WorldRay)
{
	checkf(DrawnTriangleSetComponent != nullptr, TEXT("Initialize() not called on UMeshTopologySelectionMechanic."));

	FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	HilightSelection.Clear();
	FVector3d LocalPosition, LocalNormal;
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(CameraState.bIsOrthographic);
	bool bHit = TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, HilightSelection, LocalPosition, LocalNormal);

	// Don't hover highlight a selection that we already selected, because people didn't like that
	if (PersistentSelection.Contains(HilightSelection))
	{
		HilightSelection.Clear();
	}

	return bHit;
}


bool UBoundarySelectionMechanic::UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut)
{
	FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	const FGroupTopologySelection PreviousSelection = PersistentSelection;

	FVector3d LocalPosition, LocalNormal;
	FGroupTopologySelection Selection;
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(CameraState.bIsOrthographic);
	if (TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, Selection, LocalPosition, LocalNormal))
	{
		LocalHitPositionOut = LocalPosition;
		LocalHitNormalOut = LocalNormal;
	}

	if (ShouldAddToSelectionFunc())
	{
		if (ShouldRemoveFromSelectionFunc())
		{
			PersistentSelection.Toggle(Selection);
		}
		else
		{
			PersistentSelection.Append(Selection);
		}
	}
	else if (ShouldRemoveFromSelectionFunc())
	{
		PersistentSelection.Remove(Selection);
	}
	else
	{
		PersistentSelection = Selection;
	}

	if (PersistentSelection != PreviousSelection)
	{
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
		return true;
	}

	return false;
}


#undef LOCTEXT_NAMESPACE
