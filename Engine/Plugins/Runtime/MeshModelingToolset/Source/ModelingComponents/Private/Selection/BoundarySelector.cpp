// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/BoundarySelector.h"
#include "MeshBoundaryLoops.h"
#include "ToolDataVisualizer.h"
#include "ToolSceneQueriesUtil.h"
#include "Mechanics/RectangleMarqueeMechanic.h" // FCameraRectangle

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FBoundarySelector"

bool FBoundaryTopologyProvider::GetLoopTangent(int LoopID, FVector3d& TangentOut) const
{
	check(LoopID >= 0 && LoopID < BoundaryLoops->Loops.Num());

	const FEdgeLoop& Loop = BoundaryLoops->Loops[LoopID];
	FVector3d StartPos = BoundaryLoops->Mesh->GetVertex(Loop.Vertices[0]);
	FVector3d EndPos = BoundaryLoops->Mesh->GetVertex(Loop.Vertices[Loop.Vertices.Num() - 1]);

	TangentOut = EndPos - StartPos;
	const bool bNormalized = TangentOut.Normalize(100 * FMathd::ZeroTolerance);

	if (bNormalized)
	{
		return true;
	}
	else
	{
		TangentOut = FVector3d::UnitX();
		return false;
	}
}

double FBoundaryTopologyProvider::GetLoopArcLength(int32 LoopID, TArray<double>* PerVertexLengthsOut) const
{
	check(BoundaryLoops);
	check(LoopID >= 0 && LoopID < BoundaryLoops->Loops.Num());
	const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
	check(Mesh);

	const TArray<int32>& Vertices = GetGroupEdgeVertices(LoopID);
	int32 NumV = Vertices.Num();
	if (PerVertexLengthsOut != nullptr)
	{
		PerVertexLengthsOut->SetNum(NumV);
		(*PerVertexLengthsOut)[0] = 0.0;
	}
	double AccumLength = 0;
	for (int32 k = 1; k < NumV; ++k)
	{
		AccumLength += Distance(Mesh->GetVertex(Vertices[k]), Mesh->GetVertex(Vertices[k - 1]));
		if (PerVertexLengthsOut != nullptr)
		{
			(*PerVertexLengthsOut)[k] = AccumLength;
		}
	}
	return AccumLength;
}


FVector3d FBoundaryTopologyProvider::GetLoopMidpoint(int32 LoopID, double* ArcLengthOut, TArray<double>* PerVertexLengthsOut) const
{
	check(BoundaryLoops);
	check(LoopID >= 0 && LoopID < BoundaryLoops->Loops.Num());
	const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
	check(Mesh);

	const TArray<int32>& Vertices = GetGroupEdgeVertices(LoopID);
	int32 NumV = Vertices.Num();

	// trivial case
	if (NumV == 2)
	{
		FVector3d A(Mesh->GetVertex(Vertices[0])), B(Mesh->GetVertex(Vertices[1]));
		if (ArcLengthOut)
		{
			*ArcLengthOut = Distance(A, B);
		}
		if (PerVertexLengthsOut)
		{
			(*PerVertexLengthsOut).SetNum(2);
			(*PerVertexLengthsOut)[0] = 0;
			(*PerVertexLengthsOut)[1] = Distance(A, B);
		}
		return (A + B) * 0.5;
	}

	// if we want lengths anyway we can avoid second loop
	if (PerVertexLengthsOut)
	{
		double Len = GetLoopArcLength(LoopID, PerVertexLengthsOut);
		if (ArcLengthOut)
		{
			*ArcLengthOut = Len;
		}
		Len /= 2;
		int32 k = 0;
		while ((*PerVertexLengthsOut)[k] < Len)
		{
			k++;
		}
		int32 kprev = k - 1;
		double a = (*PerVertexLengthsOut)[k - 1], b = (*PerVertexLengthsOut)[k];
		double t = (Len - a) / (b - a);
		FVector3d A(Mesh->GetVertex(Vertices[k - 1])), B(Mesh->GetVertex(Vertices[k]));
		return Lerp(A, B, t);
	}

	// compute arclen and then walk forward until we get halfway
	double Len = GetLoopArcLength(LoopID);
	if (ArcLengthOut)
	{
		*ArcLengthOut = Len;
	}
	Len /= 2;
	double AccumLength = 0;
	for (int32 k = 1; k < NumV; ++k)
	{
		double NewLen = AccumLength + Distance(Mesh->GetVertex(Vertices[k]), Mesh->GetVertex(Vertices[k - 1]));
		if (NewLen > Len)
		{
			double t = (Len - AccumLength) / (NewLen - AccumLength);
			FVector3d A(Mesh->GetVertex(Vertices[k - 1])), B(Mesh->GetVertex(Vertices[k]));
			return Lerp(A, B, t);
		}
		AccumLength = NewLen;
	}

	// somehow failed?
	return (Mesh->GetVertex(Vertices[0]) + Mesh->GetVertex(Vertices[NumV - 1])) * 0.5;
}



FFrame3d FBoundaryTopologyProvider::GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame) const
{
	check(BoundaryLoops);
	const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
	check(Mesh);


	int32 NumLoops = Selection.SelectedEdgeIDs.Num();
	FFrame3d StartFrame = (InitialLocalFrame) ? (*InitialLocalFrame) : FFrame3d();

	if (NumLoops == 1)
	{
		int32 LoopID = Selection.GetASelectedEdgeID();
		int32 MeshEdgeID = BoundaryLoops->Loops[LoopID].Edges[0];

		// align Z axis of frame to face normal of one of the connected faces. 
		FIndex2i EdgeTris = Mesh->GetEdgeT(MeshEdgeID);
		int32 UseFace = (EdgeTris.B != IndexConstants::InvalidID) ? FMath::Min(EdgeTris.A, EdgeTris.B)
			: EdgeTris.A;
		FVector3d FaceNormal = Mesh->GetTriNormal(UseFace);
		if (FaceNormal.Length() > 0.1)
		{
			StartFrame.AlignAxis(2, FaceNormal);
		}

		// align X axis along the edge, around the aligned Z axis
		FVector3d Tangent;
		if (GetLoopTangent(LoopID, Tangent))
		{
			StartFrame.ConstrainedAlignAxis(0, Tangent, StartFrame.Z());
		}

		StartFrame.Origin = GetLoopMidpoint(LoopID);
		return StartFrame;
	}

	// If we have multiple loops, just align the frame with the world axes

	const int NumSelectedLoops = Selection.SelectedEdgeIDs.Num();

	FVector3d AccumulatedOrigin = FVector3d::Zero();
	for (int32 LoopID : Selection.SelectedEdgeIDs)
	{
		const FEdgeLoop& Loop = BoundaryLoops->Loops[LoopID];
		FVector3d StartPos = Mesh->GetVertex(Loop.Vertices[0]);
		FVector3d EndPos = Mesh->GetVertex(Loop.Vertices[Loop.Vertices.Num() - 1]);
		AccumulatedOrigin += 0.5 * (StartPos + EndPos);
	}

	check(NumSelectedLoops > 1);

	FFrame3d AccumulatedFrame;
	AccumulatedOrigin /= (double)NumSelectedLoops;
	AccumulatedFrame = FFrame3d(AccumulatedOrigin, FQuaterniond::Identity());

	return AccumulatedFrame;
}

FAxisAlignedBox3d FBoundaryTopologyProvider::GetSelectionBounds(
	const FGroupTopologySelection& Selection,
	TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const
{
	check(BoundaryLoops);
	const FDynamicMesh3* Mesh = BoundaryLoops->Mesh;
	check(Mesh);

	if (ensure(!Selection.IsEmpty()) == false)
	{
		return Mesh->GetBounds();
	}

	FAxisAlignedBox3d Bounds = FAxisAlignedBox3d::Empty();
	for (int32 EdgeID : Selection.SelectedEdgeIDs)
	{
		const FEdgeLoop& Loop = BoundaryLoops->Loops[EdgeID];
		for (int32 vid : Loop.Vertices)
		{
			Bounds.Contain(TransformFunc(Mesh->GetVertex(vid)));
		}
	}

	return Bounds;
}


FBoundarySelector::FBoundarySelector(const FDynamicMesh3* MeshIn, const FMeshBoundaryLoops* BoundaryLoopsIn) :
	FMeshTopologySelector()
{
	Mesh = MeshIn;
	TopologyProvider = MakeUnique<FBoundaryTopologyProvider>(BoundaryLoopsIn);
	bGeometryInitialized = false;
	bGeometryUpToDate = false;
}

void FBoundarySelector::DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle)
{
	FLinearColor UseColor = Renderer->LineColor;
	float LineWidth = Renderer->LineThickness;

	for (int EdgeID : Selection.SelectedEdgeIDs)
	{
		const TArray<int>& Vertices = TopologyProvider->GetGroupEdgeVertices(EdgeID);
		int NV = Vertices.Num() - 1;

		// Draw the edge, but also draw the endpoints in ortho mode (to make projected edges visible)
		FVector A = (FVector)Mesh->GetVertex(Vertices[0]);
		if (CameraState->bIsOrthographic)
		{
			Renderer->DrawPoint(A, UseColor, 10, false);
		}

		for (int k = 0; k < NV; ++k)
		{
			FVector B = (FVector)Mesh->GetVertex(Vertices[k + 1]);
			Renderer->DrawLine(A, B, UseColor, LineWidth, false);
			A = B;
		}
		FVector B = (FVector)Mesh->GetVertex(Vertices[0]);
		Renderer->DrawLine(A, B, UseColor, LineWidth, false);

		if (CameraState->bIsOrthographic)
		{
			Renderer->DrawPoint(A, UseColor, LineWidth, false);
		}
	}
}


#undef LOCTEXT_NAMESPACE
