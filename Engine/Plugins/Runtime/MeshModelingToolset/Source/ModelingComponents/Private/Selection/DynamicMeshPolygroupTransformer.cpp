// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/DynamicMeshPolygroupTransformer.h"
#include "Selections/GeometrySelectionUtil.h"
#include "ToolContextInterfaces.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "Changes/MeshVertexChange.h"
#include "GroupTopology.h"
#include "Operations/GroupTopologyDeformer.h"
#include "ToolDataVisualizer.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FDynamicMeshPolygroupTransformer"

void FDynamicMeshPolygroupTransformer::BeginTransform(const UE::Geometry::FGeometrySelection& Selection)
{
	if (!ensure(Selection.TopologyType == EGeometryTopologyType::Polygroup))
	{
		bFallbackToSimpleTransform = true;
		FBasicDynamicMeshSelectionTransformer::BeginTransform(Selection);
		return;
	}

	const FGroupTopology* UseTopology = Selector->GetGroupTopology();

	LinearDeformer = MakePimpl<FGroupTopologyDeformer>();

	TSet<int32> VertexIDs;
	Selector->GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		// danger! this currently holds onto SourceMesh pointer...
		// Also this is expensive, would be nice if we could save it across multiple transforms...
		LinearDeformer->Initialize(&SourceMesh, UseTopology);

		LinearDeformer->SetActiveHandleFromSelection(Selection);

		// get set of selected vertex IDs, and then vector (only converting to vector for VertexToTriangleOneRing...)
		UE::Geometry::EnumeratePolygroupSelectionVertices(Selection, SourceMesh, UseTopology, FTransform::Identity,
			[&](uint32 VertexID, const FVector3d& Position) { VertexIDs.Add((int32)VertexID); }
		);

		NumSelectedMeshVertices = VertexIDs.Num();
		MeshVertices = VertexIDs.Array();

		int32 NumWeightedMeshVertices = NumSelectedMeshVertices + Selection.SoftSelectedVertices.Num();
		MeshVertices.Reserve(NumWeightedMeshVertices);
		SoftSelectedVertexWeights.Reserve(Selection.SoftSelectedVertices.Num());
		
		for (const TPair<uint64, double>& WeightedVertex : Selection.SoftSelectedVertices)
		{
			MeshVertices.Add(FGeoSelectionID(WeightedVertex.Key).GeometryID);
			SoftSelectedVertexWeights.Add(WeightedVertex.Value);
		}

		int32 NumVertices = MeshVertices.Num();
		InitialPositions.SetNum(NumVertices);
		for (int32 k = 0; k < NumVertices; ++k)
		{
			InitialPositions[k] = SourceMesh.GetVertex(MeshVertices[k]);
			ROIMap.Add(MeshVertices[k], k);
		}

		UpdatedPositions = InitialPositions;

		OverlayNormalsArray = LinearDeformer->GetModifiedOverlayNormals().Array();

		ActiveSelectionEdges.Reset();
		ActiveSelectionVertices.Reset();
		UE::Geometry::EnumeratePolygroupSelectionElements( Selection, SourceMesh, UseTopology,
			[&](int32 VertexID, FVector3d Position) { ActiveSelectionVertices.Add(VertexID); },
			[&](int32 EdgeID, const FSegment3d&) { ActiveSelectionEdges.Add(SourceMesh.GetEdgeV(EdgeID)); },
			[&](int32 TriangleID, const FTriangle3d&) {},
			/*ApplyTransform*/nullptr, /*bMapFacesToEdgeLoops*/true);

		ActiveROIEdges.Reset();
		LinearDeformer->EnumerateROIEdges([&](const FEdgeSpan& Span) {
			for (int32 EdgeID : Span.Edges)
			{
				ActiveROIEdges.Add(SourceMesh.GetEdgeV(EdgeID));
			}
		});
	});

	ActiveVertexChange = MakePimpl<FMeshVertexChangeBuilder>(EMeshVertexChangeComponents::VertexPositions | EMeshVertexChangeComponents::OverlayNormals);

	//UpdatePendingVertexChange(false);
	Selector->GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		ActiveVertexChange->SaveVertices(&SourceMesh, LinearDeformer->GetModifiedVertices(), true);
		ActiveVertexChange->SaveOverlayNormals(&SourceMesh, LinearDeformer->GetModifiedOverlayNormals(), true);
	});
}

void FDynamicMeshPolygroupTransformer::UpdateTransform( 
	TFunctionRef<FVector3d(int32 VertexID, double Weight,const FVector3d& InitialPosition, const FTransform& WorldTransform)> PositionTransformFunc  )
{
	if (bFallbackToSimpleTransform)
	{
		FBasicDynamicMeshSelectionTransformer::UpdateTransform(PositionTransformFunc);
		return;
	}


	int32 N = MeshVertices.Num();
	FTransform WorldTransform = Selector->GetLocalToWorldTransform();

	for (int32 k = 0; k < N; ++k)
	{
		double Weight = 1.0f;
		if (k >= NumSelectedMeshVertices)
		{
			Weight = SoftSelectedVertexWeights[(k - NumSelectedMeshVertices)];
		}
		UpdatedPositions[k] = PositionTransformFunc(MeshVertices[k], Weight, InitialPositions[k], WorldTransform);
	}

	Selector->GetDynamicMesh()->EditMesh([&](FDynamicMesh3& EditMesh)
	{

		// UEditMeshPolygonsTool::ComputeUpdate_Gizmo() calls ClearSolution if the transform was zero...

		LinearDeformer->UpdateSolution(&EditMesh, [&](FDynamicMesh3* TargetMesh, int VertIdx)
		{
			if ( int32* ArrayIndex = ROIMap.Find(VertIdx) )
			{
				double Weight = 1.0f;
				if (*ArrayIndex >= NumSelectedMeshVertices)
				{
					Weight = SoftSelectedVertexWeights[(*ArrayIndex - NumSelectedMeshVertices)];
				}
				return PositionTransformFunc( VertIdx, Weight, InitialPositions[*ArrayIndex], WorldTransform );
			}
			else
			{
				return TargetMesh->GetVertex(VertIdx);
			}
		});

		FMeshNormals::RecomputeOverlayElementNormals(EditMesh, OverlayNormalsArray);

	}, EDynamicMeshChangeType::DeformationEdit,
	   EDynamicMeshAttributeChangeFlags::VertexPositions | EDynamicMeshAttributeChangeFlags::NormalsTangents, false);
}


void FDynamicMeshPolygroupTransformer::EndTransform(IToolsContextTransactionsAPI* TransactionsAPI)
{
	if (bFallbackToSimpleTransform)
	{
		FBasicDynamicMeshSelectionTransformer::EndTransform(TransactionsAPI);
		return;
	}

	//UpdatePendingVertexChange(true);
	Selector->GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		ActiveVertexChange->SaveVertices(&SourceMesh, LinearDeformer->GetModifiedVertices(), false);
		ActiveVertexChange->SaveOverlayNormals(&SourceMesh, LinearDeformer->GetModifiedOverlayNormals(), false);
	});

	if (TransactionsAPI != nullptr)
	{
		TUniquePtr<FToolCommandChange> Result = MoveTemp(ActiveVertexChange->Change);
		TransactionsAPI->AppendChange(Selector->GetDynamicMesh(), MoveTemp(Result), LOCTEXT("DynamicMeshTransformChange", "Transform"));
	}

	ActiveVertexChange.Reset();

	if (OnEndTransformFunc)
	{
		OnEndTransformFunc(TransactionsAPI);
	}
}



void FDynamicMeshPolygroupTransformer::PreviewRender(IToolsContextRenderAPI* RenderAPI)
{
	if (bFallbackToSimpleTransform)
	{
		FBasicDynamicMeshSelectionTransformer::PreviewRender(RenderAPI);
		return;
	}
	if (!bEnableSelectionTransformDrawing) return;
	if (!LinearDeformer.IsValid()) return;

	FToolDataVisualizer Visualizer;
	Visualizer.bDepthTested = false;
	Visualizer.BeginFrame(RenderAPI);

	Visualizer.PushTransform(Selector->GetLocalToWorldTransform());

	const FDynamicMesh3* Mesh = LinearDeformer->GetMesh();

	Visualizer.SetPointParameters(FLinearColor(0, 0.3f, 0.95f, 1), 5.0f);
	for (int32 VertexID : ActiveSelectionVertices)
	{
		Visualizer.DrawPoint(Mesh->GetVertex(VertexID));
	}

	Visualizer.SetLineParameters(FLinearColor(0, 0.3f, 0.95f, 1), 3.0f);
	for (FIndex2i EdgeV : ActiveSelectionEdges)
	{
		Visualizer.DrawLine(Mesh->GetVertex(EdgeV.A), Mesh->GetVertex(EdgeV.B));
	}
	Visualizer.SetLineParameters(FLinearColor(0, 0.3f, 0.95f, 1), 1.0f);
	for (FIndex2i EdgeV : ActiveROIEdges)
	{
		Visualizer.DrawLine(Mesh->GetVertex(EdgeV.A), Mesh->GetVertex(EdgeV.B));
	}

	Visualizer.EndFrame();
}




#undef LOCTEXT_NAMESPACE 