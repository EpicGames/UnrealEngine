// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/DynamicMeshSelector.h"
#include "Selections/GeometrySelectionUtil.h"
#include "ToolContextInterfaces.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "DynamicMesh/ColliderMesh.h"
#include "GroupTopology.h"
#include "Spatial/SegmentTree3.h"
#include "ToolSceneQueriesUtil.h"
#include "Selection/DynamicMeshPolygroupTransformer.h"
#include "Selection/ToolSelectionUtil.h"
#include "ToolDataVisualizer.h"
#include "Parameterization/MeshDijkstra.h"
#include "Spatial/SparseDynamicPointOctree3.h"
#include "Async/ParallelFor.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FBaseDynamicMeshSelector"


FBaseDynamicMeshSelector::~FBaseDynamicMeshSelector()
{
	if (TargetMesh.IsValid())
	{
		ensureMsgf(false, TEXT("FBaseDynamicMeshSelector was not properly Shutdown!"));
		FBaseDynamicMeshSelector::Shutdown();
	}
}


void FBaseDynamicMeshSelector::Initialize(
	FGeometryIdentifier SourceGeometryIdentifierIn,
	UDynamicMesh* TargetMeshIn, 
	TUniqueFunction<UE::Geometry::FTransformSRT3d()> GetWorldTransformFuncIn)
{
	check(TargetMeshIn != nullptr);

	SourceGeometryIdentifier = SourceGeometryIdentifierIn;
	TargetMesh = TargetMeshIn;
	GetWorldTransformFunc = MoveTemp(GetWorldTransformFuncIn);

	RegisterMeshChangedHandler();
}

void FBaseDynamicMeshSelector::RegisterMeshChangedHandler()
{
	TargetMesh_OnMeshChangedHandle = TargetMesh->OnMeshChanged().AddLambda([this](UDynamicMesh* Mesh, FDynamicMeshChangeInfo ChangeInfo)
	{
		InvalidateOnMeshChange(ChangeInfo);
	});
}

void FBaseDynamicMeshSelector::InvalidateOnMeshChange(FDynamicMeshChangeInfo ChangeInfo)
{
	ColliderMesh.Reset();
	GroupEdgeSegmentTree.Reset();
	PathFinder.Reset();
	CachedVolumeDistanceToSelection.Reset();

	const bool bTopologyMayHaveChanged =
		   ChangeInfo.Type != EDynamicMeshChangeType::MeshVertexChange
		&& ChangeInfo.Type != EDynamicMeshChangeType::DeformationEdit;
	if (bTopologyMayHaveChanged)
	{
		GroupTopology.Reset();
	}

	// publish geometry-modified event
	NotifyGeometryModified();
}

bool FBaseDynamicMeshSelector::SupportsSleep() const
{
	// can only sleep if target mesh is valid
	return TargetMesh.IsValid();
}

void FBaseDynamicMeshSelector::Shutdown()
{
	if (TargetMesh.IsValid())
	{
		TargetMesh->OnMeshChanged().Remove(TargetMesh_OnMeshChangedHandle);
		TargetMesh_OnMeshChangedHandle.Reset();
		
		ColliderMesh.Reset();
		GroupTopology.Reset();
		GroupEdgeSegmentTree.Reset();
		PathFinder.Reset();
		CachedVolumeDistanceToSelection.Reset();
	}
	TargetMesh = nullptr;
}

bool FBaseDynamicMeshSelector::Sleep()
{
	if (TargetMesh.IsValid() == false)
	{
		return false;
	}

	TargetMesh->OnMeshChanged().Remove(TargetMesh_OnMeshChangedHandle);
	TargetMesh_OnMeshChangedHandle.Reset();

	SleepingTargetMesh = TargetMesh;
	TargetMesh = nullptr;

	ColliderMesh.Reset();
	GroupTopology.Reset();
	GroupEdgeSegmentTree.Reset();
	PathFinder.Reset();
	CachedVolumeDistanceToSelection.Reset();

	return true;
}

bool FBaseDynamicMeshSelector::Restore()
{
	if (SleepingTargetMesh.IsValid() == false)
	{
		return false;
	}

	TargetMesh = SleepingTargetMesh.Get();
	SleepingTargetMesh = nullptr;

	RegisterMeshChangedHandler();

	return true;
}


void FBaseDynamicMeshSelector::InitializeSelectionFromPredicate(
	FGeometrySelection& SelectionOut,
	TFunctionRef<bool(UE::Geometry::FGeoSelectionID)> SelectionIDPredicate,
	EInitializeSelectionMode InitializeMode,
	const FGeometrySelection* ReferenceSelection)
{
	if (IsLockable() && IsLocked()) return;

	const FGroupTopology* UseGroupTopology = ( SelectionOut.TopologyType == EGeometryTopologyType::Polygroup ) ? 
		GetGroupTopology() : nullptr;

	if (InitializeMode != EInitializeSelectionMode::All && ! ensure(ReferenceSelection != nullptr) )
	{
		return;
	}

	GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
	{ 
		if (InitializeMode == EInitializeSelectionMode::Connected)
		{
			UE::Geometry::MakeSelectAllConnectedSelection(Mesh, UseGroupTopology, *ReferenceSelection,
				SelectionIDPredicate, 
				[](FGeoSelectionID, FGeoSelectionID) { return true; },
				SelectionOut);
		}
		else if (InitializeMode == EInitializeSelectionMode::AdjacentToBorder)
		{
			UE::Geometry::FGeometrySelectionEditor TmpReadOnlyEditor;
			TmpReadOnlyEditor.Initialize( const_cast<FGeometrySelection*>(ReferenceSelection), 
				(ReferenceSelection->TopologyType == EGeometryTopologyType::Polygroup) );
			UE::Geometry::MakeBoundaryConnectedSelection(Mesh, UseGroupTopology, *ReferenceSelection,
				SelectionIDPredicate, SelectionOut);			
		}
		else   // All
		{
			UE::Geometry::MakeSelectAllSelection(Mesh, UseGroupTopology, SelectionIDPredicate, SelectionOut);
		}
	});
}



void FBaseDynamicMeshSelector::UpdateSelectionFromSelection(
	const FGeometrySelection& FromSelection,
	bool bAllowConversion,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionDelta* SelectionDelta )
{
	if (IsLockable() && IsLocked()) return;

	if ( FromSelection.IsSameType(SelectionEditor.GetSelection()) )
	{
		UE::Geometry::UpdateSelectionWithNewElements(&SelectionEditor, UpdateConfig.ChangeType,
			FromSelection.Selection.Array(), SelectionDelta);
		return;
	}

	if (bAllowConversion == false)
	{
		return;
	}

	bool bConverted = false;
	GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
	{ 
		FGeometrySelection TempSelection;
		TempSelection.InitializeTypes(SelectionEditor.GetElementType(), SelectionEditor.GetTopologyType());
		bConverted = UE::Geometry::ConvertSelection(Mesh, GetGroupTopology(), FromSelection, TempSelection, EEnumerateSelectionConversionParams::ContainSelection);		// todo do not always need group topology here...
		if ( bConverted )
		{
			UE::Geometry::UpdateSelectionWithNewElements(&SelectionEditor, UpdateConfig.ChangeType,
				TempSelection.Selection.Array(), SelectionDelta);
		}
	});
	if (bConverted)
	{
		return;
	}

	// todo: support additional conversion types here
	//ensure(false);
}



bool FBaseDynamicMeshSelector::RayHitTest(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionHitQueryConfig QueryConfig,
	FInputRayHit& HitResultOut )
{
	HitResultOut = FInputRayHit();
	if (IsLockable() && IsLocked()) return false;

	// todo: do we need to sometimes ignore surface hits here? maybe for Volumes in (eg) edge mode...

	IMeshSpatial::FQueryOptions SpatialQueryOptions;
	if (!QueryConfig.bHitBackFaces)
	{
		SpatialQueryOptions.TriangleFilterF = [&](int32 Tid) {
			return GetColliderMesh()->GetTriNormal(Tid).Dot(RayInfo.WorldRay.Direction) < 0;
		};
	}
	FTransformSRT3d WorldTransform = GetWorldTransformFunc();
	FRay3d LocalRay = WorldTransform.InverseTransformRay(RayInfo.WorldRay);
	double RayHitT; int32 HitTriangleID; FVector3d HitBaryCoords;
	if (GetColliderMesh()->FindNearestHitTriangle(LocalRay, RayHitT, HitTriangleID, HitBaryCoords, SpatialQueryOptions))
	{
		HitResultOut.bHit = true;
		HitResultOut.HitIdentifier = HitTriangleID;
		HitResultOut.HitOwner = TargetMesh.Get();
		FVector3d WorldPosition = WorldTransform.TransformPosition(LocalRay.PointAt(RayHitT));
		HitResultOut.HitDepth = RayInfo.WorldRay.GetParameter(WorldPosition);
	}
	else if ( QueryConfig.TopologyType == EGeometryTopologyType::Polygroup && QueryConfig.ElementType == EGeometryElementType::Edge )
	{
		auto EdgeHitToleranceTest = [RayInfo, WorldTransform](int32 ID, const FVector3d& A, const FVector3d& B)
		{
			return ToolSceneQueriesUtil::PointSnapQuery(RayInfo.CameraState,
				WorldTransform.TransformPosition(A), WorldTransform.TransformPosition(B));
		};
		FSegmentTree3::FSegment Segment;
		FSegmentTree3::FRayNearestSegmentInfo SegmentInfo;
		if (GetGroupEdgeSpatial()->FindNearestVisibleSegmentHitByRay(LocalRay, EdgeHitToleranceTest, Segment, SegmentInfo))
		{
			HitResultOut.bHit = true;
			HitResultOut.HitIdentifier = Segment.ID;
			HitResultOut.HitOwner = TargetMesh.Get();
			FVector3d WorldPosition = WorldTransform.TransformPosition(LocalRay.PointAt(SegmentInfo.RayParam));
			HitResultOut.HitDepth = RayInfo.WorldRay.GetParameter(WorldPosition);
		}
	}
	return HitResultOut.bHit;
}


void FBaseDynamicMeshSelector::UpdateSelectionViaRaycast(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	if (IsLockable() && IsLocked()) return;

	if (SelectionEditor.GetTopologyType() == EGeometryTopologyType::Triangle)
	{
		UpdateSelectionViaRaycast_MeshTopology(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
		return;
	}
	check(SelectionEditor.GetTopologyType() == EGeometryTopologyType::Polygroup);	// assume this...

	if (SelectionEditor.GetElementType() == EGeometryElementType::Edge)
	{
		UpdateSelectionViaRaycast_GroupEdges(RayInfo, SelectionEditor, UpdateConfig, ResultOut);
	}
	else
	{
		FRay3d LocalRay = GetWorldTransformFunc().InverseTransformRay(RayInfo.WorldRay);
		UE::Geometry::UpdateGroupSelectionViaRaycast(
			GetColliderMesh(), GetGroupTopology(), &SelectionEditor,
			LocalRay, UpdateConfig, ResultOut);
	}
}

void FBaseDynamicMeshSelector::UpdateSelectionViaRaycast_MeshTopology(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	FRay3d LocalRay = GetWorldTransformFunc().InverseTransformRay(RayInfo.WorldRay);
	UE::Geometry::UpdateTriangleSelectionViaRaycast(
		GetColliderMesh(), &SelectionEditor,
		LocalRay, UpdateConfig, ResultOut);

	// necessary so that edge is recognized from both 'sides' since edge is stored as MeshTriEdgeID.Encoded in SelectionEditor
	// therefore need to include both of the TriEdgeIDs which the edge belongs to to the SelectionEditor
	if (SelectionEditor.GetElementType() == EGeometryElementType::Edge)
	{
		auto AffectBothTriEdgeIDs = [this, &SelectionEditor, &ResultOut](const EGeometrySelectionChangeType ChangeType)
		{
			ensure(ChangeType == EGeometrySelectionChangeType::Add || ChangeType == EGeometrySelectionChangeType::Remove);

			const TArray<uint64>& DeltaElements = (ChangeType == EGeometrySelectionChangeType::Add ? ResultOut.SelectionDelta.Added : ResultOut.SelectionDelta.Removed);
			
			TArray<uint64> ElementsToUpdateInSelection;
			for (const uint64 Element : DeltaElements)
			{
				FMeshTriEdgeID TriEdgeID(FGeoSelectionID(Element).GeometryID);
				TargetMesh->ProcessMesh([TriEdgeID, &ElementsToUpdateInSelection](const UE::Geometry::FDynamicMesh3& SourceMesh)
				{
					// the added or removed EdgeID
					const int32 EdgeID = SourceMesh.IsTriangle(TriEdgeID.TriangleID) ? SourceMesh.GetTriEdge(TriEdgeID.TriangleID, TriEdgeID.TriEdgeIndex) : IndexConstants::InvalidID;
					if (SourceMesh.IsEdge(EdgeID))
					{
						SourceMesh.EnumerateTriEdgeIDsFromEdgeID(EdgeID,
							[TriEdgeID, &ElementsToUpdateInSelection](const FMeshTriEdgeID OtherTriEdgeID)
							{
								// avoid adding to the selection the edge which already exists in selection
								// OR removing from the selection the edge which has already been removed
								if (OtherTriEdgeID.TriangleID == TriEdgeID.TriangleID)
								{
									return;
								}
								ElementsToUpdateInSelection.Add(OtherTriEdgeID.Encoded());
							});
					}
				});
			}
			// adds/removes from SelectionEditor and adds to ResultOut.SelectionDelta.Added/Removed
			UpdateSelectionWithNewElements(&SelectionEditor, ChangeType, ElementsToUpdateInSelection, &ResultOut.SelectionDelta);
		};

		// selecting or deselecting (whichever applicable) the secondary TriEdgeID
		AffectBothTriEdgeIDs(EGeometrySelectionChangeType::Add);
		AffectBothTriEdgeIDs(EGeometrySelectionChangeType::Remove);
	}
}


void FBaseDynamicMeshSelector::UpdateSelectionViaRaycast_GroupEdges(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut)
{
	FTransformSRT3d WorldTransform = GetWorldTransformFunc();
	FRay3d LocalRay = WorldTransform.InverseTransformRay(RayInfo.WorldRay);

	auto EdgeHitToleranceTest = [RayInfo, WorldTransform](int32 ID, const FVector3d& A, const FVector3d& B)
	{
		return ToolSceneQueriesUtil::PointSnapQuery(RayInfo.CameraState,
			WorldTransform.TransformPosition(A), WorldTransform.TransformPosition(B));
	};
	
	IMeshSpatial::FQueryOptions SpatialQueryOptions;
	if (!SelectionEditor.GetQueryConfig().bHitBackFaces)
	{
		SpatialQueryOptions.TriangleFilterF = [&](int32 Tid) {
			return GetColliderMesh()->GetTriNormal(Tid).Dot(LocalRay.Direction) < 0;
		};
	}
	
	if (SelectionEditor.GetQueryConfig().bOnlyVisible)
	{
		// Compute mesh hit and then find nearest-segment to the mesh hit point. This filters out 'backface' hits
		// and avoids complexity of determing "nearest visible" segment. However this will not work for segments
		// on the mesh silhouette, or if the mesh surface is hidden, so this path should be optional...maybe controlled by the the SelectionEditor?
		double RayHitT; int32 HitTriangleID; FVector3d HitBaryCoords;
		bool bSurfaceMeshHit = GetColliderMesh()->FindNearestHitTriangle(LocalRay, RayHitT, HitTriangleID, HitBaryCoords, SpatialQueryOptions);
		if (bSurfaceMeshHit)
		{
			FVector3d SurfaceHitPos = LocalRay.PointAt(RayHitT);
			FSegmentTree3::FSegment Segment;
			if (GetGroupEdgeSpatial()->FindNearestSegment(SurfaceHitPos, Segment))
			{
				FVector3d SegmentPoint = Segment.Segment.NearestPoint(SurfaceHitPos);
				if (EdgeHitToleranceTest(0, SurfaceHitPos, SegmentPoint))
				{
					int32 GroupEdgeID = GetGroupTopology()->FindGroupEdgeID(Segment.ID);
					if (GroupEdgeID >= 0)
					{
						FMeshTriEdgeID TriEdgeID;
						GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) { TriEdgeID = Mesh.GetTriEdgeIDFromEdgeID(Segment.ID); });
						FGeoSelectionID SelectionID(TriEdgeID.Encoded(), GroupEdgeID);
						ResultOut.bSelectionModified = UpdateSelectionWithNewElements(&SelectionEditor, UpdateConfig.ChangeType, TArray<uint64>{SelectionID.Encoded()}, & ResultOut.SelectionDelta);
						ResultOut.bSelectionMissed = false;
					}
				}
			}
			// if we took this path we are not going to consider ray-nearest point
			return;
		}
	}

	// find nearest segment to ray
	FSegmentTree3::FSegment Segment;
	FSegmentTree3::FRayNearestSegmentInfo SegmentInfo;
	if (GetGroupEdgeSpatial()->FindNearestVisibleSegmentHitByRay(LocalRay, EdgeHitToleranceTest, Segment, SegmentInfo, SpatialQueryOptions))
	{
		int32 GroupEdgeID = GetGroupTopology()->FindGroupEdgeID(Segment.ID);
		if (GroupEdgeID >= 0)
		{
			FMeshTriEdgeID TriEdgeID;
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) { TriEdgeID = Mesh.GetTriEdgeIDFromEdgeID(Segment.ID); });
			FGeoSelectionID SelectionID(TriEdgeID.Encoded(), GroupEdgeID);
			ResultOut.bSelectionModified = UpdateSelectionWithNewElements(&SelectionEditor, UpdateConfig.ChangeType, TArray<uint64>{SelectionID.Encoded()}, & ResultOut.SelectionDelta);
			ResultOut.bSelectionMissed = false;
		}
	}
}




void FBaseDynamicMeshSelector::GetSelectionPreviewForRaycast(
	const FWorldRayQueryInfo& RayInfo,
	FGeometrySelectionEditor& PreviewEditor)
{
	if (IsLockable() && IsLocked()) return;

	FGeometrySelectionUpdateResult UpdateResult;
	UpdateSelectionViaRaycast(RayInfo, PreviewEditor,
		FGeometrySelectionUpdateConfig(),	// defaults to add
		UpdateResult);
}



void FBaseDynamicMeshSelector::UpdateSelectionViaShape(
	const FWorldShapeQueryInfo& ShapeInfo,
	FGeometrySelectionEditor& SelectionEditor,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut )
{
	ResultOut.bSelectionModified = false;
	ResultOut.bSelectionMissed = true;

	if (IsLockable() && IsLocked()) return;

	FTransformSRT3d WorldTransform = GetWorldTransformFunc();
	TArray<uint64> InsideElements;

	// todo: support bHitBackFaces
	// todo: this should be made more efficient using BVHs...

	if ( SelectionEditor.GetTopologyType() == EGeometryTopologyType::Triangle )
	{
		if ( SelectionEditor.GetElementType() == EGeometryElementType::Vertex )
		{
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
			{
				TArray<uint64> InsidePoints;
				for ( int32 vid : Mesh.VertexIndicesItr() )
				{
					FVector3d WorldPos = WorldTransform.TransformPosition(Mesh.GetVertex(vid));
					if ( ShapeInfo.Convex.IntersectPoint(WorldPos) )
					{
						InsideElements.Add( FGeoSelectionID::MeshVertex(vid).Encoded() );
					}
				}
			});
		}
		else if ( SelectionEditor.GetElementType() == EGeometryElementType::Edge )
		{
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
			{
				TArray<uint64> InsideEdges;
				for ( int32 eid : Mesh.EdgeIndicesItr() )
				{
					FVector3d A,B;
					Mesh.GetEdgeV(eid, A,B);
					A = WorldTransform.TransformPosition(A);
					B = WorldTransform.TransformPosition(B);
					if ( ShapeInfo.Convex.IntersectLineSegment(A,B) )
					{
						InsideElements.Add( FGeoSelectionID::MeshEdge( Mesh.GetTriEdgeIDFromEdgeID(eid) ).Encoded() );
					}
				}
			});
		}
		else if (SelectionEditor.GetElementType() == EGeometryElementType::Face)
		{
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
			{
				TArray<uint64> InsideFaces;
				for ( int32 tid : Mesh.TriangleIndicesItr() )
				{
					FVector3d A,B,C;
					Mesh.GetTriVertices(tid, A,B,C);
					A = WorldTransform.TransformPosition(A);
					B = WorldTransform.TransformPosition(B);
					C = WorldTransform.TransformPosition(C);
					bool bFullyContained = false;
					if ( ShapeInfo.Convex.IntersectTriangle(A,B,C, bFullyContained) )
					{
						InsideElements.Add( FGeoSelectionID::MeshTriangle(tid).Encoded() );
					}
				}
			});
		}
	}
	else if ( SelectionEditor.GetTopologyType() == EGeometryTopologyType::Polygroup )
	{
		const FGroupTopology* UseGroupTopology = GetGroupTopology();

		if ( SelectionEditor.GetElementType() == EGeometryElementType::Vertex )
		{
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
			{
				for (const FGroupTopology::FCorner& Corner : UseGroupTopology->Corners)
				{
					FVector3d WorldPos = WorldTransform.TransformPosition(Mesh.GetVertex(Corner.VertexID));
					if ( ShapeInfo.Convex.IntersectPoint(WorldPos) )
					{
						InsideElements.Add( FGeoSelectionID(Corner.VertexID, UseGroupTopology->GetCornerIDFromVertexID(Corner.VertexID)).Encoded() );
					}				
				}
			});

		}
		else if (SelectionEditor.GetElementType() == EGeometryElementType::Edge)
		{
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
			{
				int32 NumEdges = UseGroupTopology->Edges.Num();
				for ( int32 GroupEdgeID = 0; GroupEdgeID < NumEdges; ++GroupEdgeID)
				{
					const FGroupTopology::FGroupEdge& Edge = UseGroupTopology->Edges[GroupEdgeID];
					bool bAnyIntersects = false;
					for (int32 eid : Edge.Span.Edges)
					{
						FVector3d A,B;
						Mesh.GetEdgeV(eid, A,B);
						A = WorldTransform.TransformPosition(A);
						B = WorldTransform.TransformPosition(B);
						if (ShapeInfo.Convex.IntersectLineSegment(A,B))
						{
							bAnyIntersects = true;
							break;
						}
					}
					if (bAnyIntersects)
					{
						FMeshTriEdgeID EdgeTriID = Mesh.GetTriEdgeIDFromEdgeID(Edge.Span.Edges[0]);
						InsideElements.Add( FGeoSelectionID( EdgeTriID.Encoded(), GroupEdgeID).Encoded());
					}
				}
			});
		}
		else if (SelectionEditor.GetElementType() == EGeometryElementType::Face)
		{
			GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh) 
			{
				for (const FGroupTopology::FGroup& Group : UseGroupTopology->Groups)
				{
					bool bAnyIntersects = false;
					for (int32 tid : Group.Triangles)
					{
						FVector3d A,B,C;
						Mesh.GetTriVertices(tid, A,B,C);
						A = WorldTransform.TransformPosition(A);
						B = WorldTransform.TransformPosition(B);
						C = WorldTransform.TransformPosition(C);
						bool bFullyContained = false;
						if (ShapeInfo.Convex.IntersectTriangle(A,B,C, bFullyContained))
						{
							bAnyIntersects = true;
							break;
						}
					}
					if (bAnyIntersects)
					{
						InsideElements.Add( FGeoSelectionID(Group.Triangles[0], Group.GroupID).Encoded());
					}
				}
			});
		}
	}

	if (InsideElements.Num() > 0 )
	{
		ResultOut.bSelectionModified = UpdateSelectionWithNewElements(&SelectionEditor, UpdateConfig.ChangeType, InsideElements, &ResultOut.SelectionDelta);
		ResultOut.bSelectionMissed = false;
	}
}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
void FBaseDynamicMeshSelector::UpdateSoftSelection(
	FGeometrySelection& Selection,
	const FUpdateSoftSelectionConfig& InSoftSelectionConfig
	)
{
	Selection.SoftSelectedVertices.Reset();
	
	if (InSoftSelectionConfig.bRecomputeDistances)
	{
		PathFinder.Reset();
		CachedVolumeDistanceToSelection.Reset();
	}
	
	if (!InSoftSelectionConfig.bEnabled)
	{
		return;
	}

	switch (InSoftSelectionConfig.DistanceMode)
	{
	case ESoftSelectionDistanceMode::Volume:
		UpdateSoftSelection_Volume(Selection, InSoftSelectionConfig);
		break;
	case ESoftSelectionDistanceMode::Surface:
	default:
		UpdateSoftSelection_Surface(Selection, InSoftSelectionConfig);
		break;
	}
}

void FBaseDynamicMeshSelector::UpdateSoftSelection_Surface(
	FGeometrySelection& Selection,
	const FUpdateSoftSelectionConfig& InSoftSelectionConfig
	)
{
	if (!PathFinder.IsValid())
	{
		// seed a Dijkstra path finder with the selected vertices
		PathFinder = MakePimpl<TMeshDijkstra<FDynamicMesh3>>(GetDynamicMesh()->GetMeshPtr());
		TArray<TMeshDijkstra<FDynamicMesh3>::FSeedPoint> SeedPoints;

		TSet<int32> VertexIDs;
		GetDynamicMesh()->ProcessMesh([&Selection, &VertexIDs, this](const FDynamicMesh3& SourceMesh)
			{
				// get set of selected vertex IDs, and then vector (only converting to vector for VertexToTriangleOneRing...)
				if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
				{
					UE::Geometry::EnumeratePolygroupSelectionVertices(Selection, SourceMesh, GetGroupTopology(), FTransform::Identity,
						[&VertexIDs](uint32 VertexID, const FVector3d& Position) { VertexIDs.Add((int32)VertexID); }
					);
				}
				else
				{
					UE::Geometry::EnumerateTriangleSelectionVertices(Selection, SourceMesh, nullptr,
						[&VertexIDs](uint32 VertexID, const FVector3d& Position) { VertexIDs.Add((int32)VertexID); }
					);
				}
			});
	
		for (const int32 SelectedVertex : VertexIDs)
		{
			SeedPoints.Add({ SelectedVertex, SelectedVertex, 0 });
		}
	
	
		PathFinder->ComputeToMaxDistance(SeedPoints, TNumericLimits<double>::Max());	
	}


	PathFinder->EnumerateComputedPoint([&Selection, &InSoftSelectionConfig, this](TMeshDijkstra<FDynamicMesh3>::FComputedPoint InPoint)
		{
			// Skip points already in the selection
			if (InPoint.bSeedPoint)
			{
				return;
			}

			const double& Radius = InSoftSelectionConfig.Radius;
			
			if (InPoint.GraphDistance > Radius)
			{
				return;
			}
			
			FGeoSelectionID SelectionID;
			bool bAddPoint = false;
			if (Selection.TopologyType == EGeometryTopologyType::Triangle)
			{
				SelectionID = FGeoSelectionID((uint32)InPoint.PointID);
				bAddPoint = true;
			}
			else if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
			{
				const int32 CornerID = GetGroupTopology()->GetCornerIDFromVertexID(InPoint.PointID);
				if (CornerID != IndexConstants::InvalidID)
				{
					bAddPoint = true;
					SelectionID = FGeoSelectionID(InPoint.PointID, CornerID);
				}
			}
			
			if (bAddPoint)
			{
				uint64 SelectionIDEncoded = SelectionID.Encoded();

				double NormalizedDistance = Radius > UE_SMALL_NUMBER ? InPoint.GraphDistance / Radius : 0.0 ;

				double Weight = FMath::CubicInterp(1.0, 0.0, 0.0, 0.0, NormalizedDistance);
				
				Selection.SoftSelectedVertices.Add(SelectionIDEncoded, Weight);
			}
		});
}

void FBaseDynamicMeshSelector::UpdateSoftSelection_Volume(
	FGeometrySelection& Selection,
	const FUpdateSoftSelectionConfig& InSoftSelectionConfig
	)
{
	GetDynamicMesh()->ProcessMesh([&Selection, &InSoftSelectionConfig, this](const FDynamicMesh3& SourceMesh)
	{
		if (CachedVolumeDistanceToSelection.Num() != SourceMesh.MaxVertexID())
		{
			CachedVolumeDistanceToSelection.Reset();

			TArray<TPair<int32, FVector3d>> SeedVertices;
			TSet<int32> SeedVertexIDSet;
			auto AccumulateSeed = [&SeedVertices, &SeedVertexIDSet](uint32 VertexID, const FVector3d& Position)
			{
				bool bAlreadySeen = false;
				SeedVertexIDSet.Add((int32)VertexID, &bAlreadySeen);
				if (!bAlreadySeen)
				{
					SeedVertices.Emplace((int32)VertexID, Position);
				}
			};

			if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
			{
				UE::Geometry::EnumeratePolygroupSelectionVertices(Selection, SourceMesh, GetGroupTopology(), FTransform::Identity, AccumulateSeed);
			}
			else
			{
				UE::Geometry::EnumerateTriangleSelectionVertices(Selection, SourceMesh, nullptr, AccumulateSeed);
			}

			if (SeedVertices.Num() == 0)
			{
				return;
			}

			FAxisAlignedBox3d UnionBox;
			for (const TPair<int32, FVector3d>& Seed : SeedVertices)
			{
				UnionBox.Contain(Seed.Value);
			}
			FSparseDynamicPointOctree3 SeedOctree;

			const double SeedExtent = FMath::Max(UnionBox.MaxDim(), 1.0);
			SeedOctree.ConfigureFromPointCountEstimate(SeedExtent, SeedVertices.Num());
			for (int32 SeedIdx = 0; SeedIdx < SeedVertices.Num(); ++SeedIdx)
			{
				SeedOctree.InsertPoint(SeedIdx, SeedVertices[SeedIdx].Value);
			}

			const double DistanceThreshold = FMath::Max(SourceMesh.GetBounds().DiagonalLength() * 2.0, 1.0);
			CachedVolumeDistanceToSelection.Init(TNumericLimits<double>::Max(), SourceMesh.MaxVertexID());

			ParallelFor(SourceMesh.MaxVertexID(), [&](int32 VertexID)
			{
				if (!SourceMesh.IsVertex(VertexID))
				{
					return;
				}
				if (SeedVertexIDSet.Contains(VertexID))
				{
					return;
				}

				const FVector3d VertexPos = SourceMesh.GetVertex(VertexID);
				const int32 NearestSeedIdx = SeedOctree.FindClosestPoint(
					VertexPos, DistanceThreshold,
					[](int32) { return true; },
					[&SeedVertices, &VertexPos](int32 SeedIdx) -> double
					{
						return FVector3d::DistSquared(VertexPos, SeedVertices[SeedIdx].Value);
					});

				if (NearestSeedIdx != INDEX_NONE)
				{
					CachedVolumeDistanceToSelection[VertexID] = FVector3d::Distance(VertexPos, SeedVertices[NearestSeedIdx].Value);
				}
			});
		}

		const double Radius = InSoftSelectionConfig.Radius;
		if (Radius <= 0.0)
		{
			return;
		}

		const FGroupTopology* const PolygroupTopology =
			(Selection.TopologyType == EGeometryTopologyType::Polygroup) ? GetGroupTopology() : nullptr;

		for (const int32 VertexID : SourceMesh.VertexIndicesItr())
		{
			const double Distance = CachedVolumeDistanceToSelection[VertexID];
			if (Distance > Radius)
			{
				continue;
			}

			FGeoSelectionID SelectionID;
			bool bAddPoint = false;
			if (Selection.TopologyType == EGeometryTopologyType::Triangle)
			{
				SelectionID = FGeoSelectionID((uint32)VertexID);
				bAddPoint = true;
			}
			else if (PolygroupTopology != nullptr)
			{
				const int32 CornerID = PolygroupTopology->GetCornerIDFromVertexID(VertexID);
				if (CornerID != IndexConstants::InvalidID)
				{
					bAddPoint = true;
					SelectionID = FGeoSelectionID((uint32)VertexID, CornerID);
				}
			}

			if (bAddPoint)
			{
				const double NormalizedDistance = Radius > UE_SMALL_NUMBER ? Distance / Radius : 0.0;
				const double Weight = FMath::CubicInterp(1.0, 0.0, 0.0, 0.0, NormalizedDistance);
				Selection.SoftSelectedVertices.Add(SelectionID.Encoded(), Weight);
			}
		}
	});
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

bool FBaseDynamicMeshSelector::ConvertSelection(
	const FGeometrySelection& FromSelectionIn,
	FGeometrySelection& ToSelectionOut,
	const EEnumerateSelectionConversionParams ConversionParams)
{
	return UE::Geometry::ConvertSelection(GetDynamicMesh()->GetMeshRef(), GetGroupTopology(), FromSelectionIn, ToSelectionOut, ConversionParams);	
}


void FBaseDynamicMeshSelector::GetSelectionFrame(const FGeometrySelection& Selection, FFrame3d& SelectionFrame, bool bTransformToWorld)
{
	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		const FGroupTopology* Topology = GetGroupTopology();
		FGroupTopologySelection TopoSelection;
		TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
		{
			UE::Geometry::ConvertPolygroupSelectionToTopologySelection(Selection, SourceMesh, Topology, TopoSelection);
		});

		SelectionFrame = Topology->GetSelectionFrame(TopoSelection);
	}
	else
	{
		TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
		{
			UE::Geometry::GetTriangleSelectionFrame(Selection, SourceMesh, SelectionFrame);
		});
	}

	if (bTransformToWorld)
	{
		SelectionFrame.Transform(GetLocalToWorldTransform());
	}
}

void FBaseDynamicMeshSelector::GetTargetFrame(const FGeometrySelection& Selection, FFrame3d& SelectionFrame)
{
	SelectionFrame.Transform(GetLocalToWorldTransform());
	TQuaternion<double> TargetRotation = SelectionFrame.Rotation;

	// Places gizmo at the selection's accumulated origin
	GetSelectionFrame(Selection, SelectionFrame, true);

	// Uses object rotation instead of accumulated normals from selection
	SelectionFrame.Rotation = TargetRotation;
}

void FBaseDynamicMeshSelector::AccumulateSelectionBounds(const FGeometrySelection& Selection, FGeometrySelectionBounds& BoundsInOut, bool bTransformToWorld)
{
	FTransform UseTransform = (bTransformToWorld) ? GetLocalToWorldTransform() : FTransform::Identity;
	UE::Geometry::FAxisAlignedBox3d TargetWorldBounds = UE::Geometry::FAxisAlignedBox3d::Empty();
	int32 ElementCount = 0;

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		const FGroupTopology* Topology = GetGroupTopology();
		TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
		{
			UE::Geometry::EnumeratePolygroupSelectionVertices(Selection, SourceMesh, GetGroupTopology(), UseTransform,
				[&](uint32 VertexID, const FVector3d& Position) { 
					TargetWorldBounds.Contain(Position); 
					ElementCount++;
				}
			);
		});
	}
	else
	{
		TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
		{
			UE::Geometry::EnumerateTriangleSelectionVertices(Selection, SourceMesh, &UseTransform,
				[&](uint32 VertexID, const FVector3d& Position) { 
					TargetWorldBounds.Contain(Position); 
					ElementCount++;
				}
			);
		});
	}

	if (ElementCount > 0)		// relying on this because in (eg) single-vertex case, the box will still be "empty"
	{
		BoundsInOut.WorldBounds.Contain(TargetWorldBounds);
	}
}



void FBaseDynamicMeshSelector::AccumulateSelectionElements(const FGeometrySelection& Selection, FGeometrySelectionElements& Elements, bool bTransformToWorld, bool bIsForPreview)
{
	// where if bIsForPreview is true, we are mapping faces to edges
	AccumulateSelectionElements(Selection, Elements, bTransformToWorld, EEnumerateSelectionMapping::Default | (bIsForPreview ? EEnumerateSelectionMapping::FacesToEdges : EEnumerateSelectionMapping::None));
}

void FBaseDynamicMeshSelector::AccumulateSelectionElements(const FGeometrySelection& Selection, FGeometrySelectionElements& Elements, bool bTransformToWorld, const EEnumerateSelectionMapping Flags)
{
	const FTransform UseWorldTransform = GetLocalToWorldTransform();
	const FTransform* ApplyTransform = (bTransformToWorld) ? &UseWorldTransform : nullptr;

	ToolSelectionUtil::AccumulateSelectionElements(
		Elements,
		Selection,
		TargetMesh->GetMeshRef(),
		Selection.TopologyType == EGeometryTopologyType::Polygroup ? GetGroupTopology() : nullptr,
		ApplyTransform,
		Flags);
}

void FBaseDynamicMeshSelector::AccumulateElementsFromPredicate(FGeometrySelectionElements& Elements, bool bTransformToWorld, bool bIsForPreview, bool bUseGroupTopology, TFunctionRef<bool(EGeometryElementType, FGeoSelectionID)> Predicate)
{
	auto AccumulateElementsOfTypeFromPredicate = [this, &Elements, &Predicate, bTransformToWorld, bIsForPreview, bUseGroupTopology](EGeometryElementType ElementType)
	{
		FGeometrySelection Selection;
		Selection.ElementType = ElementType;
		Selection.TopologyType = bUseGroupTopology ? EGeometryTopologyType::Polygroup : EGeometryTopologyType::Triangle;
		InitializeSelectionFromPredicate(Selection, [&Predicate, ElementType](FGeoSelectionID InID){ return Predicate(ElementType, InID); });
		AccumulateSelectionElements(Selection, Elements, bTransformToWorld, EEnumerateSelectionMapping::Default);
	};

	AccumulateElementsOfTypeFromPredicate(EGeometryElementType::Vertex);
	AccumulateElementsOfTypeFromPredicate(EGeometryElementType::Edge);
	AccumulateElementsOfTypeFromPredicate(EGeometryElementType::Face);
}





void FBaseDynamicMeshSelector::UpdateAfterGeometryEdit(
	IToolsContextTransactionsAPI* TransactionsAPI,
	bool bInTransaction,
	TUniquePtr<FDynamicMeshChange> DynamicMeshChange,
	FText GeometryEditTransactionString)
{
	TransactionsAPI->AppendChange(GetDynamicMesh(),
		MakeUnique<FMeshChange>(MoveTemp(DynamicMeshChange)), GeometryEditTransactionString);
}



void FBaseDynamicMeshSelector::UpdateColliderMesh()
{
	// should we transform to world??
	TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
	{
		FColliderMesh::FBuildOptions BuildOptions;
		BuildOptions.bBuildAABBTree = BuildOptions.bBuildVertexMap = BuildOptions.bBuildTriangleMap = true;
		ColliderMesh = MakePimpl<FColliderMesh>(SourceMesh, BuildOptions);
	});
}

const FColliderMesh* FBaseDynamicMeshSelector::GetColliderMesh()
{
	if (!ColliderMesh.IsValid())
	{
		UpdateColliderMesh();
	}
	return ColliderMesh.Get();
}


void FBaseDynamicMeshSelector::UpdateGroupTopology()
{
	// TODO would be preferable to not have to use the raw mesh pointer here, however
	// FGroupTopology currently needs the pointer to do mesh queries

	GroupTopology = MakePimpl<FGroupTopology>(TargetMesh->GetMeshPtr(), true);
}

const FGroupTopology* FBaseDynamicMeshSelector::GetGroupTopology()
{
	if (!GroupTopology.IsValid())
	{
		UpdateGroupTopology();
	}
	return GroupTopology.Get();
}



void FBaseDynamicMeshSelector::UpdateGroupEdgeSegmentTree()
{
	const FGroupTopology* UseGroupTopology = GetGroupTopology();

	TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
	{
		TArray<int32> EdgeIDs;
		for (const FGroupTopology::FGroupEdge& GroupEdge : UseGroupTopology->Edges)
		{
			EdgeIDs.Append(GroupEdge.Span.Edges);
		}

		GroupEdgeSegmentTree = MakePimpl<FSegmentTree3>();
		GroupEdgeSegmentTree->Build(EdgeIDs, [&SourceMesh](int32 EdgeID)
		{
			return FSegment3d( SourceMesh.GetEdgePoint(EdgeID, 0), SourceMesh.GetEdgePoint(EdgeID, 1) );
		}, SourceMesh.EdgeCount() );
	});
}

const FSegmentTree3* FBaseDynamicMeshSelector::GetGroupEdgeSpatial()
{
	if (!GroupEdgeSegmentTree.IsValid())
	{
		UpdateGroupEdgeSegmentTree();
	}
	return GroupEdgeSegmentTree.Get();
}


bool FDynamicMeshComponentSelectorFactory::CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	if (TargetIdentifier.TargetType == FGeometryIdentifier::ETargetType::PrimitiveComponent)
	{
		UDynamicMeshComponent* Component = TargetIdentifier.GetAsComponentType<UDynamicMeshComponent>();
		if (Component)
		{
			return true;
		}
	}
	return false;
}

TUniquePtr<IGeometrySelector> FDynamicMeshComponentSelectorFactory::BuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	UDynamicMeshComponent* Component = TargetIdentifier.GetAsComponentType<UDynamicMeshComponent>();
	check(Component != nullptr);

	TUniquePtr<FDynamicMeshSelector> Selector = MakeUnique<FDynamicMeshSelector>();
	Selector->Initialize(TargetIdentifier, Component->GetDynamicMesh(),
		[TargetIdentifier]() -> FTransformSRT3d
		{ 
			UDynamicMeshComponent* Component = TargetIdentifier.GetAsComponentType<UDynamicMeshComponent>();
			return IsValid(Component) ? (FTransformSRT3d)Component->GetComponentTransform() : FTransformSRT3d::Identity(); 
		}
	);
	return Selector;
}








IGeometrySelectionTransformer* FDynamicMeshSelector::InitializeTransformation(const FGeometrySelection& Selection)
{
	check(!ActiveTransformer);

	// If we are transforming a DynamicMeshComponent, we want to defer collision updates, otherwise 
	// complex collision will be rebuilt every frame
	FGeometryIdentifier ParentIdentifier = GetSourceGeometryIdentifier();
	if (ParentIdentifier.ObjectType == FGeometryIdentifier::EObjectType::DynamicMeshComponent)
	{
		ParentIdentifier.GetAsComponentType<UDynamicMeshComponent>()->SetTransientDeferCollisionUpdates(true);
	}

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		ActiveTransformer = MakeShared<FDynamicMeshPolygroupTransformer>();
	}
	else
	{
		ActiveTransformer = MakeShared<FBasicDynamicMeshSelectionTransformer>();
	}


	ActiveTransformer->Initialize(this);
	return ActiveTransformer.Get();
}

void FDynamicMeshSelector::ShutdownTransformation(IGeometrySelectionTransformer* Transformer)
{
	ActiveTransformer.Reset();

	FGeometryIdentifier ParentIdentifier = GetSourceGeometryIdentifier();
	if (ParentIdentifier.ObjectType == FGeometryIdentifier::EObjectType::DynamicMeshComponent)
	{
		ParentIdentifier.GetAsComponentType<UDynamicMeshComponent>()->SetTransientDeferCollisionUpdates(false);
		ParentIdentifier.GetAsComponentType<UDynamicMeshComponent>()->UpdateCollision(true);
	}
}






void FBasicDynamicMeshSelectionTransformer::Initialize(FBaseDynamicMeshSelector* SelectorIn)
{
	Selector = SelectorIn;
}

void FBasicDynamicMeshSelectionTransformer::BeginTransform(const FGeometrySelection& Selection)
{
	const FGroupTopology* UseTopology = nullptr;
	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		UseTopology = Selector->GetGroupTopology();
	}

	TSet<int32> VertexIDs;
	Selector->GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		// get set of selected vertex IDs, and then vector (only converting to vector for VertexToTriangleOneRing...)
		if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
		{
			UE::Geometry::EnumeratePolygroupSelectionVertices(Selection, SourceMesh, UseTopology, FTransform::Identity,
				[&](uint32 VertexID, const FVector3d& Position) { VertexIDs.Add((int32)VertexID); }
			);
		}
		else
		{
			UE::Geometry::EnumerateTriangleSelectionVertices(Selection, SourceMesh, nullptr,
				[&](uint32 VertexID, const FVector3d& Position) { VertexIDs.Add((int32)VertexID); }
			);
		}
			
		NumSelectedMeshVertices = VertexIDs.Num();
		MeshVertices = VertexIDs.Array();

		int32 NumWeightedMeshVertices = NumSelectedMeshVertices + Selection.SoftSelectedVertices.Num();
		MeshVertices.Reserve(NumWeightedMeshVertices);
		SoftSelectedVertexWeights.Reserve(Selection.SoftSelectedVertices.Num());
			
		for (const TPair<uint64, double>& WeightedVertex : Selection.SoftSelectedVertices)
		{
			MeshVertices.Add(FGeoSelectionID(WeightedVertex.Key).GeometryID);
			SoftSelectedVertexWeights.Add(WeightedVertex.Value);
		}

		// save initial positions
		InitialPositions.Reserve(MeshVertices.Num());
		for (int32 vid : MeshVertices)
		{
			InitialPositions.Add(SourceMesh.GetVertex(vid));
		}
		UpdatedPositions.SetNum(MeshVertices.Num());

		// get triangle ROI
		UE::Geometry::VertexToTriangleOneRing(&SourceMesh, MeshVertices, TriangleROI);

		// save overlay normals
		OverlayNormalsArray.Reset();
		OverlayNormalsSet.Reset();
		if (SourceMesh.HasAttributes() && SourceMesh.Attributes()->PrimaryNormals() != nullptr)
		{
			const FDynamicMeshNormalOverlay* Normals = SourceMesh.Attributes()->PrimaryNormals();
			UE::Geometry::TrianglesToOverlayElements(Normals, TriangleROI, OverlayNormalsSet);
			OverlayNormalsArray = OverlayNormalsSet.Array();
		}

		// collect selected mesh edges and vertices
		ActiveSelectionEdges.Reset();
		ActiveSelectionVertices.Reset();
		TSet<int32> SelectionEdges;
		UE::Geometry::EnumerateTriangleSelectionElements(Selection, SourceMesh, 
			[&](int32 VertexID, FVector3d Position) { ActiveSelectionVertices.Add(VertexID); },
			[&](int32 EdgeID, const FSegment3d&) { ActiveSelectionEdges.Add(SourceMesh.GetEdgeV(EdgeID)); SelectionEdges.Add(EdgeID); },
			[&](int32 TriangleID, const FTriangle3d&) {},
			/*ApplyTransform*/nullptr, /*bMapFacesToEdgeLoops*/true);

		ActiveROIEdges.Reset();
		for (int32 tid : TriangleROI)
		{
			FIndex3i TriEdges = SourceMesh.GetTriEdges(tid);
			for (int32 k = 0; k < 3; ++k)
			{
				if (SelectionEdges.Contains(TriEdges[k]) == false)
				{
					ActiveROIEdges.Add(SourceMesh.GetEdgeV(TriEdges[k]));
					SelectionEdges.Add(TriEdges[k]);
				}
			}
		}
	});

	ActiveVertexChange = MakePimpl<FMeshVertexChangeBuilder>(EMeshVertexChangeComponents::VertexPositions | EMeshVertexChangeComponents::OverlayNormals);
	UpdatePendingVertexChange(false);
}

void FBasicDynamicMeshSelectionTransformer::UpdateTransform( 
	TFunctionRef<FVector3d(int32 VertexID, double Weight, const FVector3d& InitialPosition, const FTransform& WorldTransform)> PositionTransformFunc 
)
{
	int32 N = MeshVertices.Num();
	FTransform WorldTransform = Selector->GetLocalToWorldTransform();

	for (int32 k = 0; k < N; ++k)
	{
		double Weight = 1.0f;
		if (k >= NumSelectedMeshVertices)
		{
			Weight = SoftSelectedVertexWeights[(k - NumSelectedMeshVertices)];
		}
		UpdatedPositions[k] = PositionTransformFunc(MeshVertices[k], Weight, InitialPositions[k], WorldTransform);
	}

	Selector->GetDynamicMesh()->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		for (int32 k = 0; k < N; ++k)
		{
			EditMesh.SetVertex(MeshVertices[k], UpdatedPositions[k]);
		}

		FMeshNormals::RecomputeOverlayElementNormals(EditMesh, OverlayNormalsArray);

	}, EDynamicMeshChangeType::DeformationEdit,
	   EDynamicMeshAttributeChangeFlags::VertexPositions | EDynamicMeshAttributeChangeFlags::NormalsTangents, false);
}

void FBasicDynamicMeshSelectionTransformer::UpdatePendingVertexChange(bool bFinal)
{
	// update the vertex change
	Selector->GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		ActiveVertexChange->SaveVertices(&SourceMesh, MeshVertices, !bFinal);
		ActiveVertexChange->SaveOverlayNormals(&SourceMesh, OverlayNormalsSet, !bFinal);
	});
}

void FBasicDynamicMeshSelectionTransformer::EndTransform(IToolsContextTransactionsAPI* TransactionsAPI)
{
	UpdatePendingVertexChange(true);

	if (TransactionsAPI != nullptr)
	{
		TUniquePtr<FToolCommandChange> Result = MoveTemp(ActiveVertexChange->Change);
		TransactionsAPI->AppendChange(Selector->GetDynamicMesh(), MoveTemp(Result), LOCTEXT("DynamicMeshTransformChange", "Transform"));
	}

	ActiveVertexChange.Reset();

	if (OnEndTransformFunc)
	{
		OnEndTransformFunc(TransactionsAPI);
	}
}



void FBasicDynamicMeshSelectionTransformer::PreviewRender(IToolsContextRenderAPI* RenderAPI)
{
	if (!bEnableSelectionTransformDrawing) return;

	FToolDataVisualizer Visualizer;
	Visualizer.bDepthTested = false;
	Visualizer.BeginFrame(RenderAPI);

	Visualizer.PushTransform(Selector->GetLocalToWorldTransform());

	Selector->GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& SourceMesh)
	{
		Visualizer.SetPointParameters(FLinearColor(0, 0.3f, 0.95f, 1), 5.0f);
		for (int32 VertexID : ActiveSelectionVertices)
		{
			Visualizer.DrawPoint(SourceMesh.GetVertex(VertexID));
		}

		Visualizer.SetLineParameters(FLinearColor(0, 0.3f, 0.95f, 1), 3.0f);
		for (FIndex2i EdgeV : ActiveSelectionEdges)
		{
			Visualizer.DrawLine(SourceMesh.GetVertex(EdgeV.A), SourceMesh.GetVertex(EdgeV.B));
		}
		Visualizer.SetLineParameters(FLinearColor(0, 0.3f, 0.95f, 1), 1.0f);
		for (FIndex2i EdgeV : ActiveROIEdges)
		{
			Visualizer.DrawLine(SourceMesh.GetVertex(EdgeV.A), SourceMesh.GetVertex(EdgeV.B));
		}
	});

	Visualizer.EndFrame();
}




#undef LOCTEXT_NAMESPACE 