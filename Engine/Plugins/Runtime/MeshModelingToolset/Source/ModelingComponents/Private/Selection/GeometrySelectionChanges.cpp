// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/GeometrySelectionChanges.h"
#include "Selection/GeometrySelectionManager.h"
#include "Selections/GeometrySelectionUtil.h"

#define LOCTEXT_NAMESPACE "GeometrySelectionChanges"

using namespace UE::Geometry;

void FGeometrySelectionDeltaChange::Apply(UObject* Object)
{
	UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
	if ( ensureMsgf(Manager, TEXT("FGeometrySelectionDeltaChange must be emitted on a UGeometrySelectionManager")) )
	{
		Manager->ApplyChange(this);
	}
}

void FGeometrySelectionDeltaChange::Revert(UObject* Object)
{
	UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
	if ( ensureMsgf(Manager, TEXT("FGeometrySelectionDeltaChange must be emitted on a UGeometrySelectionManager")) )
	{
		Manager->RevertChange(this);
	}
}

FString FGeometrySelectionDeltaChange::ToString() const
{
	return TEXT("FGeometrySelectionDeltaChange");
}

bool FGeometrySelectionDeltaChange::HasExpired(UObject* Object) const
{
	UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
	return (Manager == nullptr || IsValid(Manager) == false || Manager->HasBeenShutDown());
}


void FGeometrySelectionDeltaChange::ApplyChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& ApplyDelta)
{
	if (Delta.Removed.Num() > 0)
	{
		UE::Geometry::UpdateSelectionWithNewElements(Editor, EGeometrySelectionChangeType::Remove, Delta.Removed, &ApplyDelta);
	}
	if (Delta.Added.Num() > 0)
	{
		UE::Geometry::UpdateSelectionWithNewElements(Editor, EGeometrySelectionChangeType::Add, Delta.Added, &ApplyDelta);
	}
}
void FGeometrySelectionDeltaChange::RevertChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& RevertDelta)
{
	if (Delta.Added.Num() > 0)
	{
		UE::Geometry::UpdateSelectionWithNewElements(Editor, EGeometrySelectionChangeType::Remove, Delta.Added, &RevertDelta);
	}
	if (Delta.Removed.Num() > 0)
	{
		UE::Geometry::UpdateSelectionWithNewElements(Editor, EGeometrySelectionChangeType::Add, Delta.Removed, &RevertDelta);
	}
}






void FGeometrySelectionReplaceChange::Apply(UObject* Object)
{
	UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
	if ( ensureMsgf(Manager, TEXT("FGeometrySelectionReplaceChange must be emitted on a UGeometrySelectionManager")) )
	{
		Manager->ApplyChange(this);
	}
}

void FGeometrySelectionReplaceChange::Revert(UObject* Object)
{
	UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
	if ( ensureMsgf(Manager, TEXT("FGeometrySelectionReplaceChange must be emitted on a UGeometrySelectionManager")) )
	{
		Manager->RevertChange(this);
	}
}

FString FGeometrySelectionReplaceChange::ToString() const
{
	return TEXT("FGeometrySelectionReplaceChange");
}

bool FGeometrySelectionReplaceChange::HasExpired(UObject* Object) const
{
	UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
	return (Manager == nullptr || IsValid(Manager) == false || Manager->HasBeenShutDown());
}

void FGeometrySelectionReplaceChange::ApplyChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& ApplyDelta)
{
	Editor->Replace(After, ApplyDelta);
}
void FGeometrySelectionReplaceChange::RevertChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& RevertDelta)
{
	Editor->Replace(Before, RevertDelta);
}



#undef LOCTEXT_NAMESPACE