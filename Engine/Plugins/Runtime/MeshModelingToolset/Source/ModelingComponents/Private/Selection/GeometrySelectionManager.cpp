// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/GeometrySelectionManager.h"
#include "CoreGlobals.h" // for GIsTransacting
#include "Engine/Engine.h"
#include "Selection/DynamicMeshSelector.h"
#include "Selection/ToolSelectionUtil.h"
#include "Selection/SelectionEditInteractiveCommand.h"
#include "InteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "ToolContextInterfaces.h"
#include "ToolDataVisualizer.h"
#include "ToolSetupUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Util/ColorConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometrySelectionManager)

static TAutoConsoleVariable<int32> CVarGeometrySelectionManager_FullSelectionHoverHighlights(
	TEXT("modeling.Selection.FullHoverHighlights"),
	1,
	TEXT("Use full selection hover highlights instead of simplified highlights")
);



using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "UGeometrySelectionManager"


void UGeometrySelectionManager::Initialize( UInteractiveToolsContext* ToolsContextIn, IToolsContextTransactionsAPI* TransactionsAPIIn )
{
	ToolsContext = ToolsContextIn;
	TransactionsAPI = TransactionsAPIIn;
	PreviewGeometry = NewObject<UPreviewGeometry>(this);

	OnSelectionModified.AddUObject(this, &UGeometrySelectionManager::UpdateSoftSelectionOnSelectionModified);
}

void UGeometrySelectionManager::RegisterSelectorFactory(TUniquePtr<IGeometrySelectorFactory> Factory)
{
	Factories.Add(MoveTemp(Factory));
	ResetTargetCache();
}

void UGeometrySelectionManager::UnregisterSelectorFactories()
{
	Factories.Empty();
	ResetTargetCache();
}

void UGeometrySelectionManager::Shutdown()
{
	DiscardSavedSelection();
	OnSelectionModified.Clear();
	ToolsContext = nullptr;
	TransactionsAPI = nullptr;

	DisconnectPreviewGeometry();

	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		SleepOrShutdownTarget(Target, false);
	}

	ResetTargetCache();
	ActiveTargetReferences.Reset();
	ActiveTargetMap.Reset();
	UpdateSelectionRenderCacheOnTargetChange();
}

bool UGeometrySelectionManager::HasBeenShutDown() const
{
	return (ToolsContext == nullptr);
}


class FGeometrySelectionManager_SelectionTypeChange : public FToolCommandChange
{
public:
	EGeometryElementType FromElementType = EGeometryElementType::Face;
	EGeometryElementType ToElementType = EGeometryElementType::Face;

	UGeometrySelectionManager::EMeshTopologyMode FromTopologyMode = UGeometrySelectionManager::EMeshTopologyMode::None;
	UGeometrySelectionManager::EMeshTopologyMode ToTopologyMode = UGeometrySelectionManager::EMeshTopologyMode::None;

	/** Makes the change to the object */
	virtual void Apply(UObject* Object) override
	{
		// do the (default) red selectable lines/verts need to be rebuilt?
		// - ex: when moving from object mode to tri/vert/edge mode, or moving between vert and edge/face mode,
		//		 or between triangle and polygroup topology
		bool bRebuildSelectable = false;

		UGeometrySelectionManager* GeoSelectionManager = CastChecked<UGeometrySelectionManager>(Object);

		// removes existing Line/Point/Triangle Sets when moving between vertex and face/edge modes during redo
		if (
			(((ToElementType == EGeometryElementType::Vertex) && (FromElementType == EGeometryElementType::Edge || FromElementType == EGeometryElementType::Face))
			|| ((ToElementType == EGeometryElementType::Edge || ToElementType == EGeometryElementType::Face) && (FromElementType == EGeometryElementType::Vertex)))
			&& (ToTopologyMode != UGeometrySelectionManager::EMeshTopologyMode::None)
		)
		{
			GeoSelectionManager->RemoveAllSets();
			bRebuildSelectable = true;
		}

		if (FromTopologyMode != ToTopologyMode)
		{
			// when changing to Object mode, lines or verts need to be cleared
			if (ToTopologyMode == UGeometrySelectionManager::EMeshTopologyMode::None)
			{
				GeoSelectionManager->RemoveAllSets();
			}
			// in all other cases of changing topology modes, the lines/verts need to be rebuilt
			// uses a flag to preserve order of Removing Sets->Setting Element Type/Topo Mode -> Rebuild (when applicable for each step)
			else
			{
				bRebuildSelectable = true;
			}
			GeoSelectionManager->SetMeshTopologyModeInternal(ToTopologyMode);
		}
		
		if (FromElementType != ToElementType)
		{
			GeoSelectionManager->SetSelectionElementTypeInternal(ToElementType);
		}

		// if applicable, rebuilds lines/verts
		if (bRebuildSelectable)
		{
			GeoSelectionManager->RebuildSelectable();
		}
				
	}

	/** Reverts change to the object */
	virtual void Revert(UObject* Object) override
	{
		// do the (default) red selectable lines/verts need to be rebuilt?
		// - ex: when moving from object mode to tri/vert/edge mode, or moving between vert and edge/face mode
		//		 or between triangle and polygroup topology
		bool bRebuildSelectable = false;

		UGeometrySelectionManager* GeoSelectionManager = CastChecked<UGeometrySelectionManager>(Object);

		// removes existing Line/Point/Triangle Sets when moving between vertex and face/edge modes during undo
		if (
			(((ToElementType == EGeometryElementType::Vertex) && (FromElementType == EGeometryElementType::Edge || FromElementType == EGeometryElementType::Face))
			|| ((ToElementType == EGeometryElementType::Edge || ToElementType == EGeometryElementType::Face) && (FromElementType == EGeometryElementType::Vertex)))
			&& (FromTopologyMode != UGeometrySelectionManager::EMeshTopologyMode::None)
		)
		{
			GeoSelectionManager->RemoveAllSets();
			bRebuildSelectable = true;
			
		}

		if (FromTopologyMode != ToTopologyMode)
		{
			// when changing to Object mode, lines or verts need to be cleared
			if (FromTopologyMode == UGeometrySelectionManager::EMeshTopologyMode::None)
			{
				GeoSelectionManager->RemoveAllSets();
			}
			// in all other cases of changing topology modes, the lines/verts need to be rebuilt
			// uses a flag to preserve order of Removing Sets->Setting Element Type/Topo Mode -> Rebuild (when applicable for each step)
			else
			{
				bRebuildSelectable = true;
			}
			GeoSelectionManager->SetMeshTopologyModeInternal(FromTopologyMode);
		}
		
		if (FromElementType != ToElementType)
		{
			GeoSelectionManager->SetSelectionElementTypeInternal(FromElementType);
		}

		// if applicable, rebuilds lines/verts
		if (bRebuildSelectable)
		{
			GeoSelectionManager->RebuildSelectable();
		}
	}

	/** Describes this change (for debugging) */
	virtual FString ToString() const override { return TEXT("FGeometrySelectionManager_SelectionTypeChange"); }

	virtual bool HasExpired(UObject* Object) const override
	{
		UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
		return (Manager == nullptr || IsValid(Manager) == false || Manager->HasBeenShutDown());
	}
};



void UGeometrySelectionManager::SetSelectionElementTypeInternal(EGeometryElementType NewElementType)
{
	if (SelectionElementType != NewElementType)
	{
		SelectionElementType = NewElementType;

		for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
		{
			Target->Selection.ElementType = SelectionElementType;
			bool bEnableTopologyFilter = (Target->Selection.TopologyType == EGeometryTopologyType::Polygroup && Target->Selection.ElementType != EGeometryElementType::Vertex);
			Target->SelectionEditor->UpdateQueryConfig(GetCurrentSelectionQueryConfig(), bEnableTopologyFilter);
		}

		constexpr bool bMarkSelectableDirty = true;
		MarkRenderCachesDirty(bMarkSelectableDirty);
		ClearActivePreview();
	}
}


void UGeometrySelectionManager::SetSelectionElementType(EGeometryElementType NewElementType)
{
	if (SelectionElementType != NewElementType)
	{
		GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("ChangeElementType", "Selection Type"));

		if (HasSelection())
		{
			ClearSelection();
		}

		// We have to undo/redo the change to the selection type because if we want to 'undo' this later and restore
		// the current selection, we need the active element type to be correct. Note that it goes *after* the Clear
		// so that when we undo, we change to the correct type before we restore
		TUniquePtr<FGeometrySelectionManager_SelectionTypeChange> TypeChange = MakeUnique<FGeometrySelectionManager_SelectionTypeChange>();
		TypeChange->FromElementType = SelectionElementType;
		TypeChange->ToElementType = NewElementType;
		TypeChange->FromTopologyMode = TypeChange->ToTopologyMode = MeshTopologyMode;		// no-op
		GetTransactionsAPI()->AppendChange(this, MoveTemp(TypeChange), LOCTEXT("ChangeElementType", "Selection Type"));

		SetSelectionElementTypeInternal(NewElementType);

		GetTransactionsAPI()->EndUndoTransaction();
	}
}




void UGeometrySelectionManager::SetMeshTopologyModeInternal(EMeshTopologyMode NewTopologyMode)
{
	if (MeshTopologyMode != NewTopologyMode)
	{
		MeshTopologyMode = NewTopologyMode;

		for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
		{
			Target->Selection.TopologyType = GetSelectionTopologyType();
			bool bEnableTopologyFilter = (Target->Selection.TopologyType == EGeometryTopologyType::Polygroup && Target->Selection.ElementType != EGeometryElementType::Vertex);
			Target->SelectionEditor->UpdateQueryConfig(GetCurrentSelectionQueryConfig(), bEnableTopologyFilter);
		}

		MarkRenderCachesDirty();
		ClearActivePreview();
	}
}


void UGeometrySelectionManager::SetMeshTopologyMode(EMeshTopologyMode NewTopologyMode)
{
	if (MeshTopologyMode != NewTopologyMode)
	{
		GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("ChangeSelectionMode", "Selection Mode"));

		if (HasSelection())
		{
			ClearSelection();
		}

		// We have to undo/redo the change to the selection type because if we want to 'undo' this later and restore
		// the current selection, we need the active element type to be correct. Note that it goes *after* the Clear
		// so that when we undo, we change to the correct type before we restore
		TUniquePtr<FGeometrySelectionManager_SelectionTypeChange> TypeChange = MakeUnique<FGeometrySelectionManager_SelectionTypeChange>();
		TypeChange->FromTopologyMode = MeshTopologyMode;
		TypeChange->ToTopologyMode = NewTopologyMode;
		TypeChange->FromElementType = TypeChange->ToElementType = SelectionElementType;		// no-op
		GetTransactionsAPI()->AppendChange(this, MoveTemp(TypeChange),LOCTEXT("ChangeSelectionMode", "Selection Mode"));

		SetMeshTopologyModeInternal(NewTopologyMode);

		GetTransactionsAPI()->EndUndoTransaction();
	}
}

void UGeometrySelectionManager::RebuildSelectable() const
{
	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		CreateOrUpdateAllSets(CachedSelectableRenderElements[k], UnselectedParams);
	}
}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
void UGeometrySelectionManager::UpdateSoftSelectionOnSelectionModified()
{
	if (!HasActiveTargets())
	{
		return;
	}
	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	IGeometrySelector::FUpdateSoftSelectionConfig SoftSelectionConfig;
	SoftSelectionConfig.bRecomputeDistances = true;
	SoftSelectionConfig.bEnabled = bEnableSoftSelection;
	SoftSelectionConfig.Radius = SoftSelectionRadius;
	SoftSelectionConfig.DistanceMode = SoftSelectionDistanceMode;
	Target->Selector->UpdateSoftSelection(Target->Selection, SoftSelectionConfig);
	
	MarkRenderCachesDirty(false);
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS




void UGeometrySelectionManager::SetMeshSelectionTypeAndMode(EGeometryElementType NewElementType, EMeshTopologyMode NewTopologyMode, bool bConvertSelection)
{
	if (MeshTopologyMode != NewTopologyMode || SelectionElementType != NewElementType)
	{
		bool bHasSelection = HasSelection();

		// If we're converting selections, save the old one; we will re-add it after changing the mode
		TArray<FGeometrySelection> OldTypeSelections;
		if (bHasSelection && bConvertSelection)
		{
			for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
			{
				OldTypeSelections.Add(Target->Selection);
			}
		}

		ClearSelection();

		// clear preview geometry sets when in Object selection mode
		if (NewTopologyMode == EMeshTopologyMode::None && PreviewGeometry)
		{
			RemoveAllSets();
		}

		// removes existing Line/Point/Triangle Sets when moving between vertex and face/edge modes
		bool bRebuildSelectable = false;
		if (
			(((NewElementType == EGeometryElementType::Vertex) && (SelectionElementType == EGeometryElementType::Edge || SelectionElementType == EGeometryElementType::Face))
			|| ((NewElementType == EGeometryElementType::Edge || NewElementType == EGeometryElementType::Face) && (SelectionElementType == EGeometryElementType::Vertex)))
			&& (NewTopologyMode != EMeshTopologyMode::None)
		) // ensure lines not rebuilt when changing to object mode
		{
			RemoveAllSets();
			bRebuildSelectable = true;
		}

		GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("ChangeElementMethod", "Change Selection Method"));

		// We have to undo/redo the change to the selection type because if we want to 'undo' this later and restore
		// the current selection, we need the active element type to be correct. Note that it goes *after* the Clear
		// so that when we undo, we change to the correct type before we restore
		TUniquePtr<FGeometrySelectionManager_SelectionTypeChange> TypeChange = MakeUnique<FGeometrySelectionManager_SelectionTypeChange>();
		TypeChange->FromElementType = SelectionElementType;
		TypeChange->ToElementType = NewElementType;
		TypeChange->FromTopologyMode = MeshTopologyMode;
		TypeChange->ToTopologyMode = NewTopologyMode;
		GetTransactionsAPI()->AppendChange(this, MoveTemp(TypeChange), LOCTEXT("ChangeElementMethod", "Change Selection Method"));

		SetSelectionElementTypeInternal(NewElementType);
		SetMeshTopologyModeInternal(NewTopologyMode);

		if (bRebuildSelectable)
		{
			RebuildSelectable();
		}

		if (bHasSelection && bConvertSelection && ensure(ActiveTargetReferences.Num() == OldTypeSelections.Num()))
		{
			for (int32 TargetIdx = 0; TargetIdx < ActiveTargetReferences.Num(); ++TargetIdx)
			{
				// Add back the old selection, converted to the new mode/type
				TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[TargetIdx];
				FGeometrySelection InitialSelection = Target->Selection;
				FGeometrySelectionDelta AfterDelta;
				Target->Selector->UpdateSelectionFromSelection(OldTypeSelections[TargetIdx], true, *Target->SelectionEditor, FGeometrySelectionUpdateConfig{EGeometrySelectionChangeType::Replace}, &AfterDelta);
				if (!AfterDelta.IsEmpty())
				{
					TUniquePtr<FGeometrySelectionReplaceChange> NewSelectionChange = MakeUnique<FGeometrySelectionReplaceChange>();
					NewSelectionChange->Identifier = Target->TargetIdentifier;
					NewSelectionChange->After = Target->Selection;
					NewSelectionChange->Before = InitialSelection;
					GetTransactionsAPI()->AppendChange(this, MoveTemp(NewSelectionChange), LOCTEXT("ConvertSelection", "Convert Selection"));
				}
			}
		}

		GetTransactionsAPI()->EndUndoTransaction();

		OnSelectionModified.Broadcast();
	}
}


EGeometryTopologyType UGeometrySelectionManager::GetSelectionTopologyType() const
{
	if (MeshTopologyMode == EMeshTopologyMode::Polygroup)
	{
		return EGeometryTopologyType::Polygroup;
	}
	else
	{
		return EGeometryTopologyType::Triangle;
	}
}


FGeometrySelectionHitQueryConfig UGeometrySelectionManager::GetCurrentSelectionQueryConfig() const
{
	FGeometrySelectionHitQueryConfig HitQueryConfig;
	HitQueryConfig.TopologyType = GetSelectionTopologyType();
	HitQueryConfig.ElementType = GetSelectionElementType();
	HitQueryConfig.bOnlyVisible = true;
	HitQueryConfig.bHitBackFaces = GetHitBackFaces();
	return HitQueryConfig;
}


void UGeometrySelectionManager::SetHitBackFaces(const bool bInHitBackFaces)
{
	bHitBackFaces = bInHitBackFaces;
	for (const TSharedPtr<FGeometrySelectionTarget>& Target : ActiveTargetReferences)
	{
		FGeometrySelectionHitQueryConfig TargetQueryConfig = Target->SelectionEditor->GetQueryConfig();
		TargetQueryConfig.bHitBackFaces = bHitBackFaces;
		Target->SelectionEditor->UpdateQueryConfig(TargetQueryConfig, Target->SelectionEditor->GetIsTopologyIDFilteringEnabled());
	}
}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
void UGeometrySelectionManager::UpdateSoftSelectionOnConfigModified(bool bRecomputeDistances)
{
	if (!HasActiveTargets())
	{
		return;
	}
	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	IGeometrySelector::FUpdateSoftSelectionConfig Config;
	Config.bRecomputeDistances = bRecomputeDistances;
	Config.bEnabled = bEnableSoftSelection;
	Config.Radius = SoftSelectionRadius;
	Config.DistanceMode = SoftSelectionDistanceMode;
	Target->Selector->UpdateSoftSelection(Target->Selection, Config);

	MarkRenderCachesDirty(false);
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

void UGeometrySelectionManager::SetSoftSelectionEnabled(bool bEnable)
{
	if (bEnableSoftSelection != bEnable)
	{
		bEnableSoftSelection = bEnable;
		constexpr bool bRecomputeDistances = false;
		UpdateSoftSelectionOnConfigModified(bRecomputeDistances);
	}
}

void UGeometrySelectionManager::SetSoftSelectionRadius(double Radius)
{
	double ClampedRadius = FMath::Max(Radius, 0);
	if (SoftSelectionRadius != ClampedRadius)
	{
		SoftSelectionRadius = ClampedRadius;
		constexpr bool bRecomputeDistances = false;
		UpdateSoftSelectionOnConfigModified(bRecomputeDistances);
	}
}

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
void UGeometrySelectionManager::SetSoftSelectionDistanceMode(IGeometrySelector::ESoftSelectionDistanceMode InMode)
{
	if (SoftSelectionDistanceMode != InMode)
	{
		SoftSelectionDistanceMode = InMode;
		constexpr bool bRecomputeDistances = true;
		UpdateSoftSelectionOnConfigModified(bRecomputeDistances);
	}
}
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS


bool UGeometrySelectionManager::HasSelection() const
{
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->Selection.IsEmpty() == false)
		{
			return true;
		}
	}
	return false;
}


void UGeometrySelectionManager::GetActiveSelectionInfo(EGeometryTopologyType& TopologyTypeOut, EGeometryElementType& ElementTypeOut, int& NumTargetsOut, bool& bIsEmpty) const
{
	FGeometrySelectionHitQueryConfig Config = GetCurrentSelectionQueryConfig();
	TopologyTypeOut = Config.TopologyType;
	ElementTypeOut = Config.ElementType;
	NumTargetsOut = ActiveTargetReferences.Num();
	bIsEmpty = (NumTargetsOut == 0) || ActiveTargetReferences[0]->Selection.IsEmpty();
}


class FGeometrySelectionManager_ActiveTargetsChange : public FToolCommandChange
{
public:
	TArray<FGeometryIdentifier> TargetsBefore;
	TArray<FGeometryIdentifier> TargetsAfter;

	virtual void Apply(UObject* Object) override
	{
		CastChecked<UGeometrySelectionManager>(Object)->SetTargetsOnUndoRedo(TargetsAfter);
	}

	virtual void Revert(UObject* Object) override
	{
		CastChecked<UGeometrySelectionManager>(Object)->SetTargetsOnUndoRedo(TargetsBefore);
	}

	virtual FString ToString() const override { return TEXT("FGeometrySelectionManager_ActiveTargetsChange"); }

	virtual bool HasExpired(UObject* Object) const override
	{
		UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
		return (Manager == nullptr || IsValid(Manager) == false || Manager->HasBeenShutDown());
	}
};



class FGeometrySelectionManager_TargetLockStateChange : public FToolCommandChange
{
public:
	FGeometryIdentifier TargetIdentifier;
	bool bToState;

	virtual void Apply(UObject* Object) override
	{
		CastChecked<UGeometrySelectionManager>(Object)->SetTargetLockStateOnUndoRedo(TargetIdentifier, bToState);
	}

	virtual void Revert(UObject* Object) override
	{
		CastChecked<UGeometrySelectionManager>(Object)->SetTargetLockStateOnUndoRedo(TargetIdentifier, !bToState);
	}

	virtual FString ToString() const override { return TEXT("FGeometrySelectionManager_TargetLockStateChange"); }

	virtual bool HasExpired(UObject* Object) const override
	{
		UGeometrySelectionManager* Manager = Cast<UGeometrySelectionManager>(Object);
		return (Manager == nullptr || IsValid(Manager) == false || Manager->HasBeenShutDown());
	}
};



bool UGeometrySelectionManager::HasActiveTargets() const
{
	return (ActiveTargetReferences.Num() > 0);
}

bool UGeometrySelectionManager::ValidateSelectionState() const
{
	for (const TSharedPtr<FGeometrySelectionTarget>& Target : ActiveTargetReferences)
	{
		if (!Target.IsValid())
		{
			return false;
		}
		// if we have a stale target/selection object, selection state is not valid
		// Note: it is ok for the object to be explicitly null, just not stale
		if (Target->SelectionIdentifer.TargetObject.IsStale())
		{
			return false;
		}
		if (Target->TargetIdentifier.TargetObject.IsStale())
		{
			return false;
		}
	}
	return true;
}

void UGeometrySelectionManager::ClearActiveTargets()
{
	// generally at this point is it too late to clear the selection, because it will emit an
	// undo that cannot be redone later, because on redo the Targets will not exist yet
	// (one possibility would be to emit separate changes for when the target set is modified?? would that work w/ delete?? )
	ensure(HasSelection() == false);
	DiscardSavedSelection();

	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		SleepOrShutdownTarget(Target, false);
	}

	ActiveTargetReferences.Reset();
	ActiveTargetMap.Reset();

	UpdateSelectionRenderCacheOnTargetChange();

	OnSelectionModified.Broadcast();
}

bool UGeometrySelectionManager::AddActiveTarget(FGeometryIdentifier TargetIdentifier)
{
	if (ActiveTargetMap.Contains(TargetIdentifier))
	{
		return false;
	}

	// need to have a selector factory that can build for this target
	const IGeometrySelectorFactory* UseFactory = nullptr;
	for (const TUniquePtr<IGeometrySelectorFactory>& Factory : Factories)
	{
		if (Factory->CanBuildForTarget(TargetIdentifier))
		{
			UseFactory = Factory.Get();
			break;
		}
	}
	if (UseFactory == nullptr)
	{
		return false;
	}

	TSharedPtr<FGeometrySelectionTarget> SelectionTarget = GetCachedTarget(TargetIdentifier, UseFactory);
	if (SelectionTarget.IsValid() == false)
	{
		return false;
	}

	ActiveTargetMap.Add(TargetIdentifier, SelectionTarget);
	ActiveTargetReferences.Add(SelectionTarget);
	
	SelectionTarget->OnGeometryModifiedHandle = 
		SelectionTarget->Selector->GetOnGeometryModifed().AddUObject(this, &UGeometrySelectionManager::OnTargetGeometryModified);

	UpdateSelectionRenderCacheOnTargetChange();

	return true;
}


void UGeometrySelectionManager::SynchronizeActiveTargets(
	const TArray<FGeometryIdentifier>& DesiredActiveSet,
	TFunctionRef<void()> WillChangeActiveTargetsCallback)
{
	TArray<FGeometryIdentifier> Before = GetCurrentTargetIdentifiers();
	
	// currently only support single selection
	if (DesiredActiveSet.Num() == 1)
	{
		// if we do not already have this target, select it
		if ( ActiveTargetMap.Contains(DesiredActiveSet[0]) == false )
		{
			WillChangeActiveTargetsCallback();
			ClearActiveTargets();
			AddActiveTarget(DesiredActiveSet[0]);
		}
	}
	else
	{
		WillChangeActiveTargetsCallback();
		ClearActiveTargets();
	}

	TArray<FGeometryIdentifier> After = GetCurrentTargetIdentifiers();
	if (Before != After)
	{
		TUniquePtr<FGeometrySelectionManager_ActiveTargetsChange> Change = MakeUnique<FGeometrySelectionManager_ActiveTargetsChange>();
		Change->TargetsBefore = Before;
		Change->TargetsAfter = After;
		GetTransactionsAPI()->AppendChange(this, MoveTemp(Change), LOCTEXT("Change Targets", "Change Targets"));
	}
}


bool UGeometrySelectionManager::GetAnyCurrentTargetsLockable() const
{
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->Selector->IsLockable())
		{
			return true;
		}
	}
	return false;
}

bool UGeometrySelectionManager::GetAnyCurrentTargetsLocked() const
{
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->Selector->IsLockable() && Target->Selector->IsLocked())
		{
			return true;
		}
	}
	return false;
}

void UGeometrySelectionManager::SetCurrentTargetsLockState(bool bLocked)
{
	bool bInTransaction = false;

	bool bLockStateModified = false;
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->Selector->IsLockable() && Target->Selector->IsLocked() != bLocked)
		{
			Target->Selector->SetLockedState(bLocked);
			bLockStateModified = true;

			if (!bInTransaction)
			{
				GetTransactionsAPI()->BeginUndoTransaction(
					(bLocked) ? LOCTEXT("Lock Target", "Lock Target") : LOCTEXT("Unlock Target", "Unlock Target"));
				bInTransaction = true;
			}

			TUniquePtr<FGeometrySelectionManager_TargetLockStateChange> Change = MakeUnique<FGeometrySelectionManager_TargetLockStateChange>();
			Change->TargetIdentifier = Target->TargetIdentifier;
			Change->bToState = bLocked;
			GetTransactionsAPI()->AppendChange(this, MoveTemp(Change), 
				(bLocked) ? LOCTEXT("Lock Target", "Lock Target") : LOCTEXT("Unlock Target", "Unlock Target"));
		}
	}

	if (bLockStateModified)
	{
		ClearSelection();
	}

	if (bInTransaction)
	{
		GetTransactionsAPI()->EndUndoTransaction();
	}
}

void UGeometrySelectionManager::SetTargetLockStateOnUndoRedo(FGeometryIdentifier TargetIdentifier, bool bLocked)
{
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->TargetIdentifier == TargetIdentifier)
		{
			Target->Selector->SetLockedState(bLocked);
		}
	}
}


TArray<FGeometryIdentifier> UGeometrySelectionManager::GetCurrentTargetIdentifiers() const
{
	TArray<FGeometryIdentifier> Result;
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		Result.Add(Target->TargetIdentifier);
	}
	return Result;
}

void UGeometrySelectionManager::SetTargetsOnUndoRedo(TArray<FGeometryIdentifier> NewTargets)
{
	ClearActiveTargets();
	for (FGeometryIdentifier Identifier : NewTargets)
	{
		AddActiveTarget(Identifier);
	}
}



void UGeometrySelectionManager::SleepOrShutdownTarget(TSharedPtr<FGeometrySelectionTarget> Target, bool bForceShutdown)
{
	if (Target->Selector->SupportsSleep() && bForceShutdown == false)
	{
		if (Target->Selector->Sleep())
		{
			return;
		}
	}

	// if target cannot sleep or if sleeping failed, make sure it is not in the target
	// cache so that we do not try to restore it later
	TargetCache.Remove(Target->TargetIdentifier);

	Target->Selector->GetOnGeometryModifed().Remove(Target->OnGeometryModifiedHandle);
	Target->Selector->Shutdown();
}

TSharedPtr<UGeometrySelectionManager::FGeometrySelectionTarget> UGeometrySelectionManager::GetCachedTarget(FGeometryIdentifier TargetIdentifier, const IGeometrySelectorFactory* UseFactory)
{
	if (TargetCache.Contains(TargetIdentifier))
	{
		TSharedPtr<FGeometrySelectionTarget> FoundTarget = TargetCache[TargetIdentifier];
		FoundTarget->Selection.Reset();
		bool bRestored = FoundTarget->Selector->Restore();
		if (bRestored)
		{
			// ensure these are current, as they may have changed while Target was asleep
			FoundTarget->Selection.ElementType = GetSelectionElementType();
			FoundTarget->Selection.TopologyType = GetSelectionTopologyType();
			bool bEnableTopologyFilter = (FoundTarget->Selection.TopologyType == EGeometryTopologyType::Polygroup && FoundTarget->Selection.ElementType != EGeometryElementType::Vertex);
			FoundTarget->SelectionEditor->UpdateQueryConfig(GetCurrentSelectionQueryConfig(), bEnableTopologyFilter);

			return FoundTarget;
		}
		else
		{
			// if restore failed, something is wrong w/ TargetCache, remove this Target
			TargetCache.Remove(TargetIdentifier);
		}
	}

	// if we are in a situation where we don't have a cache, currently we need the Factory to exist?
	if (UseFactory == nullptr)
	{
		return nullptr;
	}

	// selector has to be built properly
	TUniquePtr<IGeometrySelector> Selector = UseFactory->BuildForTarget(TargetIdentifier);
	if (Selector.IsValid() == false)
	{
		return nullptr;
	}

	TSharedPtr<FGeometrySelectionTarget> SelectionTarget = MakeShared<FGeometrySelectionTarget>();
	SelectionTarget->Selector = MoveTemp(Selector);
	SelectionTarget->TargetIdentifier = TargetIdentifier;
	SelectionTarget->SelectionIdentifer = SelectionTarget->Selector->GetIdentifier();
	SelectionTarget->Selection.ElementType = GetSelectionElementType();
	SelectionTarget->Selection.TopologyType = GetSelectionTopologyType();

	SelectionTarget->SelectionEditor = MakeUnique<FGeometrySelectionEditor>();
	FGeometrySelectionHitQueryConfig HitQueryConfig = GetCurrentSelectionQueryConfig();
	bool bEnableTopologyFilter = (HitQueryConfig.TopologyType == EGeometryTopologyType::Polygroup && HitQueryConfig.ElementType != EGeometryElementType::Vertex);
	SelectionTarget->SelectionEditor->Initialize(&SelectionTarget->Selection, HitQueryConfig, bEnableTopologyFilter);

	if (SelectionTarget->Selector->SupportsSleep())
	{
		TargetCache.Add(TargetIdentifier, SelectionTarget);
	}
	
	SelectionTarget->Selector->GetOnSelectableElementsChanged().AddWeakLambda(this, [this]()
	{
		RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::UnselectedCachesDirty;
		RebuildSelectableRenderCache();
	});
	
	return SelectionTarget;
}

void UGeometrySelectionManager::ResetTargetCache()
{
	// SleepOrShutdownTarget may modify TargetCache
	TArray<TSharedPtr<FGeometrySelectionTarget>> ToShutdown;
	TargetCache.GenerateValueArray(ToShutdown);
	for (TSharedPtr<FGeometrySelectionTarget> Target : ToShutdown)
	{
		SleepOrShutdownTarget(Target, true);
	}
	TargetCache.Reset();
}



bool UGeometrySelectionManager::RayHitTest(
	const FRay3d& WorldRay,
	FInputRayHit& HitResultOut )
{
	HitResultOut = FInputRayHit();
	if (ActiveTargetReferences.Num() == 0)
	{
		return false;
	}

	IGeometrySelector::FWorldRayQueryInfo RayQueryInfo;
	RayQueryInfo.WorldRay = WorldRay;
	IToolsContextQueriesAPI* QueryAPI = this->ToolsContext->ToolManager->GetContextQueriesAPI();
	QueryAPI->GetCurrentViewState(RayQueryInfo.CameraState);

	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	FGeometrySelectionHitQueryConfig HitQueryConfig = GetCurrentSelectionQueryConfig();
	bool bHit = Target->Selector->RayHitTest(RayQueryInfo, HitQueryConfig, HitResultOut);
	if (bHit)
	{
		HitResultOut.HitOwner = ActiveTargetReferences[0].Get();
		HitResultOut.HitObject = (ActiveTargetReferences[0]->TargetIdentifier.TargetType == FGeometryIdentifier::ETargetType::PrimitiveComponent) ?
			ActiveTargetReferences[0]->TargetIdentifier.TargetObject : nullptr;
	}

	// currently only going to support one object, not sure how to support more yet...
	return bHit;
}


void UGeometrySelectionManager::ClearSelection(bool bSaveSelectionBeforeClear)
{
	bool bTransact = true;
	
#if WITH_EDITORONLY_DATA
	// Cannot update the selection if we're already in a transaction (can happen e.g. when we undo out of a tool)
	if (GIsTransacting)
	{
		bTransact = false;
	}
#endif
	
	if (!HasSelection())
	{
		return;
	}

	if (bSaveSelectionBeforeClear)
	{
		SaveCurrentSelection();
	}

	if (bTransact)
	{
		GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("ClearSelection", "Clear Selection"));
	}

	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		FGeometrySelectionDelta ClearDelta;
		Target->SelectionEditor->ClearSelection(ClearDelta);
		if (ClearDelta.IsEmpty() == false)
		{
			TUniquePtr<FGeometrySelectionDeltaChange> ClearChange = MakeUnique<FGeometrySelectionDeltaChange>();
			ClearChange->Identifier = Target->TargetIdentifier;
			ClearChange->Delta = MoveTemp(ClearDelta);

			if (bTransact)
			{
				GetTransactionsAPI()->AppendChange(this, MoveTemp(ClearChange), LOCTEXT("ClearSelection", "Clear Selection"));
			}
		}
	}

	if (bTransact)
	{
		GetTransactionsAPI()->EndUndoTransaction();
	}

	MarkRenderCachesDirty(false);
	OnSelectionModified.Broadcast();
}


void UGeometrySelectionManager::UpdateSelectionViaRaycast(
	const FRay3d& WorldRay,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut
)
{
	ResultOut.bSelectionModified = false;

	if (ActiveTargetReferences.Num() == 0)
	{
		return;
	}

	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	IGeometrySelector::FWorldRayQueryInfo RayQueryInfo;
	RayQueryInfo.WorldRay = WorldRay;
	IToolsContextQueriesAPI* QueryAPI = this->ToolsContext->ToolManager->GetContextQueriesAPI();
	QueryAPI->GetCurrentViewState(RayQueryInfo.CameraState);

	Target->Selector->UpdateSelectionViaRaycast(
		RayQueryInfo, *Target->SelectionEditor, UpdateConfig, ResultOut
	);

	if (ResultOut.bSelectionModified)
	{
		TUniquePtr<FGeometrySelectionDeltaChange> DeltaChange = MakeUnique<FGeometrySelectionDeltaChange>();
		DeltaChange->Identifier = Target->TargetIdentifier;
		DeltaChange->Delta = ResultOut.SelectionDelta;

		GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("UpdateSelectionViaRaycast", "Change Selection"));
		GetTransactionsAPI()->AppendChange(this, MoveTemp(DeltaChange), LOCTEXT("UpdateSelectionViaRaycast", "Change Selection"));
		GetTransactionsAPI()->EndUndoTransaction();

		MarkRenderCachesDirty(false);
		OnSelectionModified.Broadcast();
	}
	else if (ResultOut.bSelectionMissed && UpdateConfig.ChangeType == EGeometrySelectionChangeType::Replace)
	{
		ClearSelection();
	}
}


void UGeometrySelectionManager::UpdateSelectionViaConvex(
	const FConvexVolume& ConvexVolume,
	const FGeometrySelectionUpdateConfig& UpdateConfig,
	FGeometrySelectionUpdateResult& ResultOut )
{
	ResultOut.bSelectionModified = false;

	if (ActiveTargetReferences.Num() == 0)
	{
		return;
	}

	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	IGeometrySelector::FWorldShapeQueryInfo ShapeQueryInfo;
	ShapeQueryInfo.Convex = ConvexVolume;
	IToolsContextQueriesAPI* QueryAPI = this->ToolsContext->ToolManager->GetContextQueriesAPI();
	QueryAPI->GetCurrentViewState(ShapeQueryInfo.CameraState);

	Target->Selector->UpdateSelectionViaShape(
		ShapeQueryInfo, *Target->SelectionEditor, UpdateConfig, ResultOut );

	if (ResultOut.bSelectionModified)
	{
		TUniquePtr<FGeometrySelectionDeltaChange> DeltaChange = MakeUnique<FGeometrySelectionDeltaChange>();
		DeltaChange->Identifier = Target->TargetIdentifier;
		DeltaChange->Delta = ResultOut.SelectionDelta;

		GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("UpdateSelectionViaConvex", "Change Selection"));
		GetTransactionsAPI()->AppendChange(this, MoveTemp(DeltaChange), LOCTEXT("UpdateSelectionViaConvex", "Change Selection"));
		GetTransactionsAPI()->EndUndoTransaction();

		MarkRenderCachesDirty(false);
		OnSelectionModified.Broadcast();
	}
	else if (ResultOut.bSelectionMissed && UpdateConfig.ChangeType == EGeometrySelectionChangeType::Replace)
	{
		ClearSelection();
	}
}



bool UGeometrySelectionManager::CanBeginTrackedSelectionChange() const
{
	return ActiveTargetReferences.Num() > 0 && bInTrackedSelectionChange == false;
}

bool UGeometrySelectionManager::BeginTrackedSelectionChange(FGeometrySelectionUpdateConfig UpdateConfig, bool bClearOnBegin)
{
	if (ensureMsgf(CanBeginTrackedSelectionChange(), TEXT("Cannot begin Selection Change - validate CanBeginTrackedSelectionChange() before calling BeginTrackedSelectionChange()")) == false)
	{
		return false;
	}

	GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("ChangeSelection", "Change Selection"));
	bInTrackedSelectionChange = true;

	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	ActiveTrackedUpdateConfig = UpdateConfig;
	bSelectionModifiedDuringTrackedChange = false;

	// if we are doing a Replace selection, we want to clear on initialization...
	InitialTrackedDelta = FGeometrySelectionDelta();
	if (bClearOnBegin)
	{
		Target->SelectionEditor->ClearSelection(InitialTrackedDelta);
		bSelectionModifiedDuringTrackedChange = true;
	}

	ActiveTrackedSelection = Target->Selection;
	ActiveTrackedDelta = FGeometrySelectionDelta();

	if (bClearOnBegin && InitialTrackedDelta.IsEmpty() == false)
	{
		MarkRenderCachesDirty(false);
		OnSelectionModified.Broadcast();
	}

	return true;
}

void UGeometrySelectionManager::AccumulateSelectionUpdate_Raycast(
	const FRay3d& WorldRay,
	FGeometrySelectionUpdateResult& ResultOut )
{
	if (!ensure(bInTrackedSelectionChange)) return;

	// currently only going to support one object, not sure how to support more yet...
	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	IGeometrySelector::FWorldRayQueryInfo RayQueryInfo;
	RayQueryInfo.WorldRay = WorldRay;
	IToolsContextQueriesAPI* QueryAPI = this->ToolsContext->ToolManager->GetContextQueriesAPI();
	QueryAPI->GetCurrentViewState(RayQueryInfo.CameraState);

	Target->Selector->UpdateSelectionViaRaycast(
		RayQueryInfo, *Target->SelectionEditor, ActiveTrackedUpdateConfig, ResultOut );

	if (ResultOut.bSelectionModified)
	{
		bSelectionModifiedDuringTrackedChange = true;
		ActiveTrackedDelta.Added.Append( ResultOut.SelectionDelta.Added );
		ActiveTrackedDelta.Removed.Append( ResultOut.SelectionDelta.Removed );

		MarkRenderCachesDirty(false);
		OnSelectionModified.Broadcast();
	}
}

void UGeometrySelectionManager::EndTrackedSelectionChange()
{
	if ( ensure(bInTrackedSelectionChange) )
	{
		if (bSelectionModifiedDuringTrackedChange)
		{
			FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

			if (InitialTrackedDelta.IsEmpty() == false)
			{
				TUniquePtr<FGeometrySelectionDeltaChange> InitialDeltaChange = MakeUnique<FGeometrySelectionDeltaChange>();
				InitialDeltaChange->Identifier = Target->TargetIdentifier;
				InitialDeltaChange->Delta = MoveTemp(InitialTrackedDelta);
				GetTransactionsAPI()->AppendChange(this, MoveTemp(InitialDeltaChange), LOCTEXT("ChangeSelection", "Change Selection"));
			}

			if (ActiveTrackedDelta.IsEmpty() == false)
			{
				TUniquePtr<FGeometrySelectionDeltaChange> AccumDeltaChange = MakeUnique<FGeometrySelectionDeltaChange>();
				AccumDeltaChange->Identifier = Target->TargetIdentifier;
				AccumDeltaChange->Delta = MoveTemp(ActiveTrackedDelta);
				GetTransactionsAPI()->AppendChange(this, MoveTemp(AccumDeltaChange), LOCTEXT("ChangeSelection", "Change Selection"));
			}
		}

		GetTransactionsAPI()->EndUndoTransaction();
		bInTrackedSelectionChange = false;
	}
}


bool UGeometrySelectionManager::SetSelectionForComponent(UPrimitiveComponent* Component, const FGeometrySelection& NewSelection)
{
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->TargetIdentifier.TargetObject == Component)
		{
			FGeometrySelection InitialSelection = Target->Selection;
			FGeometrySelectionDelta AfterDelta;
			Target->Selector->UpdateSelectionFromSelection(
				NewSelection, true, *Target->SelectionEditor, 
				FGeometrySelectionUpdateConfig{EGeometrySelectionChangeType::Replace}, &AfterDelta);
			if ( AfterDelta.IsEmpty() == false )
			{
				TUniquePtr<FGeometrySelectionReplaceChange> NewSelectionChange = MakeUnique<FGeometrySelectionReplaceChange>();
				NewSelectionChange->Identifier = Target->TargetIdentifier; //Target->Selector->GetIdentifier();
				NewSelectionChange->After = Target->Selection;
				NewSelectionChange->Before = InitialSelection;
				GetTransactionsAPI()->AppendChange(this, MoveTemp(NewSelectionChange), LOCTEXT("NewSelection", "New Selection"));

				MarkRenderCachesDirty(false);
				OnSelectionModified.Broadcast();
			}
			return true;
		}
	}
	return false;
}

void UGeometrySelectionManager::SaveCurrentSelection()
{
	SavedSelection.Reset();
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		SavedSelection.Targets.Add(Target->TargetIdentifier.TargetObject);
		SavedSelection.Selections.Add(Target->Selection);
	}
}

bool UGeometrySelectionManager::RestoreSavedSelection()
{
#if WITH_EDITORONLY_DATA
	// Cannot update the selection if we're already in a transaction (can happen e.g. when we undo out of a tool)
	if (GIsTransacting)
	{
		DiscardSavedSelection();
		return false;
	}
#endif

	check(SavedSelection.Targets.Num() == SavedSelection.Selections.Num());
	GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("RestoreSelection", "Restore Selection"));

	bool bSuccess = true;
	for (int32 TargetIdx = 0; TargetIdx < SavedSelection.Targets.Num(); ++TargetIdx)
	{
		if (!SavedSelection.Targets[TargetIdx].IsValid())
		{
			bSuccess = false;
			continue;
		}
		const FGeometrySelection& NewSelection = SavedSelection.Selections[TargetIdx];
		bool bFound = false;
		for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
		{
			if (SavedSelection.Targets[TargetIdx] == Target->TargetIdentifier.TargetObject)
			{
				FGeometrySelection InitialSelection = Target->Selection;
				FGeometrySelectionDelta AfterDelta;
				Target->Selector->UpdateSelectionFromSelection(
					NewSelection, true, *Target->SelectionEditor,
					FGeometrySelectionUpdateConfig{ EGeometrySelectionChangeType::Replace }, &AfterDelta);
				if (AfterDelta.IsEmpty() == false)
				{
					TUniquePtr<FGeometrySelectionReplaceChange> NewSelectionChange = MakeUnique<FGeometrySelectionReplaceChange>();
					NewSelectionChange->Identifier = Target->TargetIdentifier;
					NewSelectionChange->After = Target->Selection;
					NewSelectionChange->Before = InitialSelection;
					GetTransactionsAPI()->AppendChange(this, MoveTemp(NewSelectionChange), LOCTEXT("RestoreSelection", "Restore Selection"));

					MarkRenderCachesDirty(false);
					OnSelectionModified.Broadcast();
				}
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			bSuccess = false;
		}
	}

	GetTransactionsAPI()->EndUndoTransaction();

	DiscardSavedSelection();

	return bSuccess;
}

void UGeometrySelectionManager::DiscardSavedSelection()
{
	SavedSelection.Empty();
}

bool UGeometrySelectionManager::HasSavedSelection()
{
	return !SavedSelection.Selections.IsEmpty();
}


bool UGeometrySelectionManager::UpdateSelectionPreviewViaRaycast(
	const FRay3d& WorldRay )
{
	if (ActiveTargetReferences.Num() == 0)
	{
		return false;
	}
	
	// currently only going to support one object, not sure how to support more yet...

	FGeometrySelectionTarget* Target = ActiveTargetReferences[0].Get();

	IGeometrySelector::FWorldRayQueryInfo RayQueryInfo;
	RayQueryInfo.WorldRay = WorldRay;
	IToolsContextQueriesAPI* QueryAPI = this->ToolsContext->ToolManager->GetContextQueriesAPI();
	QueryAPI->GetCurrentViewState(RayQueryInfo.CameraState);

	FGeometrySelectionPreview NewPreview( *(Target->SelectionEditor) );
	Target->Selector->GetConditionalSelectionPreviewForRaycast( RayQueryInfo, NewPreview, &Target->Selection );
	if ( ! UE::Geometry::AreSelectionsIdentical(NewPreview.PreviewSelection, ActivePreviewSelection) )
	{
		ActivePreviewSelection = MoveTemp(NewPreview.PreviewSelection);
		
		// Initialize [Un]SelectedActivePreviewSelection(s) so that they are of the correct Topology and Geometry type, then clear them
		SelectedActivePreviewSelection = MoveTemp(NewPreview.PreviewSelection);
		UnselectedActivePreviewSelection = MoveTemp(NewPreview.PreviewSelection);
		SelectedActivePreviewSelection.Reset();
		UnselectedActivePreviewSelection.Reset();

		if (MeshTopologyMode == EMeshTopologyMode::Polygroup)
		{
			// Get all polygroup IDs in current preview selection
			TSet<uint32> SelectedGroupIDs;
			for (const uint64 ID : Target->Selection.Selection)
			{
				SelectedGroupIDs.Add(FGeoSelectionID(ID).TopologyID);
			}

			// Get GroupID of active preview selection (hovered items)
			for (const uint64 ID : ActivePreviewSelection.Selection)
			{
				const uint32 TopoID = FGeoSelectionID(ID).TopologyID;

				// add to selection according to if an element with the GroupID is already selected
				if (SelectedGroupIDs.Contains(TopoID))
				{
					SelectedActivePreviewSelection.Selection.Add(ID);
				}
				else
				{
					UnselectedActivePreviewSelection.Selection.Add(ID);
				}
			}
		}
		// Triangle Topology mode is more straightforward
		else if (MeshTopologyMode == EMeshTopologyMode::Triangle)
		{
			for (const uint64 ID : ActivePreviewSelection.Selection)
			{
				if (Target->Selection.Selection.Contains(ID))
				{
					SelectedActivePreviewSelection.Selection.Add(ID);
				}
				else
				{
					UnselectedActivePreviewSelection.Selection.Add(ID);
				}
			}
		}
		CachedSelectedPreviewRenderElements.Reset();
		CachedUnselectedPreviewRenderElements.Reset();

		RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::PreviewCachesDirty;

		RemoveSets(HoverOverSelectedParams.Identifiers);
		RemoveSets(HoverOverUnselectedParams.Identifiers);
	}

	return (ActivePreviewSelection.IsEmpty() == false);
}

void UGeometrySelectionManager::ClearSelectionPreview()
{
	ClearActivePreview();
}

bool UGeometrySelectionManager::GetSelectionBounds(FGeometrySelectionBounds& BoundsOut) const
{
	BoundsOut = FGeometrySelectionBounds();

	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		Target->Selector->AccumulateSelectionBounds(Target->Selection, BoundsOut, true);
	}

	return (BoundsOut.WorldBounds.IsEmpty() == false);
}


void UGeometrySelectionManager::GetSelectionWorldFrame(FFrame3d& SelectionFrame) const
{
	SelectionFrame = FFrame3d();
	if (HasSelection())
	{
		// only handling this case for now
		//if (ActiveTargetReferences.Num() == 1)
		TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[0];
		Target->Selector->GetSelectionFrame(Target->Selection, SelectionFrame, true);
	}
}

void UGeometrySelectionManager::GetTargetWorldFrame(FFrame3d& SelectionFrame) const
{
	SelectionFrame = FFrame3d();
	if (HasSelection())
	{
		// only handling one target for now
		//if (ActiveTargetReferences.Num() == 1)
		TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[0];
		Target->Selector->GetTargetFrame(Target->Selection, SelectionFrame);
	}
}

bool UGeometrySelectionManager::HasSelectionForComponent(UPrimitiveComponent* Component) const
{
	if (HasSelection())
	{
		for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
		{
			if (Target->TargetIdentifier.TargetObject == Component)
			{
				return Target->Selection.IsEmpty();
			}
		}
	}
	return false;
}

bool UGeometrySelectionManager::GetSelectionForComponent(UPrimitiveComponent* Component, FGeometrySelection& SelectionOut) const
{
	if (HasSelection())
	{
		for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
		{
			if (Target->TargetIdentifier.TargetObject == Component)
			{
				SelectionOut = Target->Selection;
				return ! Target->Selection.IsEmpty();
			}
		}
	}
	return false;
}



bool UGeometrySelectionManager::BeginTransformation()
{
	if (!ensure(IsInActiveTransformation() == false))
	{
		return false;
	}
	if (HasSelection() == false)
	{
		return false;
	}

	bool bHaveTransformers = false;
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		IGeometrySelectionTransformer* Transformer = Target->Selector->InitializeTransformation(Target->Selection);
		if (Transformer != nullptr)
		{
			Transformer->BeginTransform(Target->Selection);
			ActiveTransformations.Add(Transformer);
			bHaveTransformers = true;
		}
		else
		{
			ActiveTransformations.Add(nullptr);
		}
	}

	if (!bHaveTransformers)
	{
		ActiveTransformations.Reset();
		return false;
	}

	return true;
}

void UGeometrySelectionManager::UpdateTransformation(
	TFunctionRef<FVector3d(int32 VertexID, const FVector3d&, const FTransform&)> PositionTransformFunc)
{
	auto PositionTransformFuncWithUnusedWeight = [&PositionTransformFunc](int32 VertexID, double Weight, const FVector3d& Position, const FTransform& Transform)
		{
			return PositionTransformFunc(VertexID, Position, Transform);
		};
	UpdateTransformation(PositionTransformFuncWithUnusedWeight);
}

void UGeometrySelectionManager::UpdateTransformation( 
	TFunctionRef<FVector3d(int32 VertexID, double Weight, const FVector3d&, const FTransform&)> PositionTransformFunc )
{
	if ( ! ensure(IsInActiveTransformation()) ) return;

	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		if (ActiveTransformations[k] != nullptr)
		{
			ActiveTransformations[k]->UpdateTransform(PositionTransformFunc);
		}
	}

	RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::SelectionCachesDirty;
	RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::SoftSelectionCachesDirty;
}

void UGeometrySelectionManager::EndTransformation()
{
	if ( ! ensure(IsInActiveTransformation()) ) return;

	// Take ownership of ActiveTransformations locally before iterating. If EndTransform synchronously pumps
	// messages (e.g. a slow-task dialog shown during PostEditChange triggers focus loss, which routes through
	// UInputRouter::ForceTerminateAll back into OnEndGizmoTransform), any reentrant call to EndTransformation
	// will see IsInActiveTransformation() return false and bail at the ensure above.
	TArray<IGeometrySelectionTransformer*> LocalTransformations = MoveTemp(ActiveTransformations);

	GetTransactionsAPI()->BeginUndoTransaction(LOCTEXT("EndTransformation", "Transform Selection"));

	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		if (LocalTransformations[k] != nullptr)
		{
			LocalTransformations[k]->EndTransform(GetTransactionsAPI());

			ActiveTargetReferences[k]->Selector->ShutdownTransformation(LocalTransformations[k]);
		}
	}

	GetTransactionsAPI()->EndUndoTransaction();

	RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::SelectionCachesDirty;
	RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::SoftSelectionCachesDirty;
}

void UGeometrySelectionManager::UpdateTransformation(
	const FTransform& StartTransform,
	const FVector& TranslationToApply,
	const FQuat& RotationToApply,
	const FVector& ScaleToApply,
	bool bWorldSpaceScale
	)
{
	if (bWorldSpaceScale)
	{
		// For a world frame gizmo, the scaling needs to happen in world aligned gizmo space, but the 
		// rotation is still encoded in the local gizmo frame change.
		UpdateTransformation([&StartTransform, &TranslationToApply, &RotationToApply, &ScaleToApply](int32 VertexID, double Weight, const FVector3d& InitialPosition, const FTransform3d& WorldTransform)
			{
				FVector3d PosLocal = InitialPosition;
				FVector3d PosWorld = WorldTransform.TransformPosition(PosLocal);
				FVector3d PosWorldGizmo = PosWorld - StartTransform.GetLocation();

				FVector3d NewPosWorld = FQuat::Slerp(FQuat::Identity, RotationToApply, Weight) * (((ScaleToApply * Weight ) + 1.0f) * PosWorldGizmo) + (StartTransform.GetLocation() + TranslationToApply * Weight);
				FVector3d NewPosLocal = WorldTransform.InverseTransformPosition(NewPosWorld);
				return NewPosLocal;
			});	
	}
	else
	{
		UpdateTransformation([&StartTransform, &TranslationToApply, &RotationToApply, &ScaleToApply](int32 VertexID, double Weight, const FVector3d& InitialPosition, const FTransform3d& WorldTransform)
			{
				// For a local gizmo, we just get the coordinates in the original frame, scale in that frame,
				// then interpret them as coordinates in the new frame.
				FVector3d PosLocal = InitialPosition;
				FVector3d PosWorld = WorldTransform.TransformPosition(PosLocal);
				FVector3d PosGizmo = StartTransform.InverseTransformPosition(PosWorld);
				PosGizmo = ((ScaleToApply * Weight ) + 1.0f) * PosGizmo;
				FVector3d NewPosWorld = FQuat::Slerp(FQuat::Identity, RotationToApply, Weight) * StartTransform.GetRotation() * PosGizmo + (StartTransform.GetLocation() + TranslationToApply * Weight);
				FVector3d NewPosLocal = WorldTransform.InverseTransformPosition(NewPosWorld);
				return NewPosLocal;
			});
	}	
}



bool UGeometrySelectionManager::CanExecuteSelectionCommand(UGeometrySelectionEditCommand* Command)
{
	if (SelectionArguments == nullptr)
	{
		SelectionArguments = NewObject<UGeometrySelectionEditCommandArguments>();
	}

	bool bCanExecute = true;
	bool bHaveSelections = false;
	ProcessActiveSelections([&](FGeometrySelectionHandle Handle)
	{
		SelectionArguments->SelectionHandle = Handle;
		SelectionArguments->SetTransactionsAPI(TransactionsAPI);
		bCanExecute = bCanExecute && Command->CanExecuteCommand(SelectionArguments);
		bHaveSelections = true;
	});

	return ( bHaveSelections || (Command->AllowEmptySelection() && HasActiveTargets() && MeshTopologyMode != EMeshTopologyMode::None ) ) && bCanExecute;
}

void UGeometrySelectionManager::ExecuteSelectionCommand(UGeometrySelectionEditCommand* Command)
{
	if (SelectionArguments == nullptr)
	{
		SelectionArguments = NewObject<UGeometrySelectionEditCommandArguments>();
	}

	// open transaction to wrap the entire set of Commands and selection changes
	FText CommandText = Command->GetCommandShortString();
	GetTransactionsAPI()->BeginUndoTransaction(CommandText);

	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->Selection.IsEmpty() && Command->AllowEmptySelection() == false) continue;

		// TODO: can use Command->IsModifySelectionCommand() to check if this is a command that only affects selection
		// and not geometry. In that case we can skip the intermediate clear-selection and emit a single change.

		// When initially executing the command, we do not clear the selection, because we pass it to the command.
		// However, when we later *undo* any changes emitted by the command, we need to restore the selection aftewards.
		// So we emit a clearing change here, so that undo un-clears. 
		// When we later Redo, it is also necessary to Clear as otherwise an invalid Selection might hang around.
		// Note that this must happen *before* the Command. The Command will not be re-executed, only its emitted Changes,
		// so it will not be holding onto the active Selection on Redo later
		// (if that becomes necessary, this sequence of changes will need to become more complicated....)
		TUniquePtr<FGeometrySelectionReplaceChange> ClearChange = MakeUnique<FGeometrySelectionReplaceChange>();
		ClearChange->Identifier = Target->TargetIdentifier; //Target->Selector->GetIdentifier();
		ClearChange->Before = Target->Selection;
		ClearChange->After.InitializeTypes(ClearChange->Before);
		GetTransactionsAPI()->AppendChange(this, MoveTemp(ClearChange), LOCTEXT("ClearSelection", "Clear Selection"));

		// q: we could clear the selection here, and pass the Handle a copy. Perhaps safer?
		UInteractiveCommandResult* ResultPtr = nullptr;
		SelectionArguments->SelectionHandle = FGeometrySelectionHandle{ Target->Selector->GetIdentifier(), &Target->Selection, Target->Selector.Get() };
		SelectionArguments->ElementType = SelectionElementType;
		SelectionArguments->TopologyMode = GetSelectionTopologyType();
		SelectionArguments->SetTransactionsAPI(TransactionsAPI);

		if (Command->CanExecuteCommand(SelectionArguments))
		{
			Command->ExecuteCommand(SelectionArguments, &ResultPtr);
		}

		// actually clear selection after executing command. 
		FGeometrySelectionDelta ClearDelta;
		Target->SelectionEditor->ClearSelection(ClearDelta);

		// if selection returned a result, and it was a non-empty selection, select it
		if ( UGeometrySelectionEditCommandResult* SelectionResult = Cast<UGeometrySelectionEditCommandResult>(ResultPtr) )
		{
			if (SelectionResult->OutputSelection.IsEmpty() == false )
			{
				FGeometrySelectionDelta AfterDelta;
				Target->Selector->UpdateSelectionFromSelection(
					SelectionResult->OutputSelection, true, *Target->SelectionEditor, 
					FGeometrySelectionUpdateConfig{EGeometrySelectionChangeType::Add}, &AfterDelta);
				if (Target->Selection.IsEmpty() == false)
				{
					TUniquePtr<FGeometrySelectionReplaceChange> NewSelectionChange = MakeUnique<FGeometrySelectionReplaceChange>();
					NewSelectionChange->Identifier = Target->TargetIdentifier; //Target->Selector->GetIdentifier();
					NewSelectionChange->After = Target->Selection;
					NewSelectionChange->Before.InitializeTypes(Target->Selection);
					GetTransactionsAPI()->AppendChange(this, MoveTemp(NewSelectionChange), LOCTEXT("NewSelection", "New Selection"));
				}
			}
		}
	}

	GetTransactionsAPI()->EndUndoTransaction();

	// assume marking render caches (except selectable) is true for now
	MarkRenderCachesDirty(false);
	OnSelectionModified.Broadcast();
}


void UGeometrySelectionManager::ProcessActiveSelections(TFunctionRef<void(FGeometrySelectionHandle)> ProcessFunc)
{
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		if (Target->Selection.IsEmpty() == false)
		{
			FGeometrySelectionHandle Handle;
			Handle.Selection = & Target->Selection;
			Handle.Identifier = Target->Selector->GetIdentifier();
			Handle.Selector = Target->Selector.Get();
			ProcessFunc(Handle);
		}
	}
}









void UGeometrySelectionManager::ApplyChange(IGeometrySelectionChange* Change)
{
	// We should not get here because selection changes should have been expired.
	if ( ! ensure(HasBeenShutDown() == false) )
	{
		return;
	}

	FGeometryIdentifier Identifer = Change->GetIdentifier();

	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		if (ActiveTargetReferences[k]->TargetIdentifier == Identifer)
		{
			FGeometrySelectionDelta ApplyDelta;
			Change->ApplyChange( ActiveTargetReferences[k]->SelectionEditor.Get(), ApplyDelta);

			if (ApplyDelta.IsEmpty() == false)
			{
				MarkRenderCachesDirty(false);
				OnSelectionModified.Broadcast();
			}

			break;
		}
	}
}


void UGeometrySelectionManager::RevertChange(IGeometrySelectionChange* Change)
{
	// We should not get here because selection changes should have been expired.
	if ( ! ensure(HasBeenShutDown() == false) )
	{
		return;
	}

	FGeometryIdentifier Identifer = Change->GetIdentifier();

	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		if (ActiveTargetReferences[k]->TargetIdentifier == Identifer) 
		{
			FGeometrySelectionDelta RevertDelta;
			Change->RevertChange( ActiveTargetReferences[k]->SelectionEditor.Get(), RevertDelta);

			if (RevertDelta.IsEmpty() == false)
			{
				MarkRenderCachesDirty(false);
				OnSelectionModified.Broadcast();
			}

			break;
		}
	}
}


void UGeometrySelectionManager::OnTargetGeometryModified(IGeometrySelector* Selector)
{
	CachedSelectableRenderElements.Reset();
	CachedSelectionRenderElements.Reset();
	CachedSoftSelectionRenderElements.Reset();
	
	CachedSelectableRenderElements.SetNum(ActiveTargetReferences.Num());
	CachedSelectionRenderElements.SetNum(ActiveTargetReferences.Num());
	CachedSoftSelectionRenderElements.SetNum(ActiveTargetReferences.Num());

	RemoveSets(UnselectedParams.Identifiers);
	RemoveSets(SelectedParams.Identifiers);
	RemoveSets(SoftSelectedParams.Identifiers);
	
	MarkRenderCachesDirty();
	ClearActivePreview();
}


void UGeometrySelectionManager::UpdateSelectionRenderCacheOnTargetChange()
{
	CachedSelectionRenderElements.Reset();
	CachedSelectableRenderElements.Reset();
	CachedSoftSelectionRenderElements.Reset();
	
	CachedSelectionRenderElements.SetNum(ActiveTargetReferences.Num());
	CachedSelectableRenderElements.SetNum(ActiveTargetReferences.Num());
	CachedSoftSelectionRenderElements.SetNum(ActiveTargetReferences.Num());

	RemoveSets(SelectedParams.Identifiers);
	RemoveSets(UnselectedParams.Identifiers);
	RemoveSets(SoftSelectedParams.Identifiers);
	
	MarkRenderCachesDirty();
	ClearActivePreview();
}

void UGeometrySelectionManager::MarkRenderCachesDirty(bool bMarkSelectableDirty)
{
	if (bMarkSelectableDirty)
	{
		RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::UnselectedCachesDirty;
	}
	RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::PreviewCachesDirty | EEnumerateRenderCachesDirtyFlags::SelectionCachesDirty | EEnumerateRenderCachesDirtyFlags::SoftSelectionCachesDirty;
}

void UGeometrySelectionManager::RebuildSelectionRenderCaches()
{
	RebuildSelectionRenderCache();
	RebuildSelectableRenderCache();
	RebuildPreviewRenderCache();
	RebuildSoftSelectionRenderCache();
}

void UGeometrySelectionManager::ClearActivePreview()
{
	ActivePreviewSelection.Reset();
	SelectedActivePreviewSelection.Reset();
	UnselectedActivePreviewSelection.Reset();
	CachedSelectedPreviewRenderElements.Reset();
	CachedUnselectedPreviewRenderElements.Reset();
	
	RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::PreviewCachesDirty;

	RemoveSets(HoverOverSelectedParams.Identifiers);
	RemoveSets(HoverOverUnselectedParams.Identifiers);
}


void UGeometrySelectionManager::RemoveAllSets() const
{
	if(!ensure(PreviewGeometry))
	{
		return;
	}
	PreviewGeometry->RemoveAllLineSets();
	PreviewGeometry->RemoveAllPointSets();
	PreviewGeometry->RemoveAllTriangleSets();
}

void UGeometrySelectionManager::RemoveSets(const TArrayView<FString>& SetIdentifiers) const
{
	
	if (PreviewGeometry)
	{
		PreviewGeometry->RemoveLineSet(SetIdentifiers[1]);
		PreviewGeometry->RemovePointSet(SetIdentifiers[0]);
		PreviewGeometry->RemoveTriangleSet(SetIdentifiers[2]);
	}
}

void UGeometrySelectionManager::CreateOrUpdateAllSets(const FGeometrySelectionElements& Elements, const FMeshElementSelectionParams SelectionParams) const
{
	UWorld* World = GetWorld();
	if (World && !PreviewGeometry->ParentActor)
	{
		PreviewGeometry->CreateInWorld(World, FTransform::Identity);
	}
	
	PreviewGeometry->CreateOrUpdateLineSet(SelectionParams.Identifiers[1], Elements.Segments.Num(), [&](int32 j, TArray<FRenderableLine>& LinesOut)
	{
		const FSegment3d Seg = Elements.Segments[j];
		LinesOut.Add(FRenderableLine(Seg.StartPoint(), Seg.EndPoint(),  SelectionParams.Color,  SelectionParams.LineThickness, SelectionParams.DepthBias));
	}, 1);

	PreviewGeometry->CreateOrUpdateTriangleSet(SelectionParams.Identifiers[2], Elements.Triangles.Num(), [&](int32 k, TArray<FRenderableTriangle>& TrianglesOut)
	{
		const FTriangle3d Triangle = Elements.Triangles[k];
		const FVector3d Normal = Triangle.Normal();
		FRenderableTriangleVertex A(Triangle.V[0], FVector2D(0,0), Normal, SelectionParams.Color);
		FRenderableTriangleVertex B(Triangle.V[1], FVector2D(1,0), Normal, SelectionParams.Color);
		FRenderableTriangleVertex C(Triangle.V[2], FVector2D(1,1), Normal, SelectionParams.Color);
		TrianglesOut.Add(FRenderableTriangle(SelectionParams.SelectionFillColor, A, B, C));
	},1 );

	PreviewGeometry->CreateOrUpdatePointSet(SelectionParams.Identifiers[0], Elements.Points.Num(), [&](int32 k, TArray<FRenderablePoint>& PointsOut)
    {
	    const FVector3d Point = Elements.Points[k];
	    PointsOut.Add(FRenderablePoint(Point, SelectionParams.Color, SelectionParams.PointSize, SelectionParams.DepthBias));
    });
}

void UGeometrySelectionManager::RebuildSelectionRenderCache()
{
	if ((RenderCachesDirtyFlags & EEnumerateRenderCachesDirtyFlags::SelectionCachesDirty) == EEnumerateRenderCachesDirtyFlags::None)
	{
		return;
	}
	check(ActiveTargetReferences.Num() == CachedSelectionRenderElements.Num());
	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[k];
		
		FGeometrySelectionElements& SelectionElements = CachedSelectionRenderElements[k];
		SelectionElements.Reset();
		Target->Selector->AccumulateSelectionElements(Target->Selection, SelectionElements, true, EEnumerateSelectionMapping::Default);
		CreateOrUpdateAllSets(SelectionElements, SelectedParams);
	}

	RenderCachesDirtyFlags &= ~EEnumerateRenderCachesDirtyFlags::SelectionCachesDirty;
}

void UGeometrySelectionManager::RebuildSelectableRenderCache()
{
	if ((RenderCachesDirtyFlags & EEnumerateRenderCachesDirtyFlags::UnselectedCachesDirty) == EEnumerateRenderCachesDirtyFlags::None || MeshTopologyMode == EMeshTopologyMode::None)
	{
		return;
	}

	check(ActiveTargetReferences.Num() == CachedSelectableRenderElements.Num());
	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[k];
		
		FGeometrySelectionElements& AllElements = CachedSelectableRenderElements[k];
		AllElements.Reset();
		
		FGeometrySelection AllElementSelection;
		AllElementSelection.ElementType = GetSelectionElementType();
		AllElementSelection.TopologyType = GetSelectionTopologyType();
			
		// Get full selection, according to the type and topo of AllElementSelection
		Target->Selector->InitializeSelectionFromPredicate(AllElementSelection, [](FGeoSelectionID) -> bool { return true; }, IGeometrySelector::EInitializeSelectionMode::All, &Target->Selection);
		Target->Selector->AccumulateSelectionElements(AllElementSelection, AllElements, true, EEnumerateSelectionMapping::FacesToEdges);
		
		CreateOrUpdateAllSets(AllElements, UnselectedParams);
	}

	RenderCachesDirtyFlags &= ~EEnumerateRenderCachesDirtyFlags::UnselectedCachesDirty;
}

void UGeometrySelectionManager::RebuildPreviewRenderCache()
{
	if ((RenderCachesDirtyFlags & EEnumerateRenderCachesDirtyFlags::PreviewCachesDirty) == EEnumerateRenderCachesDirtyFlags::None || ActiveTargetReferences.Num() == 0)
	{
		return;
	}

	// defaults to off/false; when off, will show outlines and fill color when hovering. When on/true, will only show outlines
	const TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[0];
	
	if (ActivePreviewSelection.IsEmpty() == false)
	{
		EEnumerateSelectionMapping MappingFlags = EEnumerateSelectionMapping::Default | EEnumerateSelectionMapping::FacesToEdges;
		if (CVarGeometrySelectionManager_FullSelectionHoverHighlights.GetValueOnGameThread() == 0)
		{
			// Unset FacesToFaces flag if full hover highlights are disabled
			MappingFlags &= ~EEnumerateSelectionMapping::FacesToFaces;
		}
		
		Target->Selector->AccumulateSelectionElements(SelectedActivePreviewSelection, CachedSelectedPreviewRenderElements, true, MappingFlags);
		Target->Selector->AccumulateSelectionElements(UnselectedActivePreviewSelection, CachedUnselectedPreviewRenderElements, true, MappingFlags);
		CreateOrUpdateAllSets(CachedSelectedPreviewRenderElements, HoverOverSelectedParams);
		CreateOrUpdateAllSets(CachedUnselectedPreviewRenderElements, HoverOverUnselectedParams);
	}
	RenderCachesDirtyFlags &= ~EEnumerateRenderCachesDirtyFlags::PreviewCachesDirty;
}

void UGeometrySelectionManager::RebuildSoftSelectionRenderCache()
{
	if ((RenderCachesDirtyFlags & EEnumerateRenderCachesDirtyFlags::SoftSelectionCachesDirty) == EEnumerateRenderCachesDirtyFlags::None)
	{
		return;
	}

	check(ActiveTargetReferences.Num() == CachedSoftSelectionRenderElements.Num());
	for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
	{
		TSharedPtr<FGeometrySelectionTarget> Target = ActiveTargetReferences[k];

		FGeometrySelection SoftSelectionEdges;
		SoftSelectionEdges.TopologyType = GetSelectionTopologyType();
		SoftSelectionEdges.ElementType = EGeometryElementType::Edge;
		
		Target->Selector->ConvertSelection(Target->Selection.GetSoftSelectedVertices(), SoftSelectionEdges, EEnumerateSelectionConversionParams::ExpandSelection);
		
		FGeometrySelectionElements& SoftSelectionElements = CachedSoftSelectionRenderElements[k];
		SoftSelectionElements.Reset();

		Target->Selector->AccumulateSelectionElements(SoftSelectionEdges, SoftSelectionElements, true, EEnumerateSelectionMapping::Default);

		CreateOrUpdateAllSets(SoftSelectionElements, SoftSelectedParams);
	}

	RenderCachesDirtyFlags &= ~EEnumerateRenderCachesDirtyFlags::SoftSelectionCachesDirty;
}

void UGeometrySelectionManager::DebugPrintSelection()
{
	if (ActiveTargetReferences.Num() == 0)
	{
		UE_LOGF(LogGeometry, Warning, "[SelectionManager] No Active Selection");
		return;
	}
	
	int32 NumSelected = 0;
	for (TSharedPtr<FGeometrySelectionTarget> Target : ActiveTargetReferences)
	{
		NumSelected += Target->Selection.Num();
	}
	UE_LOGF(LogGeometry, Warning, "[SelectionManager] %d selected items in %d active targets", NumSelected, ActiveTargetReferences.Num());
}


void UGeometrySelectionManager::DebugRender(IToolsContextRenderAPI* RenderAPI)
{
	// disable selection during xform to avoid overhead
	if (IsInActiveTransformation())
	{
		for (int32 k = 0; k < ActiveTargetReferences.Num(); ++k)
		{
			if (ActiveTransformations[k] != nullptr)
			{
				ActiveTransformations[k]->PreviewRender(RenderAPI);
			}
		}

		return;
	}
	if (!this->ToolsContext->ToolManager->HasAnyActiveTool())
	{
		RebuildSelectionRenderCaches();
	}
	else
	{
		// disables PreviewGeometry when in a tool
		RemoveAllSets();
		RenderCachesDirtyFlags |= EEnumerateRenderCachesDirtyFlags::UnselectedCachesDirty;
	}
}

void UGeometrySelectionManager::SetSelectionColors(
	FLinearColor UnselectedCol,
	FLinearColor HoverOverSelectedCol,
	FLinearColor HoverOverUnselectedCol,
	FLinearColor GeometrySelectedCol,
	FLinearColor SoftSelectedCol)
{
	UnselectedParams.Color = UnselectedCol.ToFColor(true);
	HoverOverSelectedParams.Color = HoverOverSelectedCol.ToFColor(true);
	HoverOverUnselectedParams.Color = HoverOverUnselectedCol.ToFColor(true);
	SelectedParams.Color = GeometrySelectedCol.ToFColor(true);
	SoftSelectedParams.Color = SoftSelectedCol.ToFColor(true);

	// on initial set up of Materials used for selection colors material (typically when entering modeling mode)
	if (UnselectedParams.SelectionFillColor == nullptr) // if one is null, they all will be
	{
		auto SetMaterial = [this] (FMeshElementSelectionParams& Params, const FLinearColor Color)
		{
			Params.SelectionFillColor =
				ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(this->ToolsContext->ToolManager, Color, Params.DepthBias, Color.A);
		};

		SetMaterial(UnselectedParams, UnselectedCol);
		SetMaterial(HoverOverUnselectedParams, HoverOverUnselectedCol);
		SetMaterial(SelectedParams, GeometrySelectedCol);
		SetMaterial(SoftSelectedParams, SoftSelectedCol);

		// to avoid flickering, this version of GetCustomTwoSidedDeptOffsetMaterial (without opacity parameter) must be called for HoverOverSelected
		HoverOverSelectedParams.SelectionFillColor =
			ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(this->ToolsContext->ToolManager, HoverOverSelectedCol, HoverOverSelectedParams.DepthBias);
	}
	// setting colors after initialization of Materials (typically using color customization in editor preferences)
	else
	{
		auto SetColorAndOpacity = [](const FMeshElementSelectionParams& Params, const FLinearColor Color)
		{
			Params.SelectionFillColor->SetScalarParameterValue("Opacity", Color.A); // no effect for HoverOverSelected
			Params.SelectionFillColor->SetVectorParameterValue("Color", Color);
		};

		SetColorAndOpacity(UnselectedParams, UnselectedCol);
		SetColorAndOpacity(HoverOverSelectedParams, HoverOverSelectedCol);
		SetColorAndOpacity(HoverOverUnselectedParams, HoverOverUnselectedCol);
		SetColorAndOpacity(SelectedParams, GeometrySelectedCol);
		SetColorAndOpacity(SoftSelectedParams, SoftSelectedCol);
	}

	// ensures that when color is changed in Editor Preferences, colors are immediately updated in the UI
	auto UpdateAllSetsColor = [this](const FMeshElementSelectionParams& Params)
	{
		UPointSetComponent* PointSet = PreviewGeometry->FindPointSet(Params.Identifiers[0]);
		ULineSetComponent* LineSet = PreviewGeometry->FindLineSet(Params.Identifiers[1]);
		UTriangleSetComponent* TriSet = PreviewGeometry->FindTriangleSet(Params.Identifiers[2]);
		if (PointSet) {	PointSet->SetAllPointsColor(Params.Color); }
		if (LineSet) { LineSet->SetAllLinesColor(Params.Color); }
		if (TriSet) { TriSet->SetAllTrianglesColor(Params.Color); }
	};

	UpdateAllSetsColor(UnselectedParams);
	UpdateAllSetsColor(HoverOverSelectedParams);
	UpdateAllSetsColor(HoverOverUnselectedParams);
	UpdateAllSetsColor(SelectedParams);
	UpdateAllSetsColor(SoftSelectedParams);
}

void UGeometrySelectionManager::DisconnectPreviewGeometry()
{
	if (PreviewGeometry)
	{
		RemoveAllSets();
		PreviewGeometry->Disconnect();
		PreviewGeometry = nullptr;
	}
}


#undef LOCTEXT_NAMESPACE
