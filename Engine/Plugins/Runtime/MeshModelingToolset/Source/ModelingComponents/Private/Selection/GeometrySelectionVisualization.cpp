// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/GeometrySelectionVisualization.h"
#include "GroupTopology.h"
#include "MeshRegionBoundaryLoops.h"
#include "SelectionSet.h"
#include "Drawing/PreviewGeometryActor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Math/Transform.h"
#include "TransformTypes.h"
#include "DynamicMesh/MeshIndexUtil.h"

using namespace UE::Geometry;

namespace
{
	int32 GetNumBoundarySegmentsEstimate(int32 NumTriangles)
	{
		int32 NumBoundarySegmentsEstimate = FMath::FloorToInt32(FMath::Sqrt((float)NumTriangles));
		return NumBoundarySegmentsEstimate;
	}
}

void UE::Geometry::InitializeGeometrySelectionVisualization(
	UPreviewGeometry* PreviewGeom,
	UGeometrySelectionVisualizationProperties* Settings,
	const FDynamicMesh3& SourceMesh,
	const FGeometrySelection& Selection,
	const FGroupTopology* GroupTopology,
	const FGeometrySelection* TriangleVertexSelection,
	const TArray<int>* TriangleROI,
	const FTransform* ApplyTransform)
{
	check(PreviewGeom);
	check(Settings);
	ensure(Selection.ElementType == (EGeometryElementType)Settings->SelectionElementType);
	ensure(Selection.TopologyType == (EGeometryTopologyType)Settings->SelectionTopologyType);

	auto AddPointNone = [](int32, const FVector3d&) {};
	auto AddSegmentNone = [](int32, const FSegment3d&) {};
	auto AddTriangleNone = [](int32, const FTriangle3d&) {};

	constexpr bool bIgnored = false;

	if (Settings->SelectionTopologyType == EGeometrySelectionTopologyType::Polygroup)
	{
		const bool bComputeGroupTopology = (GroupTopology == nullptr);
		const FGroupTopology ComputedGroupTopology(&SourceMesh, bComputeGroupTopology);
		const FGroupTopology* UseGroupTopology = bComputeGroupTopology ? &ComputedGroupTopology : GroupTopology;

		if (Settings->SelectionElementType == EGeometrySelectionElementType::Vertex)
		{


			PreviewGeom->CreateOrUpdatePointSet(TEXT("VertexSelection"), 1, [&](int32, TArray<FRenderablePoint>& Points)
			{
				auto AddPoint = [&](int32, const FVector3d& Point)
				{
					Points.Add(FRenderablePoint(Point, Settings->PointColor, Settings->PointSize, Settings->DepthBias));
				};
				EnumeratePolygroupSelectionElements(Selection, SourceMesh, UseGroupTopology, AddPoint, AddSegmentNone, AddTriangleNone, ApplyTransform, bIgnored);
			});


		}
		else if (Settings->SelectionElementType == EGeometrySelectionElementType::Edge)
		{


			PreviewGeom->CreateOrUpdateLineSet(TEXT("EdgeSelection"), 1, [&](int32, TArray<FRenderableLine>& Lines)
			{
				auto AddSegment = [&](int32 Eid, const FSegment3d& Segment)
				{
					Lines.Add(FRenderableLine(Segment.StartPoint(), Segment.EndPoint(), Settings->LineColor, Settings->LineThickness, Settings->DepthBias));
				};
				EnumeratePolygroupSelectionElements(Selection, SourceMesh, UseGroupTopology, AddPointNone, AddSegment, AddTriangleNone, ApplyTransform, bIgnored);
			});


		}
		else if (ensure(Settings->SelectionElementType == EGeometrySelectionElementType::Face))
		{


			PreviewGeom->CreateOrUpdateTriangleSet(TEXT("FaceSelection"), 1, [&](int32, TArray<FRenderableTriangle>& Triangles)
			{
				auto AddTriangle = [&](int32 Tid, const FTriangle3d& Triangle)
				{
					FVector3d Normal = Triangle.Normal();
					FRenderableTriangleVertex A(Triangle.V[0], FVector2D(0,0), Normal, Settings->FaceColor);
					FRenderableTriangleVertex B(Triangle.V[1], FVector2D(1,0), Normal, Settings->FaceColor);
					FRenderableTriangleVertex C(Triangle.V[2], FVector2D(1,1), Normal, Settings->FaceColor);
					Triangles.Add(FRenderableTriangle(Settings->GetFaceMaterial(), A, B, C));
				};
				EnumeratePolygroupSelectionElements(Selection, SourceMesh, UseGroupTopology, AddPointNone, AddSegmentNone, AddTriangle, ApplyTransform, false);
			}, Selection.Num());


		}


	}
	else if (ensure(Settings->SelectionTopologyType == EGeometrySelectionTopologyType::Triangle)) 
	{


		if (Settings->SelectionElementType == EGeometrySelectionElementType::Vertex)
		{


			PreviewGeom->CreateOrUpdatePointSet(TEXT("VertexSelection"), 1, [&](int32, TArray<FRenderablePoint>& Points)
			{
				auto AddPoint = [&](int32, const FVector3d& Point)
				{
					Points.Add(FRenderablePoint(Point, Settings->PointColor, Settings->PointSize, Settings->DepthBias));
				};

				EnumerateTriangleSelectionElements(Selection, SourceMesh, AddPoint, AddSegmentNone, AddTriangleNone, ApplyTransform, bIgnored);
			}, Selection.Num());


		}
		else if (Settings->SelectionElementType == EGeometrySelectionElementType::Edge)
		{


			PreviewGeom->CreateOrUpdateLineSet(TEXT("EdgeSelection"), 1, [&](int32, TArray<FRenderableLine>& Lines)
			{
				auto AddSegment = [&](int32, const FSegment3d& Segment)
				{
					Lines.Add(FRenderableLine(Segment.StartPoint(), Segment.EndPoint(), Settings->LineColor, Settings->LineThickness, Settings->DepthBias));
				};

				EnumerateTriangleSelectionElements(Selection, SourceMesh, AddPointNone, AddSegment, AddTriangleNone, ApplyTransform, bIgnored);
			}, Selection.Num());


		}
		else if (ensure(Settings->SelectionElementType == EGeometrySelectionElementType::Face))
		{


			PreviewGeom->CreateOrUpdateTriangleSet(TEXT("FaceSelection"), 1, [&](int32, TArray<FRenderableTriangle>& Triangles)
			{
				auto AddTriangle = [&](int32, const FTriangle3d& Triangle)
				{
					FVector3d Normal = Triangle.Normal();
					FRenderableTriangleVertex A(Triangle.V[0], FVector2D(0,0), Normal, Settings->FaceColor);
					FRenderableTriangleVertex B(Triangle.V[1], FVector2D(1,0), Normal, Settings->FaceColor);
					FRenderableTriangleVertex C(Triangle.V[2], FVector2D(1,1), Normal, Settings->FaceColor);
					Triangles.Add(FRenderableTriangle(Settings->GetFaceMaterial(), A, B, C));
				};

				EnumerateTriangleSelectionElements(Selection, SourceMesh, AddPointNone, AddSegmentNone, AddTriangle, ApplyTransform, false);
			}, Selection.Num());

		}


	}



	if (Settings->bEnableShowEdgeSelectionVertices && TriangleVertexSelection)
	{
		ensure(TriangleVertexSelection->TopologyType == EGeometryTopologyType::Triangle);
		ensure(TriangleVertexSelection->ElementType == EGeometryElementType::Vertex);

		PreviewGeom->CreateOrUpdatePointSet(TEXT("EdgeSelectionVertices"), 1, [&](int32, TArray<FRenderablePoint>& Points)
		{
			auto AddPoint = [&](int32, const FVector3d& Point)
			{
				Points.Add(FRenderablePoint(Point, Settings->LineColor, Settings->PointSize, Settings->DepthBias));
			};

			EnumerateTriangleSelectionElements(*TriangleVertexSelection, SourceMesh, AddPoint, AddSegmentNone, AddTriangleNone, ApplyTransform, bIgnored);
		}, Selection.Num());
	}



	if (Settings->bEnableShowTriangleROIBorder && TriangleROI)
	{
		TSet<int32> BorderEdges;
		FMeshRegionBoundaryLoops BoundaryLoops(&SourceMesh, *TriangleROI, true);
		for (const FEdgeLoop& Loop : BoundaryLoops.GetLoops())
		{
			BorderEdges.Append(Loop.Edges);
		}

		PreviewGeom->CreateOrUpdateLineSet(TEXT("TriangleROIBorder"), SourceMesh.MaxEdgeID(), [&](int32 Eid, TArray<FRenderableLine>& LinesOut) {
			if (BorderEdges.Contains(Eid))
			{
				FVector A, B;
				SourceMesh.GetEdgeV(Eid, A, B);
				LinesOut.Add(FRenderableLine(A, B, Settings->TriangleROIBorderColor, Settings->LineThickness, Settings->DepthBias));
			}
		}, GetNumBoundarySegmentsEstimate(TriangleROI->Num()));
	}



	Settings->bVisualizationDirty = false;
}

void UE::Geometry::UpdateGeometrySelectionVisualization(
	UPreviewGeometry* PreviewGeom,
	UGeometrySelectionVisualizationProperties* Settings)
{
	if (Settings->bVisualizationDirty)
	{


		if (Settings->SelectionElementType == EGeometrySelectionElementType::Vertex)
		{
			PreviewGeom->UpdatePointSet(TEXT("VertexSelection"), [Settings](TObjectPtr<UPointSetComponent> PointSet)
			{
				PointSet->SetVisibility(Settings->ShowVertexSelectionPointSet());
				PointSet->SetAllPointsColor(Settings->PointColor);
				PointSet->SetAllPointsSize(Settings->PointSize);
				PointSet->SetAllPointsDepthBias(Settings->DepthBias);
				PointSet->SetPointMaterial(Settings->GetPointMaterial());
			});
		}


		if (Settings->SelectionElementType == EGeometrySelectionElementType::Edge)
		{
			PreviewGeom->UpdateLineSet(TEXT("EdgeSelection"), [Settings](TObjectPtr<ULineSetComponent> LineSet)
			{
				LineSet->SetVisibility(Settings->ShowEdgeSelectionLineSet());
				LineSet->SetAllLinesColor(Settings->LineColor);
				LineSet->SetAllLinesThickness(Settings->LineThickness);
				LineSet->SetAllLinesDepthBias(Settings->DepthBias);
				LineSet->SetLineMaterial(Settings->GetLineMaterial());
			});
		}


		if (Settings->SelectionElementType == EGeometrySelectionElementType::Face)
		{
			PreviewGeom->UpdateTriangleSet(TEXT("FaceSelection"), [Settings](TObjectPtr<UTriangleSetComponent> TriangleSet)
			{
				TriangleSet->SetVisibility(Settings->ShowFaceSelectionTriangleSet());
				TriangleSet->SetAllTrianglesColor(Settings->FaceColor);
				TriangleSet->SetAllTrianglesMaterial(Settings->GetFaceMaterial());
			});
		}


		if (Settings->bEnableShowEdgeSelectionVertices)
		{
			PreviewGeom->UpdatePointSet(TEXT("EdgeSelectionVertices"), [Settings](TObjectPtr<UPointSetComponent> PointSet)
			{
				PointSet->SetVisibility(Settings->ShowEdgeSelectionVerticesPointSet());
				PointSet->SetAllPointsColor(Settings->LineColor); // line color is intentional here
				PointSet->SetAllPointsSize(Settings->PointSize);
				PointSet->SetAllPointsDepthBias(Settings->DepthBias);
				PointSet->SetPointMaterial(Settings->GetPointMaterial());
			});
		}


		if (Settings->bEnableShowTriangleROIBorder)
		{
			PreviewGeom->UpdateLineSet(TEXT("TriangleROIBorder"), [Settings](TObjectPtr<ULineSetComponent> LineSet)
			{
				LineSet->SetVisibility(Settings->ShowTriangleROIBorderLineSet());
				LineSet->SetAllLinesColor(Settings->TriangleROIBorderColor);
				LineSet->SetAllLinesThickness(Settings->LineThickness);
				LineSet->SetAllLinesDepthBias(Settings->DepthBias);
				LineSet->SetLineMaterial(Settings->GetLineMaterial());
			});
		}


		Settings->bVisualizationDirty = false;
	}
}

FAxisAlignedBox3d UE::Geometry::ComputeBoundsFromTriangleROI(
	const FDynamicMesh3& Mesh,
	const TArray<int32>& TriangleROI,
	const FTransform3d* Transform)
{
	TArray<int32> VertexROI;
	UE::Geometry::TriangleToVertexIDs(&Mesh, TriangleROI, VertexROI);
	return ComputeBoundsFromVertexROI(Mesh, VertexROI, Transform);
}

FAxisAlignedBox3d UE::Geometry::ComputeBoundsFromVertexROI(
	const FDynamicMesh3& Mesh,
	const TArray<int32>& VertexROI,
	const FTransform3d* Transform)
{
	if (Transform)
	{
		return FAxisAlignedBox3d::MakeBoundsFromIndices(VertexROI, [&Mesh, Transform](int32 Index) { return Transform->TransformPosition(Mesh.GetVertex(Index)); });
	}
	return FAxisAlignedBox3d::MakeBoundsFromIndices(VertexROI, [&Mesh](int32 Index) { return Mesh.GetVertex(Index); });
}

