// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/GroupTopologySelector.h"
#include "Mechanics/RectangleMarqueeMechanic.h" // FCameraRectangle
#include "ToolDataVisualizer.h"
#include "ToolSceneQueriesUtil.h"

#define LOCTEXT_NAMESPACE "FGroupTopologySelector"

void FGroupTopologyUtils::AddNewEdgeLoopEdgesFromCorner(int32 EdgeID, int32 CornerID, TSet<int32>& EdgeSet) const
{
	int32 LastCornerID = CornerID;
	int32 LastEdgeID = EdgeID;
	while (true)
	{
		int32 NextEid;
		if (!GetNextEdgeLoopEdge(LastEdgeID, LastCornerID, NextEid))
		{
			break; // Probably not a valence 4 corner
		}
		if (EdgeSet.Contains(NextEid))
		{
			break; // Either we finished the loop, or we'll continue it from another selection
		}

		EdgeSet.Add(NextEid);

		LastEdgeID = NextEid;
		const FGroupTopology::FGroupEdge& LastEdge = GroupTopology->Edges[LastEdgeID];
		LastCornerID = LastEdge.EndpointCorners[0] == LastCornerID ? LastEdge.EndpointCorners[1] : LastEdge.EndpointCorners[0];

		ensure(LastCornerID != IndexConstants::InvalidID);
	}
}

bool FGroupTopologyUtils::GetNextEdgeLoopEdge(int32 IncomingEdgeID, int32 CornerID, int32& NextEdgeIDOut) const
{
	// It's worth noting that the approach here breaks down in pathological cases where the same group is present
	// multiple times around a corner (i.e. the group is not contiguous, and separate islands share a corner).
	// It's not practical to worry about those cases.

	NextEdgeIDOut = IndexConstants::InvalidID;
	const FGroupTopology::FCorner& CurrentCorner = GroupTopology->Corners[CornerID];

	if (CurrentCorner.NeighbourGroupIDs.Num() != 4)
	{
		return false; // Not a valence 4 corner
	}

	// We want to stop when we hit boundary vertices because our valence-4 method of finding the next edge typically
	// breaks down at these points. In some single-hole cases we could treat the hole as a group and find the next
	// edge, but it doesn't always make sense to do that, it gets in the way of us using loop selection to select
	// hole boundaries, and it breaks down with more than one hole. Best to just stop.
	if (GroupTopology->GetMesh()->IsBoundaryVertex(GroupTopology->GetCornerVertexID(CornerID)))
	{
		return false;
	}

	const FGroupTopology::FGroupEdge& IncomingEdge = GroupTopology->Edges[IncomingEdgeID];

	// We want to find the edge that shares this corner but does not border either of the neighboring groups of
	// the incoming edge.

	for (int32 Gid : CurrentCorner.NeighbourGroupIDs)
	{
		if (Gid == IncomingEdge.Groups[0] || Gid == IncomingEdge.Groups[1])
		{
			continue; // This is one of the neighboring groups of the incoming edge
		}

		// Iterate through all edges of group
		const FGroupTopology::FGroup* Group = GroupTopology->FindGroupByID(Gid);
		for (const FGroupTopology::FGroupBoundary& Boundary : Group->Boundaries)
		{
			for (int32 Eid : Boundary.GroupEdges)
			{
				const FGroupTopology::FGroupEdge& CandidateEdge = GroupTopology->Edges[Eid];

				// Edge must share corner but not neighboring groups
				if ((CandidateEdge.EndpointCorners[0] == CornerID || CandidateEdge.EndpointCorners[1] == CornerID)
					&& CandidateEdge.Groups[0] != IncomingEdge.Groups[0] && CandidateEdge.Groups[0] != IncomingEdge.Groups[1]
					&& CandidateEdge.Groups[1] != IncomingEdge.Groups[0] && CandidateEdge.Groups[1] != IncomingEdge.Groups[1])
				{
					NextEdgeIDOut = Eid;
					return true;
				}
			}
		}
	}
	return false;
}

void FGroupTopologyUtils::AddNewBoundaryLoopEdges(int32 StartEdgeID, int32 ForwardCornerID, TSet<int32>& EdgeSet) const
{
	if (!GroupTopology->IsBoundaryEdge(StartEdgeID))
	{
		return;
	}

	int32 LastCornerID = ForwardCornerID;
	int32 LastEdgeID = StartEdgeID;
	while (true)
	{
		int32 NextEid;
		if (!GetNextBoundaryLoopEdge(LastEdgeID, LastCornerID, NextEid))
		{
			break; // Hit a bowtie
		}
		if (EdgeSet.Contains(NextEid))
		{
			break; // Either we finished the loop, or we'll continue it from another selection
		}

		EdgeSet.Add(NextEid);

		LastEdgeID = NextEid;
		const FGroupTopology::FGroupEdge& LastEdge = GroupTopology->Edges[LastEdgeID];
		LastCornerID = LastEdge.EndpointCorners[0] == LastCornerID ? LastEdge.EndpointCorners[1] : LastEdge.EndpointCorners[0];

		ensure(LastCornerID != IndexConstants::InvalidID);
	}
}

bool FGroupTopologyUtils::GetNextBoundaryLoopEdge(int32 IncomingEdgeID, int32 CornerID, int32& NextEdgeIDOut) const
{
	NextEdgeIDOut = IndexConstants::InvalidID;

	if (!GroupTopology->IsBoundaryEdge(IncomingEdgeID))
	{
		return false;
	}

	TArray<int32> IncidentEdges;
	GroupTopology->FindCornerNbrEdges(CornerID, IncidentEdges);
	
	for (int32 GroupEdgeID : IncidentEdges)
	{
		if (GroupTopology->IsBoundaryEdge(GroupEdgeID))
		{
			if (GroupEdgeID == IncomingEdgeID)
			{
				continue;
			}
			else if (NextEdgeIDOut == IndexConstants::InvalidID)
			{
				// Found a different boundary edge than IncomingEdgeID
				NextEdgeIDOut = GroupEdgeID;
			}
			else
			{
				// Must be a bowtie, because we saw a third boundary edge
				NextEdgeIDOut = IndexConstants::InvalidID;
				return false;
			}
		}
	}
	
	return ensure(NextEdgeIDOut != IndexConstants::InvalidID);
}

void FGroupTopologyUtils::AddNewEdgeRingEdges(int32 StartEdgeID, int32 ForwardGroupID, TSet<int32>& EdgeSet) const
{
	int32 CurrentEdgeID = StartEdgeID;
	int32 CurrentForwardGroupID = ForwardGroupID;
	while (true)
	{
		if (CurrentForwardGroupID == IndexConstants::InvalidID)
		{
			break; // Reached a boundary
		}

		int32 NextEdgeID;
		if (!GetQuadOppositeEdge(CurrentEdgeID, CurrentForwardGroupID, NextEdgeID))
		{
			break; // Probably not a quad
		}
		if (EdgeSet.Contains(NextEdgeID))
		{
			break; // Either we finished the loop, or we'll continue it from another selection
		}

		EdgeSet.Add(NextEdgeID);

		CurrentEdgeID = NextEdgeID;
		const FGroupTopology::FGroupEdge& Edge = GroupTopology->Edges[CurrentEdgeID];
		CurrentForwardGroupID = (Edge.Groups[0] == CurrentForwardGroupID) ? Edge.Groups[1] : Edge.Groups[0];
	}
}

bool FGroupTopologyUtils::GetQuadOppositeEdge(int32 EdgeIDIn, int32 GroupID, int32& OppositeEdgeIDOut) const
{
	const FGroupTopology::FGroup* Group = GroupTopology->FindGroupByID(GroupID);
	if (!ensure(Group))
	{
		return false;
	}

	// Find the boundary that contains this edge
	for (int32 i = 0; i < Group->Boundaries.Num(); ++i)
	{
		const FGroupTopology::FGroupBoundary& Boundary = Group->Boundaries[i];
		int32 EdgeIndex = Boundary.GroupEdges.IndexOfByKey(EdgeIDIn);
		if (EdgeIndex != INDEX_NONE)
		{
			if (Boundary.GroupEdges.Num() != 4)
			{
				return false;
			}

			OppositeEdgeIDOut = Boundary.GroupEdges[(EdgeIndex + 2) % 4];
			return true;
		}
	}
	return ensure(false); // No boundary of the given group contained the given edge
}


FIndex2i FGroupTopologyUtils::GetEdgeEndpointCorners(int EdgeID) const
{
	return GroupTopology->Edges[EdgeID].EndpointCorners;
}

FIndex2i FGroupTopologyUtils::GetEdgeGroups(int EdgeID) const
{
	return GroupTopology->Edges[EdgeID].Groups;
}


// -------------------------------------------------

bool FGroupTopologySelector::ExpandSelectionByEdgeLoops(FGroupTopologySelection& Selection)
{
	TSet<int32>& EdgeSet = Selection.SelectedEdgeIDs;
	int32 OriginalNumEdges = Selection.SelectedEdgeIDs.Num();
	TSet<int32> ExpandedEdgeSet = EdgeSet; // make a copy of the edge set, to add to during iteration
	for (int32 Eid : Selection.SelectedEdgeIDs)
	{
		const FIndex2i EndpointCorners = GroupTopologyUtils.GetEdgeEndpointCorners(Eid);
		if (EndpointCorners[0] == IndexConstants::InvalidID)
		{
			continue; // This FGroupEdge is a loop unto itself (and already in our selection, since we're looking at it).
		}

		// Go forward and backward adding edges
		GroupTopologyUtils.AddNewEdgeLoopEdgesFromCorner(Eid, EndpointCorners[0], ExpandedEdgeSet);
		GroupTopologyUtils.AddNewEdgeLoopEdgesFromCorner(Eid, EndpointCorners[1], ExpandedEdgeSet);
	}
	EdgeSet = MoveTemp(ExpandedEdgeSet); // update the selection edges

	return EdgeSet.Num() > OriginalNumEdges;
}

bool FGroupTopologySelector::ExpandSelectionByBoundaryLoops(FGroupTopologySelection& Selection)
{
	TSet<int32>& EdgeSet = Selection.SelectedEdgeIDs;
	int32 OriginalNumEdges = Selection.SelectedEdgeIDs.Num();
	TSet<int32> ExpandedEdgeSet = EdgeSet; // make a copy of the edge set, to add to during iteration
	for (int32 Eid : Selection.SelectedEdgeIDs)
	{
		if (!GroupTopologyUtils.GroupTopology->IsBoundaryEdge(Eid))
		{
			continue;
		}

		const FIndex2i EndpointCorners = GroupTopologyUtils.GetEdgeEndpointCorners(Eid);
		if (EndpointCorners[0] == IndexConstants::InvalidID)
		{
			continue; // This FGroupEdge is a loop unto itself (and already in our selection, since we're looking at it).
		}

		// Go forward and backward adding edges
		GroupTopologyUtils.AddNewBoundaryLoopEdges(Eid, EndpointCorners[0], ExpandedEdgeSet);
		GroupTopologyUtils.AddNewBoundaryLoopEdges(Eid, EndpointCorners[1], ExpandedEdgeSet);
	}
	EdgeSet = MoveTemp(ExpandedEdgeSet); // update the selection edges

	return EdgeSet.Num() > OriginalNumEdges;
}

bool FGroupTopologySelector::ExpandSelectionByEdgeRings(FGroupTopologySelection& Selection)
{
	TSet<int32>& EdgeSet = Selection.SelectedEdgeIDs;
	int32 OriginalNumEdges = Selection.SelectedEdgeIDs.Num();
	TSet<int32> ExpandedEdgeSet = EdgeSet; // make a copy of the edge set, to add to during iteration
	for (int32 Eid : Selection.SelectedEdgeIDs)
	{
		const FIndex2i EdgeGroups = GroupTopologyUtils.GetEdgeGroups(Eid);

		// Go forward and backward adding edges
		if (EdgeGroups[0] != IndexConstants::InvalidID)
		{
			GroupTopologyUtils.AddNewEdgeRingEdges(Eid, EdgeGroups[0], ExpandedEdgeSet);
		}
		if (EdgeGroups[0] != IndexConstants::InvalidID)
		{
			GroupTopologyUtils.AddNewEdgeRingEdges(Eid, EdgeGroups[1], ExpandedEdgeSet);
		}
	}
	EdgeSet = MoveTemp(ExpandedEdgeSet); // update the selection edges

	return EdgeSet.Num() > OriginalNumEdges;
}

FGroupTopologySelector::FGroupTopologySelector(const FDynamicMesh3* Mesh, const FGroupTopology* Topology) : 
	FMeshTopologySelector()
{
	Initialize(Mesh, Topology);
}

void FGroupTopologySelector::Initialize(const FDynamicMesh3* MeshIn, const FGroupTopology* TopologyIn)
{
	Mesh = MeshIn;
	TopologyProvider = MakeUnique<FGroupTopologyProvider>(TopologyIn);
	GroupTopologyUtils.GroupTopology = TopologyIn;
	bGeometryInitialized = false;
	bGeometryUpToDate = false;
}


void FGroupTopologySelector::DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle)
{
	FLinearColor UseColor = Renderer->LineColor;
	float LineWidth = Renderer->LineThickness;

	if (CornerDrawStyle == ECornerDrawStyle::Point)
	{
		for (int CornerID : Selection.SelectedCornerIDs)
		{
			int VertexID = TopologyProvider->GetCornerVertexID(CornerID);
			FVector Position = (FVector)Mesh->GetVertex(VertexID);

			Renderer->DrawPoint(Position, Renderer->PointColor, Renderer->PointSize, false);
		}
	}
	else // ECornerDrawStyle::Circle
	{
		for (int CornerID : Selection.SelectedCornerIDs)
		{
			int VertexID = TopologyProvider->GetCornerVertexID(CornerID);
			FVector Position = (FVector)Mesh->GetVertex(VertexID);
			FVector WorldPosition = Renderer->TransformP(Position);

			// Depending on whether we're in an orthographic view or not, we set the radius based on visual angle or based on ortho 
			// viewport width (divided into 90 segments like the FOV is divided into 90 degrees).
			float Radius = (CameraState->bIsOrthographic) ? CameraState->OrthoWorldCoordinateWidth * 0.5 / 90.0
				: (float)ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(*CameraState, (FVector3d)WorldPosition, 0.5);
			Renderer->DrawViewFacingCircle(Position, Radius, 16, UseColor, LineWidth, false);
		}
	}


	for (int EdgeID : Selection.SelectedEdgeIDs)
	{
		const TArray<int>& Vertices = TopologyProvider->GetGroupEdgeVertices(EdgeID);
		int NV = Vertices.Num() - 1;

		// Draw the edge, but also draw the endpoints in ortho mode (to make projected edges visible)
		FVector A = (FVector)Mesh->GetVertex(Vertices[0]);
		if (CameraState->bIsOrthographic)
		{
			Renderer->DrawPoint(A, UseColor, 10, false);
		}
		for (int k = 0; k < NV; ++k)
		{
			FVector B = (FVector)Mesh->GetVertex(Vertices[k + 1]);
			Renderer->DrawLine(A, B, UseColor, LineWidth, false);
			A = B;
		}
		if (CameraState->bIsOrthographic)
		{
			Renderer->DrawPoint(A, UseColor, LineWidth, false);
		}
	}

	// We are not responsible for drawing the faces, but do draw the sides of the faces in ortho mode to make them visible
	// when they project to an edge.
	if (CameraState->bIsOrthographic && Selection.SelectedGroupIDs.Num() > 0)
	{
		TopologyProvider->ForGroupSetEdges(Selection.SelectedGroupIDs,
			[&UseColor, LineWidth, Renderer, this](int EdgeID)
		{
			const TArray<int>& Vertices = TopologyProvider->GetGroupEdgeVertices(EdgeID);
			int NV = Vertices.Num() - 1;
			FVector A = (FVector)Mesh->GetVertex(Vertices[0]);
			for (int k = 0; k < NV; ++k)
			{
				FVector B = (FVector)Mesh->GetVertex(Vertices[k + 1]);
				Renderer->DrawLine(A, B, UseColor, LineWidth, false);
				A = B;
			}
		});
	}
}


#undef LOCTEXT_NAMESPACE
