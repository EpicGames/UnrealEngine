// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/MeshTopologySelectionMechanic.h"

#include "BaseBehaviors/SingleClickOrDragBehavior.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "Mechanics/PolyLassoMarqueeMechanic.h"
#include "Polygon2.h"
#include "Selection/PersistentMeshSelection.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"
#include "Util/ColorConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshTopologySelectionMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UMeshTopologySelectionMechanic"

void UMeshTopologySelectionMechanicProperties::InvertSelection()
{
	if (Mechanic.IsValid())
	{
		Mechanic->InvertSelection();
	}
}

void UMeshTopologySelectionMechanicProperties::SelectAll()
{
	if (Mechanic.IsValid())
	{
		Mechanic->SelectAll();
	}
}

UMeshTopologySelectionMechanic::~UMeshTopologySelectionMechanic()
{
	checkf(PreviewGeometryActor == nullptr, TEXT("Shutdown() should be called before UMeshTopologySelectionMechanic is destroyed."));
}


UMeshTopologySelectionMechanicProperties* UMeshTopologySelectionMechanic::CreateProperties()
{
	return NewObject<UMeshTopologySelectionMechanicProperties>(this);
}

void UMeshTopologySelectionMechanic::Setup(UInteractiveTool* ParentToolIn)
{
	UInteractionMechanic::Setup(ParentToolIn);

	HoverBehavior = NewObject<UMouseHoverBehavior>();
	// We use modifiers on hover to change the highlighting according to what can be selected  	
	HoverBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	HoverBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	HoverBehavior->Initialize(this);
	HoverBehavior->SetDefaultPriority(BasePriority);
	ParentToolIn->AddInputBehavior(HoverBehavior, this);

	MarqueeMechanic = NewObject<URectangleMarqueeMechanic>();
	MarqueeMechanic->bUseExternalClickDragBehavior = true;
	MarqueeMechanic->Setup(ParentToolIn);
	MarqueeMechanic->OnDragRectangleStarted.AddUObject(this, &UMeshTopologySelectionMechanic::OnDragRectangleStarted);
	MarqueeMechanic->OnDragRectangleChanged.AddUObject(this, &UMeshTopologySelectionMechanic::OnDragRectangleChanged);
	MarqueeMechanic->OnDragRectangleFinished.AddUObject(this, &UMeshTopologySelectionMechanic::OnDragRectangleFinished);
	MarqueeMechanic->SetBasePriority(BasePriority.MakeLower());

	// Set up control points mechanic
	PolyLassoMechanic = NewObject<UPolyLassoMarqueeMechanic>(this);
	PolyLassoMechanic->Setup(ParentToolIn);
	PolyLassoMechanic->SetIsEnabled(false);
	PolyLassoMechanic->SpacingTolerance = 10.0f;
	PolyLassoMechanic->OnDrawPolyLassoStarted.AddUObject(this, &UMeshTopologySelectionMechanic::OnPolyLassoStarted);
	PolyLassoMechanic->OnDrawPolyLassoChanged.AddUObject(this, &UMeshTopologySelectionMechanic::OnPolyLassoChanged);
	PolyLassoMechanic->OnDrawPolyLassoFinished.AddUObject(this, &UMeshTopologySelectionMechanic::OnPolyLassoFinished);
	PolyLassoMechanic->SetBasePriority(BasePriority.MakeLower());

	ClickOrDragBehavior = NewObject<USingleClickOrDragInputBehavior>();
	ClickOrDragBehavior->Initialize(this, MarqueeMechanic);
	ClickOrDragBehavior->Modifiers.RegisterModifier(ShiftModifierID, FInputDeviceState::IsShiftKeyDown);
	ClickOrDragBehavior->Modifiers.RegisterModifier(CtrlModifierID, FInputDeviceState::IsCtrlKeyDown);
	ClickOrDragBehavior->SetDefaultPriority(BasePriority);
	ParentTool->AddInputBehavior(ClickOrDragBehavior, this);

	Properties = CreateProperties();
	Properties->Initialize(this);
	if (bAddSelectionFilterPropertiesToParentTool)
	{
		AddToolPropertySource(Properties);
	}
	Properties->WatchProperty(Properties->bSelectVertices, [this](bool bSelectVertices) {
		UpdateSelectorEnabled();
	});
	Properties->WatchProperty(Properties->bSelectEdges, [this](bool bSelectVertices) { 
		UpdateSelectorEnabled();
	});
	Properties->WatchProperty(Properties->bSelectFaces, [this](bool bSelectFaces) {
		UpdateSelectorEnabled();
	});
	Properties->WatchProperty(Properties->bSelectEdgeLoops, [this](bool bSelectEdgeLoops) {
		UpdateSelectorEnabled();
	});
	Properties->WatchProperty(Properties->bSelectEdgeRings, [this](bool bSelectEdgeRings) {
		UpdateSelectorEnabled();
	});
	Properties->WatchProperty(Properties->SelectionDragTool, [this](EMeshTopologySelectionMechanicDragTool DragTool) {
		UpdateSelectorEnabled();
	});

	// set up visualizers
	PolyEdgesRenderer.LineColor = FLinearColor::Red;
	PolyEdgesRenderer.LineThickness = 2.0;
	PolyEdgesRenderer.PointColor = FLinearColor::Red;
	PolyEdgesRenderer.PointSize = 8.0f;
	HilightRenderer.LineColor = FLinearColor::Green;
	HilightRenderer.LineThickness = 4.0f;
	HilightRenderer.PointColor = FLinearColor::Green;
	HilightRenderer.PointSize = 10.0f;
	SelectionRenderer.LineColor = LinearColors::Gold3f();
	SelectionRenderer.LineThickness = 4.0f;
	SelectionRenderer.PointColor = LinearColors::Gold3f();
	SelectionRenderer.PointSize = 10.0f;

	float HighlightedFacePercentDepthOffset = 0.5f;
	HighlightedFaceMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor::Green, ParentToolIn->GetToolManager(), HighlightedFacePercentDepthOffset);
	// The rest of the highlighting setup has to be done in Initialize(), since we need the world to set up our drawing component.
}

void UMeshTopologySelectionMechanic::Shutdown()
{
	if (PreviewGeometryActor)
	{
		PreviewGeometryActor->Destroy();
		PreviewGeometryActor = nullptr;
		DrawnTriangleSetComponent = nullptr;
	}
}

void UMeshTopologySelectionMechanic::DisableBehaviors(UInteractiveTool* ParentToolIn)
{
	ParentToolIn->RemoveInputBehaviorsBySource(this);
	ParentToolIn->RemoveInputBehaviorsBySource(MarqueeMechanic);
	ParentToolIn->RemoveInputBehaviorsBySource(PolyLassoMechanic);

	// TODO: Is it worth adding a way to remove the property watchers for marquee?
}

void UMeshTopologySelectionMechanic::SetIsEnabled(bool bBehaviorEnabledIn, bool bRenderTopologyIn)
{
	bIsEnabled = bBehaviorEnabledIn;
	bRenderTopology = bRenderTopologyIn;
	UpdateSelectorEnabled();
}

void UMeshTopologySelectionMechanic::SetTransform(const FTransform3d& InTargetTransform)
{
	TargetTransform = InTargetTransform;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UMeshTopologySelectionMechanic::SetMarqueeSelectionUpdateType(EMarqueeSelectionUpdateType InType)
{
	MarqueeSelectionUpdateType = InType;
	switch (MarqueeSelectionUpdateType)
	{
	case EMarqueeSelectionUpdateType::OnDrag:
		SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType::OnDrag);
		break;
	case EMarqueeSelectionUpdateType::OnRelease:
		SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType::OnRelease);
		break;
	case EMarqueeSelectionUpdateType::OnTickAndRelease:
		SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType::OnTickAndRelease);
		break;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UMeshTopologySelectionMechanic::SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType InType)
{
	SelectionDragToolUpdateType = InType;
}

void UMeshTopologySelectionMechanic::SetBasePriority(const FInputCapturePriority &Priority)
{
	BasePriority = Priority;
	if (ClickOrDragBehavior)
	{
		ClickOrDragBehavior->SetDefaultPriority(Priority);
	}
	if (HoverBehavior)
	{
		HoverBehavior->SetDefaultPriority(Priority);
	}
	const FInputCapturePriority MarqueeAndLassoPriority = Priority.MakeLower();
	if (MarqueeMechanic)
	{
		MarqueeMechanic->SetBasePriority(MarqueeAndLassoPriority);
	}
	if (PolyLassoMechanic)
	{
		PolyLassoMechanic->SetBasePriority(MarqueeAndLassoPriority);
	}
}

TPair<FInputCapturePriority, FInputCapturePriority> UMeshTopologySelectionMechanic::GetPriorityRange() const
{
	TPair<FInputCapturePriority, FInputCapturePriority> Result;
	Result.Key = BasePriority;
	Result.Value = MarqueeMechanic->GetPriorityRange().Value;
	return Result;
}

void UMeshTopologySelectionMechanic::Render(IToolsContextRenderAPI* RenderAPI)
{
	// Cache the view camera state so we can use for snapping/etc.
	GetParentTool()->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
	
	if (bIsEnabled)
	{
		MarqueeMechanic->Render(RenderAPI);
		PolyLassoMechanic->Render(RenderAPI);
	}
	
	if (!bRenderTopology) 
	{
		return;
	}

	FViewCameraState RenderCameraState = RenderAPI->GetCameraState();

	const FDynamicMesh3* TargetMesh = this->Mesh;
	const FTransform Transform = static_cast<FTransform>(TargetTransform);
	
	const FTopologyProvider* TopologyProvider = TopoSelector->GetTopologyProvider();

	PolyEdgesRenderer.BeginFrame(RenderAPI, RenderCameraState);
	PolyEdgesRenderer.SetTransform(Transform);
	if (bShowEdges)
	{
		for (int EdgeID = 0; EdgeID < TopologyProvider->GetNumEdges(); ++EdgeID)
		{
			FVector3d A, B;
			for (int eid : TopologyProvider->GetGroupEdgeEdges(EdgeID))
			{
				TargetMesh->GetEdgeV(eid, A, B);
				PolyEdgesRenderer.DrawLine(A, B);
			}
		}
	}
	if (bShowSelectableCorners)
	{
		for (int CornerID = 0; CornerID < TopologyProvider->GetNumCorners(); ++CornerID)
		{
			FVector3d A = TargetMesh->GetVertex(TopologyProvider->GetCornerVertexID(CornerID));
			PolyEdgesRenderer.DrawPoint(A);
		}
	}
	PolyEdgesRenderer.EndFrame();

	if (PersistentSelection.IsEmpty() == false)
	{
		SelectionRenderer.BeginFrame(RenderAPI, RenderCameraState);
		SelectionRenderer.SetTransform(Transform);
		TopoSelector->DrawSelection(PersistentSelection, &SelectionRenderer, &RenderCameraState);
		SelectionRenderer.EndFrame();
	}

	HilightRenderer.BeginFrame(RenderAPI, RenderCameraState);
	HilightRenderer.SetTransform(Transform);
	TopoSelector->DrawSelection(HilightSelection, &HilightRenderer, &RenderCameraState, FGroupTopologySelector::ECornerDrawStyle::Circle);
	HilightRenderer.EndFrame();
}

void UMeshTopologySelectionMechanic::DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI)
{
	if (!bIsEnabled)
	{
		return;
	}
	
	MarqueeMechanic->DrawHUD(Canvas, RenderAPI);
	PolyLassoMechanic->DrawHUD(Canvas, RenderAPI);
}

void UMeshTopologySelectionMechanic::Tick(float DeltaTime)
{
	Super::Tick(DeltaTime);

	if (SelectionDragToolUpdateType == ESelectionDragToolUpdateType::OnTickAndRelease && PendingSelectionFunction)
	{
		PendingSelectionFunction();
		PendingSelectionFunction.Reset();
	}
}

void UMeshTopologySelectionMechanic::ClearHighlight()
{
	checkf(DrawnTriangleSetComponent != nullptr, TEXT("Initialize() not called on UMeshTopologySelectionMechanic."));

	HilightSelection.Clear();
	DrawnTriangleSetComponent->Clear();
	CurrentlyHighlightedGroups.Empty();
}


void UMeshTopologySelectionMechanic::NotifyMeshChanged(bool bTopologyModified)
{
	ClearHighlight();
	TopoSelector->Invalidate(true, bTopologyModified);
	if (bTopologyModified)
	{
		PersistentSelection.Clear();
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
	}
}


bool UMeshTopologySelectionMechanic::TopologyHitTest(const FRay& WorldRay, FHitResult& OutHit, bool bUseOrthoSettings)
{
	FGroupTopologySelection Selection;
	return TopologyHitTest(WorldRay, OutHit, Selection, bUseOrthoSettings);
}

bool UMeshTopologySelectionMechanic::TopologyHitTest(const FRay& WorldRay, FHitResult& OutHit, FGroupTopologySelection& OutSelection, bool bUseOrthoSettings)
{
	// Note: this function should remain callable even if the mechanic is disabled, though client
	//  could reach in to use the TopoSelector directly.

	FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	FVector3d LocalPosition, LocalNormal;
	int32 EdgeSegmentId; // Only used if hit is an edge
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(bUseOrthoSettings);
	if (TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, OutSelection, LocalPosition, LocalNormal, &EdgeSegmentId) == false)
	{
		return false;
	}

	if (OutSelection.SelectedCornerIDs.Num() > 0)
	{
		OutHit.FaceIndex = OutSelection.GetASelectedCornerID();
		OutHit.Distance = LocalRay.GetParameter(LocalPosition);
		OutHit.ImpactPoint = (FVector)TargetTransform.TransformPosition(LocalRay.PointAt(OutHit.Distance));
	}
	else if (OutSelection.SelectedEdgeIDs.Num() > 0)
	{
		OutHit.FaceIndex = OutSelection.GetASelectedEdgeID();
		OutHit.Distance = LocalRay.GetParameter(LocalPosition);
		OutHit.ImpactPoint = (FVector)TargetTransform.TransformPosition(LocalRay.PointAt(OutHit.Distance));
		OutHit.Item = EdgeSegmentId;
	}
	else
	{
		FDynamicMeshAABBTree3* Spatial = GetSpatialFunc();
		int HitTID = Spatial->FindNearestHitTriangle(LocalRay);
		if (HitTID != IndexConstants::InvalidID)
		{
			FTriangle3d Triangle;
			Spatial->GetMesh()->GetTriVertices(HitTID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
			FIntrRay3Triangle3d Query(LocalRay, Triangle);
			if (Query.Find())
			{
				OutHit.FaceIndex = HitTID;
				OutHit.Distance = (float)Query.RayParameter;
				OutHit.Normal = (FVector)TargetTransform.TransformVectorNoScale(Spatial->GetMesh()->GetTriNormal(HitTID));
				OutHit.ImpactPoint = (FVector)TargetTransform.TransformPosition(LocalRay.PointAt(Query.RayParameter));
			}
			else
			{
				return false;
			}
		}
	}
	return true;
}

void UMeshTopologySelectionMechanic::HandleRectangleChanged(const FCameraRectangle& InRectangle)
{
	if (!bIsEnabled)
	{
		return;
	}
	
	FGroupTopologySelection RectangleSelection;

	TopoSelector->FindSelectedElement(PreDragTopoSelectorSettings, InRectangle, TargetTransform,
		RectangleSelection, &TriIsOccludedCache);

	if (ShouldAddToSelectionFunc())
	{
		PersistentSelection = PreDragPersistentSelection;
		if (ShouldRemoveFromSelectionFunc())
		{
			PersistentSelection.Toggle(RectangleSelection);
		}
		else
		{
			PersistentSelection.Append(RectangleSelection);
		}
	}
	else if (ShouldRemoveFromSelectionFunc())
	{
		PersistentSelection = PreDragPersistentSelection;
		PersistentSelection.Remove(RectangleSelection);
	}
	else
	{
		// Neither key pressed.
		PersistentSelection = RectangleSelection;
	}

	// If we modified the currently selected edges/vertices, they will be properly displayed in our
	// Render() call. However, the mechanic is not responsible for face highlighting, so if we modified
	// that, we need to notify the user so that they can update the highlighting (since OnSelectionChanged
	// only gets broadcast at rectangle end).
	if ((!PersistentSelection.SelectedGroupIDs.IsEmpty() || !LastUpdateRectangleSelection.SelectedGroupIDs.IsEmpty()) // if groups are involved
		&& PersistentSelection != LastUpdateRectangleSelection)
	{
		LastUpdateRectangleSelection = PersistentSelection;
		OnFaceSelectionPreviewChanged.Broadcast();
	}
}

FGroupTopologySelector::FSelectionSettings UMeshTopologySelectionMechanic::GetTopoSelectorSettings(bool bUseOrthoSettings)
{
	FGroupTopologySelector::FSelectionSettings Settings;

	Settings.bEnableFaceHits = Properties->bSelectFaces;
	Settings.bEnableEdgeHits = Properties->bSelectEdges || Properties->bSelectEdgeLoops || Properties->bSelectEdgeRings;
	Settings.bEnableCornerHits = Properties->bSelectVertices;
	Settings.bHitBackFaces = Properties->bHitBackFaces;

	if (!PersistentSelection.IsEmpty() && (ShouldAddToSelectionFunc() || ShouldRemoveFromSelectionFunc()))
	{
		// If we have a selection and we're adding/removing/toggling elements make sure we only hit elements with compatible types
		Settings.bEnableFaceHits = Settings.bEnableFaceHits && PersistentSelection.SelectedGroupIDs.Num() > 0;
		Settings.bEnableEdgeHits = Settings.bEnableEdgeHits && PersistentSelection.SelectedEdgeIDs.Num() > 0;
		Settings.bEnableCornerHits = Settings.bEnableCornerHits && PersistentSelection.SelectedCornerIDs.Num() > 0;
	}

	if (bUseOrthoSettings)
	{
		Settings.bPreferProjectedElement = Properties->bPreferProjectedElement;
		Settings.bSelectDownRay = Properties->bSelectDownRay;
		Settings.bIgnoreOcclusion = Properties->bIgnoreOcclusion;
	}

	return Settings;
}

bool UMeshTopologySelectionMechanic::HasSelection() const
{
	return PersistentSelection.IsEmpty() == false;
}

void UMeshTopologySelectionMechanic::SetSelection(const FGroupTopologySelection& Selection, bool bBroadcast)
{
	PersistentSelection = Selection;
	SelectionTimestamp++;
	if (bBroadcast)
	{
		OnSelectionChanged.Broadcast();
	}
}


void UMeshTopologySelectionMechanic::ClearSelection()
{
	PersistentSelection.Clear();
	SelectionTimestamp++;
	OnSelectionChanged.Broadcast();
}

void UMeshTopologySelectionMechanic::InvertSelection()
{
	if (PersistentSelection.IsEmpty())
	{
		SelectAll();
		return;
	}

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("SelectionChange", "Selection"));
	BeginChange();

	const FGroupTopologySelection PreviousSelection = PersistentSelection;
	PersistentSelection.Clear();

	const FTopologyProvider* TopologyProvider = TopoSelector->GetTopologyProvider();

	if (!PreviousSelection.SelectedCornerIDs.IsEmpty())
	{
		for (int32 CornerID = 0; CornerID < TopologyProvider->GetNumCorners(); ++CornerID)
		{
			if (!PreviousSelection.SelectedCornerIDs.Contains(CornerID))
			{
				PersistentSelection.SelectedCornerIDs.Add(CornerID);
			}
		}
	}
	else if (!PreviousSelection.SelectedEdgeIDs.IsEmpty())
	{
		for (int32 EdgeID = 0; EdgeID < TopologyProvider->GetNumEdges(); ++EdgeID)
		{
			if (!PreviousSelection.SelectedEdgeIDs.Contains(EdgeID))
			{
				PersistentSelection.SelectedEdgeIDs.Add(EdgeID);
			}
		}
	}
	else if (!PreviousSelection.SelectedGroupIDs.IsEmpty())
	{
		for (int GroupIndex = 0; GroupIndex < TopologyProvider->GetNumGroups(); ++GroupIndex)
		{
			const int GroupID = TopologyProvider->GetGroupIDAt(GroupIndex);
			if (!PreviousSelection.SelectedGroupIDs.Contains(GroupID))
			{
				PersistentSelection.SelectedGroupIDs.Add(GroupID);
			}
		}
	}

	SelectionTimestamp++;
	OnSelectionChanged.Broadcast();
	EndChangeAndEmitIfModified();
	ParentTool->GetToolManager()->EndUndoTransaction();
}

void UMeshTopologySelectionMechanic::SelectAll()
{
	const FGroupTopologySelection PreviousSelection = PersistentSelection;

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("SelectionChange", "Selection"));
	BeginChange();

	auto SelectAllIndices = [](int32 MaxExclusiveIndex, TSet<int32>& ContainerOut)
	{
		for (int32 i = 0; i < MaxExclusiveIndex; ++i)
		{
			ContainerOut.Add(i);
		}
	};

	PersistentSelection.Clear();

	const FTopologyProvider* TopologyProvider = TopoSelector->GetTopologyProvider();

	// Select based on settings, prefering corners to edges to groups (since this is the preference we have
	// elsewhere, eg in marquee).
	if (Properties->bSelectVertices)
	{
		SelectAllIndices(TopologyProvider->GetNumCorners(), PersistentSelection.SelectedCornerIDs);
	}
	else if (Properties->bSelectEdges || Properties->bSelectEdgeLoops || Properties->bSelectEdgeRings)
	{
		SelectAllIndices(TopologyProvider->GetNumEdges(), PersistentSelection.SelectedEdgeIDs);
	}
	else if (Properties->bSelectFaces)
	{
		for (int GroupIndex = 0; GroupIndex < TopologyProvider->GetNumGroups(); ++GroupIndex)
		{
			const int GroupID = TopologyProvider->GetGroupIDAt(GroupIndex);
			PersistentSelection.SelectedGroupIDs.Add(GroupID);
		}
	}

	SelectionTimestamp++;
	OnSelectionChanged.Broadcast();
	
	if (PreviousSelection != PersistentSelection)
	{
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
	}
	
	EndChangeAndEmitIfModified();
	ParentTool->GetToolManager()->EndUndoTransaction();
}

FInputRayHit UMeshTopologySelectionMechanic::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	if (!bIsEnabled)
	{
		return FInputRayHit(); // bHit is false
	}

	FHitResult OutHit;
	FGroupTopologySelection Selection;
	if (TopologyHitTest(ClickPos.WorldRay, OutHit, Selection, CameraState.bIsOrthographic))
	{
		return FInputRayHit(OutHit.Distance);
	}

	// Return a hit so we always capture and can clear the selection
	return FInputRayHit(TNumericLimits<float>::Max());
}

void UMeshTopologySelectionMechanic::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (!ensure(bIsEnabled))
	{
		return;
	}

	// update selection
	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("SelectionChange", "Selection"));
	BeginChange();

	// This will fire off a OnSelectionChanged delegate.
	UpdateSelection(ClickPos.WorldRay, LastClickedHitPosition, LastClickedHitNormal);

	EndChangeAndEmitIfModified();
	ParentTool->GetToolManager()->EndUndoTransaction();
}

FInputRayHit UMeshTopologySelectionMechanic::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FHitResult OutHit;
	if (bIsEnabled && TopologyHitTest(PressPos.WorldRay, OutHit))
	{
		return FInputRayHit(OutHit.Distance);
	}
	return FInputRayHit(); // bHit is false
}

void UMeshTopologySelectionMechanic::OnBeginHover(const FInputDeviceRay& DevicePos)
{
}

bool UMeshTopologySelectionMechanic::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	if (!bIsEnabled)
	{
		return false;
	}

	UpdateHighlight(DevicePos.WorldRay);
	return true;
}

void UMeshTopologySelectionMechanic::OnEndHover()
{
	ClearHighlight();
}

void UMeshTopologySelectionMechanic::OnUpdateModifierState(int ModifierID, bool bIsOn)
{
	switch (ModifierID)
	{
	case ShiftModifierID:
		bShiftToggle = bIsOn;
		break;
	case CtrlModifierID:
		bCtrlToggle = bIsOn;
		break;
	default:
		break;
	}
}

//////////////////////////////////////////////////////////////////////////////////////////////////////

void UMeshTopologySelectionMechanic::OnSelectionToolDragStarted()
{
	bCurrentlyMarqueeDragging = true;

	PendingSelectionFunction.Reset();

	ParentTool->GetToolManager()->BeginUndoTransaction(LOCTEXT("SelectionChange", "Selection"));
	BeginChange();

	PreDragPersistentSelection = PersistentSelection;
	LastUpdateRectangleSelection = PersistentSelection;
	PreDragTopoSelectorSettings = GetTopoSelectorSettings(false);

	PreDragTopoSelectorSettings.bIgnoreOcclusion = Properties->bIgnoreOcclusionForDragTools;
}

void UMeshTopologySelectionMechanic::OnSelectionToolDragFinished()
{
	if (PendingSelectionFunction)
	{
		PendingSelectionFunction();
		PendingSelectionFunction.Reset();
	}

	bCurrentlyMarqueeDragging = false;

	TriIsOccludedCache.Reset();

	if (PersistentSelection != PreDragPersistentSelection)
	{
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
	}

	EndChangeAndEmitIfModified();
	ParentTool->GetToolManager()->EndUndoTransaction();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////

void UMeshTopologySelectionMechanic::OnDragRectangleStarted()
{
	OnSelectionToolDragStarted();
}

void UMeshTopologySelectionMechanic::OnDragRectangleChanged(const FCameraRectangle& CurrentRectangle)
{
	if (SelectionDragToolUpdateType == ESelectionDragToolUpdateType::OnDrag)
	{
		HandleRectangleChanged(CurrentRectangle);
	}
	else
	{
		// defer the selection on tick or release  
		PendingSelectionFunction = [this, CurrentRectangle]()
		{
			HandleRectangleChanged(CurrentRectangle);
		};
	}
}

void UMeshTopologySelectionMechanic::OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled)
{
	OnSelectionToolDragFinished();
}

//////////////////////////////////////////////////////////////////////////////////////////////////////

void UMeshTopologySelectionMechanic::OnPolyLassoStarted()
{
	OnSelectionToolDragStarted();
}

void UMeshTopologySelectionMechanic::OnPolyLassoChanged(const FCameraPolyLasso& InLasso)
{
	if (SelectionDragToolUpdateType == ESelectionDragToolUpdateType::OnDrag)
	{
		HandleLassoChanged(InLasso);
	}
	else
	{
		// Defer the selection on tick or release  
		PendingSelectionFunction = [this, InLasso]()
			{
				HandleLassoChanged(InLasso);
			};
	}
}

namespace MeshTopologySelectionMechanic::Private
{

	static FCameraPolyLasso MakeClosedLasso(const FCameraPolyLasso& InLasso)
	{
		FCameraPolyLasso ClosedLasso{ InLasso };

		if (ClosedLasso.Polyline.Num() > 0)
		{
			const FVector2D FirstPoint = ClosedLasso.Polyline[0];
			if (FirstPoint != ClosedLasso.Polyline.Last())
			{
				ClosedLasso.Polyline.Add(FirstPoint);
			}
		}
		return ClosedLasso;
	}
}

void UMeshTopologySelectionMechanic::OnPolyLassoFinished(const FCameraPolyLasso& InLasso, bool bCanceled)
{
	OnSelectionToolDragFinished();
}

void UMeshTopologySelectionMechanic::HandleLassoChanged(const FCameraPolyLasso& InLasso)
{
	if (!bIsEnabled)
	{
		return;
	}

	FCameraPolyLasso ClosedLasso = MeshTopologySelectionMechanic::Private::MakeClosedLasso(InLasso);
	if (ClosedLasso.Polyline.Num() < 3)
	{
		// nothing to select really
		return;
	}

	FGroupTopologySelection RectangleSelection;

	TopoSelector->FindSelectedElement(PreDragTopoSelectorSettings, ClosedLasso, TargetTransform,
		RectangleSelection, &TriIsOccludedCache);

	if (ShouldAddToSelectionFunc())
	{
		PersistentSelection = PreDragPersistentSelection;
		if (ShouldRemoveFromSelectionFunc())
		{
			PersistentSelection.Toggle(RectangleSelection);
		}
		else
		{
			PersistentSelection.Append(RectangleSelection);
		}
	}
	else if (ShouldRemoveFromSelectionFunc())
	{
		PersistentSelection = PreDragPersistentSelection;
		PersistentSelection.Remove(RectangleSelection);
	}
	else
	{
		// Neither key pressed.
		PersistentSelection = RectangleSelection;
	}

	// If we modified the currently selected edges/vertices, they will be properly displayed in our
	// Render() call. However, the mechanic is not responsible for face highlighting, so if we modified
	// that, we need to notify the user so that they can update the highlighting (since OnSelectionChanged
	// only gets broadcast at rectangle end).
	if ((!PersistentSelection.SelectedGroupIDs.IsEmpty() || !LastUpdateRectangleSelection.SelectedGroupIDs.IsEmpty()) // if groups are involved
		&& PersistentSelection != LastUpdateRectangleSelection)
	{
		LastUpdateRectangleSelection = PersistentSelection;
		OnFaceSelectionPreviewChanged.Broadcast();
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UMeshTopologySelectionMechanic::UpdateSelectorEnabled()
{
	UpdateMarqueeEnabled();
	UpdateLassoEnabled();
}

void UMeshTopologySelectionMechanic::UpdateMarqueeEnabled()
{
	const bool bIsMarqueeEnabled = bIsEnabled
		&& Properties->SelectionDragTool == EMeshTopologySelectionMechanicDragTool::Marquee
		&& (Properties->bSelectVertices || Properties->bSelectEdges || Properties->bSelectFaces
			|| Properties->bSelectEdgeLoops || Properties->bSelectEdgeRings);
	if (bIsMarqueeEnabled != MarqueeMechanic->IsEnabled())
	{
		MarqueeMechanic->SetIsEnabled(bIsMarqueeEnabled);
		if (bIsMarqueeEnabled)
		{
			ClickOrDragBehavior->SetDragTarget(MarqueeMechanic.Get());
		}
	}
}

void UMeshTopologySelectionMechanic::UpdateLassoEnabled()
{
	const bool bIsPolyLassoEnabled = bIsEnabled
		&& Properties->SelectionDragTool == EMeshTopologySelectionMechanicDragTool::Lasso
		&& (Properties->bSelectVertices || Properties->bSelectEdges || Properties->bSelectFaces
			|| Properties->bSelectEdgeLoops || Properties->bSelectEdgeRings);
	if (bIsPolyLassoEnabled != PolyLassoMechanic->IsEnabled())
	{
		PolyLassoMechanic->SetIsEnabled(bIsPolyLassoEnabled);
		if (bIsPolyLassoEnabled)
		{
			ClickOrDragBehavior->SetDragTarget(PolyLassoMechanic.Get());
		}
	}
}

void UMeshTopologySelectionMechanic::RenderMarquee(IToolsContextRenderAPI* RenderAPI)
{
	if (MarqueeMechanic)
	{
		MarqueeMechanic->Render(RenderAPI);
	}
}

void UMeshTopologySelectionMechanic::BeginChange()
{
	// If you hit this ensure, either you didn't match a BeginChange with an EndChange(), or you
	// interleaved two actions that both needed a BeginChange(). For instance, are you sure you
	// are not performing actions while a marquee rectangle is active?
	ensure(ActiveChange.IsValid() == false);
	ActiveChange = MakeUnique<FMeshTopologySelectionMechanicSelectionChange>();
	ActiveChange->Before = PersistentSelection;
	ActiveChange->Timestamp = SelectionTimestamp;
}

TUniquePtr<FToolCommandChange> UMeshTopologySelectionMechanic::EndChange()
{
	if (ensure(ActiveChange.IsValid()) && SelectionTimestamp != ActiveChange->Timestamp)
	{
		ActiveChange->After = PersistentSelection;
		return MoveTemp(ActiveChange);
	}
	ActiveChange = TUniquePtr<FMeshTopologySelectionMechanicSelectionChange>();
	return TUniquePtr<FToolCommandChange>();
}

bool UMeshTopologySelectionMechanic::EndChangeAndEmitIfModified()
{
	if (ensure(ActiveChange.IsValid()) && SelectionTimestamp != ActiveChange->Timestamp)
	{
		ActiveChange->After = PersistentSelection;
		GetParentTool()->GetToolManager()->EmitObjectChange(this, MoveTemp(ActiveChange),
			LOCTEXT("SelectionChangeMessage", "Selection Change"));
		return true;
	}
	ActiveChange = TUniquePtr<FMeshTopologySelectionMechanicSelectionChange>();
	return false;
}

void UMeshTopologySelectionMechanic::GetClickedHitPosition(FVector3d& PositionOut, FVector3d& NormalOut) const
{
	PositionOut = LastClickedHitPosition;
	NormalOut = LastClickedHitNormal;
}

FFrame3d UMeshTopologySelectionMechanic::GetSelectionFrame(bool bWorld, FFrame3d* InitialLocalFrame) const
{
	FFrame3d UseFrame;
	if (PersistentSelection.IsEmpty() == false)
	{
		UseFrame = TopoSelector->GetTopologyProvider()->GetSelectionFrame(PersistentSelection, InitialLocalFrame);
	}

	if (bWorld)
	{
		UseFrame.Transform(TargetTransform);
	}

	return UseFrame;
}

FAxisAlignedBox3d UMeshTopologySelectionMechanic::GetSelectionBounds(bool bWorld) const
{
	if ( ! PersistentSelection.IsEmpty() )
	{
		const FTopologyProvider* TopologyProvider = TopoSelector->GetTopologyProvider();

		if (bWorld)
		{
			return TopologyProvider->GetSelectionBounds(PersistentSelection, [this](const FVector3d& Pos) { return TargetTransform.TransformPosition(Pos); });
		}
		else
		{
			return TopologyProvider->GetSelectionBounds(PersistentSelection, [this](const FVector3d& Pos) { return Pos; });
		}
	}
	else 
	{
		if (bWorld)
		{
			return FAxisAlignedBox3d(Mesh->GetBounds(), TargetTransform);
		}
		else
		{
			return Mesh->GetBounds();
		}
	}
}

void UMeshTopologySelectionMechanic::SetShowSelectableCorners(bool bShowCorners)
{
	bShowSelectableCorners = bShowCorners;
}



void FMeshTopologySelectionMechanicSelectionChange::Apply(UObject* Object)
{
	UMeshTopologySelectionMechanic* Mechanic = Cast<UMeshTopologySelectionMechanic>(Object);
	if (Mechanic)
	{
		Mechanic->PersistentSelection = After;
		Mechanic->OnSelectionChanged.Broadcast();
	}
}
void FMeshTopologySelectionMechanicSelectionChange::Revert(UObject* Object)
{
	UMeshTopologySelectionMechanic* Mechanic = Cast<UMeshTopologySelectionMechanic>(Object);
	if (Mechanic)
	{
		Mechanic->PersistentSelection = Before;
		Mechanic->OnSelectionChanged.Broadcast();
	}
}
FString FMeshTopologySelectionMechanicSelectionChange::ToString() const
{
	return TEXT("FMeshTopologySelectionMechanicSelectionChange");
}

void UMeshTopologySelectionMechanic::Initialize(const FDynamicMesh3* MeshIn,
	FTransform3d TargetTransformIn,
	UWorld* WorldIn,
	TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFuncIn)
{
	this->Mesh = MeshIn;
	this->TargetTransform = TargetTransformIn;

	this->GetSpatialFunc = GetSpatialSourceFuncIn;
	TopoSelector->SetSpatialSource(GetSpatialFunc);
	TopoSelector->PointsWithinToleranceTest = [this](const FVector3d& Position1, const FVector3d& Position2, double TolScale) {
		if (CameraState.bIsOrthographic)
		{
			// We could just always use ToolSceneQueriesUtil::PointSnapQuery. But in ortho viewports, we happen to know
			// that the only points that we will ever give this function will be the closest points between a ray and
			// some geometry, meaning that the vector between them will be orthogonal to the view ray. With this knowledge,
			// we can do the tolerance computation more efficiently than PointSnapQuery can, since we don't need to project
			// down to the view plane.
			// As in PointSnapQuery, we convert our angle-based tolerance to one we can use in an ortho viewport (instead of
			// dividing our field of view into 90 visual angle degrees, we divide the plane into 90 units).
			float OrthoTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * CameraState.OrthoWorldCoordinateWidth / 90.0;
			OrthoTolerance *= TolScale;
			return DistanceSquared(TargetTransform.TransformPosition(Position1), TargetTransform.TransformPosition(Position2)) < OrthoTolerance * OrthoTolerance;
		}
		else
		{
			return ToolSceneQueriesUtil::PointSnapQuery(CameraState,
				TargetTransform.TransformPosition(Position1), TargetTransform.TransformPosition(Position2),
				ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD() * TolScale);
		}
	};

	// Set up the component we use to draw highlighted triangles. Only needs to be done once, not when the mesh
	// changes (we are assuming that we won't swap worlds without creating a new mechanic).
	if (PreviewGeometryActor == nullptr)
	{
		FRotator Rotation(0.0f, 0.0f, 0.0f);
		FActorSpawnParameters SpawnInfo;;
		PreviewGeometryActor = WorldIn->SpawnActor<APreviewGeometryActor>(FVector::ZeroVector, Rotation, SpawnInfo);

		DrawnTriangleSetComponent = NewObject<UTriangleSetComponent>(PreviewGeometryActor);
		PreviewGeometryActor->SetRootComponent(DrawnTriangleSetComponent);
		DrawnTriangleSetComponent->RegisterComponent();
	}

	PreviewGeometryActor->SetActorTransform((FTransform)TargetTransformIn);

	DrawnTriangleSetComponent->Clear();
	CurrentlyHighlightedGroups.Empty();
}

#undef LOCTEXT_NAMESPACE
