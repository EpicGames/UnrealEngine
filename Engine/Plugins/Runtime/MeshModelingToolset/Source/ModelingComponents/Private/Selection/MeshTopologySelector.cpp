// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/MeshTopologySelector.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Spatial/SpatialInterfaces.h"
#include "Engine/Polys.h"
#include "Mechanics/RectangleMarqueeMechanic.h" // FCameraRectangle
#include "Mechanics/PolyLassoMarqueeMechanic.h" // FCameraPolyLasso
#include "ConvexVolume.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FMeshTopologySelector"

// Local utility function forward declarations
bool IsOccluded(const FGeometrySet3::FNearest& ClosestElement, const FVector3d& ViewOrigin, const FDynamicMeshAABBTree3* Spatial, bool bBackFacesOcclude);
bool IsOccluded(const FVector3d& Point, const FVector3d& ViewOrigin, const FDynamicMeshAABBTree3* Spatial, bool bBackFacesOcclude);

FMeshTopologySelector::FMeshTopologySelector()
{
	// initialize to sane values
	PointsWithinToleranceTest =
		[](const FVector3d& A, const FVector3d& B, double TolScale) { return Distance(A, B) < (TolScale*1.0); };
	GetSpatial =
		[]() { return nullptr; };
}


void FMeshTopologySelector::Invalidate(bool bTopologyDeformed, bool bTopologyModified)
{
	if (bTopologyDeformed)
	{
		bGeometryUpToDate = false;
	}
	if (bTopologyModified)
	{
		bGeometryUpToDate = bGeometryInitialized = false;
	}
}


const FGeometrySet3& FMeshTopologySelector::GetGeometrySet()
{
	if (bGeometryInitialized == false)
	{
		GeometrySet.Reset();
		int NumCorners = TopologyProvider->GetNumCorners();
		for (int CornerID = 0; CornerID < NumCorners; ++CornerID)
		{
			FVector3d Position = Mesh->GetVertex(TopologyProvider->GetCornerVertexID(CornerID));
			GeometrySet.AddPoint(CornerID, Position);
		}
		int NumEdges = TopologyProvider->GetNumEdges();
		for (int EdgeID = 0; EdgeID < NumEdges; ++EdgeID)
		{
			FPolyline3d Polyline;
			TopologyProvider->GetEdgePolyline(EdgeID, Polyline);
			GeometrySet.AddCurve(EdgeID, Polyline);
		}

		bGeometryInitialized = true;
		bGeometryUpToDate = true;
	}

	if (bGeometryUpToDate == false)
	{
		int NumCorners = TopologyProvider->GetNumCorners();
		for (int CornerID = 0; CornerID < NumCorners; ++CornerID)
		{
			FVector3d Position = Mesh->GetVertex(TopologyProvider->GetCornerVertexID(CornerID));
			GeometrySet.UpdatePoint(CornerID, Position);
		}
		int NumEdges = TopologyProvider->GetNumEdges();
		for (int EdgeID = 0; EdgeID < NumEdges; ++EdgeID)
		{
			FPolyline3d Polyline;
			TopologyProvider->GetEdgePolyline(EdgeID, Polyline);
			GeometrySet.UpdateCurve(EdgeID, Polyline);
		}
		bGeometryUpToDate = true;
	}

	return GeometrySet;
}

bool FMeshTopologySelector::FindSelectedElement(const FSelectionSettings& Settings, const FRay3d& Ray,
	FGroupTopologySelection& ResultOut, FVector3d& SelectedPositionOut, FVector3d& SelectedNormalOut, int32* EdgeSegmentIdOut)
{
	// These get used for finding intersections with triangles and corners/edges, respectively.
	FDynamicMeshAABBTree3* Spatial = GetSpatial();
	const FGeometrySet3& TopoSpatial = GetGeometrySet();

	IMeshSpatial::FQueryOptions SpatialQueryOptions;
	if (Spatial && !Settings.bHitBackFaces)
	{
		SpatialQueryOptions.TriangleFilterF = [Spatial, &Ray](int32 Tid) {
			return Spatial->GetMesh()->GetTriNormal(Tid).Dot(Ray.Direction) < 0;
		};
	}

	// We start by intersecting with the mesh triangles because even when selecting corners or edges, we set
	// the normal based on the true triangle that we hit. If we end up with a simple face selection, we will
	// end up using this result.
	double RayParameter = -1;
	int HitTriangleID = IndexConstants::InvalidID;
	FVector3d TriangleHitPos = FVector3d::ZeroVector;
	bool bActuallyHitSurface = (Spatial != nullptr) ? Spatial->FindNearestHitTriangle(Ray, RayParameter, HitTriangleID, SpatialQueryOptions) : false;
	if (bActuallyHitSurface)
	{
		TriangleHitPos = Ray.PointAt(RayParameter);
		SelectedNormalOut = Mesh->GetTriNormal(HitTriangleID);
	}
	else
	{
		SelectedNormalOut = FVector3d::UnitZ();
	}
	bool bHaveFaceHit = (bActuallyHitSurface && Settings.bEnableFaceHits);
	
	// Deal with corner hits first (and edges that project to a corner)
	FGroupTopologySelection CornerResults;
	FVector3d CornerPosition = FVector3d::ZeroVector;
	int32 CornerSegmentEdgeID = 0;
	bool bHaveCornerHit = false;
	if (Settings.bEnableCornerHits || (Settings.bEnableEdgeHits && Settings.bPreferProjectedElement))
	{
		if (DoCornerBasedSelection(Settings, Ray, Spatial, TopoSpatial, CornerResults, CornerPosition, &CornerSegmentEdgeID))
		{
			bHaveCornerHit = true;
		}
	}

	// If corner selection didn't yield results, try edge selection
	FGroupTopologySelection EdgeResults;
	FVector3d EdgePosition = FVector3d::ZeroVector;
	int32 EdgeSegmentEdgeID = 0;
	bool bHaveEdgeHit = false;
	if (Settings.bEnableEdgeHits || (Settings.bEnableFaceHits && Settings.bPreferProjectedElement))
	{
		if (DoEdgeBasedSelection(Settings, Ray, Spatial, TopoSpatial, EdgeResults, EdgePosition, &EdgeSegmentEdgeID))
		{
			bHaveEdgeHit = true;
		}
	}

	// If we have multiple types of hits, we need a prioritization to resolve between them. If we have a face hit,
	//  we will still prefer corner and then edge, but we will lower our tolerance so that it is easier to select
	//  the insides of small triangles on a mesh. The tolerance is different for corners and edges because we
	//  expect the corners to be drawn a bit bigger (these specific values are tuned for PolyEdit)
	if (bHaveFaceHit)
	{
		constexpr double CornerFaceResolveProximityTolerance = 0.25;
		constexpr double EdgeFaceResolveProximityTolerance = 0.20;
		if (bHaveCornerHit)
		{
			bHaveCornerHit = PointsWithinToleranceTest(CornerPosition, Ray.ClosestPoint(CornerPosition), 
				CornerFaceResolveProximityTolerance);
			bHaveEdgeHit = bHaveEdgeHit && !bHaveCornerHit;
		}
		if (bHaveEdgeHit)
		{
			ensure(!bHaveCornerHit); // cleared above, or else cleared bHaveEdgeHit
			bHaveEdgeHit = PointsWithinToleranceTest(EdgePosition, Ray.ClosestPoint(EdgePosition),
				EdgeFaceResolveProximityTolerance);
		}
	}
	// If we have both edge and corner, we prefer corner, but with a lower tolerance so that we can hit
	//  more of the middle portion of the edge.
	else if (bHaveCornerHit && bHaveEdgeHit)
	{
		constexpr double CornerEdgeResolveProximityTolerance = 0.75;
		bHaveCornerHit = PointsWithinToleranceTest(CornerPosition, Ray.ClosestPoint(CornerPosition), CornerEdgeResolveProximityTolerance);
		bHaveEdgeHit = !bHaveCornerHit;
	}

	if (bHaveCornerHit)
	{
		ResultOut = CornerResults;
		SelectedPositionOut = CornerPosition;
		if (EdgeSegmentIdOut != nullptr)
		{
			*EdgeSegmentIdOut = CornerSegmentEdgeID;
		}
		return true;
	}
	if (bHaveEdgeHit)
	{
		ResultOut = EdgeResults;
		SelectedPositionOut = EdgePosition;
		if (EdgeSegmentIdOut != nullptr)
		{
			*EdgeSegmentIdOut = EdgeSegmentEdgeID;
		}
		return true;
	}
	if (bHaveFaceHit)
	{
		// If we still haven't found a selection, go ahead and select the face that we found earlier
		ResultOut.SelectedGroupIDs.Add(TopologyProvider->GetGroupIDForTriangle(HitTriangleID));
		SelectedPositionOut = TriangleHitPos;
		return true;
	}


	return false;
}

bool FMeshTopologySelector::DoCornerBasedSelection(const FSelectionSettings& Settings,
	const FRay3d& Ray, FDynamicMeshAABBTree3* Spatial, const FGeometrySet3& TopoSpatial,
	FGroupTopologySelection& ResultOut, FVector3d& SelectedPositionOut, int32 *EdgeSegmentIdOut) const
{
	// These will store our results, depending on whether we select all along the ray or not.
	FGeometrySet3::FNearest SingleElement;
	// ElementsWithinTolerance gives all nearby elements returned by the topology selector, whereas DownRayElements filters these to just
	// the closest element and those that project directly onto it (and it only stores the IDs, since we don't care about anything else).
	TArray<FGeometrySet3::FNearest> ElementsWithinTolerance;
	TArray<int32> DownRayElements;

	auto LocalTolTest = [this](const FVector3d& A, const FVector3d& B) { return PointsWithinToleranceTest(A, B, 1.0); };

	// Start by getting the closest element
	const FGeometrySet3::FNearest* ClosestElement = nullptr;
	if (!Settings.bSelectDownRay)
	{
		if (TopoSpatial.FindNearestPointToRay(Ray, SingleElement, LocalTolTest))
		{
			ClosestElement = &SingleElement;
		}
	}
	else
	{
		// We're collecting all corners within tolerance, but we still need the closest element
		if (TopoSpatial.CollectPointsNearRay(Ray, ElementsWithinTolerance, LocalTolTest))
		{
			double MinRayT = TNumericLimits<double>::Max();
			for (const FGeometrySet3::FNearest& Element : ElementsWithinTolerance)
			{
				if (Element.RayParam < MinRayT)
				{
					MinRayT = Element.RayParam;
					ClosestElement = &Element;
				}
			}
		}//end if found corners
	}//end if selecting down ray

	// Bail early if we haven't found a corner
	if (ClosestElement == nullptr)
	{
		return false;
	}

	// Also bail if the closest element is not visible.
	if (!Settings.bIgnoreOcclusion && IsOccluded(*ClosestElement, Ray.Origin, Spatial, Settings.bHitBackFaces))
	{
		return false;
	}

	// The closest point is already found
	SelectedPositionOut = ClosestElement->NearestGeoPoint;

	// If we have other corners, we actually need to filter them to only those that lie in line with the closest element. Note that
	// this would be done differently depending on whether we're dealing with an orthographic or perspective view: in an
	// orthographic view, we need to see that they lie on a ray with view ray direction and closest corner origin, whereas in
	// perspective, they would need to lie on a ray from camer through closest corner (which will differ due to tolerance).
	// Because the "select down ray" behavior is only useful in orthographic viewports in the first place, we do it that way.
	if (Settings.bSelectDownRay)
	{
		DownRayElements.Add(ClosestElement->ID);
		for (const FGeometrySet3::FNearest& Element : ElementsWithinTolerance)
		{
			if (ClosestElement == &Element)
			{
				continue; // Already added
			}

			// Make sure that closest corner to current element is parallel with view ray
			FVector3d ClosestTowardElement = UE::Geometry::Normalized(Element.NearestGeoPoint - ClosestElement->NearestGeoPoint);
			// There would usually be one more abs() in here, but we know that other elements are down ray direction
			if (abs(ClosestTowardElement.Dot(Ray.Direction) - 1.0) < KINDA_SMALL_NUMBER)
			{
				DownRayElements.Add(Element.ID);
			}
		}//end assembling aligned corners
	}

	// Try to select edges that project to corners.
	if (Settings.bPreferProjectedElement && Settings.bEnableEdgeHits)
	{
		TSet<int32> AddedTopologyEdges;

		// See if the closest vertex has an attached edge that is colinear with the view ray. Due to the fact that
		// topology "edges" are actually polylines, we could actually have more than one even for the closest corner
		// (if the "edge" towards us curves away). We'll only grab an edge down our view ray.
		int32 ClosestVid = TopologyProvider->GetCornerVertexID(ClosestElement->ID);
		for (int32 Eid : Mesh->VtxEdgesItr(ClosestVid))
		{
			FDynamicMesh3::FEdge Edge = Mesh->GetEdge(Eid);
			int32 OtherVid = (Edge.Vert.A == ClosestVid) ? Edge.Vert.B : Edge.Vert.A;
			FVector3d EdgeVector = UE::Geometry::Normalized(Mesh->GetVertex(OtherVid) - Mesh->GetVertex(ClosestVid));
			if (abs(EdgeVector.Dot(Ray.Direction) - 1.0) < KINDA_SMALL_NUMBER)
			{
				int TopologyEdgeIndex = TopologyProvider->FindGroupEdgeID(Eid);
				if (TopologyEdgeIndex >= 0)
				{
					ResultOut.SelectedEdgeIDs.Add(TopologyEdgeIndex);
					AddedTopologyEdges.Add(TopologyEdgeIndex);

					if (EdgeSegmentIdOut)
					{
						TopologyProvider->GetGroupEdgeEdges(TopologyEdgeIndex).Find(Eid, *EdgeSegmentIdOut);
					}

					break;
				}
			}
		}

		// If relevant, get all the other colinear edges
		if (Settings.bSelectDownRay && AddedTopologyEdges.Num() > 0)
		{
			for (int i = 1; i < DownRayElements.Num(); ++i) // skip 0 because it is closest and done
			{
				// Look though any attached edges.
				for (int32 Eid : Mesh->VtxEdgesItr(TopologyProvider->GetCornerVertexID(DownRayElements[i])))
				{
					FDynamicMesh3::FEdge Edge = Mesh->GetEdge(Eid);
					FVector3d EdgeVector = UE::Geometry::Normalized(Mesh->GetVertex(Edge.Vert.A) - Mesh->GetVertex(Edge.Vert.B));

					// Compare absolute value of dot product to 1. We already made sure that one of the vertices is in line
					// with the closest vertex earlier.
					if (abs(abs(EdgeVector.Dot(Ray.Direction)) - 1.0) < KINDA_SMALL_NUMBER)
					{
						int TopologyEdgeIndex = TopologyProvider->FindGroupEdgeID(Eid);
						if (TopologyEdgeIndex >= 0 && !AddedTopologyEdges.Contains(TopologyEdgeIndex))
						{
							ResultOut.SelectedEdgeIDs.Add(TopologyEdgeIndex);
							AddedTopologyEdges.Add(TopologyEdgeIndex);
							// Don't break here because we may have parallel edges in both directions, since we aren't
							// going through the vertices in a particular order.
						}
					}
				}//end checking edges
			}//end going through corners
		}

		// If we found edges, we're done
		if (AddedTopologyEdges.Num() > 0)
		{
			return true;
		}
	}//end selecting projected edges

	// If getting projected edges didn't work out, go ahead and add the corners.
	if (Settings.bEnableCornerHits)
	{
		if (Settings.bSelectDownRay)
		{
			for (int32 Id : DownRayElements)
			{
				ResultOut.SelectedCornerIDs.Add(Id);
			}
		}
		else
		{
			ResultOut.SelectedCornerIDs.Add(ClosestElement->ID);
		}
		return true;
	}

	return false;
}

bool FMeshTopologySelector::DoEdgeBasedSelection(const FSelectionSettings& Settings, const FRay3d& Ray,
	FDynamicMeshAABBTree3* Spatial, const FGeometrySet3& TopoSpatial,
	FGroupTopologySelection& ResultOut, FVector3d& SelectedPositionOut, int32* EdgeSegmentIdOut) const
{
	// These will store our results, depending on whether we select all along the ray or not.
	FGeometrySet3::FNearest SingleElement;
	// ElementsWithinTolerance gives all nearby elements returned by the topology selector, whereas DownRayElements filters these to just
	// the closest element and those that project directly onto it (it stores the polyline and segment IDs, which are all we care about).
	TArray<FGeometrySet3::FNearest> ElementsWithinTolerance;
	TArray<FIndex2i> DownRayElements;

	auto LocalTolTest = [this](const FVector3d& A, const FVector3d& B) { return PointsWithinToleranceTest(A, B, 1.0); };

	// Start by getting the closest element
	const FGeometrySet3::FNearest* ClosestElement = nullptr;
	if (!Settings.bSelectDownRay)
	{
		if (TopoSpatial.FindNearestCurveToRay(Ray, SingleElement, LocalTolTest))
		{
			ClosestElement = &SingleElement;
		}
	}
	else
	{
		// Need all curves within tolerance, but also need to know the closest.
		if (TopoSpatial.CollectCurvesNearRay(Ray, ElementsWithinTolerance, LocalTolTest))
		{
			double MinRayT = TNumericLimits<double>::Max();
			for (const FGeometrySet3::FNearest& Element : ElementsWithinTolerance)
			{
				if (Element.RayParam < MinRayT)
				{
					MinRayT = Element.RayParam;
					ClosestElement = &Element;
				}
			}
		}//end if found edges
	}//end if selecting down ray

	// Bail early if we haven't found at least one edge
	if (ClosestElement == nullptr)
	{
		return false;
	}

	// Also bail if the closest element is not visible.
	if (!Settings.bIgnoreOcclusion && IsOccluded(*ClosestElement, Ray.Origin, Spatial, Settings.bHitBackFaces))
	{
		return false;
	}

	// The closest point is already found
	SelectedPositionOut = ClosestElement->NearestGeoPoint;

	// If we have other edges, we need to filter them to only those that project onto the closest element. This would be done
	// differently for perspective cameras vs orthographic projection, but since the behavior is only useful in ortho mode,
	// we do it that way.
	if (Settings.bSelectDownRay)
	{
		// Closest element is a given
		DownRayElements.Add(FIndex2i(ClosestElement->ID, ClosestElement->PolySegmentIdx));

		// We want edges that lie in a plane through the closest edge that is coplanar with the view direction. If we had wanted 
		// to do this for perspective mode, we would have picked a plane through the edge and the camera origin.
		int32 ClosestEid = TopologyProvider->GetGroupEdgeEdges(ClosestElement->ID)[ClosestElement->PolySegmentIdx];
		FDynamicMesh3::FEdge ClosestEdge = Mesh->GetEdge(ClosestEid);
		UE::Geometry::FPlane3d PlaneThroughClosestEdge(Mesh->GetVertex(ClosestEdge.Vert.A), Mesh->GetVertex(ClosestEdge.Vert.B), 
			Mesh->GetVertex(ClosestEdge.Vert.A) + Ray.Direction);

		for (const FGeometrySet3::FNearest& Element : ElementsWithinTolerance)
		{
			if (ClosestElement == &Element)
			{
				continue; // already added
			}

			// See if the edge endpoints lie in the plane
			int32 Eid = TopologyProvider->GetGroupEdgeEdges(Element.ID)[Element.PolySegmentIdx];
			FDynamicMesh3::FEdge Edge = Mesh->GetEdge(Eid);
			if (abs(PlaneThroughClosestEdge.DistanceTo(Mesh->GetVertex(Edge.Vert.A))) < KINDA_SMALL_NUMBER
				&& abs(PlaneThroughClosestEdge.DistanceTo(Mesh->GetVertex(Edge.Vert.B))) < KINDA_SMALL_NUMBER)
			{
				DownRayElements.Add(FIndex2i(Element.ID, Element.PolySegmentIdx));
			}
		}
	}

	// Try to select faces that project to the closest edge
	if (Settings.bPreferProjectedElement && Settings.bEnableFaceHits)
	{
		TSet<int32> AddedGroups;

		// Start with the closest edge
		int32 Eid = TopologyProvider->GetGroupEdgeEdges(ClosestElement->ID)[ClosestElement->PolySegmentIdx];
		FDynamicMesh3::FEdge Edge = Mesh->GetEdge(Eid);
		FVector3d VertA = Mesh->GetVertex(Edge.Vert.A);
		FVector3d VertB = Mesh->GetVertex(Edge.Vert.B);


		// Grab a plane through the two verts that contains the ray direction (again, this is assuming that we'd
		// only do this in ortho mode, otherwise the plane would go through the ray origin.
		UE::Geometry::FPlane3d PlaneThroughClosestEdge(VertA, VertB, VertA + Ray.Direction);

		// Checking that the face is coplanar simply entails checking that the opposite vert is in the plane. However,
		// it is possible even for the closest edge to have multiple coplanar faces if a group going towards
		// the camera curves away. We need to be able to select just one, and we want to select the one that
		// extends down the view ray. For that, we'll make sure that a vector to the opposite vertex lies on the same
		// side of the edge as the view ray.
		FVector3d EdgeVector = VertB - VertA;
		FVector3d EdgeVecCrossDirection = EdgeVector.Cross(Ray.Direction);
		FIndex2i OppositeVids = Mesh->GetEdgeOpposingV(Eid);

		FVector3d OppositeVert = Mesh->GetVertex(OppositeVids.A);
		if (abs(PlaneThroughClosestEdge.DistanceTo(OppositeVert)) < KINDA_SMALL_NUMBER
			&& EdgeVector.Cross(OppositeVert - VertA).Dot(EdgeVecCrossDirection) > 0)
		{
			int GroupId = TopologyProvider->GetGroupIDForTriangle(Edge.Tri.A);
			ResultOut.SelectedGroupIDs.Add(GroupId);
			AddedGroups.Add(GroupId);
		}
		else if (OppositeVids.B != FDynamicMesh3::InvalidID)
		{
			OppositeVert = Mesh->GetVertex(OppositeVids.B);

			if (abs(PlaneThroughClosestEdge.DistanceTo(OppositeVert)) < KINDA_SMALL_NUMBER
				&& EdgeVector.Cross(OppositeVert - VertA).Dot(EdgeVecCrossDirection) > 0)
			{
				int GroupId = TopologyProvider->GetGroupIDForTriangle(Edge.Tri.B);
				ResultOut.SelectedGroupIDs.Add(GroupId);
				AddedGroups.Add(GroupId);
			}
		}

		// If relevant, get all the other coplanar faces
		if (Settings.bSelectDownRay && AddedGroups.Num() > 0)
		{
			for (int i = 1; i < DownRayElements.Num(); ++i) // skip 0 because it is closest and done
			{
				// We already made sure that all these edges are coplanar, so we'll just be checking opposite verts.
				Eid = TopologyProvider->GetGroupEdgeEdges(DownRayElements[i].A)[DownRayElements[i].B];
				Edge = Mesh->GetEdge(Eid);
				OppositeVids = Mesh->GetEdgeOpposingV(Eid);

				// No need to check directionality of faces here, just need them to be coplanar.
				if (abs(PlaneThroughClosestEdge.DistanceTo(Mesh->GetVertex(OppositeVids.A))) < KINDA_SMALL_NUMBER)
				{
					int GroupId = TopologyProvider->GetGroupIDForTriangle(Edge.Tri.A);
					if (!AddedGroups.Contains(GroupId))
					{
						ResultOut.SelectedGroupIDs.Add(GroupId);
						AddedGroups.Add(GroupId);
					}
				}
				if (OppositeVids.B != FDynamicMesh3::InvalidID
					&& abs(PlaneThroughClosestEdge.DistanceTo(Mesh->GetVertex(OppositeVids.B))) < KINDA_SMALL_NUMBER)
				{
					int GroupId = TopologyProvider->GetGroupIDForTriangle(Edge.Tri.B);
					if (!AddedGroups.Contains(GroupId))
					{
						ResultOut.SelectedGroupIDs.Add(GroupId);
						AddedGroups.Add(GroupId);
					}
				}
			}//end going through other edges
		}

		// If we selected faces, we're done
		if (AddedGroups.Num() > 0)
		{
			return true;
		}
	}

	// If we didn't end up selecting projected faces, and we have edges to select, select them
	if (Settings.bEnableEdgeHits)
	{
		if (Settings.bSelectDownRay)
		{
			for (const FIndex2i& ElementTuple : DownRayElements)
			{
				ResultOut.SelectedEdgeIDs.Add(ElementTuple.A);
			}
		}
		else
		{
			ResultOut.SelectedEdgeIDs.Add(ClosestElement->ID);
		}

		if (EdgeSegmentIdOut)
		{
			*EdgeSegmentIdOut = ClosestElement->PolySegmentIdx;
		}

		return true;
	}
	return false;
}

bool IsOccluded(const FGeometrySet3::FNearest& ClosestElement, const FVector3d& ViewOrigin, 
	const FDynamicMeshAABBTree3* Spatial, bool bBackFacesOcclude)
{
	return IsOccluded(ClosestElement.NearestGeoPoint, ViewOrigin, Spatial, bBackFacesOcclude);
}

bool IsOccluded(const FVector3d& Point, const FVector3d& ViewOrigin, 
	const FDynamicMeshAABBTree3* Spatial, bool bBackFacesOcclude)
{
	// We will shoot a ray backwards to see if we hit something. 
	FRay3d ToEyeRay(Point, UE::Geometry::Normalized(ViewOrigin - Point), true);
	ToEyeRay.Origin += (double)(100 * FMathf::ZeroTolerance) * ToEyeRay.Direction;

	IMeshSpatial::FQueryOptions QueryOptions;
	if (!bBackFacesOcclude)
	{
		// If back faces don't occlude (ie we weren't hitting back faces on the way forward), 
		// then the direction toward the camera should be same as normal direction.
		QueryOptions.TriangleFilterF = [Spatial, &ToEyeRay](int32 Tid)
		{
			return Spatial->GetMesh()->GetTriNormal(Tid).Dot(ToEyeRay.Direction) > 0;
		};
	}

	if (Spatial->FindNearestHitTriangle(ToEyeRay, QueryOptions) >= 0)
	{
		return true;
	}
	return false;
}

bool FMeshTopologySelector::FindSelectedElement(const FSelectionSettings& Settings, 
	const FCameraRectangle& CameraRectangle, FTransform3d TargetTransform, FGroupTopologySelection& ResultOut,
	TMap<int32, bool>* TriIsOccludedCache)
{
	// One minor easy thing we can do to speed up the below is to detect cases where transform does not have a non-uniform
	// scale, and in those cases we transform the rectangle once rather than transforming all the query points/curves. 
	// This isn't worth doing for non-uniform scale transforms because the rectangle basis stops being orthonormal and 
	// becomes a problem to work with.

	ResultOut.Clear();
	FDynamicMeshAABBTree3* Spatial = GetSpatial();

	// Needed for occlusion test, which happens in local space.
	FVector3d LocalCameraOrigin = TargetTransform.InverseTransformPosition((FVector3d)CameraRectangle.CameraState.Position);
	
	// Corner selection takes priority over edges.
	if (Settings.bEnableCornerHits)
	{
		GeometrySet.ParallelFindAllPointsSatisfying(
			[&CameraRectangle, &TargetTransform, &Settings, &LocalCameraOrigin, Spatial]
			(const FVector3d& PointPosition)
			{
				FVector TransformedPoint = TargetTransform.TransformPosition(PointPosition);
				return CameraRectangle.IsProjectedPointInRectangle(TransformedPoint)
					&& (Settings.bIgnoreOcclusion || !IsOccluded(PointPosition, LocalCameraOrigin, 
						Spatial, Settings.bHitBackFaces));
			},
			ResultOut.SelectedCornerIDs);
	}

	// If we didn't get corners, look for edges.
	if (ResultOut.SelectedCornerIDs.IsEmpty() && Settings.bEnableEdgeHits)
	{
		GeometrySet.ParallelFindAllCurvesSatisfying(
			[&CameraRectangle, &TargetTransform, &Settings, &LocalCameraOrigin, Spatial](const FPolyline3d& Curve)
			{
				// Testing occlusion properly seems like it would be a pain, so we'll just consider something occluded
				// if one of the endpoints is occluded. This will handle the common case of not wanting to select a
				// hidden edge that is connected to a visible corner.
				if (!Settings.bIgnoreOcclusion 
					&& (IsOccluded(Curve.Start(), LocalCameraOrigin, Spatial, Settings.bHitBackFaces)
						|| IsOccluded(Curve.End(), LocalCameraOrigin, Spatial, Settings.bHitBackFaces)))
				{
					return false;
				}

				// Check whether any of the component segments intersect the rectangle
				const TArray<FVector3d>& Verts = Curve.GetVertices();
				FVector3d CurrentVert = TargetTransform.TransformPosition(Verts[0]);
				for (int32 i = 1; i < Verts.Num(); ++i)
				{
					FVector3d NextVert = TargetTransform.TransformPosition(Verts[i]);
					if (CameraRectangle.IsProjectedSegmentIntersectingRectangle((FVector)CurrentVert, (FVector)NextVert))
					{
						return true;
					}
					CurrentVert = NextVert;
				}
				return false;
			},
			ResultOut.SelectedEdgeIDs);
	}
	
	if (Settings.bEnableFaceHits && ResultOut.IsEmpty())
	{
		// Get a frustum volume in local space, then use it to figure out if it contains or intersects
		// the aabb tree boxes in the traversal. 
		// TODO: Frustum transformation is done the same way in FractureEditorMode.cpp- should this be placed
		// somewhere common? Not sure where to put it though.
		FConvexVolume WorldSpaceFrustum = CameraRectangle.FrustumAsConvexVolume();
		FMatrix InverseTargetTransform(FTransform(TargetTransform).ToInverseMatrixWithScale());

		FConvexVolume LocalFrustum;
		LocalFrustum.Planes.Empty(6);
		for (const FPlane& Plane : WorldSpaceFrustum.Planes)
		{
			LocalFrustum.Planes.Add(Plane.TransformBy(InverseTargetTransform));
		}
		LocalFrustum.Init();

		FDynamicMeshAABBTree3::FTreeTraversal Traversal;
		// Used as additional state for the traversal, relying on the fact that our traversal is depth-first
		// and serial. When we descend to a box that is fully contained, we use CurrentSelectAllDepth to mark
		// all deeper boxes as contained, and reset it once we return back out of that subtree (done similarly
		// in UVEditorMeshSelectionMechanic).
		int CurrentDepth = -1;
		int CurrentSelectAllDepth = TNumericLimits<int>::Max();

		Traversal.NextBoxF = [&LocalFrustum, &CurrentDepth, &CurrentSelectAllDepth](const FAxisAlignedBox3d& Box, int Depth)
		{
			CurrentDepth = Depth;
			if (Depth > CurrentSelectAllDepth)
			{
				// We're currently inside a fully-selected box
				return true;
			}
			// Reset CurrentSelectAllDepth because we must have left the cotained subtree (if it was set to begin with)
			CurrentSelectAllDepth = TNumericLimits<int>::Max();

			bool bFullyContained = false;
			bool bIntersects = LocalFrustum.IntersectBox(Box.Center(), Box.Extents(), bFullyContained);
			
			if (bFullyContained)
			{
				CurrentSelectAllDepth = Depth;
			}
			return bIntersects;
		};

		Traversal.NextTriangleF = 
			[this, &ResultOut, &CurrentDepth, &CurrentSelectAllDepth, &LocalFrustum, 
			&LocalCameraOrigin, &Spatial, &Settings, TriIsOccludedCache](int Tid)
		{
			int32 TriGroupID = TopologyProvider->GetGroupIDForTriangle(Tid);

			// If this group already got selected, no need to do anything else. Helps avoid some occlusion tests.
			if (ResultOut.SelectedGroupIDs.Contains(TriGroupID))
			{
				return;
			}

			// Apply frustum check. Skip if box was fully in frustum
			if (!(CurrentDepth >= CurrentSelectAllDepth))
			{
				// TODO: This isn't an ideal way to check whether the triangle is inside the frustum because FPoly is
				// an old class that we probably want to avoid using. It's fine enough here, but it should be replaced.
				// Related note: It may be tempting to just check for containment of the centroid, but we probably don't
				// want to do that because it makes selection behavior dependent on the tesselation of the underlying
				// group. I.e. a remeshed cube face ends up being much more reliable to select than one made of two triangles 
				// where a marquee select of even the group center could fail if it's between the tri centroids.
				FIndex3i TriVids = Mesh->GetTriangle(Tid);
				FPoly TrianglePolygon;
				TrianglePolygon.Init();
				for (int i = 0; i < 3; ++i)
				{
					TrianglePolygon.Vertices.Add((FVector3f)Mesh->GetVertex(TriVids[i]));
				}

				if (!LocalFrustum.ClipPolygon(TrianglePolygon))
				{
					// Triangle was fully outside the frustum
					return;
				}
			}

			// Apply occlusion filter. Note that the back face filter is applied via TriangleFilterF in the traversal.
			if (!Settings.bIgnoreOcclusion)
			{
				bool bTriangleIsOccluded = false;

				// See if we've already calculated occlusion for this tri.
				bool* CachedOcclusion = TriIsOccludedCache->Find(Tid);
				if (CachedOcclusion)
				{
					bTriangleIsOccluded = *CachedOcclusion;
				}
				else
				{
					bTriangleIsOccluded = IsOccluded(Mesh->GetTriCentroid(Tid), LocalCameraOrigin, Spatial, Settings.bHitBackFaces);
					TriIsOccludedCache->Add(Tid, bTriangleIsOccluded);
				}

				if (bTriangleIsOccluded)
				{
					return;
				}
			}

			ResultOut.SelectedGroupIDs.Add(TriGroupID);
		};

		// Add the back face filter, if applicable
		IMeshSpatial::FQueryOptions Options;
		if (!Settings.bHitBackFaces)
		{
			Options.TriangleFilterF = [this, &LocalCameraOrigin](int32 Tid)
			{
				return Mesh->GetTriNormal(Tid).Dot(Mesh->GetVertex(Mesh->GetTriangle(Tid)[0]) - LocalCameraOrigin) < 0;
			};
		}
		
		Spatial->DoTraversal(Traversal, Options);
	}

	return !ResultOut.IsEmpty();
}

bool FMeshTopologySelector::FindSelectedElement(const FSelectionSettings& Settings,
	const FCameraPolyLasso& CameraLasso, FTransform3d TargetTransform, FGroupTopologySelection& ResultOut,
	TMap<int32, bool>* TriIsOccludedCache)
{
	// One minor easy thing we can do to speed up the below is to detect cases where transform does not have a non-uniform
	// scale, and in those cases we transform the rectangle once rather than transforming all the query points/curves. 
	// This isn't worth doing for non-uniform scale transforms because the rectangle basis stops being orthonormal and 
	// becomes a problem to work with.

	ResultOut.Clear();
	FDynamicMeshAABBTree3* Spatial = GetSpatial();

	// Needed for occlusion test, which happens in local space.
	const FVector3d LocalCameraOrigin = TargetTransform.InverseTransformPosition((FVector3d)CameraLasso.CameraOrigin);

	auto IsPointInsidePolyLine = [](const TArray<FVector2D>& InPolyLine, const FVector2D& Vertex2DPos)
		{
			const int32 NumPolyPoints = InPolyLine.Num();
			if (NumPolyPoints > 0)
			{
				double WindingSum = 0;
				FVector2D Prev = (InPolyLine[0] - Vertex2DPos);
				FVector2D Curr = FVector2D::Zero();
				for (int32 PointIndex = 1; PointIndex < NumPolyPoints; ++PointIndex)
				{
					Curr = (InPolyLine[PointIndex] - Vertex2DPos);
					WindingSum += (double)FMathf::Atan2(Prev.X * Curr.Y - Prev.Y * Curr.X, Prev.X * Curr.X + Prev.Y * Curr.Y);
					Prev = Curr;
				}
				WindingSum /= FMathd::TwoPi;
				return FMathd::Abs(WindingSum) > 0.3;
			}
			return false;
		};

	auto VertexFilter = [&CameraLasso, &TargetTransform, &Settings, &LocalCameraOrigin, Spatial, &IsPointInsidePolyLine]
	(const FVector3d& PointPosition)

		{
			const FVector TransformedPoint = TargetTransform.TransformPosition(PointPosition);
			const FVector2D Vertex2DPos = CameraLasso.GetProjectedPoint(TransformedPoint);
			const bool bIsInside = IsPointInsidePolyLine(CameraLasso.Polyline, Vertex2DPos);
			return bIsInside
				&& (Settings.bIgnoreOcclusion || !IsOccluded(PointPosition, LocalCameraOrigin,
					Spatial, Settings.bHitBackFaces));
		};

	// Corner selection takes priority over edges.
	if (Settings.bEnableCornerHits)
	{
		GeometrySet.ParallelFindAllPointsSatisfying(VertexFilter, ResultOut.SelectedCornerIDs);
	}

	auto SegmentIntersectPolyLine = [IsPointInsidePolyLine](const TArray<FVector2D>& InPolyLine, const FSegment2d& Segment2D, bool TestPoints = true)
		{
			if (IsPointInsidePolyLine(InPolyLine, Segment2D.StartPoint()))
			{
				return true;
			}
			if (IsPointInsidePolyLine(InPolyLine, Segment2D.EndPoint()))
			{
				return true;
			}

			const int32 NumPolyPoints = InPolyLine.Num();
			if (NumPolyPoints > 1)
			{
				for (int32 PointIndex = 1; PointIndex < NumPolyPoints; ++PointIndex)
				{
					const FSegment2d PolyLineSegment(InPolyLine[PointIndex - 1], InPolyLine[PointIndex]);
					if (PolyLineSegment.Intersects(Segment2D))
					{
						return true;
					}
				}
			}
			return false;
		};

	auto CurveFilter = [&CameraLasso, &TargetTransform, &Settings, &LocalCameraOrigin, Spatial, SegmentIntersectPolyLine](const FPolyline3d& Curve)
		{
			// Testing occlusion properly seems like it would be a pain, so we'll just consider something occluded
			// if one of the endpoints is occluded. This will handle the common case of not wanting to select a
			// hidden edge that is connected to a visible corner.
			if (!Settings.bIgnoreOcclusion
				&& (IsOccluded(Curve.Start(), LocalCameraOrigin, Spatial, Settings.bHitBackFaces)
					|| IsOccluded(Curve.End(), LocalCameraOrigin, Spatial, Settings.bHitBackFaces)))
			{
				return false;
			}

			// Check whether any of the component segments intersect the rectangle
			const TArray<FVector3d>& Verts = Curve.GetVertices();
			FVector2D CurrentVert = CameraLasso.GetProjectedPoint(TargetTransform.TransformPosition(Verts[0]));
			for (int32 VertIndex = 1; VertIndex < Verts.Num(); ++VertIndex)
			{
				const FVector2D NextVert = CameraLasso.GetProjectedPoint(TargetTransform.TransformPosition(Verts[VertIndex]));
				const FSegment2d Segment(CurrentVert, NextVert);
				if (SegmentIntersectPolyLine(CameraLasso.Polyline, Segment))
				{
					// We consider an edge non occluded if at least one point is not
					// This is not perfect but it is a good approximation most most use case and still remain easy to evaluate
					if (Settings.bIgnoreOcclusion ||
						!IsOccluded(Verts[VertIndex - 1], LocalCameraOrigin, Spatial, Settings.bHitBackFaces) ||
						!IsOccluded(Verts[VertIndex], LocalCameraOrigin, Spatial, Settings.bHitBackFaces)
						)
					{
						return true;
					}
				}
				CurrentVert = NextVert;
			}
			return false;
		};

	// If we didn't get corners, look for edges.
	if (ResultOut.SelectedCornerIDs.IsEmpty() && Settings.bEnableEdgeHits)
	{
		GeometrySet.ParallelFindAllCurvesSatisfying(CurveFilter, ResultOut.SelectedEdgeIDs);
	}

	if (Settings.bEnableFaceHits && ResultOut.IsEmpty() && Mesh && TopologyProvider)
	{
		// check if the edges of triangle intersect with the lasso or if any points are inside of it 
		auto TriangleIntersectPolyLine = [IsPointInsidePolyLine](const TArray<FVector2D>& InPolyLine, const FVector2D& Vtx0, const FVector2D& Vtx1, const FVector2D& Vtx2)
			{
				const int32 NumPolyPoints = InPolyLine.Num();
				if (NumPolyPoints > 1)
				{
					const FSegment2d Segment0(Vtx0, Vtx1);
					const FSegment2d Segment1(Vtx1, Vtx2);
					const FSegment2d Segment2(Vtx2, Vtx0);

					FVector WindingSums{ 0,0,0 };
					for (int32 PointIndex = 1; PointIndex < NumPolyPoints; ++PointIndex)
					{
						const FVector2d PrevVert = InPolyLine[PointIndex - 1];
						const FVector2d CurrVert = InPolyLine[PointIndex + 0];
						const FSegment2d PolyLineSegment(PrevVert, CurrVert);
						if (PolyLineSegment.Intersects(Segment0) ||
							PolyLineSegment.Intersects(Segment1) ||
							PolyLineSegment.Intersects(Segment2) )
						{
							return true;
						}
						const FVector2d Prev0 = PrevVert - Vtx0;
						const FVector2d Curr0 = CurrVert - Vtx0;
						WindingSums[0] += (double)FMathf::Atan2(Prev0.X * Curr0.Y - Prev0.Y * Curr0.X, Prev0.X * Curr0.X + Prev0.Y * Curr0.Y);
						const FVector2d Prev1 = PrevVert - Vtx1;
						const FVector2d Curr1 = CurrVert - Vtx1;
						WindingSums[1] += (double)FMathf::Atan2(Prev1.X * Curr1.Y - Prev1.Y * Curr1.X, Prev1.X * Curr1.X + Prev1.Y * Curr1.Y);
						const FVector2d Prev2 = PrevVert - Vtx2;
						const FVector2d Curr2 = CurrVert - Vtx2;
						WindingSums[2] += (double)FMathf::Atan2(Prev2.X * Curr2.Y - Prev2.Y * Curr2.X, Prev2.X * Curr2.X + Prev2.Y * Curr2.Y);
					}
					WindingSums /= FMathd::TwoPi;

					return(
						FMathd::Abs(WindingSums[0]) > 0.3 ||
						FMathd::Abs(WindingSums[1]) > 0.3 ||
						FMathd::Abs(WindingSums[2]) > 0.3
						);
				}
				return false;
			};

		const int32 MaxTriangleID = Mesh->MaxTriangleID();

		TArray<bool> SelectedTriangles;
		SelectedTriangles.Init(false, MaxTriangleID);

		ParallelFor(Mesh->MaxTriangleID(), 
			[this, &SelectedTriangles/*, &VerticesInLasso*/, TriangleIntersectPolyLine, &CameraLasso, &TargetTransform](int32 TriId)
			{
				if (Mesh->IsTriangle(TriId))
				{
					const FIndex3i TriIndices = Mesh->GetTriangle(TriId);
					const FVector2D Vtx0 = CameraLasso.GetProjectedPoint(TargetTransform.TransformPosition(Mesh->GetVertex(TriIndices[0])));
					const FVector2D Vtx1 = CameraLasso.GetProjectedPoint(TargetTransform.TransformPosition(Mesh->GetVertex(TriIndices[1])));
					const FVector2D Vtx2 = CameraLasso.GetProjectedPoint(TargetTransform.TransformPosition(Mesh->GetVertex(TriIndices[2])));
					if (TriangleIntersectPolyLine(CameraLasso.Polyline, Vtx0, Vtx1, Vtx2))
					{
						SelectedTriangles[TriId] = true;
					}
				}
			});

		// Now add to the selection array 
		for (int32 TriId = 0; TriId < MaxTriangleID; ++TriId)
		{
			if (SelectedTriangles[TriId])
			{
				ResultOut.SelectedGroupIDs.Add(TopologyProvider->GetGroupIDForTriangle(TriId));
			}
		}
	}

	return !ResultOut.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
