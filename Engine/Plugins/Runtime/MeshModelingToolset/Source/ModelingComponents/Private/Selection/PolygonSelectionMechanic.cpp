// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/PolygonSelectionMechanic.h"
#include "Engine/World.h"
#include "InteractiveToolManager.h"
#include "Selection/GroupTopologySelector.h"
#include "Selection/PersistentMeshSelection.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PolygonSelectionMechanic)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UPolygonSelectionMechanic"

namespace PolygonSelectionMechanicLocals
{
	using namespace UE::Geometry;

	EGeometryElementType ToGeometryElementType(const FGroupTopologySelection& Selection)
	{
		if (!Selection.SelectedCornerIDs.IsEmpty())
		{
			return EGeometryElementType::Vertex;
		}
		else if (!Selection.SelectedEdgeIDs.IsEmpty())
		{
			return EGeometryElementType::Edge;
		}
		return EGeometryElementType::Face;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UPolygonSelectionMechanicProperties::GrowSelection()
{
	if (UPolygonSelectionMechanic* PolygonMechanic = Cast<UPolygonSelectionMechanic>(Mechanic.Get()))
	{
		PolygonMechanic->GrowSelection(/*bAsTriangleTopology*/ true);
	}
}

void UPolygonSelectionMechanicProperties::ShrinkSelection()
{
	if (UPolygonSelectionMechanic* PolygonMechanic = Cast<UPolygonSelectionMechanic>(Mechanic.Get()))
	{
		PolygonMechanic->ShrinkSelection(/*bAsTriangleTopology*/ true);
	}
}

void UPolygonSelectionMechanicProperties::FloodSelection()
{
	if (UPolygonSelectionMechanic* PolygonMechanic = Cast<UPolygonSelectionMechanic>(Mechanic.Get()))
	{
		PolygonMechanic->FloodSelection();
	}
}

void UPolygonSelectionMechanicProperties::ConvertToBorder()
{
	if (UPolygonSelectionMechanic* PolygonMechanic = Cast<UPolygonSelectionMechanic>(Mechanic.Get()))
	{
		PolygonMechanic->ConvertSelectionToBorderVertices(/*bAsTriangleTopology*/ true);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void UPolygonSelectionMechanic::Initialize(
	const FDynamicMesh3* MeshIn,
	FTransform3d TargetTransformIn,
	UWorld* WorldIn,
	const FGroupTopology* TopologyIn,
	TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFuncIn)
{
	Topology = TopologyIn;
	TopoSelector = MakeShared<FGroupTopologySelector>(MeshIn, TopologyIn);

	UMeshTopologySelectionMechanic::Initialize(MeshIn, TargetTransformIn, WorldIn, GetSpatialSourceFuncIn);
}

void UPolygonSelectionMechanic::Initialize(
	UDynamicMeshComponent* MeshComponentIn,
	const FGroupTopology* TopologyIn,
	TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFuncIn)
{
	Initialize(MeshComponentIn->GetMesh(),
		(FTransform3d)MeshComponentIn->GetComponentTransform(),
		MeshComponentIn->GetWorld(),
		TopologyIn,
		GetSpatialSourceFuncIn);
}

UMeshTopologySelectionMechanicProperties* UPolygonSelectionMechanic::CreateProperties()
{
	return NewObject<UPolygonSelectionMechanicProperties>(this);
}

void UPolygonSelectionMechanic::GetSelection_AsGroupTopology(UE::Geometry::FGeometrySelection& SelectionOut, const FCompactMaps* CompactMapsToApply) const
{
	const FGroupTopologySelection& CurSelection = PersistentSelection;
	if (SelectionOut.TopologyType != EGeometryTopologyType::Polygroup)
	{
		return;
	}
	if (SelectionOut.ElementType == EGeometryElementType::Vertex)
	{
		for (int32 CornerID : CurSelection.SelectedCornerIDs)
		{
			int32 VertexID = Topology->GetCornerVertexID(CornerID);
			if (CompactMapsToApply != nullptr)
			{
				VertexID = CompactMapsToApply->GetVertexMapping(VertexID);
			}
			SelectionOut.Selection.Add( FGeoSelectionID(VertexID, CornerID).Encoded() );
		}
	}
	else if (SelectionOut.ElementType == EGeometryElementType::Edge)
	{
		if ( CompactMapsToApply == nullptr )
		{
			for (int32 GroupEdgeID : CurSelection.SelectedEdgeIDs)
			{
				const TArray<int>& GroupEdge = Topology->GetGroupEdgeEdges(GroupEdgeID);
				FMeshTriEdgeID TriEdgeID = Topology->GetMesh()->GetTriEdgeIDFromEdgeID(GroupEdge[0]);
				SelectionOut.Selection.Add( FGeoSelectionID(TriEdgeID.Encoded(), GroupEdgeID).Encoded() );
			}
		}
		else
		{
			for (int32 GroupEdgeID : CurSelection.SelectedEdgeIDs)
			{
				const TArray<int>& GroupEdgeVerts = Topology->GetGroupEdgeVertices(GroupEdgeID);
				if (GroupEdgeVerts.Num() > 1)
				{
					int32 VID0 = CompactMapsToApply->GetVertexMapping(GroupEdgeVerts[0]);
					int32 VID1 = CompactMapsToApply->GetVertexMapping(GroupEdgeVerts[1]);
					int32 FoundEID = Topology->GetMesh()->FindEdge(VID0, VID1);
					if (FoundEID != IndexConstants::InvalidID)
					{
						FMeshTriEdgeID TriEdgeID = Topology->GetMesh()->GetTriEdgeIDFromEdgeID(FoundEID);
						SelectionOut.Selection.Add(FGeoSelectionID(TriEdgeID.Encoded(), GroupEdgeID).Encoded());
					}
				}
			}
		}
	}
	else if (SelectionOut.ElementType == EGeometryElementType::Face)
	{
		for (int32 GroupID : CurSelection.SelectedGroupIDs)
		{
			const FGroupTopology::FGroup* GroupFace = Topology->FindGroupByID(GroupID);
			if ( GroupFace )
			{
				FGeoSelectionID ID = FGeoSelectionID(GroupFace->Triangles[0], GroupFace->GroupID);
				SelectionOut.Selection.Add(ID.Encoded());
			}
		}
	}
}


void UPolygonSelectionMechanic::GetSelection_AsTriangleTopology(UE::Geometry::FGeometrySelection& SelectionOut, const FCompactMaps* CompactMapsToApply) const
{
	// note: this is currently the same code as GetSelection_AsGroupTopology() except for the topology-type verification check, and the topology type of the selected elements

	const FGroupTopologySelection& CurSelection = PersistentSelection;
	if (SelectionOut.TopologyType != EGeometryTopologyType::Triangle)
	{
		return;
	}
	if (SelectionOut.ElementType == EGeometryElementType::Vertex)
	{
		for (int32 CornerID : CurSelection.SelectedCornerIDs)
		{
			int32 VertexID = Topology->GetCornerVertexID(CornerID);
			if (CompactMapsToApply != nullptr)
			{
				VertexID = CompactMapsToApply->GetVertexMapping(VertexID);
			}
			SelectionOut.Selection.Add( FGeoSelectionID::MeshVertex(VertexID).Encoded() );
		}
	}
	else if (SelectionOut.ElementType == EGeometryElementType::Edge)
	{
		if ( CompactMapsToApply == nullptr )
		{
			for (int32 GroupEdgeID : CurSelection.SelectedEdgeIDs)
			{
				const TArray<int>& GroupEdge = Topology->GetGroupEdgeEdges(GroupEdgeID);
				FMeshTriEdgeID TriEdgeID = Topology->GetMesh()->GetTriEdgeIDFromEdgeID(GroupEdge[0]);
				SelectionOut.Selection.Add( FGeoSelectionID::MeshEdge(TriEdgeID).Encoded() );
			}
		}
		else
		{
			for (int32 GroupEdgeID : CurSelection.SelectedEdgeIDs)
			{
				const TArray<int>& GroupEdgeVerts = Topology->GetGroupEdgeVertices(GroupEdgeID);
				if (GroupEdgeVerts.Num() > 1)
				{
					int32 VID0 = CompactMapsToApply->GetVertexMapping(GroupEdgeVerts[0]);
					int32 VID1 = CompactMapsToApply->GetVertexMapping(GroupEdgeVerts[1]);
					int32 FoundEID = Topology->GetMesh()->FindEdge(VID0, VID1);
					if (FoundEID != IndexConstants::InvalidID)
					{
						FMeshTriEdgeID TriEdgeID = Topology->GetMesh()->GetTriEdgeIDFromEdgeID(FoundEID);
						SelectionOut.Selection.Add(FGeoSelectionID::MeshEdge(TriEdgeID).Encoded());
					}
				}
			}
		}
	}
	else if (SelectionOut.ElementType == EGeometryElementType::Face)
	{
		for (int32 GroupID : CurSelection.SelectedGroupIDs)
		{
			const FGroupTopology::FGroup* GroupFace = Topology->FindGroupByID(GroupID);
			if ( GroupFace )
			{
				SelectionOut.Selection.Add( FGeoSelectionID::MeshTriangle(GroupFace->Triangles[0]).Encoded() );
			}
		}
	}
}


void UPolygonSelectionMechanic::SetSelection_AsGroupTopology(const UE::Geometry::FGeometrySelection& Selection, const bool bBroadcastSelectionChanged)
{
	if (Selection.TopologyType != EGeometryTopologyType::Polygroup)
	{
		return;
	}
	
	TOptional<FGroupTopologySelection> PreviousSelection;
	if (bBroadcastSelectionChanged)
	{
		// Only remember the previous selection when broadcasting the change
		PreviousSelection = PersistentSelection;
	}
	
	PersistentSelection.Clear();

	if (Selection.ElementType == EGeometryElementType::Vertex)
	{
		for (uint64 ElementID : Selection.Selection)
		{
			int32 VertexID = FGeoSelectionID(ElementID).GeometryID;
			int32 CornerID = Topology->GetCornerIDFromVertexID(VertexID);
			if (CornerID != IndexConstants::InvalidID)
			{
				PersistentSelection.SelectedCornerIDs.Add(CornerID);
			}
		}
	}
	else if (Selection.ElementType == EGeometryElementType::Edge)
	{
		for (uint64 ElementID : Selection.Selection)
		{
			FMeshTriEdgeID TriEdgeID(FGeoSelectionID(ElementID).GeometryID);
			int32 GroupEdgeID = Topology->FindGroupEdgeID(TriEdgeID);
			if (GroupEdgeID != IndexConstants::InvalidID)
			{
				PersistentSelection.SelectedEdgeIDs.Add(GroupEdgeID);
			}
		}
	}
	else if (Selection.ElementType == EGeometryElementType::Face)
	{
		for (uint64 ElementID : Selection.Selection)
		{
			int32 TriangleID = FGeoSelectionID(ElementID).GeometryID;
			int32 GroupID = Topology->GetGroupID(TriangleID);
			if (Topology->FindGroupByID(GroupID) != nullptr)
			{
				PersistentSelection.SelectedGroupIDs.Add(GroupID);
			}
		}
	}
	
	if (bBroadcastSelectionChanged &&
		ensure(PreviousSelection.IsSet()) &&
		PersistentSelection != PreviousSelection.GetValue())
	{
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
	}
}

void UPolygonSelectionMechanic::SetSelection_AsTriangleTopology(const UE::Geometry::FGeometrySelection& Selection, const bool bBroadcastSelectionChanged)
{
	if (Selection.TopologyType != EGeometryTopologyType::Triangle)
	{
		return;
	}

	TOptional<FGroupTopologySelection> PreviousSelection;
	if (bBroadcastSelectionChanged)
	{
		// Only remember the previous selection when broadcasting the change
		PreviousSelection = PersistentSelection;
	}

	PersistentSelection.Clear();
	
	if (Selection.ElementType == EGeometryElementType::Vertex)
	{
		for (uint64 ElementID : Selection.Selection)
		{
			int32 VertexID = FGeoSelectionID(ElementID).GeometryID;
			int32 CornerID = Topology->GetCornerIDFromVertexID(VertexID);
			if (CornerID != IndexConstants::InvalidID)
			{
				PersistentSelection.SelectedCornerIDs.Add(CornerID);
			}
		}
	}
	else if (Selection.ElementType == EGeometryElementType::Edge)
	{
		for (uint64 ElementID : Selection.Selection)
		{
			FMeshTriEdgeID TriEdgeID(FGeoSelectionID(ElementID).GeometryID);
			int32 GroupEdgeID = Topology->FindGroupEdgeID(TriEdgeID);
			if (GroupEdgeID != IndexConstants::InvalidID)
			{
				PersistentSelection.SelectedEdgeIDs.Add(GroupEdgeID);
			}
		}
	}
	else if (Selection.ElementType == EGeometryElementType::Face)
	{
		for (uint64 ElementID : Selection.Selection)
		{
			int32 TriangleID = FGeoSelectionID(ElementID).GeometryID;
			int32 GroupID = Topology->GetGroupID(TriangleID);
			if (Topology->FindGroupByID(GroupID) != nullptr)
			{
				PersistentSelection.SelectedGroupIDs.Add(GroupID);
			}
		}
	}
	
	if (bBroadcastSelectionChanged &&
		ensure(PreviousSelection.IsSet()) &&
		PersistentSelection != PreviousSelection.GetValue())
	{
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
	}
}

bool UPolygonSelectionMechanic::ExecuteActionThroughGeometrySelection(bool bAsTriangleTopology, const FText& TransactionName, 
	TFunctionRef<bool(UE::Geometry::FGeometrySelection& SelectionToModifyInPlace)> SelectionProcessor)
{
	using namespace PolygonSelectionMechanicLocals;
	using namespace UE::Geometry;

	if (!Topology || !Topology->GetMesh())
	{
		return false;
	}

	FGeometrySelection GeometrySelection;
	GeometrySelection.InitializeTypes(ToGeometryElementType(PersistentSelection),
		bAsTriangleTopology ? EGeometryTopologyType::Triangle : EGeometryTopologyType::Polygroup);

	if (bAsTriangleTopology)
	{
		GetSelection_AsTriangleTopology(GeometrySelection);
	}
	else
	{
		GetSelection_AsGroupTopology(GeometrySelection);
	}

	bool bSuccess = SelectionProcessor(GeometrySelection);
	if (!bSuccess)
	{
		return false;
	}

	ParentTool->GetToolManager()->BeginUndoTransaction(TransactionName);
	BeginChange();

	if (bAsTriangleTopology)
	{
		SetSelection_AsTriangleTopology(GeometrySelection);
	}
	else
	{
		SetSelection_AsGroupTopology(GeometrySelection);
	}

	SelectionTimestamp++;
	OnSelectionChanged.Broadcast();
	EndChangeAndEmitIfModified();
	ParentTool->GetToolManager()->EndUndoTransaction();

	return true;
}

void UPolygonSelectionMechanic::GrowSelection(bool bAsTriangleTopology)
{
	using namespace UE::Geometry;

	ExecuteActionThroughGeometrySelection(bAsTriangleTopology, LOCTEXT("GrowSelectionChange", "Grow Selection"),
		[this](FGeometrySelection& GeometrySelection) 
	{
		FGeometrySelection BoundaryConnectedSelection;
		BoundaryConnectedSelection.InitializeTypes(GeometrySelection);

		bool bSuccess = MakeBoundaryConnectedSelection(*Topology->GetMesh(), Topology, GeometrySelection,
			[](FGeoSelectionID) { return true; }, BoundaryConnectedSelection);
		bSuccess = bSuccess && CombineSelectionInPlace(GeometrySelection, BoundaryConnectedSelection, 
			UE::Geometry::EGeometrySelectionCombineModes::Add);

		return bSuccess;
	});
}

void UPolygonSelectionMechanic::ShrinkSelection(bool bAsTriangleTopology)
{
	using namespace UE::Geometry;

	ExecuteActionThroughGeometrySelection(bAsTriangleTopology, LOCTEXT("ShrinkSelectionChange", "Shrink Selection"),
		[this](FGeometrySelection& GeometrySelection) 
	{
		FGeometrySelection BoundaryConnectedSelection;
		BoundaryConnectedSelection.InitializeTypes(GeometrySelection);

		bool bSuccess = MakeBoundaryConnectedSelection(*Topology->GetMesh(), Topology, GeometrySelection,
			[](FGeoSelectionID) { return true; }, BoundaryConnectedSelection);
		bSuccess = bSuccess && CombineSelectionInPlace(GeometrySelection, BoundaryConnectedSelection, 
			UE::Geometry::EGeometrySelectionCombineModes::Subtract);

		return bSuccess;
	});
}

void UPolygonSelectionMechanic::InvertSelection(bool bAsTriangleTopology)
{
	using namespace UE::Geometry;

	ExecuteActionThroughGeometrySelection(bAsTriangleTopology, LOCTEXT("InvertSelectionChange", "Invert Selection"),
		[this](FGeometrySelection& GeometrySelection)
		{
			FGeometrySelection NewSelection;
			NewSelection.InitializeTypes(GeometrySelection);

			bool bSuccess = MakeSelectAllSelection(
				*Topology->GetMesh(), Topology,
				[](FGeoSelectionID) { return true; },
				NewSelection);

			bSuccess = bSuccess && CombineSelectionInPlace(NewSelection, GeometrySelection,
				UE::Geometry::EGeometrySelectionCombineModes::Subtract);

			GeometrySelection = MoveTemp(NewSelection);

			return bSuccess;
		});
}

void UPolygonSelectionMechanic::ConvertSelectionToBorderVertices(bool bAsTriangleTopology)
{
	using namespace UE::Geometry;

	ExecuteActionThroughGeometrySelection(bAsTriangleTopology, LOCTEXT("BorderSelectionChange", "Select Border"),
		[this, bAsTriangleTopology](FGeometrySelection& GeometrySelection) 
	{
		TSet<int32> Unused;
		if (bAsTriangleTopology)
		{
			TSet<int32> BoundaryVertices;
			bool bSuccess = GetSelectionBoundaryVertices(*Topology->GetMesh(), Topology, GeometrySelection,
				BoundaryVertices, Unused);
			
			if (!bSuccess) { return false; }

			GeometrySelection.Selection.Reset();
			GeometrySelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Triangle);
			for (int32 Vid : BoundaryVertices)
			{
				GeometrySelection.Selection.Add(FGeoSelectionID::MeshVertex(Vid).Encoded());
			}
		}
		else
		{
			TSet<int32> BoundaryCorners;
			bool bSuccess = GetSelectionBoundaryCorners(*Topology->GetMesh(), Topology, GeometrySelection, 
				BoundaryCorners, Unused);

			if (!bSuccess) { return false; }

			GeometrySelection.Selection.Reset();
			GeometrySelection.InitializeTypes(EGeometryElementType::Vertex, EGeometryTopologyType::Polygroup);
			for (int32 CornerID : BoundaryCorners)
			{
				GeometrySelection.Selection.Add(FGeoSelectionID(Topology->GetCornerVertexID(CornerID), CornerID).Encoded());
			}
		}

		return true;
	});
}

void UPolygonSelectionMechanic::FloodSelection()
{
	using namespace UE::Geometry;

	ExecuteActionThroughGeometrySelection(true, LOCTEXT("FloodSelectionChange", "Flood Selection"),
		[this](FGeometrySelection& GeometrySelection)
	{
		FGeometrySelection NewSelection;
		NewSelection.InitializeTypes(GeometrySelection);
		bool bSuccess = MakeSelectAllConnectedSelection(*Topology->GetMesh(), Topology, GeometrySelection, 
			[](FGeoSelectionID) { return true; }, [](FGeoSelectionID, FGeoSelectionID) { return true; }, 
			NewSelection);

		if (!bSuccess) { return false; }
			
		GeometrySelection = NewSelection;
		return true;
	});
}


bool UPolygonSelectionMechanic::UpdateHighlight(const FRay& WorldRay)
{
	checkf(DrawnTriangleSetComponent != nullptr, TEXT("Initialize() not called on UMeshTopologySelectionMechanic."));

	FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	HilightSelection.Clear();
	FVector3d LocalPosition, LocalNormal;
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(CameraState.bIsOrthographic);
	bool bHit = TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, HilightSelection, LocalPosition, LocalNormal);

	TSharedPtr<FGroupTopologySelector> GroupTopoSelector = StaticCastSharedPtr<FGroupTopologySelector>(TopoSelector);

	if (HilightSelection.SelectedEdgeIDs.Num() > 0 && Properties->bSelectEdgeRings && ShouldSelectEdgeRingsFunc())
	{
		GroupTopoSelector->ExpandSelectionByEdgeRings(HilightSelection);
	}
	if (HilightSelection.SelectedEdgeIDs.Num() > 0 && Properties->bSelectEdgeLoops && ShouldSelectEdgeLoopsFunc())
	{
		GroupTopoSelector->ExpandSelectionByEdgeLoops(HilightSelection);
		GroupTopoSelector->ExpandSelectionByBoundaryLoops(HilightSelection);
	}

	// Don't hover highlight a selection that we already selected, because people didn't like that
	if (PersistentSelection.Contains(HilightSelection))
	{
		HilightSelection.Clear();
	}

	// Currently we draw highlighted edges/vertices differently from highlighted faces. Edges/vertices
	// get drawn in the Render() call, so it is sufficient to just update HighlightSelection above.
	// Faces, meanwhile, get placed into a Component that is rendered through the normal rendering system.
	// So, we need to update the component when the highlighted selection changes.

	// Put hovered groups in a set to easily compare to current
	TSet<int> NewlyHighlightedGroups;
	NewlyHighlightedGroups.Append(HilightSelection.SelectedGroupIDs);

	// See if we're currently highlighting any groups that we're not supposed to
	if (!NewlyHighlightedGroups.Includes(CurrentlyHighlightedGroups))
	{
		DrawnTriangleSetComponent->Clear();
		CurrentlyHighlightedGroups.Empty();
	}

	// See if we need to add any groups
	if (!CurrentlyHighlightedGroups.Includes(NewlyHighlightedGroups))
	{
		// Add triangles for each new group
		for (int Gid : HilightSelection.SelectedGroupIDs)
		{
			if (!CurrentlyHighlightedGroups.Contains(Gid))
			{
				for (int32 Tid : Topology->GetGroupTriangles(Gid))
				{
					// We use the triangle normals because the normal overlay isn't guaranteed to be valid as we edit the mesh
					FVector3d TriangleNormal = Mesh->GetTriNormal(Tid);

					// The UV's and colors here don't currently get used by HighlightedFaceMaterial, but we set them anyway
					FIndex3i VertIndices = Mesh->GetTriangle(Tid);
					DrawnTriangleSetComponent->AddTriangle(FRenderableTriangle(HighlightedFaceMaterial,
						FRenderableTriangleVertex((FVector)Mesh->GetVertex(VertIndices.A), (FVector2D)Mesh->GetVertexUV(VertIndices.A), (FVector)TriangleNormal, FLinearColor(Mesh->GetVertexColor(VertIndices.A)).ToFColor(true)),
						FRenderableTriangleVertex((FVector)Mesh->GetVertex(VertIndices.B), (FVector2D)Mesh->GetVertexUV(VertIndices.B), (FVector)TriangleNormal, FLinearColor(Mesh->GetVertexColor(VertIndices.B)).ToFColor(true)),
						FRenderableTriangleVertex((FVector)Mesh->GetVertex(VertIndices.C), (FVector2D)Mesh->GetVertexUV(VertIndices.C), (FVector)TriangleNormal, FLinearColor(Mesh->GetVertexColor(VertIndices.C)).ToFColor(true))));
				}

				CurrentlyHighlightedGroups.Add(Gid);
			}
		}//end iterating through groups
	}//end if groups need to be added

	return bHit;
}


bool UPolygonSelectionMechanic::UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut)
{
	FRay3d LocalRay(TargetTransform.InverseTransformPosition((FVector3d)WorldRay.Origin),
		TargetTransform.InverseTransformVector((FVector3d)WorldRay.Direction));
	UE::Geometry::Normalize(LocalRay.Direction);

	const FGroupTopologySelection PreviousSelection = PersistentSelection;

	FVector3d LocalPosition, LocalNormal;
	FGroupTopologySelection Selection;
	FGroupTopologySelector::FSelectionSettings TopoSelectorSettings = GetTopoSelectorSettings(CameraState.bIsOrthographic);
	if (TopoSelector->FindSelectedElement(TopoSelectorSettings, LocalRay, Selection, LocalPosition, LocalNormal))
	{
		LocalHitPositionOut = LocalPosition;
		LocalHitNormalOut = LocalNormal;

		TSharedPtr<FGroupTopologySelector> GroupTopoSelector = StaticCastSharedPtr<FGroupTopologySelector>(TopoSelector);

		if (Selection.SelectedEdgeIDs.Num() > 0 && Properties->bSelectEdgeRings && ShouldSelectEdgeRingsFunc())
		{
			GroupTopoSelector->ExpandSelectionByEdgeRings(Selection);
		}
		if (Selection.SelectedEdgeIDs.Num() > 0 && Properties->bSelectEdgeLoops && ShouldSelectEdgeLoopsFunc())
		{
			GroupTopoSelector->ExpandSelectionByEdgeLoops(Selection);
			GroupTopoSelector->ExpandSelectionByBoundaryLoops(Selection);
		}
	}

	if (ShouldAddToSelectionFunc())
	{
		if (ShouldRemoveFromSelectionFunc())
		{
			PersistentSelection.Toggle(Selection);
		}
		else
		{
			PersistentSelection.Append(Selection);
		}
	}
	else if (ShouldRemoveFromSelectionFunc())
	{
		PersistentSelection.Remove(Selection);
	}
	else
	{
		PersistentSelection = Selection;
	}

	if (PersistentSelection != PreviousSelection)
	{
		SelectionTimestamp++;
		OnSelectionChanged.Broadcast();
		return true;
	}

	return false;
}


#undef LOCTEXT_NAMESPACE
