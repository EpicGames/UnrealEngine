// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/SelectClickedAction.h"
#include "Engine/HitResult.h"

FInputRayHit FSelectClickedAction::DoRayCast(const FInputDeviceRay& ClickPos, bool callbackOnHit)
{
	FHitResult Result;
	const TArray<const UPrimitiveComponent*>* IgnoreComponents = VisibleComponentsToIgnore.Num() == 0 ? nullptr : &VisibleComponentsToIgnore;
	const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = InvisibleComponentsToHitTest.Num() == 0 ? nullptr : &InvisibleComponentsToHitTest;
	bool bHitWorld = (SnapManager != nullptr) ?
		                 ToolSceneQueriesUtil::FindNearestVisibleObjectHit(SnapManager, Result, ClickPos.WorldRay, IgnoreComponents, InvisibleComponentsToInclude) :
		                 ToolSceneQueriesUtil::FindNearestVisibleObjectHit(World, Result, ClickPos.WorldRay, IgnoreComponents, InvisibleComponentsToInclude);

	if (callbackOnHit && bHitWorld && OnClickedPositionFunc != nullptr)
	{
		OnClickedPositionFunc(Result);
	}
	return (bHitWorld) ? FInputRayHit(Result.Distance) : FInputRayHit();
}
