// Copyright Epic Games, Inc. All Rights Reserved.

#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/PersistentMeshSelection.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "ContextObjectStore.h"
#include "GroupTopology.h"
#include "Selection/GeometrySelectionManager.h"

using namespace UE::Geometry;


bool UE::Geometry::HaveAvailableGeometrySelection(const FToolBuilderState& SceneState)
{
	UGeometrySelectionManager* SelectionManager = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>();
	return (SelectionManager != nullptr && SelectionManager->HasSelection());
}


bool UE::Geometry::GetCurrentGeometrySelectionForTarget(const FToolBuilderState& SceneState, UToolTarget* Target, FGeometrySelection& SelectionOut)
{
	return GetCurrentGeometrySelectionForTarget(SceneState.ToolManager, Target, SelectionOut);
}


bool UE::Geometry::GetCurrentGeometrySelectionForTarget(UInteractiveToolManager* ToolManager, UToolTarget* Target, FGeometrySelection& SelectionOut)
{
	UGeometrySelectionManager* SelectionManager = ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>();
	if (SelectionManager == nullptr)
	{
		return false;
	}
	IPrimitiveComponentBackedTarget* TargetInterface = Cast<IPrimitiveComponentBackedTarget>(Target);
	if (TargetInterface == nullptr)
	{
		return false;
	}

	bool bHaveSelection = SelectionManager->GetSelectionForComponent(TargetInterface->GetOwnerComponent(), SelectionOut);
	return bHaveSelection;
}

bool UE::Geometry::GetCurrentSelectionWorldFrameBounds(const FToolBuilderState& SceneState, FFrame3d& OutWorldFrame, FAxisAlignedBox3d& OutWorldBounds, bool& bOutIsElementSelection)
{
	UGeometrySelectionManager* SelectionManager = SceneState.ToolManager->GetContextObjectStore()->FindContext<UGeometrySelectionManager>();
	if (SelectionManager && SelectionManager->HasSelection()) 
	{
		bOutIsElementSelection = true;
		FGeometrySelectionBounds Bounds;
		SelectionManager->GetSelectionBounds(Bounds);
		OutWorldBounds = Bounds.WorldBounds;
		SelectionManager->GetSelectionWorldFrame(OutWorldFrame);
		return true;
	}
	else // use the selected actor/components
	{
		bOutIsElementSelection = false;
		FBox Bounds;
		for (AActor* Actor : SceneState.SelectedActors)
		{
			Bounds += Actor->GetComponentsBoundingBox(true);
		}
		for (UActorComponent* Component : SceneState.SelectedComponents)
		{
			if (UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component))
			{
				Bounds += PrimComponent->Bounds.GetBox();
			}
		}
		if (Bounds.IsValid)
		{
			OutWorldBounds = (FAxisAlignedBox3d)Bounds;
			OutWorldFrame = FFrame3d(Bounds.GetCenter());
			return true;
		}
	}
	return false;
}


bool UE::Geometry::SetToolOutputGeometrySelectionForTarget(UInteractiveTool* Tool, UToolTarget* Target, const FGeometrySelection& OutputSelection)
{
	UGeometrySelectionManager* SelectionManager = Tool->GetToolManager()->GetContextObjectStore()->FindContext<UGeometrySelectionManager>();
	if (SelectionManager == nullptr)
	{
		return false;
	}
	IPrimitiveComponentBackedTarget* TargetInterface = Cast<IPrimitiveComponentBackedTarget>(Target);
	if (TargetInterface == nullptr)
	{
		return false;
	}
	return SelectionManager->SetSelectionForComponent(TargetInterface->GetOwnerComponent(), OutputSelection);
}








namespace UELocal
{
	// Helper function for GetGroupEdgeRepresentativeVerts(). Given a loop as a list of vids, with
	// the first and last vid the same, returns the lowest vid and its lower-vid neighbor.
	FIndex2i GetLoopRepresentativeVerts(const TArray<int32>& Verts)
	{
		int32 NumVerts = Verts.Num();

		int32 MinVid = Verts[0];
		// Last vert is a repeat of first vert, so neighbor is second to last.
		int32 MinNeighbor = FMath::Min(Verts[1], Verts[NumVerts - 2]);

		for (int32 i = 1; i < NumVerts - 1; ++i)
		{
			if (Verts[i] < MinVid)
			{
				MinVid = Verts[i];
				MinNeighbor = FMath::Min(Verts[i - 1], Verts[i + 1]);
			}
		}
		return FIndex2i(MinVid, MinNeighbor); // we know that MinVid is smaller
	}
}//end namespace UELocal
using namespace UELocal;





UE::Geometry::FIndex2i UE::Geometry::GetGroupEdgeRepresentativeVerts(const FGroupTopology& TopologyIn, int GroupEdgeID, const FCompactMaps& CompactMaps)
{
	check(GroupEdgeID >= 0 && GroupEdgeID < TopologyIn.Edges.Num());

	const FGroupTopology::FGroupEdge& GroupEdge = TopologyIn.Edges[GroupEdgeID];
	const TArray<int32>& Verts = GroupEdge.Span.Vertices;

	if (GroupEdge.EndpointCorners.A != IndexConstants::InvalidID)
	{
		// Use remapped vids
		int32 FirstVid = CompactMaps.GetVertexMapping(Verts[0]);
		int32 FirstNeighbor = CompactMaps.GetVertexMapping(Verts[1]);
		int32 LastVid = CompactMaps.GetVertexMapping(Verts.Last());
		int32 LastNeighbor = CompactMaps.GetVertexMapping(Verts[Verts.Num() - 2]);

		return FirstVid < LastVid ?
			FIndex2i(FMath::Min(FirstVid, FirstNeighbor), FMath::Max(FirstVid, FirstNeighbor))
			: FIndex2i(FMath::Min(LastVid, LastNeighbor), FMath::Max(LastVid, LastNeighbor));
	}
	else
	{
		TArray<int32> RemappedVerts;
		RemappedVerts.SetNum(Verts.Num());
		for (int32 i = 0; i < Verts.Num(); ++i)
		{
			RemappedVerts[i] = CompactMaps.GetVertexMapping(Verts[i]);
		}
		return GetLoopRepresentativeVerts(RemappedVerts);
	}
}

UE::Geometry::FIndex2i UE::Geometry::GetGroupEdgeRepresentativeVerts(const FGroupTopology& TopologyIn, int GroupEdgeID)
{
	check(GroupEdgeID >= 0 && GroupEdgeID < TopologyIn.Edges.Num());

	const FGroupTopology::FGroupEdge& GroupEdge = TopologyIn.Edges[GroupEdgeID];
	const TArray<int32>& Verts = GroupEdge.Span.Vertices;

	if (GroupEdge.EndpointCorners.A != IndexConstants::InvalidID)
	{
		int32 FirstVid = Verts[0];
		int32 FirstNeighbor = Verts[1];
		int32 LastVid = Verts.Last();
		int32 LastNeighbor = Verts[Verts.Num() - 2];

		return FirstVid < LastVid ?
			FIndex2i(FMath::Min(FirstVid, FirstNeighbor), FMath::Max(FirstVid, FirstNeighbor))
			: FIndex2i(FMath::Min(LastVid, LastNeighbor), FMath::Max(LastVid, LastNeighbor));
	}
	else
	{
		return GetLoopRepresentativeVerts(Verts);
	}
}
