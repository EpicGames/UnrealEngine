// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/ToolSelectionUtil.h"
#include "InteractiveToolManager.h"
#include "GameFramework/Actor.h"

#include "TriangleTypes.h"
#include "SegmentTypes.h"
#include "GroupTopology.h"
#include "ToolDataVisualizer.h"
#include "DynamicMeshBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"

// For debug drawing
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"

using namespace UE::Geometry;

void ToolSelectionUtil::SetNewActorSelection(UInteractiveToolManager* ToolManager, AActor* Actor)
{
	FSelectedObjectsChangeList NewSelection;
	NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;
	NewSelection.Actors.Add(Actor);
	ToolManager->RequestSelectionChange(NewSelection);
}


void ToolSelectionUtil::SetNewActorSelection(UInteractiveToolManager* ToolManager, const TArray<AActor*>& Actors)
{
	FSelectedObjectsChangeList NewSelection;
	NewSelection.ModificationType = ESelectedObjectsModificationType::Replace;
	for (AActor* Actor : Actors)
	{
		NewSelection.Actors.Add(Actor);
	}
	ToolManager->RequestSelectionChange(NewSelection);
}

bool ToolSelectionUtil::AccumulateSelectionElements(
	FGeometrySelectionElements& Elements,
	const FGeometrySelection& Selection,
	const FDynamicMesh3& SourceMesh,
	const FGroupTopology* GroupTopology,
	const FTransform* ApplyTransform,
	bool bMapFacesToEdges)
{
	return AccumulateSelectionElements(
		Elements,
		Selection,
		SourceMesh,
		GroupTopology,
		ApplyTransform,
		EEnumerateSelectionMapping::Default | (bMapFacesToEdges ? EEnumerateSelectionMapping::FacesToEdges : EEnumerateSelectionMapping::None));
}

bool ToolSelectionUtil::AccumulateSelectionElements(
	FGeometrySelectionElements& Elements,
	const FGeometrySelection& Selection,
	const FDynamicMesh3& SourceMesh,
	const FGroupTopology* GroupTopology,
	const FTransform* ApplyTransform,
	const EEnumerateSelectionMapping Flags)
{
	auto AddPoint = [&Elements](uint32 Vid, const FVector3d& Point) { Elements.Points.Add(Point); };
	auto AddSegment = [&Elements](uint32 Eid, const FSegment3d& Segment) { Elements.Segments.Add(Segment); };
	auto AddTriangle = [&Elements](uint32 Tid, const FTriangle3d& Triangle) { Elements.Triangles.Add(Triangle); };

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		if (GroupTopology)
		{
			return EnumeratePolygroupSelectionElements(Selection, SourceMesh, GroupTopology,
				AddPoint, AddSegment, AddTriangle, ApplyTransform, Flags);
		}

		const FGroupTopology ComputedGroupTopology(&SourceMesh, true);
		return EnumeratePolygroupSelectionElements(Selection, SourceMesh, &ComputedGroupTopology,
			AddPoint, AddSegment, AddTriangle, ApplyTransform, Flags);
	}

	return EnumerateTriangleSelectionElements(Selection, SourceMesh, 
			AddPoint, AddSegment, AddTriangle, ApplyTransform, Flags);
}

void ToolSelectionUtil::DebugRenderGeometrySelectionElements(
	IToolsContextRenderAPI* RenderAPI,
	const FGeometrySelectionElements& Elements,
	bool bIsPreview)
{	
	DebugRender(
		RenderAPI,
		Elements,
		bIsPreview ? 1.f : 3.f,
		bIsPreview ? FLinearColor(1, 1, 0, 1) : FLinearColor(0, 0.3f, 0.95f, 1),
		bIsPreview ? 5.f : 10.f,
		bIsPreview ? FLinearColor(1, 1, 0, 1) : FLinearColor(0, 0.3f, 0.95f, 1));
}

void ToolSelectionUtil::DebugRender(
	IToolsContextRenderAPI* RenderAPI,
	const UE::Geometry::FGeometrySelectionElements& Elements,
	float LineThickness,
	FLinearColor LineColor,
	float PointSize,
	FLinearColor PointColor,
	float DepthBias,
	FLinearColor FillColor)
{
	if (RenderAPI == nullptr)
	{
		return;
	}
	
	FPrimitiveDrawInterface* CurrentPDI = RenderAPI->GetPrimitiveDrawInterface();

	// batch render all the triangles, vastly more efficient than drawing one by one!
	FDynamicMeshBuilder MeshBuilder(CurrentPDI->View->GetFeatureLevel());
	int32 DepthPriority = SDPG_World; // SDPG_Foreground;  // SDPG_World
	FVector2f UVs[3] = { FVector2f(0,0), FVector2f(0,1), FVector2f(1,1)	};
	FVector3f Normal = FVector3f(0, 0, 1);
	FVector3f Tangent = FVector3f(1, 0, 0);	
	for (const FTriangle3d& Triangle : Elements.Triangles)
	{
		int32 V0 = MeshBuilder.AddVertex(FDynamicMeshVertex((FVector3f)Triangle.V[0], Tangent, Normal, UVs[0], FColor::White));
		int32 V1 = MeshBuilder.AddVertex(FDynamicMeshVertex((FVector3f)Triangle.V[1], Tangent, Normal, UVs[1], FColor::White));
		int32 V2 = MeshBuilder.AddVertex(FDynamicMeshVertex((FVector3f)Triangle.V[2], Tangent, Normal, UVs[2], FColor::White));
		MeshBuilder.AddTriangle(V0, V1, V2);
	}
	//FMaterialRenderProxy* MaterialRenderProxy = TriangleMaterial->GetRenderProxy();		// currently does not work, material does not render

	// desaturated color to use when triangle is selected in Mesh Element Selection
	// uses ConstraintLimitMaterial bc ConstraintLimitMaterialX was used previously to get desaturated red color
	TObjectPtr<class UMaterialInstanceDynamic> MeshElementSelectionFillColor;
	MeshElementSelectionFillColor = UMaterialInstanceDynamic::Create((UMaterialInterface*)GEngine->ConstraintLimitMaterial, NULL);
	MeshElementSelectionFillColor->SetVectorParameterValue(FName("Color"), FillColor);
	MeshElementSelectionFillColor->SetScalarParameterValue(FName("Desaturation"), 0.6f);
	FMaterialRenderProxy* MaterialRenderProxy = MeshElementSelectionFillColor->GetRenderProxy();
	//FMaterialRenderProxy* MaterialRenderProxy = GEngine->ConstraintLimitMaterialX->GetRenderProxy();

	MeshBuilder.Draw(CurrentPDI, FMatrix::Identity, MaterialRenderProxy, DepthPriority, false, false);

	FToolDataVisualizer Visualizer;
	Visualizer.DepthBias = DepthBias;
	Visualizer.BeginFrame(RenderAPI);

	Visualizer.SetLineParameters(LineColor, LineThickness);
	Visualizer.SetPointParameters(PointColor, PointSize);
	
	for (const FSegment3d& Segment : Elements.Segments)
	{
		Visualizer.DrawLine(Segment.StartPoint(), Segment.EndPoint());
	}
	for (const FVector3d& Point : Elements.Points)
	{
		Visualizer.DrawPoint(Point);
	}

	Visualizer.EndFrame();
}

void FSelectionRenderHelper::Initialize(const FGeometrySelection& Selection,
	const FDynamicMesh3& SourceMesh,
	const FGroupTopology* Topology,
	const FTransform* ApplyTransform)
{
	bool bSuccess = ToolSelectionUtil::AccumulateSelectionElements(Elements, Selection, SourceMesh, Topology, ApplyTransform, EEnumerateSelectionMapping::Default | EEnumerateSelectionMapping::None);
	ensure(bSuccess);
}

void FSelectionRenderHelper::Render(IToolsContextRenderAPI* RenderAPI) const
{
	// TODO Using PDI here does not scale well for large selections. We are going to have to reimplement this with
	// something that uses (eg) a custom LineSetComponent / PointSetComponent / TriangleSetComponent depending on
	// the selection type
	ToolSelectionUtil::DebugRenderGeometrySelectionElements(RenderAPI, Elements, false);
}
