// Copyright Epic Games, Inc. All Rights Reserved.

#include "Snapping/BasePositionSnapSolver3.h"
#include "VectorUtil.h"
#include "LineTypes.h"
#include "Distance/DistLine3Ray3.h"

using namespace UE::Geometry;

FBasePositionSnapSolver3::FBasePositionSnapSolver3()
{
	// initialize to sane values
	SnapMetricFunc =
		[](const FVector3d& A, const FVector3d& B) { return Distance(A, B); };
	SnapMetricTolerance = 1.0;

	FBasePositionSnapSolver3::Reset();
}

void FBasePositionSnapSolver3::Reset()
{
	TargetPoints.Reset();
	TargetLines.Reset();
	TargetCircles.Reset();
	ResetActiveSnap();
}

void FBasePositionSnapSolver3::ResetActiveSnap()
{
	ClearActiveSnapData();
}

void FBasePositionSnapSolver3::AddPointTarget(const FVector3d& Position, int TargetID, int Priority)
{
	FSnapTargetPoint Point;
	Point.Position = Position;
	Point.TargetID = TargetID;
	Point.Priority = Priority;
	Point.bHaveCustomMetric = false;
	TargetPoints.Add(Point);
}


void FBasePositionSnapSolver3::AddPointTarget(const FVector3d& Position, int TargetID, const FCustomMetric& CustomMetric, int Priority)
{
	FSnapTargetPoint Point;
	Point.Position = Position;
	Point.TargetID = TargetID;
	Point.Priority = Priority;
	Point.bHaveCustomMetric = true;
	Point.CustomMetric = CustomMetric;
	TargetPoints.Add(Point);
}

bool FBasePositionSnapSolver3::RemovePointTargetsByID(int TargetID)
{
	bool bRemoved = false;
	int Num = TargetPoints.Num();
	for (int k = 0; k < Num; ++k)
	{
		if (TargetPoints[k].TargetID == TargetID)
		{
			TargetPoints.RemoveAtSwap(k);
			k--; Num--;
			bRemoved = true;
		}
	}
	return bRemoved;
}



void FBasePositionSnapSolver3::AddLineTarget(const FLine3d& Line, int TargetID, int Priority)
{
	FSnapTargetLine Target = { Line, TargetID, Priority };
	TargetLines.Add(Target);
}


bool FBasePositionSnapSolver3::RemoveLineTargetsByID(int TargetID)
{
	bool bRemoved = false;
	int Num = TargetLines.Num();
	for (int k = 0; k < Num; ++k)
	{
		if (TargetLines[k].TargetID == TargetID)
		{
			TargetLines.RemoveAtSwap(k);
			k--; Num--;
			bRemoved = true;
		}
	}
	return bRemoved;
}




void FBasePositionSnapSolver3::AddCircleTarget(const FCircle3d& Circle, int TargetID, int Priority)
{
	FSnapTargetCircle Target = { Circle, TargetID, Priority };
	TargetCircles.Add(Target);
}


bool FBasePositionSnapSolver3::RemoveCircleTargetsByID(int TargetID)
{
	bool bRemoved = false;
	int Num = TargetCircles.Num();
	for (int k = 0; k < Num; ++k)
	{
		if (TargetCircles[k].TargetID == TargetID)
		{
			TargetCircles.RemoveAtSwap(k);
			k--; Num--;
			bRemoved = true;
		}
	}
	return bRemoved;
}




void FBasePositionSnapSolver3::AddIgnoreTarget(int TargetID)
{
	IgnoreTargets.Add(TargetID);
}

void FBasePositionSnapSolver3::RemoveIgnoreTarget(int TargetID)
{
	IgnoreTargets.Remove(TargetID);
}

bool FBasePositionSnapSolver3::IsIgnored(int TargetID) const
{
	return IgnoreTargets.Contains(TargetID);
}


int32 FBasePositionSnapSolver3::FindIndexOfBestSnapInSet(const TArray<FSnapTargetPoint>& TestTargets, double& MinMetric, int& MinPriority,
	const TFunction<FVector3d(const FVector3d&)>& GetSnapPointFromFunc)
{
	int32 BestIndex = -1;
	int NumTargets = TestTargets.Num();
	for (int k = 0; k < NumTargets; ++k)
	{
		if (TestTargets[k].Priority > MinPriority)
		{
			continue;
		}
		if (IgnoreTargets.Contains(TestTargets[k].TargetID))
		{
			continue;
		}

		double TestMetric = SnapMetricTolerance;
		if (TestTargets[k].bHaveCustomMetric)
		{
			switch (TestTargets[k].CustomMetric.Type)
			{
			case ECustomMetricType::MinValue:
				TestMetric = FMathd::Min(SnapMetricTolerance, TestTargets[k].CustomMetric.Value);
				break;
			case ECustomMetricType::Multiplier:
				TestMetric = TestTargets[k].CustomMetric.Value * SnapMetricTolerance;
				break;
			case ECustomMetricType::ReplaceValue:
				TestMetric = TestTargets[k].CustomMetric.Value;
				break;
			}
		}

		FVector3d SnapPoint = GetSnapPointFromFunc(TestTargets[k].Position);
		double Metric = SnapMetricFunc(SnapPoint, TestTargets[k].Position);

		if (Metric < TestMetric)
		{
			if (Metric < MinMetric || TestTargets[k].Priority < MinPriority)
			{
				MinMetric = Metric;
				BestIndex = k;
				MinPriority = TestTargets[k].Priority;
			}
		}
	}
	return BestIndex;
}

const FBasePositionSnapSolver3::FSnapTargetPoint* FBasePositionSnapSolver3::FindBestSnapInSet(const TArray<FSnapTargetPoint>& TestTargets, double& MinMetric, int& MinPriority,
	const TFunction<FVector3d(const FVector3d&)>& GetSnapPointFromFunc)
{
	const FSnapTargetPoint* BestTarget = nullptr;
	int32 BestIndex = FindIndexOfBestSnapInSet(TestTargets, MinMetric, MinPriority, GetSnapPointFromFunc);
	if (BestIndex >= 0)
	{
		BestTarget = &TestTargets[BestIndex];
	}
	return BestTarget;
}


bool FBasePositionSnapSolver3::TestSnapTarget(const FSnapTargetPoint& Target, double MinMetric, int MinPriority, 
	const TFunction<FVector3d(const FVector3d&)>& GetSnapPointFromFunc)
{
	if (Target.Priority <= MinPriority)
	{
		FVector3d SnapPoint = GetSnapPointFromFunc(Target.Position);
		double Metric = SnapMetricFunc(SnapPoint, Target.Position);
		if (Metric < SnapMetricTolerance && Metric < MinMetric)
		{
			return true;
		}
	}
	return false;
}




void FBasePositionSnapSolver3::SetActiveSnapData(const FSnapTargetPoint& TargetPoint, const FVector3d& FromPoint, const FVector3d& ToPoint, double Metric)
{
	bHaveActiveSnap = true;
	ActiveSnapTarget = TargetPoint;
	ActiveSnapFromPoint = FromPoint;
	ActiveSnapToPoint = ToPoint;
	SnappedPointMetric = Metric;
}

void FBasePositionSnapSolver3::ClearActiveSnapData()
{
	bHaveActiveSnap = false;
}