// Copyright Epic Games, Inc. All Rights Reserved.

#include "Snapping/ModelingSceneSnappingManager.h"

#include "InteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "InteractiveToolObjects.h"
#include "ContextObjectStore.h"

#include "Scene/SceneGeometrySpatialCache.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "EngineUtils.h" // for TActorIterator<>
#include "Engine/StaticMesh.h"
#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Components/ShapeComponent.h"
#include "RawIndexBuffer.h"
#include "StaticMeshResources.h"
#include "UObject/UObjectGlobals.h"
#include "VectorUtil.h"
#include "SegmentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModelingSceneSnappingManager)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "USceneSnappingManager"


// defined in SceneGeometrySpatialCache.cpp
extern TAutoConsoleVariable<bool> CVarEnableModelingVolumeSnapping;



void UModelingSceneSnappingManager::Initialize(TObjectPtr<UInteractiveToolsContext> ToolsContext)
{
	ParentContext = ToolsContext;

	QueriesAPI = (ParentContext && ParentContext->ToolManager) ?
		ParentContext->ToolManager->GetContextQueriesAPI() : nullptr;

	SpatialCache = MakeShared<FSceneGeometrySpatialCache>();

#if WITH_EDITOR
	OnObjectModifiedHandler = FCoreUObjectDelegates::OnObjectModified.AddLambda([this](UObject* Object) { 
		HandleGlobalObjectModifiedDelegate(Object);
	});

	if (GEngine)
	{
		OnComponentTransformChangedHandle = GEngine->OnComponentTransformChanged().AddLambda([this](USceneComponent* Component, ETeleportType) {
			HandleGlobalComponentTransformChangedDelegate(Component);
		});
	}
#endif
}

void UModelingSceneSnappingManager::Shutdown()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedHandler);

	if (GEngine)
	{
		GEngine->OnComponentTransformChanged().Remove(OnComponentTransformChangedHandle);
	}
#endif
}


bool UModelingSceneSnappingManager::IsComponentTypeSupported(const UPrimitiveComponent* Component) const
{
	// supported types
	return ( Cast<UStaticMeshComponent>(Component) != nullptr ||
			 Cast<UBrushComponent>(Component) != nullptr ||
			 Cast<UDynamicMeshComponent>(Component) != nullptr );
}

void UModelingSceneSnappingManager::OnActorAdded(AActor* Actor, TFunctionRef<bool(UPrimitiveComponent*)> PrimitiveFilter)
{
	// TODO: additional filtering here?

	// ignore internal tool framework Actors
	if (Cast<AInternalToolFrameworkActor>(Actor) != nullptr)
	{
		return;
	}

	TArray<AActor*> AllActors;
	Actor->GetAllChildActors(AllActors, true);
	AllActors.Add(Actor);
	for (AActor* ProcessActor : AllActors)
	{
		for (UActorComponent* Component : ProcessActor->GetComponents())
		{
			UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component);
			if (PrimComponent && PrimitiveFilter(PrimComponent) )
			{
				OnComponentAdded(PrimComponent);
			}
		}
	}
}

void UModelingSceneSnappingManager::OnActorRemoved(AActor* Actor)
{
	// cannot assume actor is still valid...
	TArray<UPrimitiveComponent*> ToRemove;
	for (TPair<UPrimitiveComponent*, AActor*> Pair : ComponentToActorMap)
	{
		if (Pair.Value == Actor)
		{
			ToRemove.Add(Pair.Key);
		}
	}
	for (UPrimitiveComponent* Component : ToRemove)
	{
		OnComponentRemoved(Component);
	}
}

void UModelingSceneSnappingManager::OnComponentAdded(UPrimitiveComponent* Component)
{
	if (Component->GetOwner() == nullptr)
	{
		return;
	}
	if (IsComponentTypeSupported(Component) == false)
	{
		return;
	}

	UE::Geometry::FSceneGeometryID GeometryID;
	SpatialCache->EnableComponentTracking(Component, GeometryID);

	ComponentToActorMap.Add(Component, Component->GetOwner());

	// UDynamicMeshComponent gets special treatment because it has an explicit mesh-modified handler
	// we can rely on to detect geometry changes
	if (UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component))
	{
		DynamicMeshComponent->OnMeshChanged.AddUObject(this, &UModelingSceneSnappingManager::HandleDynamicMeshModifiedDelegate, DynamicMeshComponent);

		DynamicMeshComponents.Add(DynamicMeshComponent, TWeakObjectPtr<UDynamicMeshComponent>(DynamicMeshComponent) );
	}

}

void UModelingSceneSnappingManager::OnComponentRemoved(UPrimitiveComponent* Component)
{
	SpatialCache->DisableComponentTracking(Component);
	ComponentToActorMap.Remove(Component);

	// If this is a DynamicMeshComponent, we want to stop listening to mesh change events, but
	// we need to make sure it is still alive first.
	TWeakObjectPtr<UDynamicMeshComponent>* FoundDynamicMesh = DynamicMeshComponents.Find(Component);
	if (FoundDynamicMesh != nullptr)
	{
		if (FoundDynamicMesh->IsValid())
		{
			UDynamicMeshComponent* DynamicMeshComponent = FoundDynamicMesh->Get();
			DynamicMeshComponent->OnMeshChanged.RemoveAll(this);
		}

		DynamicMeshComponents.Remove(Component);
	}
}


void UModelingSceneSnappingManager::HandleGlobalObjectModifiedDelegate(UObject* Object)
{
	if (Cast<UDynamicMeshComponent>(Object))
	{
		return;		// have special-case handling
	}

	UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Object);
	if (PrimComponent)
	{
		OnComponentModified(PrimComponent);
	}
}

void UModelingSceneSnappingManager::OnComponentModified(UActorComponent* Component)
{
	const bool bDeferRebuild = false;
	UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component);
	if (PrimComponent)
	{
		SpatialCache->NotifyGeometryUpdate(PrimComponent, bDeferRebuild);
	}
}


void UModelingSceneSnappingManager::HandleGlobalComponentTransformChangedDelegate(USceneComponent* Component)
{
	UPrimitiveComponent* PrimComponent = Cast<UPrimitiveComponent>(Component);
	if (PrimComponent && ComponentToActorMap.Contains(PrimComponent))
	{
		SpatialCache->NotifyTransformUpdate(PrimComponent);
	}
}


void UModelingSceneSnappingManager::HandleDynamicMeshModifiedDelegate(UDynamicMeshComponent* Component)
{
	if (bQueueModifiedDynamicMeshUpdates)
	{
		PendingModifiedDynamicMeshes.Add(Component);
	}
	else
	{
		const bool bDeferRebuild = false;
		SpatialCache->NotifyGeometryUpdate(Component, bDeferRebuild);
	}
}





void UModelingSceneSnappingManager::PauseSceneGeometryUpdates()
{
	ensure(bQueueModifiedDynamicMeshUpdates == false);
	bQueueModifiedDynamicMeshUpdates = true;
}


void UModelingSceneSnappingManager::UnPauseSceneGeometryUpdates(bool bImmediateRebuilds)
{
	if (ensure(bQueueModifiedDynamicMeshUpdates))
	{
		for (UDynamicMeshComponent* Component : PendingModifiedDynamicMeshes)
		{
			SpatialCache->NotifyGeometryUpdate(Component, !bImmediateRebuilds);
		}
		PendingModifiedDynamicMeshes.Reset();
		bQueueModifiedDynamicMeshUpdates = false;
	}
}



void UModelingSceneSnappingManager::BuildSpatialCacheForWorld(
	UWorld* World,
	TFunctionRef<bool(AActor*)> ActorFilter,
	TFunctionRef<bool(UPrimitiveComponent*)> PrimitiveFilter)
{
	for ( TActorIterator<AActor> ActorItr(World); ActorItr; ++ActorItr)
	{
		if (ActorFilter(*ActorItr))
		{
			OnActorAdded(*ActorItr, PrimitiveFilter);
		}
	}
}


bool UModelingSceneSnappingManager::ExecuteSceneSnapQuery(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& Results) const
{
	switch (Request.RequestType)
	{
	case ESceneSnapQueryType::Position:
		return ExecuteSceneSnapQueryPosition(Request, Results);
	case ESceneSnapQueryType::Rotation:
		return ExecuteSceneSnapQueryRotation(Request, Results);
	default:
		check(!"Only Position and Rotation Snap Queries are supported");
	}
	return false;
}


bool UModelingSceneSnappingManager::ExecuteSceneSnapQueryRotation(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& Results) const
{
	if (!QueriesAPI)
	{
		return false;
	}

	if ((Request.TargetTypes & ESceneSnapQueryTargetType::Grid) != ESceneSnapQueryTargetType::None)
	{
		FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();
		if (SnappingConfig.bEnableRotationGridSnapping)
		{
			FRotator Rotator(Request.DeltaRotation);
			FRotator RotGrid = Request.RotGridSize.Get(SnappingConfig.RotationGridAngles);
			Rotator = Rotator.GridSnap(RotGrid);

			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::Grid;
			SnapResult.DeltaRotation = Rotator.Quaternion();
			Results.Add(SnapResult);
			return true;
		}
	}
	return false;
}





static bool FindNearestVisibleObjectHit_Internal(
	UWorld* World, 
	FHitResult& HitResultOut, 
	const FRay3d& Ray,
	bool bIsSceneGeometrySnapQuery, 
	const FSceneQueryVisibilityFilter& VisibilityFilter,
	UE::Geometry::FSceneGeometrySpatialCache* SpatialCache)
{
	FCollisionObjectQueryParams ObjectQueryParams(FCollisionObjectQueryParams::AllObjects);
	FCollisionQueryParams QueryParams = FCollisionQueryParams::DefaultQueryParam;
	QueryParams.bTraceComplex = true;
	QueryParams.bReturnFaceIndex = bIsSceneGeometrySnapQuery;

	// TODO: QueryParams.AddIgnoredComponents ?

	TArray<FHitResult> OutHits;
	FVector RayEnd = (FVector)(Ray.PointAt(HALF_WORLD_MAX));
	if (World->LineTraceMultiByObjectType(OutHits, (FVector)Ray.Origin, RayEnd, ObjectQueryParams, QueryParams) == false)
	{
		return false;
	}

	bool bEnableVolumes = CVarEnableModelingVolumeSnapping.GetValueOnAnyThread();

	double NearestVisible = TNumericLimits<double>::Max();
	for (const FHitResult& CurResult : OutHits)
	{
		// if we have hit Component in the SpatialCache, prefer to use that
		if (SpatialCache && SpatialCache->HaveCacheForComponent(CurResult.GetComponent()) )
		{
			continue;
		}

		// filtering out any volume hits here will disable volume snapping
		// shape components are also commonly used to represent simple volumes, so if we are filtering out volume hits, we filter out both brush and shape components
		if (bEnableVolumes == false && (Cast<UBrushComponent>(CurResult.GetComponent()) != nullptr || Cast<UShapeComponent>(CurResult.GetComponent()) != nullptr))
		{
			continue;
		}

		if (CurResult.Distance < NearestVisible && VisibilityFilter.IsVisible(CurResult.Component.Get()) )
		{
			HitResultOut = CurResult;
			NearestVisible = CurResult.Distance;
		}
	}

	return NearestVisible < TNumericLimits<double>::Max();
}


template<typename VectorType>
bool GetComponentHitTriangle_Internal(FHitResult HitResult, VectorType* TriVertices)
{
	if (HitResult.Component.IsValid() == false || HitResult.FaceIndex < 0)
	{
		return false;
	}

	UStaticMeshComponent* Component = Cast<UStaticMeshComponent>(HitResult.Component.Get());
	if (!Component)
	{
		return false;
	}
	
	// physics collision data is created from StaticMesh RenderData
	// so use HitResult.FaceIndex to extract triangle from the mesh
	// (note: this may be incorrect if there are multiple sections...in that case I think we have to
	//  first find section whose accumulated index range would contain .FaceIndexX)
	UStaticMesh* StaticMesh = Component->GetStaticMesh();

	// Get the lowest available render LOD.
	// If the Minimum LOD index is not zero, their might be a lower LOD available, but it will not be used for rendering in the viewport.
	const int32 MinLODIdx = StaticMesh->GetMinLODIdx();
	const FStaticMeshLODResources* LOD = StaticMesh->GetRenderData()->GetCurrentFirstLOD(MinLODIdx);
	if (!LOD)
	{
		return false;
	}

	const FIndexArrayView Indices = LOD->IndexBuffer.GetArrayView();
	const FPositionVertexBuffer& Vertices = LOD->VertexBuffers.PositionVertexBuffer;

	const int32 TriIdx = 3 * HitResult.FaceIndex;
	if (TriIdx + 2 >= Indices.Num())
	{
		return false;
	}

	const uint32 Idx[3] = { Indices[TriIdx], Indices[TriIdx + 1], Indices[TriIdx + 2] };
	const uint32 NumVertices = Vertices.GetNumVertices();
	if (Idx[0] >= NumVertices || Idx[1] >= NumVertices || Idx[2] >= NumVertices)
	{
		return false;
	}

	const FVector Positions[3] = {
		static_cast<FVector>(Vertices.VertexPosition(Idx[0])),
		static_cast<FVector>(Vertices.VertexPosition(Idx[1])),
		static_cast<FVector>(Vertices.VertexPosition(Idx[2]))
	};

	// transform to world space
	const FTransform ComponentTransform = Component->GetComponentTransform();
	TriVertices[0] = static_cast<VectorType>(ComponentTransform.TransformPosition(Positions[0]));
	TriVertices[1] = static_cast<VectorType>(ComponentTransform.TransformPosition(Positions[1]));
	TriVertices[2] = static_cast<VectorType>(ComponentTransform.TransformPosition(Positions[2]));

	return true;
}




bool UModelingSceneSnappingManager::ExecuteSceneHitQuery(const FSceneHitQueryRequest& Request, FSceneHitQueryResult& ResultOut) const
{
	if (!QueriesAPI)
	{
		return false;
	}

	FRay3d WorldRay(Request.WorldRay);

	double MinCurrentHitDist = TNumericLimits<double>::Max();

	FViewCameraState ViewState;
	QueriesAPI->GetCurrentViewState(ViewState);

	FSceneGeometryPoint HitPoint;
	FSceneGeometryID HitIdentifier;
	bool bHitSceneCache = SpatialCache->FindNearestHit(WorldRay, HitPoint, HitIdentifier, &Request.VisibilityFilter );
	if (bHitSceneCache)
	{
		FVector3d A, B, C;
		SpatialCache->GetGeometry(HitIdentifier, HitPoint.GeometryType, HitPoint.GeometryIndex, true, A, B, C);
		MinCurrentHitDist = HitPoint.RayDistance;
		ResultOut.TargetActor = HitPoint.Actor;
		ResultOut.TargetComponent = HitPoint.Component;
		ResultOut.HitTriIndex = HitPoint.GeometryIndex;
		ResultOut.Position = HitPoint.WorldPoint;
		ResultOut.Normal = VectorUtil::Normal(A, B, C);
		ResultOut.TriVertices[0] = A;
		ResultOut.TriVertices[1] = B;
		ResultOut.TriVertices[2] = C;

		ResultOut.InitializeHitResult(Request);
	}


	FHitResult HitResult;
	bool bHitWorld = FindNearestVisibleObjectHit_Internal(
		QueriesAPI->GetCurrentEditingWorld(), 
		HitResult, WorldRay, 
		true, Request.VisibilityFilter, SpatialCache.Get());

	if (bHitWorld && HitResult.Distance < MinCurrentHitDist )
	{
		MinCurrentHitDist =  HitResult.Distance;
		ResultOut.TargetActor = HitResult.GetActor();
		ResultOut.TargetComponent = HitResult.GetComponent();
		ResultOut.HitTriIndex = HitResult.FaceIndex;
		ResultOut.Position = (FVector3d)HitResult.Location;
		ResultOut.Normal = (FVector3d)HitResult.Normal;
		ResultOut.HitResult = HitResult;
		if (Request.bWantHitGeometryInfo)
		{ 
			GetComponentHitTriangle_Internal<FVector3d>(HitResult, ResultOut.TriVertices);
		}
	}

	return (MinCurrentHitDist < TNumericLimits<double>::Max());
}



bool UModelingSceneSnappingManager::ExecuteSceneSnapQueryPosition(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& Results) const
{
	if (!QueriesAPI)
	{
		return false;
	}

	int FoundResultCount = 0;

	if ((Request.TargetTypes & ESceneSnapQueryTargetType::Grid) != ESceneSnapQueryTargetType::None)
	{
		FToolContextSnappingConfiguration SnappingConfig = QueriesAPI->GetCurrentSnappingSettings();

		if (SnappingConfig.bEnablePositionGridSnapping)
		{
			FSceneSnapQueryResult SnapResult;
			SnapResult.TargetType = ESceneSnapQueryTargetType::Grid;

			FVector GridSize = Request.GridSize.Get(SnappingConfig.PositionGridDimensions);

			SnapResult.Position.X = (FVector::FReal)UE::Geometry::SnapToIncrement(Request.Position.X, GridSize.X);
			SnapResult.Position.Y = (FVector::FReal)UE::Geometry::SnapToIncrement(Request.Position.Y, GridSize.Y);
			SnapResult.Position.Z = (FVector::FReal)UE::Geometry::SnapToIncrement(Request.Position.Z, GridSize.Z);

			Results.Add(SnapResult);
			FoundResultCount++;
		}
	}

	FViewCameraState ViewState;
	QueriesAPI->GetCurrentViewState(ViewState);

	//
	// Run a snap query by casting ray into the world.
	// If a hit is found, we look up what triangle was hit, and then test its vertices and edges
	//


	// try snapping to vertices
	auto TrySnapToVertex = [](const FSceneSnapQueryRequest& RequestIn, const FRay3d& WorldRay, FSceneSnapQueryResult& SnapResult, double& SmallestSnapAngle)
	{
		if ((RequestIn.TargetTypes & ESceneSnapQueryTargetType::MeshVertex) != ESceneSnapQueryTargetType::None)
		{
			for (int j = 0; j < 3; ++j)
			{
				double VisualAngle = UE::Geometry::VectorUtil::OpeningAngleD((FVector3d)RequestIn.Position, (FVector3d)SnapResult.TriVertices[j], WorldRay.Origin);
				if (VisualAngle < SmallestSnapAngle)
				{
					SmallestSnapAngle = VisualAngle;
					SnapResult.Position = SnapResult.TriVertices[j];
					SnapResult.TargetType = ESceneSnapQueryTargetType::MeshVertex;
					SnapResult.TriSnapIndex = j;
				}
			}
		}
	};


	auto TrySnapToEdge = [](const FSceneSnapQueryRequest& RequestIn, const FVector& SurfacePoint, const FRay3d& WorldRay, FSceneSnapQueryResult& SnapResult, double& SmallestSnapAngle)
	{
		if ( ((RequestIn.TargetTypes & ESceneSnapQueryTargetType::MeshEdge) != ESceneSnapQueryTargetType::None) &&
			  (SnapResult.TargetType != ESceneSnapQueryTargetType::MeshVertex) )
		{
			for (int j = 0; j < 3; ++j)
			{
				UE::Geometry::FSegment3d Segment(SnapResult.TriVertices[j], SnapResult.TriVertices[(j+1)%3]);
				FVector3d EdgeNearestPt = Segment.NearestPoint(SurfacePoint);
				double VisualAngle = UE::Geometry::VectorUtil::OpeningAngleD(RequestIn.Position, EdgeNearestPt, WorldRay.Origin);
				if (VisualAngle < SmallestSnapAngle )
				{
					SmallestSnapAngle = VisualAngle;
					SnapResult.Position = EdgeNearestPt;
					SnapResult.TargetType = ESceneSnapQueryTargetType::MeshEdge;
					SnapResult.TriSnapIndex = j;
				}
			}
		}
	};


	FVector3d RayStart = (FVector3d)ViewState.Position;
	FVector3d RayDirection = (FVector3d)Request.Position - RayStart;
	FRay3d WorldRay(RayStart, Normalized(RayDirection));

	FSceneQueryVisibilityFilter TmpFilter;
	TmpFilter.ComponentsToIgnore = Request.ComponentsToIgnore;
	TmpFilter.InvisibleComponentsToInclude = Request.InvisibleComponentsToInclude;

	// cast ray into scene cache
	FSceneGeometryPoint HitPoint;
	FSceneGeometryID HitIdentifier;
	bool bHitSceneCache = SpatialCache->FindNearestHit(WorldRay, HitPoint, HitIdentifier, &TmpFilter);

	// cast ray into world
	FHitResult HitResult;
	bool bHitWorld = FindNearestVisibleObjectHit_Internal(
		QueriesAPI->GetCurrentEditingWorld(), 
		HitResult, WorldRay, 
		true, TmpFilter, SpatialCache.Get());

	// if we hit both, determine which is closer
	if (bHitSceneCache && bHitWorld)
	{
		if (HitPoint.RayDistance < HitResult.Distance)
		{
			bHitWorld = false;
		}
		else
		{
			bHitSceneCache = false;
		}
	}

	// Try to snap based on whichever thing was hit.
	// Might be possible to combine parts of these blocks.

	if (bHitSceneCache)
	{
		// is this check necessary? isn't this always going to be zero for a scene hit??
		double VisualAngle = UE::Geometry::VectorUtil::OpeningAngleD((FVector3d)Request.Position, HitPoint.WorldPoint, RayStart);
		if (VisualAngle < (double)Request.VisualAngleThresholdDegrees)
		{
			FVector3d A, B, C;
			SpatialCache->GetGeometry(HitIdentifier, HitPoint.GeometryType, HitPoint.GeometryIndex, true, A, B, C);

			FSceneSnapQueryResult SnapResult;
			SnapResult.TriVertices[0] = (FVector)A;
			SnapResult.TriVertices[1] = (FVector)B;
			SnapResult.TriVertices[2] = (FVector)C;

			double SmallestAngle = Request.VisualAngleThresholdDegrees;

			// try snapping to vertices
			TrySnapToVertex(Request, WorldRay, SnapResult, SmallestAngle);

			// try snapping to nearest points on edges
			TrySnapToEdge(Request, HitPoint.WorldPoint, WorldRay, SnapResult, SmallestAngle);

			// if we found a valid snap, return it
			if (SmallestAngle < (double)Request.VisualAngleThresholdDegrees)
			{
				SnapResult.TargetActor = HitPoint.Actor;
				SnapResult.TargetComponent = HitPoint.Component;
				Results.Add(SnapResult);
				FoundResultCount++;
			}

		}
	}


	if (bHitWorld)
	{
		// is this check necessary? isn't this always going to be zero for a scene hit??
		double VisualAngle = UE::Geometry::VectorUtil::OpeningAngleD((FVector3d)Request.Position, (FVector3d)HitResult.ImpactPoint, RayStart);
		if (VisualAngle < (double)Request.VisualAngleThresholdDegrees)
		{
			FSceneSnapQueryResult SnapResult;
			if ( GetComponentHitTriangle_Internal<FVector>(HitResult, SnapResult.TriVertices) )
			{
				double SmallestAngle = Request.VisualAngleThresholdDegrees;

				// try snapping to vertices
				TrySnapToVertex(Request, WorldRay, SnapResult, SmallestAngle);

				// try snapping to nearest points on edges
				TrySnapToEdge(Request, HitResult.ImpactPoint, WorldRay, SnapResult, SmallestAngle);

				// if we found a valid snap, return it
				if (SmallestAngle < (double)Request.VisualAngleThresholdDegrees)
				{
					SnapResult.TargetActor = HitResult.HitObjectHandle.FetchActor();
					SnapResult.TargetComponent = HitResult.Component.Get();
					Results.Add(SnapResult);
					FoundResultCount++;
				}
			}
		}
	}

	return (FoundResultCount > 0);
}


bool UE::Geometry::RegisterSceneSnappingManager(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UModelingSceneSnappingManager* Found = ToolsContext->ContextObjectStore->FindContext<UModelingSceneSnappingManager>();
		if (Found == nullptr)
		{
			UModelingSceneSnappingManager* SelectionManager = NewObject<UModelingSceneSnappingManager>(ToolsContext->ToolManager);
			if (ensure(SelectionManager))
			{
				SelectionManager->Initialize(ToolsContext);
				ToolsContext->ContextObjectStore->AddContextObject(SelectionManager);
				return true;
			}
			else
			{
				return false;
			}
		}
		return true;
	}
	return false;
}



bool UE::Geometry::DeregisterSceneSnappingManager(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UModelingSceneSnappingManager* Found = ToolsContext->ContextObjectStore->FindContext<UModelingSceneSnappingManager>();
		if (Found != nullptr)
		{
			Found->Shutdown();
			ToolsContext->ContextObjectStore->RemoveContextObject(Found);
		}
		return true;
	}
	return false;
}


UModelingSceneSnappingManager* UE::Geometry::FindModelingSceneSnappingManager(UInteractiveToolManager* ToolManager)
{
	if (ensure(ToolManager))
	{
		UModelingSceneSnappingManager* Found = ToolManager->GetContextObjectStore()->FindContext<UModelingSceneSnappingManager>();
		if (Found != nullptr)
		{
			return Found;
		}
	}
	return nullptr;
}


#undef LOCTEXT_NAMESPACE
