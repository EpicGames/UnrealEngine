// Copyright Epic Games, Inc. All Rights Reserved.


#include "Snapping/PointPlanarSnapSolver.h"

#include "LineTypes.h"
#include "Quaternion.h"
#include "ToolDataVisualizer.h"

using namespace UE::Geometry;

FPointPlanarSnapSolver::FPointPlanarSnapSolver()
{
	Plane = FFrame3d();
}

void FPointPlanarSnapSolver::Reset()
{
	FBasePositionSnapSolver3::Reset();
	PointHistory.Reset();
	ResetGenerated();
}

/** Clears any existing generated snap targets. */
void FPointPlanarSnapSolver::ResetGenerated()
{
	GeneratedLines.Reset();
	IntersectionPoints.Reset();
	IntersectionSecondLinePointers.Reset();
	GeneratedTargets.Reset();
}

void FPointPlanarSnapSolver::UpdatePointHistory(const TArray<FVector3d>& Points)
{
	PointHistory = Points;
	ResetGenerated();
}

void FPointPlanarSnapSolver::UpdatePointHistory(const TArray<FVector3f>& Points)
{
	PointHistory.Reset();
	int Num = Points.Num();
	for (int k = 0; k < Num; ++k)
	{
		PointHistory.Add((FVector3d)Points[k]);
	}
	ResetGenerated();
}

void FPointPlanarSnapSolver::AppendHistoryPoint(const FVector3d& Point)
{
	PointHistory.Add(Point);
	ResetGenerated();
}

void FPointPlanarSnapSolver::InsertHistoryPoint(const FVector3d& Point, int32 Index)
{
	PointHistory.Insert(Point, Index);
	ResetGenerated();
}

void FPointPlanarSnapSolver::RemoveHistoryPoint(int32 Index)
{
	PointHistory.RemoveAt(Index);
	ResetGenerated();
}


void FPointPlanarSnapSolver::RegenerateTargetLines(bool bCardinalAxes, bool bLastHistorySegment)
{
	ResetGenerated();

	int NumHistoryPts = PointHistory.Num();
	if (NumHistoryPts == 0)
	{
		return;
	}

	FVector3d LastPt = PointHistory[NumHistoryPts - 1];

	if (bCardinalAxes)
	{
		FSnapTargetLine AxisLine;
		AxisLine.TargetID = CardinalAxisTargetID;
		AxisLine.Priority = CardinalAxisPriority;
		AxisLine.Line = FLine3d(LastPt, Plane.X());
		GeneratedLines.Add(AxisLine);
		AxisLine.Line = FLine3d(LastPt, Plane.Y());
		GeneratedLines.Add(AxisLine);
	}

	if (bLastHistorySegment && NumHistoryPts > 1)
	{
		FSnapTargetLine SegLine;
		SegLine.TargetID = LastSegmentTargetID;
		SegLine.Priority = LastSegmentPriority;
		SegLine.Line = FLine3d::FromPoints(LastPt, PointHistory[NumHistoryPts-2]);
		GeneratedLines.Add(SegLine);
		SegLine.Line.Direction = FQuaterniond(Plane.Z(), 90, true) * SegLine.Line.Direction;
		GeneratedLines.Add(SegLine);
	}
}

void FPointPlanarSnapSolver::RegenerateTargetLinesAround(int32 HistoryIndex, bool bWrapAround, bool bGenerateIntersections)
{
	ResetGenerated();

	// We allow indices one to each side of our history to allow snapping based on endpoints.
	if (PointHistory.Num() == 0 || HistoryIndex < -1 || HistoryIndex > PointHistory.Num())
	{
		return;
	}

	// Templates for the objects that we'll be adding
	FSnapTargetLine AxisLine;
	AxisLine.TargetID = CardinalAxisTargetID;
	AxisLine.Priority = CardinalAxisPriority;

	// Add lines based on previous point
	if (HistoryIndex > 0 || (bWrapAround && HistoryIndex == 0)) // Disallowing -1 here
	{
		int32 PreviousIndex = (HistoryIndex + PointHistory.Num() - 1) % PointHistory.Num();

		AxisLine.Line = FLine3d(PointHistory[PreviousIndex], Plane.X());
		GeneratedLines.Add(AxisLine);
		AxisLine.Line = FLine3d(PointHistory[PreviousIndex], Plane.Y());
		GeneratedLines.Add(AxisLine);
	}

	
	if (HistoryIndex < PointHistory.Num() - 1 || (bWrapAround && HistoryIndex == PointHistory.Num() - 1)) // Disallowing PointHistory.Num())
	{
		int32 NextIndex = (HistoryIndex + 1) % PointHistory.Num();

		AxisLine.Line = FLine3d(PointHistory[NextIndex], Plane.X());
		GeneratedLines.Add(AxisLine);
		AxisLine.Line = FLine3d(PointHistory[NextIndex], Plane.Y());
		GeneratedLines.Add(AxisLine);

	}

	if (bGenerateIntersections)
	{
		GenerateLineIntersectionTargets();
	}
}

void FPointPlanarSnapSolver::GenerateLineIntersectionTargets()
{
	IntersectionPoints.Reset();
	IntersectionSecondLinePointers.Reset();

	const double IN_PLANE_THRESHOLD = KINDA_SMALL_NUMBER * 10;

	FSnapTargetPoint Intersection;
	Intersection.TargetID = IntersectionTargetID;
	
	// Accumulate all lines in the plane so we can intersect them with each other. Also keep a 1:1
	// array with pointers to the original 3-d lines so we can link the intersection points to them.
	TArray<const FSnapTargetLine*> OriginalLinePointers;
	TArray<FLine2d> LinesInPlane;
	for (const FSnapTargetLine& Line : GeneratedLines)
	{
		FVector3d OriginInPlane = Plane.ToFramePoint(Line.Line.Origin);
		if (OriginInPlane.Z < IN_PLANE_THRESHOLD)
		{
			FVector3d DirectionInPlane = Plane.ToFrameVector(Line.Line.Direction);
			if (DirectionInPlane.Z < IN_PLANE_THRESHOLD)
			{
				LinesInPlane.Add(FLine2d(
					FVector2d(OriginInPlane.X, OriginInPlane.Y), FVector2d(DirectionInPlane.X, DirectionInPlane.Y)));
				OriginalLinePointers.Add(&Line);
			}
		}
	}
	for (const FSnapTargetLine& Line : TargetLines)
	{
		FVector3d OriginInPlane = Plane.ToFramePoint(Line.Line.Origin);
		if (OriginInPlane.Z < IN_PLANE_THRESHOLD)
		{
			FVector3d DirectionInPlane = Plane.ToFrameVector(Line.Line.Direction);
			if (DirectionInPlane.Z < IN_PLANE_THRESHOLD)
			{
				LinesInPlane.Add(FLine2d(
					FVector2d(OriginInPlane.X, OriginInPlane.Y), FVector2d(DirectionInPlane.X, DirectionInPlane.Y)));
				OriginalLinePointers.Add(&Line);
			}
		}
	}

	// Intersect all lines with each other
	for (int i = 0; i < LinesInPlane.Num(); ++i)
	{
		for (int j = i; j < LinesInPlane.Num(); ++j)
		{
			FVector2d IntersectionPoint;
			if (LinesInPlane[i].IntersectionPoint(LinesInPlane[j], IntersectionPoint))
			{
				Intersection.Position = Plane.FromFramePoint(FVector3d(IntersectionPoint.X, IntersectionPoint.Y, 0.0));

				// Lines in plane are 1:1 with our OriginalLinePointers array, which has more info per line.
				// Store the more important priority line in the snap target, and the second line in the 
				// IntersectionSecondLinePointers array.

				bool bImportanceIsSwapped = OriginalLinePointers[i]->Priority > OriginalLinePointers[j]->Priority;
				const FSnapTargetLine* MoreImportantLine = bImportanceIsSwapped ? OriginalLinePointers[j] : OriginalLinePointers[i];
				const FSnapTargetLine* SecondLine = bImportanceIsSwapped ? OriginalLinePointers[i] : OriginalLinePointers[j];

				Intersection.Priority = MoreImportantLine->Priority - IntersectionPriorityDelta;
				Intersection.SnapLine = MoreImportantLine->Line;
				IntersectionSecondLinePointers.Add(SecondLine);

				IntersectionPoints.Add(Intersection);
			}
		}
	}
}

void FPointPlanarSnapSolver::GenerateTargets(const FVector3d& PointIn)
{
	GeneratedTargets.Reset();

	int NumSnapLines = GeneratedLines.Num();

	// nearest-point-on-line snaps
	for (int j = 0; j < NumSnapLines; ++j)
	{
		FSnapTargetPoint Target;
		Target.Position = GeneratedLines[j].Line.NearestPoint(PointIn);
		Target.TargetID = GeneratedLines[j].TargetID;
		Target.Priority = GeneratedLines[j].Priority;
		Target.bIsSnapLine = true;
		Target.SnapLine = GeneratedLines[j].Line;
		GeneratedTargets.Add(Target);
	}

	for (int j = 0; j < TargetLines.Num(); ++j)
	{
		FSnapTargetPoint Target;
		Target.Position = TargetLines[j].Line.NearestPoint(PointIn);
		Target.TargetID = TargetLines[j].TargetID;
		Target.Priority = TargetLines[j].Priority;
		Target.bIsSnapLine = true;
		Target.SnapLine = TargetLines[j].Line;
		GeneratedTargets.Add(Target);
	}

	// length-along-line snaps
	if (bEnableSnapToKnownLengths)
	{
		int NumHistoryPts = PointHistory.Num();
		for (int j = 0; j < NumHistoryPts - 1; ++j)
		{
			double SegLength = Distance(PointHistory[j], PointHistory[j+1]);
			for (int k = 0; k < NumSnapLines; ++k)
			{
				FSnapTargetPoint Target;
				Target.TargetID = GeneratedLines[k].TargetID;
				Target.Priority = GeneratedLines[k].Priority - KnownLengthPriorityDelta;
				Target.bIsSnapLine = true;
				Target.SnapLine = GeneratedLines[k].Line;
				Target.bIsSnapDistance = true;
				Target.SnapDistanceID = j;

				Target.Position = GeneratedLines[k].Line.PointAt(SegLength);
				GeneratedTargets.Add(Target);
				Target.Position = GeneratedLines[k].Line.PointAt(-SegLength);
				GeneratedTargets.Add(Target);
			}
		}
	}
}



void FPointPlanarSnapSolver::UpdateSnappedPoint(const FVector3d& PointIn)
{
	double MinMetric = TNumericLimits<double>::Max();
	static constexpr int MIN_PRIORITY = TNumericLimits<int>::Max();
	int BestPriority = MIN_PRIORITY;
	const FSnapTargetPoint* BestSnapTarget = nullptr;
	bActiveSnapIsIntersection = false;

	GenerateTargets(PointIn);

	TFunction<FVector3d(const FVector3d&)> GetSnapFromPointFunc = [&PointIn](const FVector3d& Point)
	{
		return PointIn;
	};

	const FSnapTargetPoint* BestPointTarget = FindBestSnapInSet(TargetPoints, MinMetric, BestPriority, GetSnapFromPointFunc);
	if (BestPointTarget != nullptr)
	{
		BestSnapTarget = BestPointTarget;
	}

	const FSnapTargetPoint* BestLineTarget = FindBestSnapInSet(GeneratedTargets, MinMetric, BestPriority, GetSnapFromPointFunc);
	if (BestLineTarget != nullptr)
	{
		BestSnapTarget = BestLineTarget;
	}

	int32 BestIntersectionIndex = FindIndexOfBestSnapInSet(IntersectionPoints, MinMetric, BestPriority, GetSnapFromPointFunc);
	if (BestIntersectionIndex >= 0)
	{
		BestSnapTarget = &IntersectionPoints[BestIntersectionIndex];
		bActiveSnapIsIntersection = true;
		IntersectionSecondLine = IntersectionSecondLinePointers[BestIntersectionIndex]->Line;
	}

	if (bHaveActiveSnap && bEnableStableSnap && ActiveSnapTarget.bIsSnapLine == false)
	{
		if (TestSnapTarget(ActiveSnapTarget, MinMetric*StableSnapImproveThresh, BestPriority, GetSnapFromPointFunc))
		{
			return;
		}
	}

	if (BestSnapTarget != nullptr)
	{
		SetActiveSnapData(*BestSnapTarget, BestSnapTarget->Position, Plane.ToPlane(BestSnapTarget->Position, 2), MinMetric);
	}
	else
	{
		ClearActiveSnapData();
	}

}

void FPointPlanarSnapSolver::DebugRender(IToolsContextRenderAPI* RenderAPI)
{
	FToolDataVisualizer Renderer;
	Renderer.BeginFrame(RenderAPI);
	double LineLength = 10000;

	int ActiveSnapID = (bHaveActiveSnap) ? GetActiveSnapTargetID() : -9999;

	for (const FSnapTargetLine& LineTarget : TargetLines)
	{
		if (IsIgnored(LineTarget.TargetID))
		{
			continue;
		}

		const FLine3d& Line = LineTarget.Line;

		FLinearColor UseColor = FColor::Cyan;
		float LineWidth = (LineTarget.TargetID == ActiveSnapID) ? Renderer.LineThickness : Renderer.LineThickness * 0.5;
		Renderer.DrawLine(Line.PointAt(-LineLength), Line.PointAt(LineLength), UseColor, LineWidth);
	}

	for (const FSnapTargetLine& LineTarget : GeneratedLines)
	{
		if (IsIgnored(LineTarget.TargetID))
		{
			continue;
		}

		const FLine3d& Line = LineTarget.Line;

		FLinearColor UseColor = FColor::Cyan;
		float LineWidth = (LineTarget.TargetID == ActiveSnapID) ? Renderer.LineThickness : Renderer.LineThickness * 0.5;
		Renderer.DrawLine(Line.PointAt(-LineLength), Line.PointAt(LineLength), UseColor, LineWidth);
	}

	for (const FSnapTargetPoint& Point : IntersectionPoints)
	{
		if (IsIgnored(Point.TargetID))
		{
			continue;
		}

		FLinearColor UseColor = FColor::Cyan;
		float LineWidth = (Point.TargetID == ActiveSnapID) ? Renderer.LineThickness : Renderer.LineThickness * 0.5;
		Renderer.DrawCircle(Point.Position, FVector3d(1, 0, 0), 3, 64,
			UseColor, LineWidth, Renderer.bDepthTested);
	}

	for (const FSnapTargetPoint& Point : TargetPoints)
	{
		if (IsIgnored(Point.TargetID))
		{
			continue;
		}

		FLinearColor UseColor = FColor::Cyan;
		float LineWidth = (Point.TargetID == ActiveSnapID) ? Renderer.LineThickness : Renderer.LineThickness * 0.5;
		Renderer.DrawCircle(Point.Position, FVector3d(1, 0, 0), 3, 64,
			UseColor, LineWidth, Renderer.bDepthTested);
	}
}
