// Copyright Epic Games, Inc. All Rights Reserved.


#include "Snapping/RaySpatialSnapSolver.h"
#include "LineTypes.h"
#include "Distance/DistLine3Ray3.h"
#include "Distance/DistLine3Circle3.h"
#include "ToolDataVisualizer.h"

using namespace UE::Geometry;


FRaySpatialSnapSolver::FRaySpatialSnapSolver()
{
}



void FRaySpatialSnapSolver::GenerateTargetPoints(const FRay3d& Ray)
{
	GeneratedTargetPoints.Reset();

	// nearest-point-on-line snaps
	int NumSnapLines = TargetLines.Num();
	for (int j = 0; j < NumSnapLines; ++j)
	{
		FSnapTargetPoint Target;
		FDistLine3Ray3d DistQuery(TargetLines[j].Line, Ray);
		DistQuery.GetSquared();
		Target.Position = DistQuery.LineClosestPoint;

		Target.TargetID = TargetLines[j].TargetID;
		Target.Priority = TargetLines[j].Priority;
		Target.bIsSnapLine = true;
		Target.SnapLine = TargetLines[j].Line;

		// if we have a point constraint function, find constraint point and then project back onto line
		if (PointConstraintFunc != nullptr)
		{
			FVector3d SnapPos = PointConstraintFunc(Target.Position);
			Target.ConstrainedPosition = TargetLines[j].Line.NearestPoint(SnapPos);
			double PosT = TargetLines[j].Line.Project(Target.Position);
			double PosConstrainT = TargetLines[j].Line.Project(Target.ConstrainedPosition);
			// its usually weird to snap from A to B if they are on opposite sides of the origin, so clamp to origin in that case
			// (todo: maybe this should be optional...makes sense when using this for interactive gizmo snapping but maybe not in other cases)
			if (PosT * PosConstrainT < 0)
			{
				Target.ConstrainedPosition = TargetLines[j].Line.Origin;
			}
			Target.bHaveConstrainedPosition = true;
		}

		GeneratedTargetPoints.Add(Target);
	}


	// nearest-point-on-circle snaps
	int NumSnapCircles = TargetCircles.Num();
	FLine3d RayLine(Ray.Origin, Ray.Direction);
	for (int j = 0; j < NumSnapCircles; ++j)
	{
		FSnapTargetPoint Target;
		FDistLine3Circle3d DistQuery(RayLine, TargetCircles[j].Circle);
		DistQuery.GetSquared();
		if (DistQuery.bIsEquiDistant || DistQuery.NumClosestPairs != 1)
		{
			continue;
		}
		Target.Position = DistQuery.CircleClosest[0];
		Target.TargetID = TargetCircles[j].TargetID;
		Target.Priority = TargetCircles[j].Priority;
		Target.bIsSnapLine = false;
		//Target.bIsSnapLine = true;
		//Target.SnapLine = TargetCircles[j].Line;
		GeneratedTargetPoints.Add(Target);
	}
}


void FRaySpatialSnapSolver::UpdateSnappedPoint(const FRay3d& RayIn)
{
	double MinMetric = TNumericLimits<double>::Max();
	static constexpr int MIN_PRIORITY = TNumericLimits<int>::Max();
	int BestPriority = MIN_PRIORITY;
	const FSnapTargetPoint* BestSnapTarget = nullptr;

	// generate snap target point for each line
	// @todo it might be more efficient to just do these inline and avoid storing them?
	GenerateTargetPoints(RayIn);

	TFunction<FVector3d(const FVector3d&)> GetSnapFromPointFunc = [&RayIn](const FVector3d& Point)
	{
		return RayIn.ClosestPoint(Point);
	};

	const FSnapTargetPoint* BestPointTarget = FindBestSnapInSet(TargetPoints, MinMetric, BestPriority, GetSnapFromPointFunc);
	if (BestPointTarget != nullptr)
	{
		BestSnapTarget = BestPointTarget;
	}

	const FSnapTargetPoint* BestLineTarget = FindBestSnapInSet(GeneratedTargetPoints, MinMetric, BestPriority, GetSnapFromPointFunc);
	if (BestLineTarget != nullptr)
	{
		BestSnapTarget = BestLineTarget;
	}

	if (bHaveActiveSnap && bEnableStableSnap && ActiveSnapTarget.bIsSnapLine == false)
	{
		if (TestSnapTarget(ActiveSnapTarget, MinMetric*StableSnapImproveThresh, BestPriority, GetSnapFromPointFunc))
		{
			return;
		}
	}

	// if we found a best-target, update our snap details
	if (BestSnapTarget != nullptr)
	{
		// if we have a constrained position for a given snap we will use that instead. This is 
		// used for grid+line snapping for example. We want to snap to the grid *after* we snap to the
		// line, otherwise things snap to weird unexpected places.
		FVector3d UseSnapPosition = (BestSnapTarget->bHaveConstrainedPosition) ? BestSnapTarget->ConstrainedPosition : BestSnapTarget->Position;
		SetActiveSnapData(*BestSnapTarget, RayIn.ClosestPoint(BestSnapTarget->Position), UseSnapPosition, MinMetric);
	}
	else
	{
		ClearActiveSnapData();
	}
}




void FRaySpatialSnapSolver::Draw(FToolDataVisualizer* Renderer, float LineLength, TMap<int,FLinearColor>* ColorMap)
{
	int ActiveSnapID = (bHaveActiveSnap) ? GetActiveSnapTargetID() : -9999;

	for (const FSnapTargetLine& LineTarget : TargetLines)
	{
		if (IsIgnored(LineTarget.TargetID))
		{
			continue;
		}

		const FLine3d& Line = LineTarget.Line;

		const FLinearColor* LineColor = (ColorMap == nullptr) ? nullptr : ColorMap->Find(LineTarget.TargetID);
		FLinearColor UseColor = (LineColor == nullptr) ? Renderer->LineColor : *LineColor;
		float LineWidth = (LineTarget.TargetID == ActiveSnapID) ? Renderer->LineThickness : Renderer->LineThickness*0.5;
		Renderer->DrawLine(Line.PointAt(-LineLength), Line.PointAt(LineLength), UseColor, LineWidth);
	}


	for (const FSnapTargetCircle& CircleTarget : TargetCircles)
	{
		if (IsIgnored(CircleTarget.TargetID))
		{
			continue;
		}

		const FCircle3d& Circle = CircleTarget.Circle;

		const FLinearColor* LineColor = (ColorMap == nullptr) ? nullptr : ColorMap->Find(CircleTarget.TargetID);
		FLinearColor UseColor = (LineColor == nullptr) ? Renderer->LineColor : *LineColor;
		float LineWidth = (CircleTarget.TargetID == ActiveSnapID) ? Renderer->LineThickness : Renderer->LineThickness*0.5;
		Renderer->DrawCircle(Circle.GetCenter(), Circle.GetNormal(), (float)Circle.Radius, 64, 
			UseColor, LineWidth, Renderer->bDepthTested);
	}


	if (bHaveActiveSnap)
	{
		if (HaveActiveSnapLine())
		{
			const FLine3d& Line = GetActiveSnapLine();
			const FLinearColor* LineColor = (ColorMap == nullptr) ? nullptr : ColorMap->Find(ActiveSnapID);
			FLinearColor UseColor = (LineColor == nullptr) ? Renderer->LineColor : *LineColor;
			Renderer->DrawLine( (FVector3d)Line.PointAt(0), GetActiveSnapToPoint(), UseColor, Renderer->LineThickness*0.5);
		}
		Renderer->DrawPoint(GetActiveSnapToPoint());
	}
}