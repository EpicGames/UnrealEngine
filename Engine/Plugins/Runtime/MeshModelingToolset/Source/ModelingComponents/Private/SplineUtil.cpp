// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineUtil.h"

#include "Components/SplineComponent.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface
#include "SceneView.h"
#include "Styling/StyleColors.h"
#include "ToolContextInterfaces.h" // IToolsContextRenderAPI
#include "DynamicMesh/DynamicMeshAABBTree3.h"

UE::Geometry::SplineUtil::FDrawSplineSettings::FDrawSplineSettings()
	: RegularColor(FStyleColors::White.GetSpecifiedColor().ToFColor(true))
	, SelectedColor(FStyleColors::AccentOrange.GetSpecifiedColor().ToFColor(true))
{
}

// Mostly copied from the editor-only FSplineComponentVisualizer
void UE::Geometry::SplineUtil::DrawSpline(const USplineComponent& SplineComp, IToolsContextRenderAPI& RenderAPI, const FDrawSplineSettings& Settings)
{
	const FSceneView* View = RenderAPI.GetSceneView();
	auto GetDashSize = [View](const FVector& Start, const FVector& End, float Scale) -> double
	{
		const double StartW = View->WorldToScreen(Start).W;
		const double EndW = View->WorldToScreen(End).W;

		const double WLimit = 10.0f;
		if (StartW > WLimit || EndW > WLimit)
		{
			return FMath::Max(StartW, EndW) * Scale;
		}

		return 0;
	};

	FPrimitiveDrawInterface* PDI = RenderAPI.GetPrimitiveDrawInterface();

	const FInterpCurveVector& SplineInfo = SplineComp.GetSplinePointsPosition();

	const float GrabHandleSize = 10.0f;

	const bool bShouldVisualizeScale = Settings.ScaleVisualizationWidth > 0;
	const float DefaultScale = Settings.ScaleVisualizationWidth;

	FVector OldKeyPos(0);
	FVector OldKeyRightVector(0);
	FVector OldKeyScale(0);

	const int32 NumPoints = SplineInfo.Points.Num();
	const int32 NumSegments = SplineInfo.bIsLooped ? NumPoints : NumPoints - 1;
	for (int32 KeyIdx = 0; KeyIdx < NumSegments + 1; KeyIdx++)
	{
		const FVector NewKeyPos = SplineComp.GetLocationAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
		const FVector NewKeyRightVector = SplineComp.GetRightVectorAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
		const FVector NewKeyUpVector = SplineComp.GetUpVectorAtSplinePoint(KeyIdx, ESplineCoordinateSpace::World);
		const FVector NewKeyScale = SplineComp.GetScaleAtSplinePoint(KeyIdx) * DefaultScale;

		const FColor KeyColor = (Settings.SelectedKeys && Settings.SelectedKeys->Contains(KeyIdx)) ? Settings.SelectedColor
			: Settings.RegularColor;

		// Draw the keypoint and up/right vectors
		if (KeyIdx < NumPoints)
		{
			if (bShouldVisualizeScale)
			{
				PDI->DrawLine(NewKeyPos, NewKeyPos - NewKeyRightVector * NewKeyScale.Y, KeyColor, SDPG_Foreground);
				PDI->DrawLine(NewKeyPos, NewKeyPos + NewKeyRightVector * NewKeyScale.Y, KeyColor, SDPG_Foreground);
				PDI->DrawLine(NewKeyPos, NewKeyPos + NewKeyUpVector * NewKeyScale.Z, KeyColor, SDPG_Foreground);

				const int32 ArcPoints = 20;
				FVector OldArcPos = NewKeyPos + NewKeyRightVector * NewKeyScale.Y;
				for (int32 ArcIndex = 1; ArcIndex <= ArcPoints; ArcIndex++)
				{
					float Sin;
					float Cos;
					FMath::SinCos(&Sin, &Cos, ArcIndex * PI / ArcPoints);
					const FVector NewArcPos = NewKeyPos + Cos * NewKeyRightVector * NewKeyScale.Y + Sin * NewKeyUpVector * NewKeyScale.Z;
					PDI->DrawLine(OldArcPos, NewArcPos, KeyColor, SDPG_Foreground);
					OldArcPos = NewArcPos;
				}
			}

			PDI->DrawPoint(NewKeyPos, KeyColor, GrabHandleSize, SDPG_Foreground);
		}

		// If not the first keypoint, draw a line to the previous keypoint.
		if (KeyIdx > 0)
		{
			const FColor LineColor = Settings.RegularColor;

			// For constant interpolation - don't draw ticks - just draw dotted line.
			if (SplineInfo.Points[KeyIdx - 1].InterpMode == CIM_Constant)
			{
				const double DashSize = GetDashSize(OldKeyPos, NewKeyPos, 0.03f);
				if (DashSize > 0.0f)
				{
					DrawDashedLine(PDI, OldKeyPos, NewKeyPos, LineColor, DashSize, SDPG_World);
				}
			}
			else
			{
				// Determine the colors to use
				const bool bKeyIdxLooped = (SplineInfo.bIsLooped && KeyIdx == NumPoints);
				const int32 BeginIdx = bKeyIdxLooped ? 0 : KeyIdx;
				const int32 EndIdx = KeyIdx - 1;
				const bool bBeginSelected = Settings.SelectedKeys && Settings.SelectedKeys->Contains(BeginIdx);
				const bool bEndSelected = Settings.SelectedKeys && Settings.SelectedKeys->Contains(BeginIdx);
				const FColor BeginColor = (bBeginSelected) ? Settings.SelectedColor : Settings.RegularColor;
				const FColor EndColor = (bEndSelected) ? Settings.SelectedColor : Settings.RegularColor;

				// Find position on first keyframe.
				FVector OldPos = OldKeyPos;
				FVector OldRightVector = OldKeyRightVector;
				FVector OldScale = OldKeyScale;

				// Then draw a line for each substep.
				constexpr int32 NumSteps = 20;
				constexpr float PartialGradientProportion = 0.75f;
				constexpr int32 PartialNumSteps = (int32)(NumSteps * PartialGradientProportion);
				const float SegmentLineThickness = 0;

				for (int32 StepIdx = 1; StepIdx <= NumSteps; StepIdx++)
				{
					const float StepRatio = StepIdx / static_cast<float>(NumSteps);
					const float Key = EndIdx + StepRatio;
					const FVector NewPos = SplineComp.GetLocationAtSplineInputKey(Key, ESplineCoordinateSpace::World);
					const FVector NewRightVector = SplineComp.GetRightVectorAtSplineInputKey(Key, ESplineCoordinateSpace::World);
					const FVector NewScale = SplineComp.GetScaleAtSplineInputKey(Key) * DefaultScale;

					// creates a gradient that starts partway through the selection
					FColor StepColor;
					if (bBeginSelected == bEndSelected)
					{
						StepColor = BeginColor;
					}
					else if (bBeginSelected && StepIdx > (NumSteps - PartialNumSteps))
					{
						const float LerpRatio = (1.0f - StepRatio) / PartialGradientProportion;
						StepColor = FMath::Lerp(BeginColor.ReinterpretAsLinear(), EndColor.ReinterpretAsLinear(), LerpRatio).ToFColor(false);
					}
					else if (bEndSelected && StepIdx <= PartialNumSteps)
					{
						const float LerpRatio = 1.0f - (StepRatio / PartialGradientProportion);
						StepColor = FMath::Lerp(BeginColor.ReinterpretAsLinear(), EndColor.ReinterpretAsLinear(), LerpRatio).ToFColor(false);
					}
					else
					{
						StepColor = Settings.RegularColor; // unselected
					}

					PDI->DrawLine(OldPos, NewPos, StepColor, SDPG_Foreground, SegmentLineThickness);
					if (bShouldVisualizeScale)
					{
						PDI->DrawLine(OldPos - OldRightVector * OldScale.Y, NewPos - NewRightVector * NewScale.Y, LineColor, SDPG_Foreground);
						PDI->DrawLine(OldPos + OldRightVector * OldScale.Y, NewPos + NewRightVector * NewScale.Y, LineColor, SDPG_Foreground);

						constexpr bool bVisualizeSplineInterpolatedVectors = false;
						if (bVisualizeSplineInterpolatedVectors)
						{
							const FVector NewUpVector = SplineComp.GetUpVectorAtSplineInputKey(Key, ESplineCoordinateSpace::World);
							PDI->DrawLine(NewPos, NewPos + NewUpVector * Settings.ScaleVisualizationWidth * 0.5f, LineColor, SDPG_Foreground);
							PDI->DrawLine(NewPos, NewPos + NewRightVector * Settings.ScaleVisualizationWidth * 0.5f, LineColor, SDPG_Foreground);
						}
					}

					OldPos = NewPos;
					OldRightVector = NewRightVector;
					OldScale = NewScale;
				}
			}
		}

		OldKeyPos = NewKeyPos;
		OldKeyRightVector = NewKeyRightVector;
		OldKeyScale = NewKeyScale;
	}
}


void UE::Geometry::SplineUtil::ProjectSplineToSurface(FInterpCurveVector& OutputSpline, 
	const FInterpCurveVector& InputSpline,
	const UE::Geometry::FDynamicMeshAABBTree3& SurfaceAABBTree, 
	const FTransform& SplineTransform, 
	const FTransform& MeshTransform, 
	double DistanceThreshold,
	int32 MaxNewPoints)
{
	if (InputSpline.Points.Num() == 0)
	{
		return;
	}

	// Find the closest point on the surface given by SurfaceAABBTree to the spline point given by the parameter SplinePointT
	auto GetNearestPointOnSurface = [&SplineTransform, &MeshTransform, &SurfaceAABBTree](const FInterpCurveVector& Spline, double SplinePointT, int32* NearestTriangleID = nullptr) -> FVector3d
	{
		const FVector3d SplinePoint = Spline.Eval(SplinePointT, FVector3d(0));
		const FVector3d WorldSplinePoint = SplineTransform.TransformPosition(SplinePoint);
		const FVector3d MeshSpaceSplinePoint = MeshTransform.InverseTransformPosition(WorldSplinePoint);

		const FVector3d MeshSpaceNearestPoint = SurfaceAABBTree.FindNearestPoint(MeshSpaceSplinePoint);
		const FVector3d WorldNearestPoint = MeshTransform.TransformPosition(MeshSpaceNearestPoint);
		const FVector3d SplineNearestPoint = SplineTransform.InverseTransformPosition(WorldNearestPoint);

		if (NearestTriangleID != nullptr)
		{
			double DistToTriSq;
			*NearestTriangleID = SurfaceAABBTree.FindNearestTriangle(MeshSpaceSplinePoint, DistToTriSq);
		}

		return SplineNearestPoint;
	};

	// Find the closest point on the surface and return a new spline point there
	auto ProjectToSurface = [&SplineTransform, &MeshTransform, &SurfaceAABBTree, GetNearestPointOnSurface](const FInterpCurveVector& Spline, double SplinePointT, EInterpCurveMode InterpMode) -> FInterpCurvePointVector
	{
		int32 NearestTriangleID;
		const FVector3d SplineNearestPoint = GetNearestPointOnSurface(Spline, SplinePointT, &NearestTriangleID);

		FInterpCurvePointVector ProjectedPoint;
		ProjectedPoint.InVal = SplinePointT;
		ProjectedPoint.OutVal = SplineNearestPoint;
		ProjectedPoint.InterpMode = InterpMode;

		if (ensure(SurfaceAABBTree.GetMesh()->IsTriangle(NearestTriangleID)))
		{
			// Spline point tangent = spline derivative projected onto the tangent plane of the surface
			const FVector3d SplineDerivative = Spline.EvalDerivative(SplinePointT, FVector3d(0));
			const FVector3d TriangleNormal = SurfaceAABBTree.GetMesh()->GetTriNormal(NearestTriangleID);
			const FVector3d Tangent = SplineDerivative - SplineDerivative.Dot(TriangleNormal) * TriangleNormal;
			ProjectedPoint.ArriveTangent = 0.5 * Tangent;
			ProjectedPoint.LeaveTangent = 0.5 * Tangent;
		}

		return ProjectedPoint;
	};


	// Sample the full spline and finds the point with greatest error (distance to surface)
	auto ApproximationError = [&SplineTransform, &MeshTransform, &SurfaceAABBTree, GetNearestPointOnSurface](const FInterpCurveVector& Spline, int32 NumSamplesPerSegment = 10) -> TPair<double, double>
	{
		NumSamplesPerSegment = FMath::Max(NumSamplesPerSegment, 1);

		double MaxError = -1.0;
		double MaxErrorT = -1.0;

		for (int32 PointIndex = 0; PointIndex < Spline.Points.Num() - 1; ++PointIndex)
		{
			const int32 NextPointIndex = PointIndex + 1;

			const double MinT = Spline.Points[PointIndex].InVal;
			const double MaxT = Spline.Points[NextPointIndex].InVal;
			const double DeltaT = (MaxT - MinT) / (NumSamplesPerSegment + 1);

			for (int32 SampleIndex = 0; SampleIndex < NumSamplesPerSegment; ++SampleIndex)
			{
				const double T = MinT + DeltaT * (SampleIndex + 1);
				const FVector SplinePoint = Spline.Eval(T, FVector(0));
				const FVector3d SplineNearestPoint = GetNearestPointOnSurface(Spline, T);
				const double Error = FVector3d::Distance(SplineNearestPoint, SplinePoint);

				if (MaxError < Error)
				{
					MaxErrorT = T;
					MaxError = Error;
				}
			}
		}

		return { MaxErrorT, MaxError };
	};


	// Initialize OutputSpline using input points

	const double MaxT = InputSpline.Points.Last().InVal;
	double AvgSegmentLength = 0.0;
	for (int32 PointIndex = 0; PointIndex < InputSpline.Points.Num(); ++PointIndex)
	{
		const FInterpCurvePointVector& Point = InputSpline.Points[PointIndex];
		const FInterpCurvePointVector ProjectedPoint = ProjectToSurface(InputSpline, Point.InVal, Point.InterpMode);
		OutputSpline.Points.Add(ProjectedPoint);
	}

	// Now iteratively insert points at the max-error sample along the spline

	TPair<double, double> TAndError = ApproximationError(OutputSpline);
	double MaxErrorT = TAndError.Key;
	double MaxError = TAndError.Value;

	for (int32 InsertIteration = 0; InsertIteration < MaxNewPoints; ++InsertIteration)
	{
		if (MaxError < DistanceThreshold)
		{
			break;
		}

		const FInterpCurvePointVector ProjectedPoint = ProjectToSurface(OutputSpline, MaxErrorT, EInterpCurveMode::CIM_CurveAuto);

		const int32 NewPointIndex = OutputSpline.AddPoint(MaxErrorT, ProjectedPoint.OutVal);
		OutputSpline.Points[NewPointIndex] = ProjectedPoint;

		TAndError = ApproximationError(OutputSpline);
		MaxErrorT = TAndError.Key;
		MaxError = TAndError.Value;

		if (MaxErrorT <= 0.0 || MaxErrorT >= OutputSpline.Points.Last().InVal)
		{
			break;
		}
	}
}
