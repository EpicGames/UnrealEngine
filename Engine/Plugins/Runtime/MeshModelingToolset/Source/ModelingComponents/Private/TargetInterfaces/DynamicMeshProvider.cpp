// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetInterfaces/DynamicMeshProvider.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshProvider)

#define LOCTEXT_NAMESPACE "DynamicMeshProvider"

using namespace UE::Geometry;

FDynamicMesh3 IDynamicMeshProvider::GetDynamicMesh(bool bRequestTangents)
{
	FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = bRequestTangents;
	return GetDynamicMesh(GetMeshParams);
}

FDynamicMesh3 IDynamicMeshProvider::GetDynamicMesh(const FGetMeshParameters& InGetMeshParams)
{
	return GetDynamicMesh();
}

#undef LOCTEXT_NAMESPACE 
