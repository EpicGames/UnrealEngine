// Copyright Epic Games, Inc. All Rights Reserved.

#include <numeric>
#include "ImageCoreUtils.h"
#include "RenderUtils.h"
#include "Algo/ForEach.h"
#include "AssetUtils/Texture2DUtil.h"
#include "Engine/Texture2D.h"
#include "Logging/MessageLog.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

namespace AssetUtilsTestsLocal
{

int64 GetNumBytes(const int32 Width, const int32 Height, const ETextureSourceFormat SourceFormat)
{
	check(TSF_Invalid < SourceFormat && SourceFormat < TSF_MAX);
	return static_cast<int64>(Width) * static_cast<int64>(Height) * GTextureSourceFormats[SourceFormat].BytesPerPixel;
}

bool FormatUsesUInt8(const ETextureSourceFormat SourceFormat)
{
	return SourceFormat == TSF_G8 || SourceFormat == TSF_BGRA8 || SourceFormat == TSF_BGRE8;
}

bool FormatUsesUInt16(const ETextureSourceFormat SourceFormat)
{
	return SourceFormat == TSF_RGBA16 || SourceFormat == TSF_G16;
}

bool FormatUsesFloat(const ETextureSourceFormat SourceFormat)
{
	return SourceFormat == TSF_RGBA32F || SourceFormat == TSF_R32F;
}

bool FormatUsesFloat16(const ETextureSourceFormat SourceFormat)
{
	return SourceFormat == TSF_RGBA16F || SourceFormat == TSF_R16F;
}

TArray64<uint8> GetRandomPixels(const int32 Width, const int32 Height, const ETextureSourceFormat SourceFormat, uint32 Seed)
{
	const FRandomStream Stream(Seed);

	const int64 NumBytes = GetNumBytes(Width, Height, SourceFormat);
	TArray64<uint8> Bytes;
	Bytes.SetNumUninitialized(NumBytes);

	if (FormatUsesUInt8(SourceFormat))
	{
		for (int64 Index = 0; Index < NumBytes; ++Index)
		{
			// Assign value in [0..255]
			Bytes[Index] = static_cast<uint8>(Stream.RandRange(0, 255));
		}
	}
	else if (FormatUsesUInt16(SourceFormat))
	{
		uint16* UInt16s = reinterpret_cast<uint16*>(Bytes.GetData());
		for (int64 Index = 0, Num = NumBytes / sizeof(uint16); Index < Num; ++Index)
		{
			// Assign value in [0..65535]
			UInt16s[Index] = static_cast<uint16>(Stream.RandRange(0, 65535));
		}
	}
	else if (FormatUsesFloat(SourceFormat))
	{
		float* Floats = reinterpret_cast<float*>(Bytes.GetData());
		for (int64 Index = 0, Num = NumBytes / sizeof(float); Index < Num; ++Index)
		{
			// Assign value in [0..1024]
			Floats[Index] = Stream.RandRange(0, 1024 * 1024) / 1024.0f;
		}
	}
	else if (FormatUsesFloat16(SourceFormat))
	{
		FFloat16* HalfFloats = reinterpret_cast<FFloat16*>(Bytes.GetData());
		for (int64 Index = 0, Num = NumBytes / sizeof(FFloat16); Index < Num; ++Index)
		{
			// Assign value in [0..1024]
			HalfFloats[Index] = Stream.RandRange(0, 1024 * 1024) / 1024.0f;
		}
	}
	else
	{
		ensure(false);
	}

	return Bytes;
}

EPixelFormat GetPixelFormat(const ETextureSourceFormat SourceFormat)
{
	check(TSF_Invalid < SourceFormat && SourceFormat < TSF_MAX);
	return GTextureSourceFormats[SourceFormat].PixelFormat;
}

FImage GetImage(const int32 Width, const int32 Height, const ERawImageFormat::Type RawFormat, const bool bSRGB, const TArray64<uint8>& Pixels)
{
	FImage Image(Width, Height, RawFormat, bSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear);
	Image.RawData = Pixels;
	return Image;
}

TStrongObjectPtr<UTexture2D> GetTexture(const int32 Width, const int32 Height, const ETextureSourceFormat SourceFormat, const bool bSRGB, const TArray64<uint8>& Pixels)
{
	check(0 < Width && 0 < Height);
	check(TSF_Invalid < SourceFormat && SourceFormat < TSF_MAX);
	check(Pixels.Num() == GetNumBytes(Width, Height, SourceFormat));

	TStrongObjectPtr Texture(UTexture2D::CreateTransient(Width, Height, GetPixelFormat(SourceFormat)));
	if (Texture)
	{
		Texture->Source.Init(Width, Height, 1, 1, SourceFormat, Pixels.GetData());

		Texture->MipGenSettings = TMGS_NoMipmaps;
		Texture->CompressionSettings = TC_Default;
		Texture->SRGB = bSRGB;

		Texture->PreEditChange(nullptr);
		Texture->PostEditChange();
		Texture->FinishCachePlatformData();
	}

	return Texture;
}

}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetUtilsReadTextureTest,
                                 "MeshModeling.AssetUtils.ReadTexture",
                                 EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FAssetUtilsReadTextureTest::RunTest(const FString& Parameters)
{
	using namespace AssetUtilsTestsLocal;

	constexpr int32 Width = 512;
	constexpr int32 Height = 256;
	constexpr int64 NumPixels = Width * Height;
	constexpr uint32 Seed = 0;

	// Set this to true to time 100 iterations for each test, and print the results summary in the message log.
	constexpr bool bRunBenchmark = false;
	constexpr uint32 BenchmarkNumIterations = 100;

	// The current implementation for platform data converts the texture to TC_VectorDisplacementmap, which seems to introduce all kinds of artifacts that make
	// a comparison to the original source data intractable.
	constexpr bool bPreferPlatformData = false;

	ETextureSourceFormat SourceFormats[TSF_MAX - 1];
	Algo::ForEach(SourceFormats, [Value = TSF_Invalid + 1](ETextureSourceFormat& Format) mutable
	{
		Format = static_cast<ETextureSourceFormat>(Value++);
	});

	ETextureSourceFormat UnsupportedFormats[] = {
		TSF_RGBA8_DEPRECATED,
		TSF_RGBE8_DEPRECATED
	};

	FMessageLog MessageLog("AutomationTestingLog");

	for (const auto SourceFormat : SourceFormats)
	{
		if (Algo::Find(UnsupportedFormats, SourceFormat))
		{
			// Format not supported.
			continue;
		}

		const ERawImageFormat::Type RawImageFormat = FImageCoreUtils::ConvertToRawImageFormat(SourceFormat);
		if (!TestTrue(FString::Format(TEXT("Format {1} is a valid raw image format."), {GTextureSourceFormats[SourceFormat].Name}),
		              RawImageFormat != ERawImageFormat::Invalid))
		{
			// Invalid raw image format.
			continue;
		}

		const TArray<bool> OptionsSRGB = [RawImageFormat]
		{
			TArray<bool> Options({false});
			if (ERawImageFormat::GetFormatNeedsGammaSpace(RawImageFormat))
			{
				Options.Add(true);
			}
			return Options;
		}();

		const TArray64<uint8> Pixels = GetRandomPixels(Width, Height, SourceFormat, Seed);

		for (const bool bIsSRGB : OptionsSRGB)
		{
			auto What = [SourceFormat, bIsSRGB, bPreferPlatformData](FString Step)
			{
				return FString::Format(TEXT("{0} ({1}{2}{3})"), {
					                       Step,
					                       GTextureSourceFormats[SourceFormat].Name,
					                       bIsSRGB ? TEXT(", SRGB") : TEXT(""),
					                       bPreferPlatformData ? TEXT(", PreferPlatformData") : TEXT("")
				                       });
			};

			auto IsEqual = [](const FLinearColor& A, const FVector4f& B, const float Threshold)
			{
				return FMath::Abs(A.R - B.X) < Threshold
					&& FMath::Abs(A.G - B.Y) < Threshold
					&& FMath::Abs(A.B - B.Z) < Threshold
					&& FMath::Abs(A.A - B.W) < Threshold;
			};

			auto SetMaxError = [](float& MaxError, const FLinearColor& A, const FVector4f& B)
			{
				MaxError = FMath::Max(MaxError, FMath::Abs(A.R - B.X));
				MaxError = FMath::Max(MaxError, FMath::Abs(A.G - B.Y));
				MaxError = FMath::Max(MaxError, FMath::Abs(A.B - B.Z));
				MaxError = FMath::Max(MaxError, FMath::Abs(A.A - B.W));
			};

			const TStrongObjectPtr<UTexture2D> Texture = GetTexture(Width, Height, SourceFormat, bIsSRGB, Pixels);
			const FImage Image = GetImage(Width, Height, RawImageFormat, bIsSRGB, Pixels);

			UE::Geometry::TImageBuilder<FVector4f> Output;
			const bool bReadTextureSuccess = UE::AssetUtils::ReadTexture(Texture.Get(), Output, bPreferPlatformData);
			if (TestTrue(What("ReadTexture"), bReadTextureSuccess))
			{
				if (TestTrue(What("Correct Output Size"), NumPixels == Output.GetDimensions().Num()))
				{
					const FImage ImageRGBA32F = [&Image]
					{
						FImage ImageCopy;
						Image.CopyTo(ImageCopy, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
						return ImageCopy;
					}();

					const TArrayView64<const FLinearColor> ImageView = ImageRGBA32F.AsRGBA32F();

					int64 NumErrors = 0;
					float MaxError = 0.0f;
					for (int64 Index = 0, Num = Output.GetDimensions().Num(); Index < Num; ++Index)
					{
						const bool bIsEqual = IsEqual(ImageView[Index], Output.GetPixel(Index), 1.e-7f);
						NumErrors += !bIsEqual;
						SetMaxError(MaxError, ImageView[Index], Output.GetPixel(Index));
					}

					if (!TestTrue(What("Check Pixels"), NumErrors == 0))
					{
						MessageLog.Warning(FText::FromString(FString::Printf(
							TEXT("%s - Num Errors: %lld (%.2f%%) - Max Error: %.10f"),
							*What("Check Pixels"), NumErrors, static_cast<float>(NumErrors) / NumPixels * 100.0f, MaxError)));
					}
				}

				if constexpr (bRunBenchmark)
				{
					TArray<double> Timings;
					Timings.Reserve(BenchmarkNumIterations);
					for (int32 Iteration = 0; Iteration < BenchmarkNumIterations; ++Iteration)
					{
						const uint64 StartCycles = FPlatformTime::Cycles64();
						const bool bSuccess = UE::AssetUtils::ReadTexture(Texture.Get(), Output, bPreferPlatformData);
						const uint64 EndCycles = FPlatformTime::Cycles64();

						if (bSuccess)
						{
							Timings.Add(FPlatformTime::ToMilliseconds64(EndCycles - StartCycles) * 1000.0);
						}
					}
					if (Timings.Num())
					{
						Timings.Sort();
						MessageLog.Info(FText::FromString(FString::Printf(
							TEXT("%s Min: %.1f us, Median: %.1f us, Max: %.1f us"),
							*What("Benchmark"), Timings[0], Timings[Timings.Num() / 2], Timings[Timings.Num() - 1])));
					}
					else
					{
						MessageLog.Error(FText::FromString(FString::Printf(TEXT("%s failed!"), *What("Benchmark"))));
					}
				}
			}
		}
	}

	return true;
}

#endif
#endif
