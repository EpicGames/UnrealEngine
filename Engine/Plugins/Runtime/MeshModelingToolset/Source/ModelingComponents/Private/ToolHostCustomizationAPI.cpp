// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolHostCustomizationAPI.h"

#include "ContextObjectStore.h"
#include "InteractiveToolManager.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolHostCustomizationAPI)

TScriptInterface<IToolHostCustomizationAPI> IToolHostCustomizationAPI::Find(UInteractiveToolManager* ToolManager)
{
	if (!ensure(ToolManager))
	{
		return nullptr;
	}

	UContextObjectStore* ContextObjectStore = ToolManager->GetContextObjectStore();
	if (!ensure(ContextObjectStore))
	{
		return nullptr;
	}

	return ContextObjectStore->FindContextByClass(UToolHostCustomizationAPI::StaticClass());
}
