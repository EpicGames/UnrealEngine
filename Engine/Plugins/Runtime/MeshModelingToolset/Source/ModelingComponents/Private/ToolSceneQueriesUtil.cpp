// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolSceneQueriesUtil.h"
#include "Engine/World.h"
#include "VectorUtil.h"
#include "Quaternion.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "SceneQueries/SceneSnappingManager.h"


static double VISUAL_ANGLE_SNAP_THRESHOLD_DEG = 1.0;

double ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD()
{
	return VISUAL_ANGLE_SNAP_THRESHOLD_DEG;
}


bool ToolSceneQueriesUtil::PointSnapQuery(const UInteractiveTool* Tool, const FVector3d& Point1, const FVector3d& Point2, double VisualAngleThreshold)
{
	IToolsContextQueriesAPI* QueryAPI = Tool->GetToolManager()->GetContextQueriesAPI();
	FViewCameraState CameraState;
	QueryAPI->GetCurrentViewState(CameraState);
	return PointSnapQuery(CameraState, Point1, Point2, VisualAngleThreshold);
}

bool ToolSceneQueriesUtil::PointSnapQuery(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2, double VisualAngleThreshold)
{
	if (!CameraState.bIsOrthographic)
	{
		double UseThreshold = (VisualAngleThreshold <= 0) ? GetDefaultVisualAngleSnapThreshD() : VisualAngleThreshold;
		UseThreshold *= CameraState.GetFOVAngleNormalizationFactor();
		double VisualAngle = VectorUtil::OpeningAngleD(Point1, Point2, (FVector3d)CameraState.Position);
		return FMathd::Abs(VisualAngle) < UseThreshold;
	}
	else
	{
		// Whereas in perspective mode we can compare the angle difference to the camera, we can't do that in ortho mode, since the camera isn't a point
		// but a plane. Instead we need to project into the camera plane and measure distance here. To be analogous to our tolerance in perspective mode,
		// where we divide the FOV into 90 visual angle degrees, we divide the plane into 90 segments and use the same tolerance.
		double AngleThreshold = (VisualAngleThreshold <= 0) ? GetDefaultVisualAngleSnapThreshD() : VisualAngleThreshold;
		double OrthoThreshold = AngleThreshold * CameraState.OrthoWorldCoordinateWidth / 90.0;
		FVector3d ViewPlaneNormal = (FVector3d)CameraState.Orientation.GetForwardVector();
		FVector3d DistanceVector = Point1 - Point2;

		// Project the vector into the plane and check its length
		DistanceVector = DistanceVector - (DistanceVector).Dot(ViewPlaneNormal) * ViewPlaneNormal;
		return DistanceVector.SquaredLength() < (OrthoThreshold * OrthoThreshold);
	}
}

double ToolSceneQueriesUtil::PointSnapMetric(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2)
{
	if (!CameraState.bIsOrthographic)
	{
		double VisualAngle = VectorUtil::OpeningAngleD(Point1, Point2, (FVector3d)CameraState.Position);

		// To go from a world space angle to a 90 degree division of the view, we divide by TrueFOVDegrees/90 (our normalization factor)
		VisualAngle /= CameraState.GetFOVAngleNormalizationFactor();
		return FMathd::Abs(VisualAngle);
	}
	else
	{
		FVector3d ViewPlaneNormal = (FVector3d)CameraState.Orientation.GetForwardVector();

		// Get projected distance in the plane
		FVector3d DistanceVector = Point1 - Point2;
		DistanceVector = DistanceVector - (DistanceVector).Dot(ViewPlaneNormal) * ViewPlaneNormal;

		// We have one visual angle degree correspond to the width of the viewport divided by 90, so we divide by width/90.
		return DistanceVector.Length() * 90.0 / CameraState.OrthoWorldCoordinateWidth;
	}
}


double ToolSceneQueriesUtil::CalculateViewVisualAngleD(const UInteractiveTool* Tool, const FVector3d& Point1, const FVector3d& Point2)
{
	IToolsContextQueriesAPI* QueryAPI = Tool->GetToolManager()->GetContextQueriesAPI();
	FViewCameraState CameraState;
	QueryAPI->GetCurrentViewState(CameraState);
	return CalculateViewVisualAngleD(CameraState, Point1, Point2);
}

double ToolSceneQueriesUtil::CalculateViewVisualAngleD(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2)
{
	double VisualAngle = VectorUtil::OpeningAngleD(Point1, Point2, (FVector3d)CameraState.Position);
	return FMathd::Abs(VisualAngle);
}

double ToolSceneQueriesUtil::CalculateNormalizedViewVisualAngleD(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2)
{
	double VisualAngle = VectorUtil::OpeningAngleD(Point1, Point2, (FVector3d)CameraState.Position);
	double FOVNormalization = CameraState.GetFOVAngleNormalizationFactor();
	return FMathd::Abs(VisualAngle) / FOVNormalization;
}




double ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(const UInteractiveTool* Tool, const FVector3d& Point, double TargetVisualAngleDeg)
{
	IToolsContextQueriesAPI* QueryAPI = Tool->GetToolManager()->GetContextQueriesAPI();
	FViewCameraState CameraState;
	QueryAPI->GetCurrentViewState(CameraState);
	return CalculateDimensionFromVisualAngleD(CameraState, Point, TargetVisualAngleDeg);
}
double ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(const FViewCameraState& CameraState, const FVector3d& Point, double TargetVisualAngleDeg)
{
	FVector3d EyePos = (FVector3d)CameraState.Position;
	FVector3d PointVec = Point - EyePos;
	TargetVisualAngleDeg *= CameraState.GetFOVAngleNormalizationFactor();
	FVector3d RotPointPos = EyePos + FQuaterniond((FVector3d)CameraState.Up(), TargetVisualAngleDeg, true)*PointVec;
	double ActualAngleDeg = CalculateViewVisualAngleD(CameraState, Point, RotPointPos);
	return Distance(Point, RotPointPos) * (TargetVisualAngleDeg/ActualAngleDeg);
}



bool ToolSceneQueriesUtil::IsPointVisible(const FViewCameraState& CameraState, const FVector3d& Point)
{
	if (CameraState.bIsOrthographic == false)
	{
		FVector3d PointDir = (Point - (FVector3d)CameraState.Position);
		//@todo should use view frustum here!
		if (PointDir.Dot((FVector3d)CameraState.Forward()) < 0.25)		// ballpark estimate
		{
			return false;
		}
	}
	else
	{
		// @todo probably not always true but it's not exactly clear how ortho camera is configured...
		return true;
	}
	return true;
}

bool ToolSceneQueriesUtil::FindSceneSnapPoint(const UInteractiveTool* Tool, const FVector3d& Point, FVector3d& SnapPointOut,
	bool bVertices, bool bEdges, double VisualAngleThreshold,
	FSnapGeometry* SnapGeometry, FVector* DebugTriangleOut)
{
	FFindSceneSnapPointParams Params;

	Params.SnappingManager = USceneSnappingManager::Find(Tool->GetToolManager());
	Tool->GetToolManager()->GetContextQueriesAPI()->GetCurrentViewState(Params.CameraState);
	Params.Point = &Point;
	Params.SnapPointOut = &SnapPointOut;
	Params.bVertices = bVertices;
	Params.bEdges = bEdges;
	Params.VisualAngleThreshold = VisualAngleThreshold;
	Params.SnapGeometryOut = SnapGeometry;
	Params.DebugTriangleOut = DebugTriangleOut;

	return FindSceneSnapPoint(Params);
}

bool ToolSceneQueriesUtil::FindSceneSnapPoint(FFindSceneSnapPointParams& Params)
{
	double UseThreshold = (Params.VisualAngleThreshold <= 0) ? GetDefaultVisualAngleSnapThreshD() : Params.VisualAngleThreshold;

	if (!Params.SnappingManager)
	{
		return false;
	}

	UseThreshold *= Params.CameraState.GetFOVAngleNormalizationFactor();

	FSceneSnapQueryRequest Request;
	Request.RequestType = ESceneSnapQueryType::Position;
	Request.TargetTypes = ESceneSnapQueryTargetType::None;
	if (Params.bVertices)
	{
		Request.TargetTypes |= ESceneSnapQueryTargetType::MeshVertex;
	}
	if (Params.bEdges)
	{
		Request.TargetTypes |= ESceneSnapQueryTargetType::MeshEdge;
	}
	Request.Position = (FVector)*Params.Point;
	Request.VisualAngleThresholdDegrees = UseThreshold;
	Request.ComponentsToIgnore = Params.ComponentsToIgnore;
	Request.InvisibleComponentsToInclude = Params.InvisibleComponentsToInclude;

	TArray<FSceneSnapQueryResult> Results;
	if (Params.SnappingManager->ExecuteSceneSnapQuery(Request, Results))
	{
		*Params.SnapPointOut = (FVector3d)Results[0].Position;

		if (Params.SnapGeometryOut != nullptr)
		{
			int iSnap = Results[0].TriSnapIndex;
			Params.SnapGeometryOut->Points[0] = (FVector3d)Results[0].TriVertices[iSnap];
			Params.SnapGeometryOut->PointCount = 1;
			if (Results[0].TargetType == ESceneSnapQueryTargetType::MeshEdge)
			{
				Params.SnapGeometryOut->Points[1] = (FVector3d)Results[0].TriVertices[(iSnap+1)%3];
				Params.SnapGeometryOut->PointCount = 2;
			}
		}

		if (Params.DebugTriangleOut != nullptr)
		{
			Params.DebugTriangleOut[0] = Results[0].TriVertices[0];
			Params.DebugTriangleOut[1] = Results[0].TriVertices[1];
			Params.DebugTriangleOut[2] = Results[0].TriVertices[2];
		}

		return true;
	}
	return false;
}


bool ToolSceneQueriesUtil::FindWorldGridSnapPoint(const UInteractiveTool* Tool, const FVector3d& Point, FVector3d& GridSnapPointOut)
{
	USceneSnappingManager* SnapManager = USceneSnappingManager::Find(Tool->GetToolManager());
	if (!SnapManager)
	{
		return false;
	}

	FSceneSnapQueryRequest Request;
	Request.RequestType = ESceneSnapQueryType::Position;
	Request.TargetTypes = ESceneSnapQueryTargetType::Grid;
	Request.Position = (FVector)Point;
	TArray<FSceneSnapQueryResult> Results;
	if ( SnapManager->ExecuteSceneSnapQuery(Request, Results) )
	{
		GridSnapPointOut = (FVector3d)Results[0].Position;
		return true;
	};
	return false;
}

double ToolSceneQueriesUtil::SnapDistanceToWorldGridSize(const UInteractiveTool* Tool, const double Distance)
{
	IToolsContextQueriesAPI* QueryAPI = Tool->GetToolManager()->GetContextQueriesAPI();
	FToolContextSnappingConfiguration SnapConfig = QueryAPI->GetCurrentSnappingSettings();

	if (SnapConfig.IsPositionGridSnappingActive())
	{
		double DX = SnapConfig.PositionGridDimensions.GetMax();
		if (DX > 0.0)
		{
			int N = FMath::RoundToInt(Distance / DX);
			return N * DX;
		}
	}

	return Distance;
}




bool ToolSceneQueriesUtil::IsVisibleObjectHit(const FHitResult& HitResult)
{
	AActor* Actor = HitResult.GetActor();
	if (Actor != nullptr)
	{
#if WITH_EDITOR
		if (Actor->IsHidden() || Actor->IsHiddenEd())
		{
			return false;
		}
#else
		if (Actor->IsHidden())
		{
			return false;
		}
#endif
	}

	UPrimitiveComponent* Component = HitResult.GetComponent();
	if (Component != nullptr)
	{
#if WITH_EDITOR
		if (Component->IsVisible() == false && Component->IsVisibleInEditor() == false)
		{
			return false;
		}
#else
		if (Component->IsVisible() == false)
		{
			return false;
		}
#endif
	}

	return true;
}




bool ToolSceneQueriesUtil::FindNearestVisibleObjectHit(UWorld* World, FHitResult& HitResultOut, const FVector& Start, const FVector& End,
	const TArray<const UPrimitiveComponent*>* IgnoreComponents, const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)
{
	FCollisionObjectQueryParams ObjectQueryParams(FCollisionObjectQueryParams::AllObjects);
	FCollisionQueryParams QueryParams = FCollisionQueryParams::DefaultQueryParam;
	QueryParams.bTraceComplex = true;

	TArray<FHitResult> OutHits;
	if (World->LineTraceMultiByObjectType(OutHits, Start, End, ObjectQueryParams, QueryParams) == false)
	{
		return false;
	}

	float NearestVisible = TNumericLimits<float>::Max();
	for (const FHitResult& CurResult : OutHits)
	{
		if (CurResult.Distance < NearestVisible)
		{
			if (IsVisibleObjectHit(CurResult) 
				|| (InvisibleComponentsToInclude && InvisibleComponentsToInclude->Contains(CurResult.GetComponent())))
			{
				if (IgnoreComponents == nullptr || IgnoreComponents->Contains(CurResult.GetComponent()) == false)
				{
					HitResultOut = CurResult;
					NearestVisible = CurResult.Distance;
				}
			}
		}
	}

	return NearestVisible < TNumericLimits<float>::Max();
}


bool ToolSceneQueriesUtil::FindNearestVisibleObjectHit(UWorld* World, FHitResult& HitResultOut, const FRay& Ray,
	const TArray<const UPrimitiveComponent*>* IgnoreComponents, const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)
{
	return FindNearestVisibleObjectHit(World, HitResultOut, Ray.Origin, Ray.PointAt(HALF_WORLD_MAX), IgnoreComponents, InvisibleComponentsToInclude);
}




bool ToolSceneQueriesUtil::FindNearestVisibleObjectHit(USceneSnappingManager* SnapManager, FHitResult& HitResultOut, const FRay& Ray,
	const TArray<const UPrimitiveComponent*>* IgnoreComponents, const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)
{
	if (!SnapManager)
	{
		return false;
	}

	FSceneHitQueryRequest Request;
	Request.WorldRay = (FRay3d)Ray;
	Request.bWantHitGeometryInfo = false;
	Request.VisibilityFilter.ComponentsToIgnore = IgnoreComponents;
	Request.VisibilityFilter.InvisibleComponentsToInclude = InvisibleComponentsToInclude;

	FSceneHitQueryResult Result;
	if ( SnapManager->ExecuteSceneHitQuery(Request, Result) )
	{
		HitResultOut = Result.HitResult;
		return true;
	};
	return false;
}



bool ToolSceneQueriesUtil::FindNearestVisibleObjectHit(const UInteractiveTool* Tool, FHitResult& HitResultOut, const FVector& Start, const FVector& End,
	const TArray<const UPrimitiveComponent*>* IgnoreComponents, const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)
{
	FRay WorldRay(Start, (End - Start), false);
	return FindNearestVisibleObjectHit(USceneSnappingManager::Find(Tool->GetToolManager()), HitResultOut, WorldRay, IgnoreComponents, InvisibleComponentsToInclude);
}


bool ToolSceneQueriesUtil::FindNearestVisibleObjectHit(const UInteractiveTool* Tool, FHitResult& HitResultOut, const FRay& Ray,
	const TArray<const UPrimitiveComponent*>* IgnoreComponents, const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)
{
	return FindNearestVisibleObjectHit(USceneSnappingManager::Find(Tool->GetToolManager()), HitResultOut, Ray, IgnoreComponents, InvisibleComponentsToInclude);
}