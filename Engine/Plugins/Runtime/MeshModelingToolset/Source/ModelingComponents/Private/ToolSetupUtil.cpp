// Copyright Epic Games, Inc. All Rights Reserved.


#include "ToolSetupUtil.h"
#include "ModelingComponentsSettings.h"
#include "Curves/CurveFloat.h"
#include "FrameTypes.h"
#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Components/BaseDynamicMeshComponent.h"
#include "PreviewMesh.h"

#include "SceneQueries/SceneSnappingManager.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "ToolContextInterfaces.h"
#include "ToolSceneQueriesUtil.h"

UMaterialInterface* ToolSetupUtil::GetDefaultMaterial()
{
	return UMaterial::GetDefaultMaterial(MD_Surface);
}


UMaterialInterface* ToolSetupUtil::GetDefaultMaterial(UInteractiveToolManager* ToolManager, UMaterialInterface* SourceMaterial)
{
	if (SourceMaterial == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return SourceMaterial;
}


UMaterialInstanceDynamic* ToolSetupUtil::GetVertexColorMaterial(UInteractiveToolManager* ToolManager, bool bTwoSided)
{
	// Unfortunately the two-sided flag is not something that we can give as a runtime 
	// parameter, so we need separate versions of the material.
	UMaterial* Material = bTwoSided ? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/MeshVertexColorMaterialTwoSided"))
		: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/MeshVertexColorMaterial"));

	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		return MatInstance;
	}
	return nullptr;
}

UMaterialInterface* ToolSetupUtil::GetVertexColorSculptMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/MeshVertexColorSculptMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}


UMaterialInterface* ToolSetupUtil::GetDefaultWorkingMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/InProgressMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}


UMaterialInstanceDynamic* ToolSetupUtil::GetDefaultWorkingMaterialInstance(UInteractiveToolManager* ToolManager)
{
	UMaterialInstanceDynamic* ResultMaterial = nullptr;
	if (UMaterialInterface* Material = GetDefaultWorkingMaterial(ToolManager))
	{
		ResultMaterial = UMaterialInstanceDynamic::Create(Material, ToolManager);
	}
	return ResultMaterial;
}


UMaterialInstanceDynamic* ToolSetupUtil::GetDefaultErrorMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterialInstanceDynamic* ResultMaterial = nullptr;
	if (UMaterialInterface* Material = GetDefaultWorkingMaterial(ToolManager))
	{
		static constexpr FLinearColor ErrorBrightColor(0.8f, 0.01552f, 0.01552f);
		static constexpr FLinearColor ErrorDarkColor(0.3386144f, 0.0141136f, 0.0141136f);
		static constexpr float ErrorTimeFactor = 2.0f;

		ResultMaterial = UMaterialInstanceDynamic::Create(Material, ToolManager);
		ResultMaterial->SetVectorParameterValue(TEXT("BrightColor"), ErrorBrightColor);
		ResultMaterial->SetVectorParameterValue(TEXT("DarkColor"), ErrorDarkColor);
		ResultMaterial->SetScalarParameterValue(TEXT("TimeFactor"), ErrorTimeFactor);
	}
	return ResultMaterial;
}


UMaterialInstanceDynamic* ToolSetupUtil::GetUVCheckerboardMaterial(double CheckerDensity)
{
	UMaterial* CheckerMaterialBase = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/CheckerMaterial"));
	if (CheckerMaterialBase != nullptr)
	{
		UMaterialInstanceDynamic* CheckerMaterial = UMaterialInstanceDynamic::Create(CheckerMaterialBase, NULL);
		if (CheckerMaterial != nullptr)
		{
			CheckerMaterial->SetScalarParameterValue("Density", CheckerDensity);
			return CheckerMaterial;
		}
	}
	return UMaterialInstanceDynamic::Create(GetDefaultMaterial(), NULL);
}



UMaterialInstanceDynamic* ToolSetupUtil::GetDefaultBrushVolumeMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/BrushIndicatorMaterial"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		return MatInstance;
	}
	return nullptr;
}

UMaterialInstanceDynamic* ToolSetupUtil::GetDefaultBrushAlphaMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/BrushAlphaIndicatorMaterial"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		return MatInstance;
	}
	return nullptr;
}


UMaterialInterface* ToolSetupUtil::GetDefaultSculptMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}



UMaterialInstanceDynamic* ToolSetupUtil::GetTransparentSculptMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, double Opacity, bool bTwoSided)
{
	// Unfortunately the two-sided flag is not something that we can give as a runtime 
	// parameter, so we need separate versions of the material.
	UMaterial* Material = bTwoSided ? LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial_TransparentTwoSided"))
		: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial_Transparent"));

	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		MatInstance->SetScalarParameterValue(TEXT("Opacity"), Opacity);
		return MatInstance;
	}
	return nullptr;
}



UMaterialInterface* ToolSetupUtil::GetImageBasedSculptMaterial(UInteractiveToolManager* ToolManager, ImageMaterialType Type)
{
	UMaterialInterface* Material = nullptr;
	if (Type == ImageMaterialType::DefaultBasic)
	{
		Material = LoadObject<UMaterialInstance>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial_Basic"));
	}
	else if (Type == ImageMaterialType::DefaultSoft)
	{
		Material = LoadObject<UMaterialInstance>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial_Soft"));
	}
	else if (Type == ImageMaterialType::TangentNormalFromView)
	{
		Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SculptMaterial_TangentNormalFromView"));
	}

	if (Material == nullptr && ToolManager != nullptr)
	{
		Material = GetDefaultSculptMaterial(ToolManager);
	}

	return Material;
}



UMaterialInstanceDynamic* ToolSetupUtil::GetCustomImageBasedSculptMaterial(UInteractiveToolManager* ToolManager, UTexture* SetImage)
{
	UMaterial* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/ImageBasedMaterial_Master"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		if (SetImage != nullptr)
		{
			MatInstance->SetTextureParameterValue(TEXT("ImageTexture"), SetImage);
		}
		return MatInstance;
	}
	return nullptr;
}



UMaterialInterface* ToolSetupUtil::GetSelectionMaterial(UInteractiveToolManager* ToolManager)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SelectionMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}


UMaterialInterface* ToolSetupUtil::GetSelectionMaterial(const FLinearColor& UseColor, UInteractiveToolManager* ToolManager, float DepthBias)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SelectionMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("ConstantColor"), UseColor);
		if (DepthBias != 0)
		{
			MatInstance->SetScalarParameterValue(TEXT("DepthBias"), DepthBias);
		}
		return MatInstance;
	}
	return Material;
}


UMaterialInstanceDynamic* ToolSetupUtil::GetSimpleCustomMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float Opacity)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SimpleBaseMaterial_Transparent"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		MatInstance->SetScalarParameterValue(TEXT("Opacity"), Opacity);
		return MatInstance;
	}
	return nullptr;
}

MODELINGCOMPONENTS_API UMaterialInstanceDynamic* ToolSetupUtil::GetSimpleCustomMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SimpleBaseMaterial"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		return MatInstance;
	}
	return nullptr;
}

UMaterialInstanceDynamic* ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias, float Opacity)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SimpleTwoSidedOffsetMaterial_Transparent"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		MatInstance->SetScalarParameterValue(TEXT("DepthBias"), DepthBias);
		MatInstance->SetScalarParameterValue(TEXT("Opacity"), Opacity);
		return MatInstance;
	}
	return nullptr;
}

MODELINGCOMPONENTS_API UMaterialInstanceDynamic* ToolSetupUtil::GetCustomTwoSidedDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SimpleTwoSidedOffsetMaterial"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		MatInstance->SetScalarParameterValue(TEXT("DepthBias"), DepthBias);
		return MatInstance;
	}
	return nullptr;
}

UMaterialInstanceDynamic* ToolSetupUtil::GetCustomDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias, float Opacity)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SimpleOffsetMaterial_Transparent"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		MatInstance->SetScalarParameterValue(TEXT("DepthBias"), DepthBias);
		MatInstance->SetScalarParameterValue(TEXT("Opacity"), Opacity);
		return MatInstance;
	}
	return nullptr;
}

MODELINGCOMPONENTS_API UMaterialInstanceDynamic* ToolSetupUtil::GetCustomDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias)
{
	UMaterialInterface* Material = LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/SimpleOffsetMaterial"));
	if (Material != nullptr)
	{
		UMaterialInstanceDynamic* MatInstance = UMaterialInstanceDynamic::Create(Material, ToolManager);
		MatInstance->SetVectorParameterValue(TEXT("Color"), Color);
		MatInstance->SetScalarParameterValue(TEXT("DepthBias"), DepthBias);
		return MatInstance;
	}
	return nullptr;
}

UMaterialInterface* ToolSetupUtil::GetDefaultEditVolumeMaterial()
{
	return LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/VolumeEditMaterial"));
}


UMaterialInterface* ToolSetupUtil::GetDefaultPointComponentMaterial(UInteractiveToolManager* ToolManager, bool bDepthTested)
{
	UMaterialInterface* Material = (bDepthTested) ?
		LoadObject<UMaterial>(nullptr, TEXT("/Engine/EngineMaterials/PointSetComponentMaterial"))
		: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}

UMaterialInterface* ToolSetupUtil::GetRoundPointComponentMaterial(UInteractiveToolManager* ToolManager, bool bDepthTested)
{
	UMaterialInterface* Material = (bDepthTested) ?
		LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial_Round"))
		: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial_Round"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}

UMaterialInterface* ToolSetupUtil::GetDefaultLineComponentMaterial(UInteractiveToolManager* ToolManager, bool bDepthTested)
{
	UMaterialInterface* Material = (bDepthTested) ?
		LoadObject<UMaterial>(nullptr, TEXT("/Engine/EngineMaterials/LineSetComponentMaterial"))
		: LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/LineSetOverlaidComponentMaterial"));
	if (Material == nullptr && ToolManager != nullptr)
	{
		// We don't seem to have a default line material to use here.
		return ToolManager->GetContextQueriesAPI()->GetStandardMaterial(EStandardToolContextMaterials::VertexColorMaterial);
	}
	return Material;
}

UCurveFloat* ToolSetupUtil::GetContrastAdjustmentCurve(UInteractiveToolManager* ToolManager)
{
	// This curve would currently be shared across any tools that need such a curve. We'll probably want to revisit this
	// once it is used in multiple tools.
	UCurveFloat* Curve = LoadObject<UCurveFloat>(nullptr, TEXT("/MeshModelingToolsetExp/Curves/ContrastAdjustmentCurve"));

	// Create a transient duplicate of the curve, as we are going to expose this curve to the user, and we
	// do not want them editing the default Asset.
	UCurveFloat* CurveCopy = DuplicateObject<UCurveFloat>(Curve, GetTransientPackage());

	return CurveCopy;
}




void ToolSetupUtil::ApplyRenderingConfigurationToPreview(UBaseDynamicMeshComponent* Component, UToolTarget* SourceTarget)
{
	UModelingComponentsSettings* Settings = GetMutableDefault<UModelingComponentsSettings>();
	bool bEnableRaytracingSupport = (Settings != nullptr) ? Settings->bEnableRayTracingWhileEditing : false;


	if (bEnableRaytracingSupport)
	{
		Component->SetEnableRaytracing(bEnableRaytracingSupport);
	}
}

void ToolSetupUtil::ApplyRenderingConfigurationToPreview(UPreviewMesh* PreviewMesh, UToolTarget* SourceTarget)
{
	UBaseDynamicMeshComponent* Component = Cast<UBaseDynamicMeshComponent>(PreviewMesh->GetRootComponent());
	if (!Component)
	{
		return;
	}
	ApplyRenderingConfigurationToPreview(Component, SourceTarget);
}

UE::Geometry::FFrame3d ToolSetupUtil::GetDefaultWorldReferenceFrame(UInteractiveToolManager* ToolManager, UE::Geometry::FQuaterniond DefaultOrientation, double DefaultPlacementDistance)
{
	using namespace UE::Geometry;

	FFrame3d Frame;
	FAxisAlignedBox3d Bounds;
	bool bIsElementSelection;
	FToolBuilderState SceneState;
	ToolManager->GetContextQueriesAPI()->GetCurrentSelectionState(SceneState);
	if (GetCurrentSelectionWorldFrameBounds(SceneState, Frame, Bounds, bIsElementSelection))
	{
		if (bIsElementSelection)
		{
			return Frame;
		}
		else
		{
			Frame.Rotation = DefaultOrientation;
			return Frame;
		}
	}
	else
	{
		FViewCameraState CameraState;
		ToolManager->GetContextQueriesAPI()->GetCurrentViewState(CameraState);
		FHitResult WorldHitResult;
		FRay Ray(CameraState.Position, CameraState.Forward(), true);
		double HitParam = DefaultPlacementDistance;
		if (ToolSceneQueriesUtil::FindNearestVisibleObjectHit(USceneSnappingManager::Find(ToolManager), WorldHitResult, Ray))
		{
			HitParam = (double)Ray.GetParameter(WorldHitResult.ImpactPoint);
		}

		return FFrame3d(CameraState.Position + CameraState.Forward() * HitParam, DefaultOrientation);
	}
}
