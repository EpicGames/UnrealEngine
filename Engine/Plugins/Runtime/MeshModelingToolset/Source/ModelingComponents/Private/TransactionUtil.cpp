// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransactionUtil.h"

#include "InteractiveToolManager.h"

namespace UE::TransactionUtil
{

	void FLongTransactionTracker::Open(FText TransactionName, UInteractiveToolManager* ToolManager)
	{
		if (OpenCount == 0) // only actually 'begin' when opening the outer-most scope
		{
			ToolManager->BeginUndoTransaction(TransactionName);
		}
		OpenCount++;

		// After opening, expect from 1 to MaxOpen to be open
		ensureMsgf(OpenCount > 0 && OpenCount <= MaxOpen, TEXT("Opened new long transaction (%s); found unexpected current open count: %d (Expected from 1 to %d)"), *TransactionName.ToString(), OpenCount, MaxOpen);
	}

	
	void FLongTransactionTracker::Close(UInteractiveToolManager* ToolManager)
	{
		// Before closing, expect from 1 to MaxOpen to be open
		ensureMsgf(OpenCount > 0 && OpenCount <= MaxOpen, TEXT("Closing long transaction; found unexpected current open count: %d (Expected from 1 to %d)"), OpenCount, MaxOpen);

		OpenCount--;
		if (OpenCount == 0) // only actually 'end' when closing the outer-most scope 
		{
			ToolManager->EndUndoTransaction();
		}
	}

	void FLongTransactionTracker::CloseAll(UInteractiveToolManager* ToolManager)
	{
		// When closing all, expect 0 to MaxOpen to be open
		ensureMsgf(OpenCount >= 0 && OpenCount <= MaxOpen, TEXT("Closing any/all long transactions; found unexpected current open count: %d (Expected from 0 to %d)"), OpenCount, MaxOpen);

		if (OpenCount > 0)
		{
			ToolManager->EndUndoTransaction();
		}
		OpenCount = 0;
	}

}
