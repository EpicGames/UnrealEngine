// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transforms/MultiTransformer.h"
#include "BaseGizmos/TransformGizmoUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MultiTransformer)

using namespace UE::Geometry;


void UMultiTransformer::Setup(UInteractiveGizmoManager* GizmoManagerIn, IToolContextTransactionProvider* TransactionProviderIn)
{
	GizmoManager = GizmoManagerIn;
	TransactionProvider = TransactionProviderIn;

	ActiveGizmoFrame = FFrame3d();
	ActiveGizmoScale = FVector3d::One();

	ActiveMode = EMultiTransformerMode::DefaultGizmo;

	// Create a new TransformGizmo and associated TransformProxy. The TransformProxy will not be the
	// parent of any Components in this case, we just use it's transform and change delegate.
	TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->SetTransform(ActiveGizmoFrame.ToFTransform());
	UpdateShowGizmoState(true);

	// listen for changes to the proxy and update the transform frame when that happens
	TransformProxy->OnTransformChanged.AddUObject(this, &UMultiTransformer::OnProxyTransformChanged);
	TransformProxy->OnBeginTransformEdit.AddUObject(this, &UMultiTransformer::OnBeginProxyTransformEdit);
	TransformProxy->OnEndTransformEdit.AddUObject(this, &UMultiTransformer::OnEndProxyTransformEdit);
	TransformProxy->OnPivotChanged.AddWeakLambda(this, [this](UTransformProxy* Proxy, FTransform Transform) {
		ActiveGizmoFrame = FFrame3d(Transform);
		ActiveGizmoScale = FVector3d(Transform.GetScale3D());
	});
	TransformProxy->OnEndPivotEdit.AddWeakLambda(this, [this](UTransformProxy* Proxy) {
		OnEndPivotEdit.Broadcast();
	});
}



void UMultiTransformer::Shutdown()
{
	GizmoManager->DestroyAllGizmosByOwner(this);
}


void UMultiTransformer::SetOverrideGizmoCoordinateSystem(EToolContextCoordinateSystem CoordSystem)
{
	if (GizmoCoordSystem != CoordSystem || bForceGizmoCoordSystem == false)
	{
		bForceGizmoCoordSystem = true;
		GizmoCoordSystem = CoordSystem;
		if (TransformGizmo != nullptr)
		{
			UpdateShowGizmoState(false);
			UpdateShowGizmoState(true);
		}
	}
}

void UMultiTransformer::SetEnabledGizmoSubElements(ETransformGizmoSubElements EnabledSubElements)
{
	if (ActiveGizmoSubElements != EnabledSubElements)
	{
		ActiveGizmoSubElements = EnabledSubElements;
		if (TransformGizmo != nullptr)
		{
			UpdateShowGizmoState(false);
			UpdateShowGizmoState(true);
		}
	}
}

void UMultiTransformer::SetMode(EMultiTransformerMode NewMode)
{
	if (NewMode != ActiveMode)
	{
		if (NewMode == EMultiTransformerMode::DefaultGizmo)
		{
			UpdateShowGizmoState(true);
		}
		else
		{
			UpdateShowGizmoState(false);
		}
		ActiveMode = NewMode;
	}
}


void UMultiTransformer::SetGizmoVisibility(bool bVisible)
{
	if (bShouldBeVisible != bVisible)
	{
		bShouldBeVisible = bVisible;
		if (TransformGizmo != nullptr)
		{
			TransformGizmo->SetVisibility(bVisible);
		}
	}
}

void UMultiTransformer::SetGizmoRepositionable(bool bOn)
{
	if (bRepositionableGizmo != bOn)
	{
		bRepositionableGizmo = bOn;
		if (TransformGizmo)
		{
			UpdateShowGizmoState(false);
			UpdateShowGizmoState(true);
		}
	}
}

EToolContextCoordinateSystem UMultiTransformer::GetGizmoCoordinateSystem()
{ 
	return TransformGizmo ? TransformGizmo->CurrentCoordinateSystem : GizmoCoordSystem; 
}


void UMultiTransformer::SetSnapToWorldGridSourceFunc(TUniqueFunction<bool()> EnableSnapFunc)
{
	EnableSnapToWorldGridFunc = MoveTemp(EnableSnapFunc);
}

void UMultiTransformer::SetIsNonUniformScaleAllowedFunction(TFunction<bool()> IsNonUniformScaleAllowedIn)
{
	IsNonUniformScaleAllowed = IsNonUniformScaleAllowedIn;
	if (TransformGizmo != nullptr)
	{
		TransformGizmo->SetIsNonUniformScaleAllowedFunction(IsNonUniformScaleAllowed);
	}
}

void UMultiTransformer::SetDisallowNegativeScaling(bool bDisallow)
{
	bDisallowNegativeScaling = bDisallow;
	if (TransformGizmo != nullptr)
	{
		TransformGizmo->SetDisallowNegativeScaling(bDisallowNegativeScaling);
	}
}

void UMultiTransformer::AddAlignmentMechanic(UDragAlignmentMechanic* AlignmentMechanic)
{
	DragAlignmentMechanic = AlignmentMechanic;
	if (TransformGizmo)
	{
		DragAlignmentMechanic->AddToGizmo(TransformGizmo);
	}
}

void UMultiTransformer::Tick(float DeltaTime)
{
	if (TransformGizmo != nullptr)
	{
		// todo this
		TransformGizmo->bSnapToWorldGrid =
			(EnableSnapToWorldGridFunc) ? EnableSnapToWorldGridFunc() : false;
	}
}


void UMultiTransformer::InitializeGizmoPositionFromWorldFrame(const FFrame3d& Frame, bool bResetScale)
{
	ActiveGizmoFrame = Frame;
	if (bResetScale)
	{
		ActiveGizmoScale = FVector3d::One();
	}

	if (TransformGizmo != nullptr)
	{
		// this resets the child scale to one
		TransformGizmo->ReinitializeGizmoTransform(ActiveGizmoFrame.ToFTransform());
	}
}



void UMultiTransformer::UpdateGizmoPositionFromWorldFrame(const FFrame3d& Frame, bool bResetScale)
{
	ActiveGizmoFrame = Frame;
	if (bResetScale)
	{
		ActiveGizmoScale = FVector3d::One();
	}

	if (TransformGizmo != nullptr)
	{
		// this resets the child scale to one
		TransformGizmo->SetNewGizmoTransform(ActiveGizmoFrame.ToFTransform());
	}
}


void UMultiTransformer::ResetScale()
{
	ActiveGizmoScale = FVector3d::One();
	if (TransformGizmo != nullptr)
	{
		TransformGizmo->SetNewChildScale(FVector::OneVector);
	}
}


void UMultiTransformer::OnProxyTransformChanged(UTransformProxy* Proxy, FTransform Transform)
{
	ActiveGizmoFrame = FFrame3d(Transform);
	ActiveGizmoScale = FVector3d(Transform.GetScale3D());
	OnTransformUpdated.Broadcast();
}


void UMultiTransformer::OnBeginProxyTransformEdit(UTransformProxy* Proxy)
{
	bInGizmoEdit = true;
	OnTransformStarted.Broadcast();
}

void UMultiTransformer::OnEndProxyTransformEdit(UTransformProxy* Proxy)
{
	bInGizmoEdit = false;
	OnTransformCompleted.Broadcast();
}



void UMultiTransformer::UpdateShowGizmoState(bool bNewVisibility)
{
	if (bNewVisibility == false)
	{
		GizmoManager->DestroyAllGizmosByOwner(this);
		TransformGizmo = nullptr;
	}
	else
	{
		check(TransformGizmo == nullptr);
		TransformGizmo = bRepositionableGizmo ? UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GizmoManager, ActiveGizmoSubElements, this)
			: UE::TransformGizmoUtil::CreateCustomTransformGizmo(GizmoManager, ActiveGizmoSubElements, this);
		if (bForceGizmoCoordSystem)
		{
			TransformGizmo->bUseContextCoordinateSystem = false;
			TransformGizmo->CurrentCoordinateSystem = GizmoCoordSystem;
		}
		if (IsNonUniformScaleAllowed)
		{
			TransformGizmo->SetIsNonUniformScaleAllowedFunction(IsNonUniformScaleAllowed);
		}
		TransformGizmo->SetDisallowNegativeScaling(bDisallowNegativeScaling);
		if (DragAlignmentMechanic)
		{
			DragAlignmentMechanic->AddToGizmo(TransformGizmo);
		}
		TransformGizmo->SetActiveTarget(TransformProxy, TransactionProvider);
		TransformGizmo->ReinitializeGizmoTransform(ActiveGizmoFrame.ToFTransform());
		TransformGizmo->SetVisibility(bShouldBeVisible);
	}
}

