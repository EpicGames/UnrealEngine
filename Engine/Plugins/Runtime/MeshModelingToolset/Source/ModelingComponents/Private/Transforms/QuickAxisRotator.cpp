// Copyright Epic Games, Inc. All Rights Reserved.


#include "Transforms/QuickAxisRotator.h"
#include "ToolSceneQueriesUtil.h"

using namespace UE::Geometry;


constexpr int RotateXSnapID = FRaySpatialSnapSolver::BaseExternalLineID + 1;
constexpr int RotateYSnapID = FRaySpatialSnapSolver::BaseExternalLineID + 2;
constexpr int RotateZSnapID = FRaySpatialSnapSolver::BaseExternalLineID + 3;
constexpr int RotateOriginSnapID = FRaySpatialSnapSolver::BaseExternalPointID + 1;


void FQuickAxisRotator::Initialize()
{
	MoveAxisSolver.SnapMetricTolerance = ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	MoveAxisSolver.SnapMetricFunc = [this](const FVector3d& Position1, const FVector3d& Position2) 
	{
		return ToolSceneQueriesUtil::CalculateNormalizedViewVisualAngleD(this->CameraState, Position1, Position2);
	};

	QuickAxisRenderer.SetLineParameters(FLinearColor(240, 240, 16), 6);
	QuickAxisRenderer.SetPointParameters(FLinearColor(240, 240, 240), 8);
	QuickAxisRenderer.bDepthTested = false;
	AxisColorMap.Add(RotateXSnapID, FLinearColor::Red);
	AxisColorMap.Add(RotateYSnapID, FLinearColor::Green);
	AxisColorMap.Add(RotateZSnapID, FLinearColor::Blue);

	QuickAxisPreviewRenderer.SetLineParameters(FLinearColor(240, 240, 16), 1);
	QuickAxisPreviewRenderer.SetPointParameters(FLinearColor(240, 240, 240), 1);
	QuickAxisPreviewRenderer.bDepthTested = false;

	AxisFrameWorld = FFrame3d();
}



void FQuickAxisRotator::SetActiveFrameFromWorldAxes(const FVector3d& Origin)
{
	AxisFrameWorld = FFrame3d(Origin);
	UpdateSnapAxes();
}


void FQuickAxisRotator::SetActiveWorldFrame(const FFrame3d& Frame)
{
	AxisFrameWorld = Frame;
	UpdateSnapAxes();
}


void FQuickAxisRotator::SetActiveFrameFromWorldNormal(const FVector3d& Origin, const FVector3d& Normal, bool bAlignToWorldAxes)
{
	AxisFrameWorld = FFrame3d(Origin);
	AxisFrameWorld.AlignAxis(2, Normal);

	if (bAlignToWorldAxes)
	{
		double XDot = FMathd::Abs(AxisFrameWorld.X().Dot(FVector3d::UnitX()));
		double YDot = FMathd::Abs(AxisFrameWorld.Y().Dot(FVector3d::UnitY()));
		if (XDot > YDot)
		{
			AxisFrameWorld.ConstrainedAlignAxis(0, FVector3d::UnitX(), Normal);
		}
		else
		{
			AxisFrameWorld.ConstrainedAlignAxis(1, FVector3d::UnitY(), Normal);
		}
	}

	UpdateSnapAxes();
}


void FQuickAxisRotator::UpdateActiveFrameOrigin(const FVector3d& NewOrigin)
{
	AxisFrameWorld.Origin = NewOrigin;
	UpdateSnapAxes();
}



void FQuickAxisRotator::UpdateSnapAxes()
{
	MoveAxisSolver.Reset();

	constexpr double UseRadius = 50.0;

	const FFrame3d RotXFrame(AxisFrameWorld.Origin, AxisFrameWorld.GetAxis(0));
	MoveAxisSolver.AddCircleTarget(FCircle3d(RotXFrame, UseRadius), RotateXSnapID, 100);

	const FFrame3d RotYFrame(AxisFrameWorld.Origin, AxisFrameWorld.GetAxis(1));
	MoveAxisSolver.AddCircleTarget(FCircle3d(RotYFrame, UseRadius), RotateYSnapID, 100);

	const FFrame3d RotZFrame(AxisFrameWorld.Origin, AxisFrameWorld.GetAxis(2));
	MoveAxisSolver.AddCircleTarget(FCircle3d(RotZFrame, UseRadius), RotateZSnapID, 100);

	MoveAxisSolver.AddPointTarget(AxisFrameWorld.Origin, RotateOriginSnapID,
		FRaySpatialSnapSolver::FCustomMetric::Replace(0.5*ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD()), 50);
}


void FQuickAxisRotator::SetAxisLock()
{
	check(HaveActiveSnapRotation());
	int ActiveSnapTargetID = MoveAxisSolver.GetActiveSnapTargetID();
	IgnoredAxes = FIndex3i(RotateXSnapID, RotateYSnapID, RotateZSnapID);
	for ( int j = 0; j < 3; ++j )
	{
		if (IgnoredAxes[j] != ActiveSnapTargetID)
		{
			MoveAxisSolver.AddIgnoreTarget(IgnoredAxes[j]);
		}
		else
		{
			IgnoredAxes[j] = -1;
		}
	}
	bHaveLockedToAxis = true;
}

void FQuickAxisRotator::ClearAxisLock()
{
	for (int j = 0; j < 3; ++j)
	{
		MoveAxisSolver.RemoveIgnoreTarget(IgnoredAxes[j]);
		IgnoredAxes[j] = -1;
	}
	bHaveLockedToAxis = false;
}


bool FQuickAxisRotator::UpdateSnap(const FRay3d& Ray, FVector3d& SnapPointOut)
{
	MoveAxisSolver.UpdateSnappedPoint(Ray);
	if (MoveAxisSolver.HaveActiveSnap())
	{
		SnapPointOut = MoveAxisSolver.GetActiveSnapToPoint();
		return true;
	}
	return false;
}

bool FQuickAxisRotator::HaveActiveSnap() const
{
	return MoveAxisSolver.HaveActiveSnap();
}


bool FQuickAxisRotator::HaveActiveSnapRotation() const
{
	int ActiveSnapTargetID = MoveAxisSolver.GetActiveSnapTargetID();
	return MoveAxisSolver.HaveActiveSnap() &&
		(ActiveSnapTargetID == RotateXSnapID || ActiveSnapTargetID == RotateYSnapID || ActiveSnapTargetID == RotateZSnapID);
}


FFrame3d FQuickAxisRotator::GetActiveRotationFrame() const
{
	if (HaveActiveSnapRotation() == false)
	{
		return FFrame3d(AxisFrameWorld.Origin);
	}

	int UseAxisIndex = (MoveAxisSolver.GetActiveSnapTargetID() - FRaySpatialSnapSolver::BaseExternalLineID - 1);
	return FFrame3d(AxisFrameWorld.Origin, AxisFrameWorld.GetAxis(UseAxisIndex));
}


void FQuickAxisRotator::UpdateCameraState(const FViewCameraState& CameraStateIn)
{
	this->CameraState = CameraStateIn;
}


void FQuickAxisRotator::Render(IToolsContextRenderAPI* RenderAPI)
{
	double CurViewSizeFactor = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, AxisFrameWorld.Origin, 1.0);

	QuickAxisRenderer.BeginFrame(RenderAPI);
	MoveAxisSolver.Draw(&QuickAxisRenderer, 3*CurViewSizeFactor, &AxisColorMap);
	QuickAxisRenderer.EndFrame();
}


void FQuickAxisRotator::PreviewRender(IToolsContextRenderAPI* RenderAPI)
{
	double CurViewSizeFactor = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, AxisFrameWorld.Origin, 1.0);

	QuickAxisPreviewRenderer.BeginFrame(RenderAPI);
	MoveAxisSolver.Draw(&QuickAxisPreviewRenderer, 2 * CurViewSizeFactor);
	QuickAxisPreviewRenderer.EndFrame();
}

void FQuickAxisRotator::Reset()
{
	MoveAxisSolver.Reset();
	ClearAxisLock();
}
