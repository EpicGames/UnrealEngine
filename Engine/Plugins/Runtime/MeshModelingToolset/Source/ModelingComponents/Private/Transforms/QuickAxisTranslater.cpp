// Copyright Epic Games, Inc. All Rights Reserved.


#include "Transforms/QuickAxisTranslater.h"
#include "ToolSceneQueriesUtil.h"

using namespace UE::Geometry;


constexpr int TranslateXSnapID = FRaySpatialSnapSolver::BaseExternalLineID + 1;
constexpr int TranslateYSnapID = FRaySpatialSnapSolver::BaseExternalLineID + 2;
constexpr int TranslateZSnapID = FRaySpatialSnapSolver::BaseExternalLineID + 3;
constexpr int TranslateOriginSnapID = FRaySpatialSnapSolver::BaseExternalPointID + 1;


void FQuickAxisTranslater::Initialize()
{
	MoveAxisSolver.SnapMetricTolerance = 9999; // ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD();
	MoveAxisSolver.SnapMetricFunc = [this](const FVector3d& Position1, const FVector3d& Position2) 
	{
		return ToolSceneQueriesUtil::CalculateNormalizedViewVisualAngleD(this->CameraState, Position1, Position2);
	};

	QuickAxisRenderer.SetLineParameters(FLinearColor(240, 240, 16), 6);
	QuickAxisRenderer.SetPointParameters(FLinearColor(240, 240, 240), 8);
	QuickAxisRenderer.bDepthTested = false;
	AxisColorMap.Add(TranslateXSnapID, FLinearColor::Red);
	AxisColorMap.Add(TranslateYSnapID, FLinearColor::Green);
	AxisColorMap.Add(TranslateZSnapID, FLinearColor::Blue);

	QuickAxisPreviewRenderer.SetLineParameters(FLinearColor(240, 240, 16), 1);
	QuickAxisPreviewRenderer.SetPointParameters(FLinearColor(240, 240, 240), 1);
	QuickAxisPreviewRenderer.bDepthTested = false;

	AxisFrameWorld = FFrame3d();
}



void FQuickAxisTranslater::SetActiveFrameFromWorldAxes(const FVector3d& Origin)
{
	AxisFrameWorld = FFrame3d(Origin);
	UpdateSnapAxes();
}


void FQuickAxisTranslater::SetActiveWorldFrame(const FFrame3d& Frame)
{
	AxisFrameWorld = Frame;
	UpdateSnapAxes();
}


void FQuickAxisTranslater::SetActiveFrameFromWorldNormal(const FVector3d& Origin, const FVector3d& Normal, bool bAlignToWorldAxes)
{
	AxisFrameWorld = FFrame3d(Origin);
	AxisFrameWorld.AlignAxis(2, Normal);

	if (bAlignToWorldAxes)
	{
		double XDot = FMathd::Abs(AxisFrameWorld.X().Dot(FVector3d::UnitX()));
		double YDot = FMathd::Abs(AxisFrameWorld.Y().Dot(FVector3d::UnitY()));
		if (XDot > YDot)
		{
			AxisFrameWorld.ConstrainedAlignAxis(0, FVector3d::UnitX(), Normal);
		}
		else
		{
			AxisFrameWorld.ConstrainedAlignAxis(1, FVector3d::UnitY(), Normal);
		}
	}

	UpdateSnapAxes();
}


void FQuickAxisTranslater::UpdateActiveFrameOrigin(const FVector3d& NewOrigin)
{
	AxisFrameWorld.Origin = NewOrigin;
	UpdateSnapAxes();
}



bool ProjectToCameraPlanePos(const FVector3d& ScenePos, const FFrame3d& CameraPlaneFrame, const FVector3d& CameraPos, FVector2d& CamPlanePos)
{
	FVector3d HitPos;
	if (CameraPlaneFrame.RayPlaneIntersection(CameraPos, UE::Geometry::Normalized(ScenePos - CameraPos), 2, HitPos) == false)
	{
		return false;
	}
	CamPlanePos = CameraPlaneFrame.ToPlaneUV(HitPos, 2);
	return true;
}


void FQuickAxisTranslater::UpdateSnapAxes()
{
	// cos(angle) tolerances 
	static const double ForwardDotTol = FMathd::Cos(15.0 * FMathd::DegToRad);
	static const double OverlapDotTol = FMathd::Cos(15.0 * FMathd::DegToRad);

	MoveAxisSolver.Reset();

	// extract camera vectors and make camera plane frame
	FVector3d CameraPosition = (FVector3d)CameraState.Position;
	FVector3d Forward = (FVector3d)CameraState.Forward();
	FFrame3d CameraPlaneFrame(CameraPosition + 0.1*Forward, Forward);  // 0.1 is arbitrary here, just needs to be in front of camera

	// desired X/Y/Z axes of transformer
	FVector3d AxisX = AxisFrameWorld.GetAxis(0);
	FVector3d AxisY = AxisFrameWorld.GetAxis(1);
	FVector3d AxisZ = AxisFrameWorld.GetAxis(2);
	bool IgnoreAxis[3] = { false, false, false };

	// LocalForward is vector from eye to target position, use this as "forward"
	FVector3d LocalForward = UE::Geometry::Normalized(AxisFrameWorld.Origin - CameraPosition);
	double DirectionDotX = AxisX.Dot(LocalForward);
	double DirectionDotY = AxisY.Dot(LocalForward);
	double DirectionDotZ = AxisZ.Dot(LocalForward);
	
	// if axis is too aligned w/ forward vec, it is unstable and we want to ignore it
	if (FMathd::Abs(DirectionDotX) > ForwardDotTol)
	{
		IgnoreAxis[0] = true;
	}
	if (FMathd::Abs(DirectionDotY) > ForwardDotTol)
	{
		IgnoreAxis[1] = true;
	}
	if (FMathd::Abs(DirectionDotZ) > ForwardDotTol)
	{
		IgnoreAxis[2] = true;
	}

	// construct projections of axes on camera plane. Because of perspective this is not
	// a simple projection so just project 3D points and then convert back to 2D directions
	FVector2d CamPosCenter, CamPosAxisX, CamPosAxisY, CamPosAxisZ;
	bool bAllHit = 
		ProjectToCameraPlanePos(AxisFrameWorld.Origin, CameraPlaneFrame, CameraPosition, CamPosCenter)
		&& ProjectToCameraPlanePos(AxisFrameWorld.Origin + 10 * AxisX, CameraPlaneFrame, CameraPosition, CamPosAxisX)	// 10 is arbitrary
		&& ProjectToCameraPlanePos(AxisFrameWorld.Origin + 10 * AxisY, CameraPlaneFrame, CameraPosition, CamPosAxisY)
		&& ProjectToCameraPlanePos(AxisFrameWorld.Origin + 10 * AxisZ, CameraPlaneFrame, CameraPosition, CamPosAxisZ);
	if (bAllHit == false)
	{
		return;		// abort for now...
	}
	FVector2d PlaneAxisX = UE::Geometry::Normalized(CamPosAxisX - CamPosCenter);
	FVector2d PlaneAxisY = UE::Geometry::Normalized(CamPosAxisY - CamPosCenter);
	FVector2d PlaneAxisZ = UE::Geometry::Normalized(CamPosAxisZ - CamPosCenter);

	// if angle between these 2D projections of axes is too small, then axes are visually overlapping
	// from this view position, and snapping will be unstable. So only keep the one pointing "towards" camera.
	double AngleXYDot = (IgnoreAxis[0] || IgnoreAxis[1]) ? 0.0 : FMathd::Abs(PlaneAxisX.Dot(PlaneAxisY));
	double AngleXZDot = (IgnoreAxis[0] || IgnoreAxis[2]) ? 0.0 : FMathd::Abs(PlaneAxisX.Dot(PlaneAxisZ));
	double AngleYZDot = (IgnoreAxis[1] || IgnoreAxis[2]) ? 0.0 : FMathd::Abs(PlaneAxisY.Dot(PlaneAxisZ));
	if (AngleXYDot > OverlapDotTol)
	{
		int which = (FMathd::Sign(DirectionDotX) < FMathd::Sign(DirectionDotY)) ? 0 : 1;
		IgnoreAxis[which] = true;
	}
	if (AngleXZDot > OverlapDotTol)
	{
		int which = (FMathd::Sign(DirectionDotX) < FMathd::Sign(DirectionDotZ)) ? 0 : 2;
		IgnoreAxis[which] = true;
	}
	if (AngleYZDot > OverlapDotTol)
	{
		int which = (FMathd::Sign(DirectionDotY) < FMathd::Sign(DirectionDotZ)) ? 1 : 2;
		IgnoreAxis[which] = true;
	}

	// emit the axes that we didn't ignore (should always be at least 2)
	FIndex3i SnapIDs(TranslateXSnapID, TranslateYSnapID, TranslateZSnapID);
	for (int j = 0; j < 3; ++j)
	{
		if (IgnoreAxis[j] == false)
		{
			FVector3d Direction = AxisFrameWorld.GetAxis(j);
			MoveAxisSolver.AddLineTarget(FLine3d(AxisFrameWorld.Origin, Direction), SnapIDs[j], 100);
		}
	}

	MoveAxisSolver.AddPointTarget(AxisFrameWorld.Origin, TranslateOriginSnapID,
		FRaySpatialSnapSolver::FCustomMetric::Replace(0.5*ToolSceneQueriesUtil::GetDefaultVisualAngleSnapThreshD()), 50);
}





bool FQuickAxisTranslater::UpdateSnap(const FRay3d& Ray, FVector3d& SnapPointOut,
	TFunction<FVector3d(const FVector3d&)> PositionConstraintFunc)
{
	bool bSetConstraintFunc = false;
	if (PositionConstraintFunc != nullptr)
	{
		check(MoveAxisSolver.PointConstraintFunc == nullptr);
		MoveAxisSolver.PointConstraintFunc = PositionConstraintFunc;
		bSetConstraintFunc = true;
	}

	MoveAxisSolver.UpdateSnappedPoint(Ray);

	if (bSetConstraintFunc)
	{
		MoveAxisSolver.PointConstraintFunc = nullptr;
	}

	if (MoveAxisSolver.HaveActiveSnap())
	{
		SnapPointOut = MoveAxisSolver.GetActiveSnapToPoint();
		return true;
	}
	return false;
}

bool FQuickAxisTranslater::HaveActiveSnap() const
{
	return MoveAxisSolver.HaveActiveSnap();
}



void FQuickAxisTranslater::UpdateCameraState(const FViewCameraState& CameraStateIn)
{
	this->CameraState = CameraStateIn;
}


void FQuickAxisTranslater::Render(IToolsContextRenderAPI* RenderAPI)
{
	double CurViewSizeFactor = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, AxisFrameWorld.Origin, 1.0);

	QuickAxisRenderer.BeginFrame(RenderAPI);
	MoveAxisSolver.Draw(&QuickAxisRenderer, 3*CurViewSizeFactor, &AxisColorMap);
	QuickAxisRenderer.EndFrame();
}


void FQuickAxisTranslater::PreviewRender(IToolsContextRenderAPI* RenderAPI)
{
	double CurViewSizeFactor = ToolSceneQueriesUtil::CalculateDimensionFromVisualAngleD(CameraState, AxisFrameWorld.Origin, 1.0);

	QuickAxisPreviewRenderer.BeginFrame(RenderAPI);
	MoveAxisSolver.Draw(&QuickAxisPreviewRenderer, 2 * CurViewSizeFactor);
	QuickAxisPreviewRenderer.EndFrame();
}


void FQuickAxisTranslater::Reset()
{
	MoveAxisSolver.Reset();
}
