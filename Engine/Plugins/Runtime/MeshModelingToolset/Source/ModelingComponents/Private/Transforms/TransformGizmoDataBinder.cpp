// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transforms/TransformGizmoDataBinder.h"

#include "BaseGizmos/TransformGizmoUtil.h" // UCombinedTransformGizmoContextObject
#include "BaseGizmos/TransformProxy.h"
#include "VectorTypes.h"

#define LOCTEXT_NAMESPACE "TransformGizmoDataBinder"

// Key function for going from gizmo to values
void FTransformGizmoDataBinder::UpdateDataValuesFromGizmo()
{
	if (!CurrentlyTrackedGizmo.IsValid() || !ensure(CurrentlyTrackedGizmo->ActiveTarget))
	{
		return;
	}

	FTransform CurrentTransform = CurrentlyTrackedGizmo->ActiveTarget->GetTransform();
	FVector3d CurrentTranslation;
	FRotator CurrentRotation;
	if (bUsingDeltaMode)
	{
		*BoundScale = CurrentTransform.GetScale3D() / DeltaStartTransform.GetScale3D();

		if (CurrentlyTrackedGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::Local)
		{
			// Here we want the delta from the start transform in the coordinate system of the start transform
			CurrentRotation = FRotator(DeltaStartTransform.GetRotation().Inverse() * CurrentTransform.GetRotation());
			CurrentTranslation = DeltaStartTransform.GetRotation().UnrotateVector(CurrentTransform.GetTranslation() - DeltaStartTransform.GetTranslation());
		}
		else
		{
			// Get the delta from the start transform in the world coordinate system
			CurrentRotation = FRotator(CurrentTransform.GetRotation() * DeltaStartTransform.GetRotation().Inverse());
			CurrentTranslation = CurrentTransform.GetTranslation() - DeltaStartTransform.GetTranslation();
		}
	}
	else
	{
		TOptional<FTransform> ReferenceTransformToUse;
		if (CurrentlyTrackedGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::Local && CurrentCustomLocalReferenceTransform.IsSet())
		{
			ReferenceTransformToUse = CurrentCustomLocalReferenceTransform;
		}

		if (ReferenceTransformToUse.IsSet())
		{
			const FTransform& ReferenceTransform = ReferenceTransformToUse.GetValue();

			*BoundScale = CurrentTransform.GetScale3D() / ReferenceTransform.GetScale3D();
			CurrentRotation = FRotator(ReferenceTransform.GetRotation().Inverse() * CurrentTransform.GetRotation());
			CurrentTranslation = ReferenceTransform.GetRotation().UnrotateVector(CurrentTransform.GetTranslation() - ReferenceTransform.GetTranslation());
		}
		else
		{
			CurrentTranslation = CurrentTransform.GetTranslation();
			CurrentRotation = FRotator(CurrentTransform.GetRotation());
			*BoundScale = CurrentTransform.GetScale3D();
		}
	}

	*BoundTranslation = ActualToBoundConversion ? ActualToBoundConversion(CurrentTranslation) : CurrentTranslation;

	// We don't want to update the rotation if it is actually the same as what the user typed in previously.
	// Note that FRotator::Equals doesn't actually seem to do a proper comparison of rotations within a tolerance, despite what 
	// its header comment promises, hence us using FQuat::Equals here.
	FQuat ExistingRotation = FQuat::MakeFromEuler(*BoundEulerAngles);
	if (!ExistingRotation.Equals(CurrentRotation.Quaternion(), KINDA_SMALL_NUMBER))
	{
		*BoundEulerAngles = CurrentRotation.Euler();
	}

	LastTranslation = *BoundTranslation;
	LastEulerAngles = *BoundEulerAngles;
	LastScale = *BoundScale;
}

// Key function for going from values to gizmo
FTransform FTransformGizmoDataBinder::GetGizmoTransformFromDataValues(bool bLocalCoordinates)
{
	FVector3d CurrentTranslation = BoundToActualConversion ? BoundToActualConversion(*BoundTranslation) : *BoundTranslation;
	FRotator CurrentRotation = FRotator::MakeFromEuler(*BoundEulerAngles);
	FTransform CurrentUITransform(CurrentRotation, CurrentTranslation, *BoundScale);
	FTransform ResultTransform;
	if (bUsingDeltaMode)
	{
		ResultTransform.SetScale3D(DeltaStartTransform.GetScale3D() * *BoundScale);

		if (bLocalCoordinates)
		{
			ResultTransform.SetRotation(DeltaStartTransform.GetRotation() * CurrentRotation.Quaternion());
			ResultTransform.SetTranslation(DeltaStartTransform.GetTranslation()
				+ DeltaStartTransform.GetRotation().RotateVector(CurrentTranslation));
		}
		else
		{
			ResultTransform.SetRotation(CurrentRotation.Quaternion() * DeltaStartTransform.GetRotation());
			ResultTransform.SetTranslation(DeltaStartTransform.GetTranslation() + CurrentTranslation);
		}
	}
	else
	{
		TOptional<FTransform> ReferenceTransformToUse;
		if (bLocalCoordinates && CurrentCustomLocalReferenceTransform.IsSet())
		{
			ReferenceTransformToUse = CurrentCustomLocalReferenceTransform;
		}

		if (ReferenceTransformToUse.IsSet())
		{
			const FTransform& ReferenceTransform = ReferenceTransformToUse.GetValue();

			ResultTransform.SetScale3D(ReferenceTransform.GetScale3D() * *BoundScale);
			ResultTransform.SetRotation(ReferenceTransform.GetRotation() * CurrentRotation.Quaternion());
			ResultTransform.SetTranslation(ReferenceTransform.GetTranslation()
				+ ReferenceTransform.GetRotation().RotateVector(CurrentTranslation));
		}
		else
		{
			ResultTransform = CurrentUITransform;
		}
	}

	return ResultTransform;
}


void FTransformGizmoDataBinder::SetTrackedGizmo(UCombinedTransformGizmo* Gizmo)
{
	if (CurrentlyTrackedGizmo != Gizmo && ensure(Gizmo && Gizmo->ActiveTarget && BoundGizmos.Contains(Gizmo)))
	{
		CurrentlyTrackedGizmo = Gizmo;

		ETransformGizmoSubElements GizmoElements = CurrentlyTrackedGizmo->GetGizmoElements();

		CurrentCustomLocalReferenceTransform = Gizmo->GetDisplaySpaceTransform();
		if (!CurrentCustomLocalReferenceTransform.IsSet())
		{
			CurrentCustomLocalReferenceTransform = DefaultCustomLocalReferenceTransform;
		}

		LastCoordinateSystem = CurrentlyTrackedGizmo->CurrentCoordinateSystem;

		bCurrentGizmoLacksDegreeOfFreedom = ((GizmoElements & ETransformGizmoSubElements::TranslateAllAxes) != ETransformGizmoSubElements::TranslateAllAxes)
			|| ((GizmoElements & ETransformGizmoSubElements::RotateAllAxes) != ETransformGizmoSubElements::RotateAllAxes);

		bCurrentGizmoOnlyHasUniformScale = (GizmoElements & ETransformGizmoSubElements::ScaleUniform) != ETransformGizmoSubElements::None
			&& (GizmoElements & ETransformGizmoSubElements::ScaleAllAxes) == ETransformGizmoSubElements::None
			&& (GizmoElements & ETransformGizmoSubElements::ScaleAllPlanes) == ETransformGizmoSubElements::None;

		// See header comment for ShouldDestinationModeBeAllowed
		if (bAvoidDestinationModeWhenUnsafe && !ShouldDestinationModeBeAllowed() && !bUsingDeltaMode)
		{
			ResetToDeltaMode();
		}
		else if (bUsingDeltaMode)
		{
			ResetToDeltaMode();
		}
		else
		{
			UpdateDataValuesFromGizmo();
		}

		OnTrackedGizmoChanged.Broadcast(Gizmo);
	}
}

void FTransformGizmoDataBinder::SetToDeltaMode(const FTransform& StartTransform)
{
	bUsingDeltaMode = true;
	DeltaStartTransform = StartTransform;

	UpdateDataValuesFromGizmo();
}

void FTransformGizmoDataBinder::ResetToDeltaMode()
{
	bUsingDeltaMode = true;

	FTransform StartTransform;
	if (CurrentlyTrackedGizmo.IsValid() && ensure(CurrentlyTrackedGizmo->ActiveTarget))
	{
		StartTransform = CurrentlyTrackedGizmo->ActiveTarget->GetTransform();
	}
	DeltaStartTransform = StartTransform;

	// UI values should be at zeroes
	*BoundTranslation = ActualToBoundConversion ? ActualToBoundConversion(FVector::Zero()) : FVector::Zero();
	*BoundEulerAngles = FVector::Zero();
	*BoundScale = FVector::One();

	LastTranslation = *BoundTranslation;
	LastEulerAngles = *BoundEulerAngles;
	LastScale = *BoundScale;
}

void FTransformGizmoDataBinder::SetToDestinationMode()
{
	bUsingDeltaMode = false;
	UpdateDataValuesFromGizmo();
}

bool FTransformGizmoDataBinder::HasVisibleGizmo()
{
	return CurrentlyTrackedGizmo.IsValid() && CurrentlyTrackedGizmo->ActiveTarget && CurrentlyTrackedGizmo->IsVisible();
}

bool FTransformGizmoDataBinder::ShouldDestinationModeBeAllowed()
{
	return !(bCurrentGizmoLacksDegreeOfFreedom
		&& !CurrentCustomLocalReferenceTransform.IsSet()
		&& CurrentlyTrackedGizmo.IsValid()
		&& CurrentlyTrackedGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::Local);
}


// Functions to listen to gizmo changes:

void FTransformGizmoDataBinder::OnProxyBeginTransformEdit(UTransformProxy* TransformProxy, UCombinedTransformGizmo* Gizmo)
{
	if (!TransformProxy)
	{
		checkSlow(false);
		return;
	}

	if (bIgnoreCallbackForDebouncing)
	{
		return;
	}

	bGizmoIsBeingDragged = true;

	if (bChangeDisplayedGizmoOnDrag && CurrentlyTrackedGizmo != Gizmo)
	{
		SetTrackedGizmo(Gizmo);
	}

	if (CurrentlyTrackedGizmo == Gizmo && bUsingDeltaMode)
	{
		// Reset delta mode at the start of each drag
		ResetToDeltaMode();
	}
}

void FTransformGizmoDataBinder::OnProxyTransformChanged(UTransformProxy* TransformProxy, FTransform Transform, UCombinedTransformGizmo* Gizmo)
{
	if (!TransformProxy)
	{
		checkSlow(false);
		return;
	}

	if (bIgnoreCallbackForDebouncing || CurrentlyTrackedGizmo != Gizmo)
	{
		return;
	}

	if (!bGizmoIsBeingDragged && bUsingDeltaMode)
	{
		// In delta mode, if we got some kind of update that didn't come from a drag (so we haven't reset it yet), we probably
		// need to reset delta mode. However it's worth seeing whether the update actually changes our transform, because sometimes
		// tools issue updates that keep the transform the same (for instance, to make sure that the scale is all 1's). In that case
		// we don't want to reset anything.
		FTransform TransformFromUI = GetGizmoTransformFromDataValues(CurrentlyTrackedGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::Local);

		if (!TransformFromUI.GetRotation().Equals(CurrentlyTrackedGizmo->ActiveTarget->GetTransform().GetRotation(), KINDA_SMALL_NUMBER)
			|| !TransformFromUI.GetTranslation().Equals(CurrentlyTrackedGizmo->ActiveTarget->GetTransform().GetTranslation(), KINDA_SMALL_NUMBER)
			|| !TransformFromUI.GetScale3D().Equals(CurrentlyTrackedGizmo->ActiveTarget->GetTransform().GetScale3D(), KINDA_SMALL_NUMBER))
		{
			ResetToDeltaMode();
		}
	}
	else
	{
		// If we're not in delta mode or if we're dragging, no need to figure out whether the update changes anything, just do
		// the UI update (which involves much of the same work).
		UpdateDataValuesFromGizmo();
	}
}

void FTransformGizmoDataBinder::OnProxyEndTransformEdit(UTransformProxy* TransformProxy, UCombinedTransformGizmo* Gizmo)
{
	bGizmoIsBeingDragged = false;
}

void FTransformGizmoDataBinder::OnDisplaySpaceTransformChanged(UCombinedTransformGizmo* Gizmo, TOptional<FTransform> Transform)
{
	if (ensure(Gizmo) && CurrentlyTrackedGizmo == Gizmo)
	{
		CurrentCustomLocalReferenceTransform = Transform;
		if (!CurrentCustomLocalReferenceTransform.IsSet())
		{
			CurrentCustomLocalReferenceTransform = DefaultCustomLocalReferenceTransform;
		}
		UpdateDataValuesFromGizmo();
	}
}

void FTransformGizmoDataBinder::OnGizmoVisibilityChanged(UCombinedTransformGizmo* GizmoIn, bool bVisible)
{
	if (!ensure(GizmoIn))
	{
		return;
	}

	// If we don't have a gizmo to track, and this one is becoming visible, track it.
	if (!HasVisibleGizmo() && bVisible)
	{
		SetTrackedGizmo(GizmoIn);
	}
	// If we currently track this gizmo, and it's becoming invisible, see if there's
	// another one we can track instead.
	else if (CurrentlyTrackedGizmo == GizmoIn && !bVisible)
	{
		for (TWeakObjectPtr<UCombinedTransformGizmo> Gizmo : BoundGizmos)
		{
			if (Gizmo.IsValid() && Gizmo->ActiveTarget && Gizmo->IsVisible())
			{
				SetTrackedGizmo(Gizmo.Get());
				break;
			}
		}
	}
}


// Functions for listening to value changes:

void FTransformGizmoDataBinder::BeginDataEditSequence()
{
	bInDataEditSequence = true;

	// Used for proporitionally changing all results during a drag. Only relevant to scaling if
	// we're only allowed to uniformly scale, but too messy to special case that.
	ProportionalDragInitialVector = *BoundScale;

	if (CurrentlyTrackedGizmo.IsValid() && CurrentlyTrackedGizmo->ActiveTarget)
	{
		TGuardValue<bool> DebounceGuard(bIgnoreCallbackForDebouncing, true);
		CurrentlyTrackedGizmo->BeginTransformEditSequence();
	}
}

void FTransformGizmoDataBinder::UpdateAfterDataEdit()
{
	if (bGizmoIsBeingDragged)
	{
		// Generally it's unsafe to be trying to make changes while the gizmo is being dragged. As an example,
		// note that in a UI, if we type something and immediately start dragging the gizmo, the update from
		// the typing focus loss may actually come after the gizmo has already issued a BeginTransformEdit call.
		// This will cause problems since we would end up trying to nest our own Begin/EndTransformEditSequence
		// calls inside the gizmo's.
		return;
	}

	// See if everything actually stayed the same. This is actually an important early-out because we don't want
	// to emit undo/redo in this situation, and it arises in UI when a user click to edit a field but then doesn't.

	bool bScaleChanged = !BoundScale->Equals(LastScale, KINDA_SMALL_NUMBER); // saved since used further too

	if (!bScaleChanged
		&& BoundTranslation->Equals(LastTranslation, KINDA_SMALL_NUMBER)
		&& BoundEulerAngles->Equals(LastEulerAngles, KINDA_SMALL_NUMBER))
	{
		return;
	}

	// Apply proportionality to the scale, if relevant

	auto ShouldEnforceUniformScale = [this]() {
		return bEnforceUniformScaleConstraintsIfPresent && CurrentlyTrackedGizmo.IsValid()
			&& !CurrentlyTrackedGizmo->IsNonUniformScaleAllowed();
	};

	if (bScaleChanged && (bCurrentGizmoOnlyHasUniformScale || ShouldEnforceUniformScale()))
	{
		// Figure out which component changed.
		FVector3d Difference = *BoundScale - LastScale;
		int32 DifferenceMaxElementIndex = UE::Geometry::MaxAbsElementIndex(Difference);

		FVector3d ReferenceVectorToUse = bInDataEditSequence ? ProportionalDragInitialVector : LastScale;
		if (ReferenceVectorToUse[DifferenceMaxElementIndex] == 0)
		{
			// It's not clear how to apply proporitional scale if the scrubbed/changed value was zero, but the editor 
			// goes the route of changing it to be 1 beforehand and scaling the result. So, we do the same.
			ReferenceVectorToUse[DifferenceMaxElementIndex] = 1;
		}

		*BoundScale = ReferenceVectorToUse * ((*BoundScale)[DifferenceMaxElementIndex] / ReferenceVectorToUse[DifferenceMaxElementIndex]);
	}

	// Apply the bound values to the gizmo
	if (CurrentlyTrackedGizmo.IsValid() && CurrentlyTrackedGizmo->ActiveTarget)
	{
		TGuardValue<bool> DebounceGuard(bIgnoreCallbackForDebouncing, true);

		if (!bInDataEditSequence && bTriggerSequenceBookendsForNonSequenceUpdates)
		{
			CurrentlyTrackedGizmo->BeginTransformEditSequence();

			CurrentlyTrackedGizmo->UpdateTransformDuringEditSequence(
				GetGizmoTransformFromDataValues(CurrentlyTrackedGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::Local));

			CurrentlyTrackedGizmo->EndTransformEditSequence();
			UpdateDataValuesFromGizmo();
		}
		else
		{
			CurrentlyTrackedGizmo->UpdateTransformDuringEditSequence(
				GetGizmoTransformFromDataValues(CurrentlyTrackedGizmo->CurrentCoordinateSystem == EToolContextCoordinateSystem::Local));
		}
	}

	LastTranslation = *BoundTranslation;
	LastEulerAngles = *BoundEulerAngles;
	LastScale = *BoundScale;
}

void FTransformGizmoDataBinder::EndDataEditSequence()
{
	bInDataEditSequence = false;
	if (CurrentlyTrackedGizmo.IsValid() && CurrentlyTrackedGizmo->ActiveTarget)
	{
		TGuardValue<bool> DebounceGuard(bIgnoreCallbackForDebouncing, true);
		CurrentlyTrackedGizmo->EndTransformEditSequence();

		// This is necesarry in case tools modify the gizmo transform after EndTransformEditSequence, for instance
		// PolyEd resetting the scale after each drag...
		UpdateDataValuesFromGizmo();
	}
}



void FTransformGizmoDataBinder::BindToInitializedGizmo(UCombinedTransformGizmo* Gizmo, UTransformProxy*)
{
	if (!ensure(Gizmo && Gizmo->ActiveTarget && !BoundGizmos.Contains(Gizmo)))
	{
		return;
	}

	Gizmo->ActiveTarget->OnBeginTransformEdit.AddSP(this, &FTransformGizmoDataBinder::OnProxyBeginTransformEdit, Gizmo);
	Gizmo->ActiveTarget->OnTransformChanged.AddSP(this, &FTransformGizmoDataBinder::OnProxyTransformChanged, Gizmo);
	Gizmo->ActiveTarget->OnEndTransformEdit.AddSP(this, &FTransformGizmoDataBinder::OnProxyEndTransformEdit, Gizmo);

	Gizmo->ActiveTarget->OnBeginPivotEdit.AddSP(this, &FTransformGizmoDataBinder::OnProxyBeginTransformEdit, Gizmo);
	Gizmo->ActiveTarget->OnPivotChanged.AddSP(this, &FTransformGizmoDataBinder::OnProxyTransformChanged, Gizmo);
	Gizmo->ActiveTarget->OnEndPivotEdit.AddSP(this, &FTransformGizmoDataBinder::OnProxyEndTransformEdit, Gizmo);

	Gizmo->OnDisplaySpaceTransformChanged.AddSP(this, &FTransformGizmoDataBinder::OnDisplaySpaceTransformChanged);
	Gizmo->OnVisibilityChanged.AddSP(this, &FTransformGizmoDataBinder::OnGizmoVisibilityChanged);

	BoundGizmos.Add(Gizmo);

	if (!HasVisibleGizmo())
	{
		SetTrackedGizmo(Gizmo);
	}
}

void FTransformGizmoDataBinder::UnbindFromGizmo(UCombinedTransformGizmo* Gizmo, UTransformProxy*)
{
	if (!Gizmo)
	{
		return;
	}

	// Note about this ensure: it might seem ok to unbind once the ActiveTarget is no longer around,
	//  but it can be dangerous if the proxy is reused elsewhere instead of being immediately destroyed,
	//  causing us to continue to trigger these callbacks.
	ensure(Gizmo->ActiveTarget
		// The second condition here is for cases where we destroyed a gizmo without actually ever calling
		//  SetActiveTarget on it, meaning that we never fully bound to it in the first place.
		|| !BoundGizmos.Contains(Gizmo));

	if (Gizmo->ActiveTarget)
	{
		Gizmo->ActiveTarget->OnBeginTransformEdit.RemoveAll(this);
		Gizmo->ActiveTarget->OnTransformChanged.RemoveAll(this);
		Gizmo->ActiveTarget->OnEndTransformEdit.RemoveAll(this);

		Gizmo->ActiveTarget->OnBeginPivotEdit.RemoveAll(this);
		Gizmo->ActiveTarget->OnPivotChanged.RemoveAll(this);
		Gizmo->ActiveTarget->OnEndPivotEdit.RemoveAll(this);
	}

	Gizmo->OnDisplaySpaceTransformChanged.RemoveAll(this);
	Gizmo->OnVisibilityChanged.RemoveAll(this);

	BoundGizmos.Remove(Gizmo);

	if (CurrentlyTrackedGizmo == Gizmo)
	{
		CurrentlyTrackedGizmo.Reset();
		CurrentCustomLocalReferenceTransform.Reset();
	}
}

void FTransformGizmoDataBinder::BindToGizmoContextObject(UCombinedTransformGizmoContextObject* ContextObject)
{
	if (!ensure(IsValid(ContextObject)))
	{
		return;
	}

	ContextObject->OnGizmoCreated.AddSP(this, &FTransformGizmoDataBinder::BindToUninitializedGizmo);

	ContextObjectsToUnregisterWith.Add(ContextObject);
}

void FTransformGizmoDataBinder::BindToUninitializedGizmo(UCombinedTransformGizmo* Gizmo)
{
	Gizmo->OnSetActiveTarget.AddSP(this, &FTransformGizmoDataBinder::BindToInitializedGizmo);
	Gizmo->OnAboutToClearActiveTarget.AddSP(this, &FTransformGizmoDataBinder::UnbindFromGizmo);
}

FTransformGizmoDataBinder::~FTransformGizmoDataBinder()
{
	Reset();
}

void FTransformGizmoDataBinder::Reset()
{
	// Deinitialize while walking through a copied list of gizmos so we don't try
	// to delete from under ourselves.
	TArray<TWeakObjectPtr<UCombinedTransformGizmo>> Gizmos = BoundGizmos.Array();
	for (TWeakObjectPtr<UCombinedTransformGizmo> Gizmo : Gizmos)
	{
		if (!Gizmo.IsValid())
		{
			continue;
		}

		UnbindFromGizmo(Gizmo.Get(), Gizmo->ActiveTarget);
	}
	BoundGizmos.Empty();

	CurrentlyTrackedGizmo.Reset();
	CurrentCustomLocalReferenceTransform.Reset();
	DefaultCustomLocalReferenceTransform.Reset();

	BoundTranslation = VectorsToUseIfUnbound;
	BoundEulerAngles = VectorsToUseIfUnbound + 1;
	BoundScale = VectorsToUseIfUnbound + 2;

	for (TWeakObjectPtr<UCombinedTransformGizmoContextObject>& ContextObject : ContextObjectsToUnregisterWith)
	{
		if (ContextObject.IsValid())
		{
			ContextObject->OnGizmoCreated.RemoveAll(this);
		}
	}
	ContextObjectsToUnregisterWith.Empty();

	OnTrackedGizmoChanged.Clear();
}

void FTransformGizmoDataBinder::InitializeBoundVectors(FVector3d* Translation, FVector3d* RotationEulerAngles, FVector3d* Scale)
{
	if (Translation)
	{
		BoundTranslation = Translation;
	}
	if (RotationEulerAngles)
	{
		BoundEulerAngles = RotationEulerAngles;
	}
	if (Scale)
	{
		BoundScale = Scale;
	}
}

void FTransformGizmoDataBinder::UpdateCoordinateSystem()
{
	// When the coordinate system switches, we need to reset the delta mode because otherwise we will try to apply
	// the existing delta in a new direction from the old reference point, and the gizmo will jump.
	if (CurrentlyTrackedGizmo.IsValid() && CurrentlyTrackedGizmo->CurrentCoordinateSystem != LastCoordinateSystem)
	{
		if (bUsingDeltaMode)
		{
			ResetToDeltaMode();
		}

		LastCoordinateSystem = CurrentlyTrackedGizmo->CurrentCoordinateSystem;
		UpdateDataValuesFromGizmo();
	}
}

#undef LOCTEXT_NAMESPACE