// Copyright Epic Games, Inc. All Rights Reserved.

#include "WeightMapUtil.h"
#include "MeshDescription.h"


void UE::WeightMaps::FindVertexWeightMaps(const FMeshDescription* Mesh, TArray<FName>& PropertyNamesOut)
{
	const TAttributesSet<FVertexID>& VertexAttribs = Mesh->VertexAttributes();
	VertexAttribs.ForEach([&](const FName AttributeName, auto AttributesRef)
	{
		if (VertexAttribs.HasAttributeOfType<float>(AttributeName))
		{
			PropertyNamesOut.Add(AttributeName);
		}
	});
}


bool UE::WeightMaps::GetVertexWeightMap(const FMeshDescription* Mesh, FName AttributeName, FIndexedWeightMap1f& WeightMap, float DefaultValue)
{
	WeightMap.DefaultValue = DefaultValue;
	TArray<float>& Values = WeightMap.Values;

	int32 VertexCount = Mesh->Vertices().Num();
	if (Values.Num() < VertexCount)
	{
		Values.SetNum(VertexCount);
	}

	const TAttributesSet<FVertexID>& VertexAttribs = Mesh->VertexAttributes();
	TVertexAttributesConstRef<float> Attrib = VertexAttribs.GetAttributesRef<float>(AttributeName);
	if (Attrib.IsValid())
	{
		for (int32 k = 0; k < VertexCount; ++k)
		{
			Values[k] = Attrib.Get(FVertexID(k));
		}
		return true;
	}
	else
	{
		for (int32 k = 0; k < VertexCount; ++k)
		{
			Values[k] = DefaultValue;
		}
		return false;
	}
}
