// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// predeclare engine types
class UStaticMesh;
class UActorComponent;
struct FMeshBuildSettings;
struct FMeshDescription;


namespace UE
{
	namespace MeshDescription
	{
		//
		// Utility functions to preprocess a MeshDescription so that it is suitable for use in Modeling Tools, ie with fully-populated
		// autogen normals/tangents/etc.
		//

		/**
		 * Populate any auto-generated attributes of a FMeshDescription that are currently invalid (eg Normals/Tangents, which may be zero until calculated)
		 */
		MODELINGCOMPONENTS_API void InitializeAutoGeneratedAttributes(FMeshDescription& Mesh, const FMeshBuildSettings* BuildSettings);

		/**
		 * Populate any auto-generated attributes of a FMeshDescription that are currently invalid (eg Normals/Tangents, which may be zero until calculated)
		 * @param StaticMesh BuildSettings for the Mesh are fetched from this UStaticMesh
		 * @param SourceLOD BuildSettings from this LOD Index are used
		 */
		MODELINGCOMPONENTS_API void InitializeAutoGeneratedAttributes(FMeshDescription& Mesh, UStaticMesh* StaticMesh, int32 SourceLOD);

		/**
		 * Populate any auto-generated attributes of a FMeshDescription that are currently invalid (eg Normals/Tangents, which may be zero until calculated)
		 * @param StaticMeshComponent BuildSettings for the Mesh are fetched from this Component's UStaticMesh
		 * @param SourceLOD BuildSettings from this LOD Index are used
		 */
		MODELINGCOMPONENTS_API void InitializeAutoGeneratedAttributes(FMeshDescription& Mesh, UActorComponent* StaticMeshComponent, int32 SourceLOD);



		//
		// Utility functions to update UStaticMesh build settings
		//

		/** How to update a boolean build setting */
		enum class EBuildSettingBoolChange
		{
			Disable,
			Enable,
			NoChange
		};

		/** Set of changes to apply to StaticMesh Build Settings */
		struct FStaticMeshBuildSettingChange
		{
			EBuildSettingBoolChange AutoGeneratedNormals = EBuildSettingBoolChange::NoChange;
			EBuildSettingBoolChange AutoGeneratedTangents = EBuildSettingBoolChange::NoChange;
			EBuildSettingBoolChange UseMikkTSpaceTangents = EBuildSettingBoolChange::NoChange;
		};

		/** Utility function to apply build settings changes to a UStaticMesh */
		MODELINGCOMPONENTS_API void ConfigureBuildSettings(UStaticMesh* StaticMesh, int32 SourceLOD,
			FStaticMeshBuildSettingChange NewSettings);
	}
}
