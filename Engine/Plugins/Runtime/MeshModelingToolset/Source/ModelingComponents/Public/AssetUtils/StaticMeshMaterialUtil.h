// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UMaterialInterface;
class UStaticMesh;
struct FStaticMaterial;


namespace UE
{
namespace AssetUtils
{
	/**
	 * Information about a mesh material slot (eg a FStaticMaterial)
	 */
	struct FStaticMeshMaterialSlot
	{
		UMaterialInterface* Material;
		FName SlotName;
	};

	/**
	 * Information about the materials assigned to a StaticMesh LOD.
	 * The StaticMesh stores a Material List and per-mesh-Section assignments into that Material List.
	 * This relationship is not necessarily 1-1, multiple Sections can use the same material, and there
	 * can be arbitrary remappings, and the storage of this information is somewhat scattered across StaticMesh.
	 * This data structure encapsulates the critical information needed to (eg) assign materials
	 * to a new mesh/component such that they match the rendered materials on an existing StaticMesh.
	 * 
	 * The GetStaticMeshAssetMaterials() function can be used to build this data structure for a given StaticMesh LOD.
	 */
	struct FStaticMeshLODMaterialSetInfo
	{
		/** MaterialSlots array is a copy of the UStaticMesh::StaticMaterials array/data */
		TArray<FStaticMeshMaterialSlot> MaterialSlots;

		/** LOD Index the data below refers to */
		int32 LODIndex = 0;
		/** Number of Sections on the mesh LOD */
		int32 NumSections = 0;
		/** The index into MaterialSlots array of the Material assigned to each Section, in linear order */
		TArray<int32> SectionSlotIndexes;
		/** The Material assigned to each Section, in linear order */
		TArray<UMaterialInterface*> SectionMaterials;
	};

	/**
	 * Extract information about the material set for a given LODIndex of a StaticMeshAsset
	 */
	MODELINGCOMPONENTS_API bool GetStaticMeshLODAssetMaterials(
		UStaticMesh* StaticMeshAsset,
		int32 LODIndex,
		FStaticMeshLODMaterialSetInfo& MaterialInfoOut);

	/**
	 * Construct the linear per-section material list for a given LODIndex of a StaticMeshAsset
	 * @param MaterialListOut the list of linear per-section indices into the Asset Material List
	 * @param MaterialIndexOut the corresponding list of linear per-section indices into the Asset Material List
	 * @param MaterialSlotNameOut the corresponding list of material slot names
	 */
	MODELINGCOMPONENTS_API bool GetStaticMeshLODMaterialListBySection(
		UStaticMesh* StaticMeshAsset,
		int32 LODIndex,
		TArray<UMaterialInterface*>& MaterialListOut,
		TArray<int32>& MaterialIndexOut,
		TArray<FName>& MaterialSlotNameOut);

	/**
	 * Generate a new unique material slot name for the given SlotMaterial and NewSlotIndex,
	 * ensuring that the name is not already used in the set of ExistingMaterials
	 */
	MODELINGCOMPONENTS_API FName GenerateNewMaterialSlotName(
		const TArray<FStaticMaterial>& ExistingMaterials,
		UMaterialInterface* SlotMaterial,
		int32 NewSlotIndex);
}
}
