// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "Image/ImageBuilder.h"
#include "Image/ImageDimensions.h"
#include "VectorTypes.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE
{
namespace Geometry
{

/**
 * Texture2DBuilder is a utility class for creating/modifying various types of UTexture2D.
 * Use Initialize() functions to configure, can either generate a new UTexture2D (in the Transient package) or modify an existing UTexture2D.
 *
 * Currently the generated UTexture2D will only have Mip 0, and only Mip 0 can be edited.
 * The generated UTexture2D has format PF_B8G8R8A8 or PF_FloatRGBA.
 *
 * Use Commit() to lock and update the texture after editing is complete. LockForEditing() can be used to re-open.
 * By default textures are locked for editing on Initialize()
 *
 * If you have generated a UTexture2D by other means, you can use the static function ::CopyPlatformDataToSourceData() to populate the 
 * Source data from the PlatformData, which is required to save it as a UAsset. 
 */
class FTexture2DBuilder
{
public:

	/** 
	 * Supported texture types.
	 */
	enum class ETextureType
	{
		Color,					// byte-BGRA SRGB
		ColorLinear,			// byte-BGRA linear
		Roughness,				// byte-BGRA linear
		Metallic,				// byte-BGRA linear
		Specular,				// byte-BGRA linear
		NormalMap,				// byte-BGRA linear
		AmbientOcclusion,		// byte-BGRA linear
		EmissiveHDR				// float-RGBA linear
	};

protected:
	FImageDimensions Dimensions;
	ETextureType BuildType = ETextureType::Color;

	UTexture2D* RawTexture2D = nullptr;
	FColor* CurrentMipData = nullptr;
	FFloat16Color* CurrentMipDataFloat16 = nullptr;
	EPixelFormat CurrentPixelFormat = PF_B8G8R8A8;

public:

	virtual ~FTexture2DBuilder()
	{
		check(IsEditable() == false);
	}

	const FImageDimensions& GetDimensions() const
	{
		return Dimensions;
	}

	const ETextureType GetTextureType() const
	{
		return BuildType;
	}

	const EPixelFormat GetTexturePixelFormat() const
	{
		return CurrentPixelFormat;
	}

	/** @return true if texture buffer is FColor/8-bit BGRA */
	const bool IsByteTexture() const
	{
		return CurrentPixelFormat == PF_B8G8R8A8;
	}

	/** @return true if texture buffer is FFloat16Color/16-bit RGBA  */
	const bool IsFloat16Texture() const
	{
		return CurrentPixelFormat == PF_FloatRGBA;
	}

	/** @return the internal texture */
	UTexture2D* GetTexture2D() const
	{
		return RawTexture2D;
	}


	/**
	 * Create a new UTexture2D configured with the given BuildType and Dimensions
	 */
	UE_API bool Initialize(ETextureType BuildTypeIn, FImageDimensions DimensionsIn);


	/**
	 * Initialize the builder with an existing UTexture2D
	 */
	UE_API bool Initialize(UTexture2D* ExistingTexture, ETextureType BuildTypeIn, bool bLockForEditing = true);


	/**
	 * Create a new UTexture2D 'over' an existing UTexture2D, with the given BuildType and Dimensions
	 */
	UE_API bool InitializeAndReplaceExistingTexture(UTexture2D* ExistingTexture, ETextureType BuildTypeIn, FImageDimensions DimensionsIn);


	/**
	 * Create a new UTexture2D 'over' an existing UTexture2D using the same texture settings
	 * as a given source UTexture2D. This method only initializes the texture settings of ExistingTexture
	 * to match those of SourceTexture. To populate texture data, use Copy(..).
	 */
	UE_API bool InitializeAndReplaceExistingTexture(UTexture2D* ExistingTexture, UTexture2D* SourceTexture);


	/**
	 * Lock the Mip 0 buffer for editing
	 * @return true if lock was successfull. Will return false if already locked.
	 */
	UE_API bool LockForEditing();

	/** @return true if the texture data is currently locked and editable */
	bool IsEditable() const
	{
		return CurrentMipData != nullptr || CurrentMipDataFloat16 != nullptr;
	}


	/**
	 * Unlock the Mip 0 buffer and update the texture rendering data.
	 * This does not PostEditChange() so any materials using this texture may not be updated, the caller must do that.
	 * @param bUpdateSourceData if true, UpdateSourceData() is called to copy the current PlatformData to the SourceData
	 */
	UE_API void Commit(bool bUpdateSourceData = true);


	/**
	 * Copy the current PlatformData to the UTexture2D Source Data.
	 * This does not require the texture to be locked for editing, if it is not locked, a read-only lock will be acquired as needed
	 * @warning currently assumes both buffers are the same format
	 */
	UE_API void UpdateSourceData();

	/**
	 * Cancel active editing and unlock the texture data
	 */
	UE_API void Cancel();


	/**
	 * Clear all texels in the current Mip to the clear/default color for the texture build type
	 */
	UE_API void Clear();

	/**
	 * Clear all texels in the current Mip to the given ClearColor
	 */
	UE_API void Clear(const FColor& ClearColor);

	/**
	 * Clear all texels in the current Mip to the given ClearColor
	 */
	UE_API void Clear(const FFloat16Color& ClearColor);



	/**
	 * Get the FColor texel at the given X/Y coordinates
	 */
	const FColor& GetTexel(const FVector2i& ImageCoords) const
	{
		checkSlow(IsEditable() && IsByteTexture());
		int64 UseIndex = Dimensions.GetIndex(ImageCoords);
		return CurrentMipData[UseIndex];
	}


	/**
	 * Get the FColor texel at the given linear index
	 */
	const FColor& GetTexel(int64 LinearIndex) const
	{
		checkSlow(IsEditable() && IsByteTexture());
		return CurrentMipData[LinearIndex];
	}


	/**
	 * Get the FFloat16Color texel at the given X/Y coordinates
	 */
	const FFloat16Color& GetTexelFloat16(const FVector2i& ImageCoords) const
	{
		checkSlow(IsEditable() && IsFloat16Texture());
		int64 UseIndex = Dimensions.GetIndex(ImageCoords);
		return CurrentMipDataFloat16[UseIndex];
	}


	/**
	 * Get the FFloat16Color texel at the given linear index
	 */
	const FFloat16Color& GetTexelFloat16(int64 LinearIndex) const
	{
		checkSlow(IsEditable() && IsFloat16Texture());
		return CurrentMipDataFloat16[LinearIndex];
	}



	/**
	 * Set the texel at the given X/Y coordinates to the given FColor
	 */
	void SetTexel(const FVector2i& ImageCoords, const FColor& NewValue)
	{
		checkSlow(IsEditable() && IsByteTexture());
		int64 UseIndex = Dimensions.GetIndex(ImageCoords);
		CurrentMipData[UseIndex] = NewValue;
	}


	/**
	 * Set the texel at the given linear index to the given FColor
	 */
	void SetTexel(int64 LinearIndex, const FColor& NewValue)
	{
		checkSlow(IsEditable() && IsByteTexture());
		SetTexel(Dimensions.GetCoords(LinearIndex), NewValue);
	}


	/**
	 * Set the texel at the given X/Y coordinates to the given FFloat16Color
	 */
	void SetTexel(const FVector2i& ImageCoords, const FFloat16Color& NewValue)
	{
		checkSlow(IsEditable() && IsFloat16Texture());
		int64 UseIndex = Dimensions.GetIndex(ImageCoords);
		CurrentMipDataFloat16[UseIndex] = NewValue;
	}


	/**
	 * Set the texel at the given linear index to the given FFloat16Color
	 */
	void SetTexel(int64 LinearIndex, const FFloat16Color& NewValue)
	{
		checkSlow(IsEditable() && IsFloat16Texture());
		SetTexel(Dimensions.GetCoords(LinearIndex), NewValue);
	}


	/**
	 * Copy texel value from one linear index to another
	 */
	void CopyTexel(int64 FromLinearIndex, int64 ToLinearIndex)
	{
		checkSlow(IsEditable());
		if (IsByteTexture())
		{
			CurrentMipData[ToLinearIndex] = CurrentMipData[FromLinearIndex];
		}
		else
		{
			CurrentMipDataFloat16[ToLinearIndex] = CurrentMipDataFloat16[FromLinearIndex];
		}
	}


	/**
	 * populate texel values from floating-point SourceImage
	 * @param bConvertToSRGB if true, SourceImage is assumed to be Linear, and converted to SRGB for the Texture
	 */
	UE_API bool Copy(const TImageBuilder<FVector3f>& SourceImage, const bool bConvertToSRGB = false);

	/**
	 * Populate texel values from floating-point SourceImage.
	 * @param bConvertToSRGB if true, SourceImage is assumed to be Linear, and converted to SRGB for the Texture
	 */
	UE_API bool Copy(const TImageBuilder<FVector4f>& SourceImage, const bool bConvertToSRGB = false);

	/**
	 * Populate Source Data texel values from a floating point SourceImage
	 * @param SourceImage the image to copy to source data
	 * @param SourceDataFormat the texture format for the Source Data
	 * @param bConvertToSRGB if true, SourceImage is assumed to be Linear, and converted to SRGB for the Texture
	 */
	UE_API bool CopyImageToSourceData(const TImageBuilder<FVector4f>& SourceImage, const ETextureSourceFormat SourceDataFormat, const bool bConvertToSRGB = false);


	/**
	 * copy existing texel values to floating-point DestImage. Assumed to be same color space (eg values are copied without any linear/SRGB conversion)
	 */
	UE_API bool CopyTo(TImageBuilder<FVector4f>& DestImage) const;

	/**
	 * @return current locked Mip data. Nullptr if IsEditable() == false. Use IsByteTexture() / IsFloat16Texture() to determine type.
	 * @warning this point is invalid after the texture is Committed!
	 */
	const void* GetRawTexelBufferUnsafe() const
	{
		return CurrentMipData;
	}

	/**
	 * @return current locked Mip data. Nullptr if IsEditable() == false. Use IsByteTexture() / IsFloat16Texture() to determine type.
	 * @warning this point is invalid after the texture is Committed!
	 */
	void* GetRawTexelBufferUnsafe()
	{
		return CurrentMipData;
	}


	/**
	 * @return the default FColor for the current texture build type
	 */
	UE_API const FColor& GetClearColor() const;


	/**
	 * @return the default FFloat16Color for the current texture build type
	 */
	UE_API FFloat16Color GetClearColorFloat16() const;


	/**
	 * Use a FTexture2DBuilder to copy the PlatformData to the UTexture2D Source data, so it can be saved as an Asset
	 */
	static UE_API bool CopyPlatformDataToSourceData(UTexture2D* Texture, ETextureType TextureType);


	/**
	 * Create a new UTexture2D of the given TextureType from the given SourceImage
	 * @param bConvertToSRGB if true, assumption is SourceImage is Linear and we want output UTexture2D to be SRGB. Only valid for 8-bit textures.
	 * @param bPopulateSourceData if true, first the PlatformData is initialized, then it is copied to the SourceData. This is necessary if the UTexture2D is going to be stored as an Asset (but can also be done explicitly using Commit())
	 */
	template<typename PixelType>
	static UTexture2D* BuildTextureFromImage(const TImageBuilder<PixelType>& SourceImage, FTexture2DBuilder::ETextureType TextureType, const bool bConvertToSRGB, bool bPopulateSourceData = true)
	{
		FTexture2DBuilder TexBuilder;
		TexBuilder.Initialize(TextureType, SourceImage.GetDimensions());
		TexBuilder.Copy(SourceImage, bConvertToSRGB);
		TexBuilder.Commit(bPopulateSourceData);
		return TexBuilder.GetTexture2D();
	}

private:
	UE_API bool InitializeInternal(ETextureType BuildTypeIn, FImageDimensions DimensionsIn, UTexture2D* CreatedTextureIn);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
