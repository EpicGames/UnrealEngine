// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "Image/ImageDimensions.h"
#include "Image/ImageBuilder.h"
#include "Engine/Texture2D.h"
#include "Math/Color.h"



namespace UE
{
namespace AssetUtils
{
	using namespace UE::Geometry;

	/**
	 * Extract the Dimensions and image pixels from an input TextureMap
	 * By default, the "Source" texture data is read. This is only available in-Editor. At runtime, only "Platform" data is available.
	 * @param TextureMap the texture map to read
	 * @param DestImageOut the result of the texture read
	 * @param bPreferPlatformData if true, platform data will be returned even in-Editor.
	 * @return true on success
	 */
	MODELINGCOMPONENTS_API bool ReadTexture(
		UTexture2D* TextureMap,
		TImageBuilder<FVector4f>& DestImageOut,
		const bool bPreferPlatformData = false);

	/**
	 * Convert input UTexture2D to single-channel. Assumption is it has more than one channel. Red channel is used.
	 * @return true on success
	 */
	MODELINGCOMPONENTS_API bool ConvertToSingleChannel(UTexture2D* TextureMap);

	/**
	 * Issue requests to the render thread to force virtual textures to load for the given screen dimensions.
	 * @param bWaitForPrefetchToComplete if true, FlushRenderingCommands() is called to wait for the textures to finish loading
	 * @return true on success
	 */
	MODELINGCOMPONENTS_API bool ForceVirtualTexturePrefetch(FImageDimensions ScreenSpaceDimensions, bool bWaitForPrefetchToComplete = true);

	/**
	 * Save image stored in Pixels, of given Dimensions to <Project>/Intermediate/DebugSubFolder/FilenameBase_<FileCounter>.bmp
	 * If UseFileCounter is not specified, an internal static counter that is incremented each call is used.
	 */
	MODELINGCOMPONENTS_API bool SaveDebugImage(
		const TArray<FColor>& Pixels,
		FImageDimensions Dimensions,
		FString DebugSubfolder,
		FString FilenameBase,
		int32 UseFileCounter = -1);

	/**
	 * Save image stored in Pixels, of given Dimensions to <Project>/Intermediate/DebugSubFolder/FilenameBase_<FileCounter>.bmp
	 * If UseFileCounter is not specified, an internal static counter that is incremented each call is used.
	 */
	MODELINGCOMPONENTS_API bool SaveDebugImage(
		const TArray<FLinearColor>& Pixels,
		FImageDimensions Dimensions,
		bool bConvertToSRGB,
		FString DebugSubfolder,
		FString FilenameBase,
		int32 UseFileCounter = -1);

	/**
	 * Save Image to <Project>/Intermediate/DebugSubFolder/FilenameBase_<FileCounter>.bmp
	 * If UseFileCounter is not specified, an internal static counter that is incremented each call is used.
	 */
	MODELINGCOMPONENTS_API bool SaveDebugImage(
		const FImageAdapter& Image,
		bool bConvertToSRGB,
		FString DebugSubfolder,
		FString FilenameBase,
		int32 UseFileCounter = -1);
}
}

