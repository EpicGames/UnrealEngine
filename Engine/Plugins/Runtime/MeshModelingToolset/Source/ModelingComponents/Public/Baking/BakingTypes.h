// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
* Bake maps enums
*/

#include "CoreMinimal.h"

#include "BakingTypes.generated.h"

UENUM()
enum class EBakeTextureResolution 
{
	Resolution16 = 16 UMETA(DisplayName = "16 x 16"),
	Resolution32 = 32 UMETA(DisplayName = "32 x 32"),
	Resolution64 = 64 UMETA(DisplayName = "64 x 64"),
	Resolution128 = 128 UMETA(DisplayName = "128 x 128"),
	Resolution256 = 256 UMETA(DisplayName = "256 x 256"),
	Resolution512 = 512 UMETA(DisplayName = "512 x 512"),
	Resolution1024 = 1024 UMETA(DisplayName = "1024 x 1024"),
	Resolution2048 = 2048 UMETA(DisplayName = "2048 x 2048"),
	Resolution4096 = 4096 UMETA(DisplayName = "4096 x 4096"),
	Resolution8192 = 8192 UMETA(DisplayName = "8192 x 8192")
};


UENUM()
enum class EBakeTextureBitDepth
{
	ChannelBits8 UMETA(DisplayName = "8 bits/channel"),
	ChannelBits16 UMETA(DisplayName = "16 bits/channel")
};


UENUM()
enum class EBakeTextureSamplesPerPixel
{
	Sample1 = 1 UMETA(DisplayName = "1"),
	Sample4 = 4 UMETA(DisplayName = "4"),
	Sample16 = 16 UMETA(DisplayName = "16"),
	Sample64 = 64 UMETA(DisplayName = "64"),
	Sample256 = 256 UMETA(DisplayName = "256")
};

