// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Scene/SceneCapturePhotoSet.h"
#include "Sampling/MeshBakerCommon.h"
#include "Sampling/MeshMapBaker.h"
#include "Image/ImageInfilling.h"
#include "Baking/BakingTypes.h"
#include "DynamicMesh/MeshTangents.h"
#include "CoreMinimal.h"

#define UE_API MODELINGCOMPONENTS_API

class UTexture2D;
class AActor;

namespace UE
{
namespace Geometry
{






class FRenderCaptureOcclusionHandler
{
public:
	UE_API FRenderCaptureOcclusionHandler(FImageDimensions Dimensions);

	UE_API void RegisterSample(const FVector2i& ImageCoords, bool bSampleValid);

	UE_API void PushInfillRequired(bool bInfillRequired);

	UE_API void ComputeAndApplyInfill(TArray<TUniquePtr<TImageBuilder<FVector4f>>>& Images);

private:

	struct FSampleStats {
		uint16 NumValid = 0;
		uint16 NumInvalid = 0;

		// These are required by the TMarchingPixelInfill implementation
		bool operator==(const FSampleStats& Other) const;
		bool operator!=(const FSampleStats& Other) const;
		FSampleStats& operator+=(const FSampleStats& Other);
		static FSampleStats Zero();
	};

	void ComputeInfill();

	void ApplyInfill(TImageBuilder<FVector4f>& Image) const;

	// Collect some sample stats per pixel, used to determine if a pixel requires infill or not
	TImageBuilder<FSampleStats> SampleStats;

	// InfillRequire[i] indicates if the i-th image passed to ComputeAndApplyInfill needs infill
	TArray<bool> InfillRequired;

	TMarchingPixelInfill<FSampleStats> Infill;
};





class FSceneCapturePhotoSetSampler : public FMeshBakerDynamicMeshSampler
{
public:
	UE_API FSceneCapturePhotoSetSampler(
		FSceneCapturePhotoSet* SceneCapture,
		float ValidSampleDepthThreshold,
		const FDynamicMesh3* Mesh,
		const FDynamicMeshAABBTree3* Spatial,
		const FMeshTangentsd* Tangents); 

	UE_NONCOPYABLE(FSceneCapturePhotoSetSampler);

	UE_API virtual bool SupportsCustomCorrespondence() const override;

	// Warning: Expects that Sample.BaseSample.SurfacePoint and Sample.BaseNormal are set when the function is called
	UE_API virtual void* ComputeCustomCorrespondence(const FMeshUVSampleInfo& SampleInfo, FMeshMapEvaluator::FCorrespondenceSample& Sample) const override;

	UE_API virtual bool IsValidCorrespondence(const FMeshMapEvaluator::FCorrespondenceSample& Sample) const override;

private:
	FSceneCapturePhotoSet* SceneCapture = nullptr;
	float ValidSampleDepthThreshold = 0;
	TFunction<bool(const FVector3d&, const FVector3d&)> VisibilityFunction;
};





struct FSceneCaptureConfig
{
	int32 RenderCaptureImageSize = 1024;
	bool bAntiAliasing = false;
	double FieldOfViewDegrees = 45.0;
	double NearPlaneDist = 1.0;

	FRenderCaptureTypeFlags Flags = FRenderCaptureTypeFlags::None();

	UE_API bool operator==(const FSceneCaptureConfig&) const;
	UE_API bool operator!=(const FSceneCaptureConfig&) const;
};

MODELINGCOMPONENTS_API
void ConfigureSceneCapture(
	FSceneCapturePhotoSet& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& Config,
	bool bAllowCancel);

MODELINGCOMPONENTS_API
FRenderCaptureTypeFlags UpdateSceneCapture(
	FSceneCapturePhotoSet& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& DesiredConfig,
	bool bAllowCancel);

MODELINGCOMPONENTS_API
FSceneCaptureConfig GetSceneCaptureConfig(
	const FSceneCapturePhotoSet& SceneCapture,
	FSceneCapturePhotoSet::ECaptureTypeStatus QueryStatus = FSceneCapturePhotoSet::ECaptureTypeStatus::Computed);

// Return a render capture baker, note the lifetime of all arguments such match the lifetime of the returned baker
MODELINGCOMPONENTS_API
TUniquePtr<FMeshMapBaker> MakeRenderCaptureBaker(
	FDynamicMesh3* BaseMesh,
	TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> BaseMeshTangents,
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> BaseMeshUVCharts,
	FSceneCapturePhotoSet* SceneCapture,
	FSceneCapturePhotoSetSampler* Sampler,
	FRenderCaptureTypeFlags PendingBake,
	int32 TargetUVLayer,
	EBakeTextureResolution TextureImageSize,
	EBakeTextureSamplesPerPixel SamplesPerPixel,
	FRenderCaptureOcclusionHandler* OcclusionHandler);

struct FRenderCaptureTextures
{
	UTexture2D* BaseColorMap = nullptr;
	UTexture2D* NormalMap = nullptr;
	UTexture2D* PackedMRSMap = nullptr;
	UTexture2D* MetallicMap = nullptr;
	UTexture2D* RoughnessMap = nullptr;
	UTexture2D* SpecularMap = nullptr;
	UTexture2D* EmissiveMap = nullptr;
	UTexture2D* OpacityMap = nullptr;
	UTexture2D* SubsurfaceColorMap = nullptr;
};

// Note: The source data in the textures is *not* updated by this function
MODELINGCOMPONENTS_API
void GetTexturesFromRenderCaptureBaker(
	FMeshMapBaker& Baker,
	FRenderCaptureTextures& TexturesOut);












////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// Everything that follows is deprecated 
////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UE_DEPRECATED(5.4, "Please use the overload passing SceneCapture by reference. Using smart pointers was an error since ownership semantics were not intended.")
MODELINGCOMPONENTS_API
void ConfigureSceneCapture(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& Config,
	bool bAllowCancel);

UE_DEPRECATED(5.4, "Please use the overload passing SceneCapture by reference. Using smart pointers was an error since ownership semantics were not intended.")
MODELINGCOMPONENTS_API
FRenderCaptureTypeFlags UpdateSceneCapture(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FSceneCaptureConfig& DesiredConfig,
	bool bAllowCancel);

UE_DEPRECATED(5.4, "Please use the overload passing SceneCapture by reference. Using smart pointers was an error since ownership semantics were not intended.")
MODELINGCOMPONENTS_API
FSceneCaptureConfig GetSceneCaptureConfig(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	FSceneCapturePhotoSet::ECaptureTypeStatus QueryStatus = FSceneCapturePhotoSet::ECaptureTypeStatus::Computed);

// Note: The source data in the textures is *not* updated by this function
UE_DEPRECATED(5.4, "Please use the overload passing Baker by reference. Using smart pointers was an error since ownership semantics were not intended.")
MODELINGCOMPONENTS_API
void GetTexturesFromRenderCaptureBaker(
	const TUniquePtr<FMeshMapBaker>& Baker,
	FRenderCaptureTextures& TexturesOut);

struct UE_DEPRECATED(5.3, "FRenderCaptureOptions is only used by deprecated functions, see those deprecation notes for more info.") MODELINGCOMPONENTS_API FRenderCaptureOptions
{
	//
	// Material approximation settings
	//

	int32 RenderCaptureImageSize = 1024;
	bool bAntiAliasing = false;

	// render capture parameters
	double FieldOfViewDegrees = 45.0;
	double NearPlaneDist = 1.0;

	//
	// Material output settings
	//

	bool bBakeBaseColor = true;
	bool bBakeRoughness = true;
	bool bBakeMetallic = true;
	bool bBakeSpecular = true;
	bool bBakeEmissive = true;
	bool bBakeNormalMap = true;
	bool bBakeOpacity = true;
	bool bBakeSubsurfaceColor = true;
	bool bBakeDeviceDepth = true;
	
	bool bUsePackedMRS = true;
};

struct UE_DEPRECATED(5.3, "FRenderCaptureUpdate is only used by deprecated functions, see those deprecation notes for more info.") MODELINGCOMPONENTS_API FRenderCaptureUpdate
{
	// Default to true so that we can use the default value to trigger all update code paths
	bool bUpdatedBaseColor = true;
	bool bUpdatedRoughness = true;
	bool bUpdatedMetallic = true;
	bool bUpdatedSpecular = true;
	bool bUpdatedEmissive = true;
	bool bUpdatedNormalMap = true;
	bool bUpdatedOpacity = true;
	bool bUpdatedSubsurfaceColor = true;
	bool bUpdatedDeviceDepth = true;
	bool bUpdatedPackedMRS = true;
};


/**
 * This function computes the SceneCapture for the first time
 */
UE_DEPRECATED(5.3, "Please use ConfigureSceneCapture and FSceneCapturePhotoSet::Compute instead.")
MODELINGCOMPONENTS_API
TUniquePtr<FSceneCapturePhotoSet> CapturePhotoSet(
	const TArray<TObjectPtr<AActor>>& Actors,
	const FRenderCaptureOptions& Options,
	FRenderCaptureUpdate& Update,
	bool bAllowCancel);

/**
* This function efficiently updates the given SceneCapture and returns a struct indicating which channels were updated
* 
* - If the requested Options and Actors have already been captured the call is a no-op
* - If the given Options disable existing capture channels then the call just clears the corresponding photo sets
* - If the given Options enable a new capture channel then the new photos set are captured and added to the SceneCapture
* - If the given Actors are different the ones cached in the SceneCapture, or if the given Options changes a parameter
*   affecting all photo sets (e.g., photo resolution), then all the photo sets are cleared and recomputed
*/
UE_DEPRECATED(5.3, "Please use UpdateSceneCapture instead.")
MODELINGCOMPONENTS_API
FRenderCaptureUpdate UpdatePhotoSets(
	const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture,
	const TArray<TObjectPtr<AActor>>& Actors,
	const FRenderCaptureOptions& Options,
	bool bAllowCancel);

UE_DEPRECATED(5.3, "Please use GetSceneCaptureConfig instead.")
MODELINGCOMPONENTS_API
FRenderCaptureOptions GetComputedPhotoSetOptions(const TUniquePtr<FSceneCapturePhotoSet>& SceneCapture);


// Return a render capture baker, note the lifetime of all arguments such match the lifetime of the returned baker
UE_DEPRECATED(5.3, "Please use the overload where PendingBake has type FRenderCaptureTypeFlags instead.")
MODELINGCOMPONENTS_API
TUniquePtr<FMeshMapBaker> MakeRenderCaptureBaker(
	FDynamicMesh3* BaseMesh,
	TSharedPtr<FMeshTangentsd, ESPMode::ThreadSafe> BaseMeshTangents,
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> BaseMeshUVCharts,
	FSceneCapturePhotoSet* SceneCapture,
	FSceneCapturePhotoSetSampler* Sampler,
	FRenderCaptureOptions PendingBake,
	int32 TargetUVLayer,
	EBakeTextureResolution TextureImageSize,
	EBakeTextureSamplesPerPixel SamplesPerPixel,
	FRenderCaptureOcclusionHandler* OcclusionHandler);

PRAGMA_ENABLE_DEPRECATION_WARNINGS

} // end namespace Geometry
} // end namespace UE

#undef UE_API
