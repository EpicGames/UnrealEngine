// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveGizmo.h"
#include "InteractiveGizmoBuilder.h"
#include "BaseGizmos/BrushStampIndicator.h"

#include "VolumetricBrushStampIndicator.generated.h"

class UObject;
class UInteractiveTool;
class UPrimitiveComponent;
class UPreviewMesh;
class UWorld;

UCLASS(MinimalAPI)
class UVolumetricBrushStampIndicatorBuilder : public UInteractiveGizmoBuilder
{
	GENERATED_BODY()
public:
	MODELINGCOMPONENTS_API virtual UInteractiveGizmo* BuildGizmo(const FToolBuilderState& SceneState) const override;
};

/*
 * UVolumetricBrushStampIndicator is a simple 3D brush indicator that can be volumetric 
 */
UCLASS(Transient, MinimalAPI)
class UVolumetricBrushStampIndicator : public UBrushStampIndicator
{
	GENERATED_BODY()

public:
	MODELINGCOMPONENTS_API virtual void Shutdown() override;

	MODELINGCOMPONENTS_API void SetVolumetricBrushVisible(bool bInVisible);

	MODELINGCOMPONENTS_API bool IsVolumetricBrushVisible();

	MODELINGCOMPONENTS_API void MakeBrushIndicatorMesh(UInteractiveTool* ParentTool, UWorld* World);

	MODELINGCOMPONENTS_API UPreviewMesh* GetPreviewMesh() const;
private:
	UPROPERTY()
	TObjectPtr<UPreviewMesh> SphereMesh;
};

