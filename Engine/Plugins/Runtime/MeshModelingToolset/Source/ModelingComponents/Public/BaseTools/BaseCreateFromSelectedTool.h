// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolManager.h"
#include "MeshOpPreviewHelpers.h"
#include "PropertySets/OnAcceptProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "BaseCreateFromSelectedTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UCombinedTransformGizmo;
class UTransformProxy;
class UBaseCreateFromSelectedTool;

/**
 * ToolBuilder for UBaseCreateFromSelectedTool
 */
UCLASS(MinimalAPI)
class UBaseCreateFromSelectedToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;

public:
	virtual TOptional<int32> MaxComponentsSupported() const { return TOptional<int32>(); }
	virtual int32 MinComponentsSupported() const { return 1; }
};


UENUM()
enum class EBaseCreateFromSelectedTargetType
{
	/** Create and write to a new object with a given name. */
	NewObject,

	/** Write to the first object in the input selection. */
	FirstInputObject,

	/** Write to the last object in the input selection. */
	LastInputObject
};


UCLASS(MinimalAPI)
class UBaseCreateFromSelectedHandleSourceProperties : public UOnAcceptHandleSourcesProperties
{
	GENERATED_BODY()
public:
	/** Defines the object the tool output is written to. */
	UPROPERTY(EditAnywhere, Category = OutputObject, meta = (DisplayName = "Write To", NoResetToDefault))
	EBaseCreateFromSelectedTargetType OutputWriteTo = EBaseCreateFromSelectedTargetType::NewObject;

	/** Base name of the newly generated object to which the output is written to. */
	UPROPERTY(EditAnywhere, Category = OutputObject, meta = (TransientToolProperty, DisplayName = "Name",
		EditCondition = "OutputWriteTo == EBaseCreateFromSelectedTargetType::NewObject", EditConditionHides, NoResetToDefault))
	FString OutputNewName;

	/** Name of the existing object to which the output is written to. */
	UPROPERTY(VisibleAnywhere, Category = OutputObject, meta = (TransientToolProperty, DisplayName = "Name",
		EditCondition = "OutputWriteTo != EBaseCreateFromSelectedTargetType::NewObject", EditConditionHides, NoResetToDefault))
	FString OutputExistingName;
};

UCLASS(MinimalAPI)
class UBaseCreateFromSelectedCollisionProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Whether to transfer collision settings and any simple collision shapes from the source object(s) to the new object. */
	UPROPERTY(EditAnywhere, Category = OutputObject)
	bool bTransferCollision = true;
};

/**
 * Properties of UI to adjust input meshes
 */
UCLASS(MinimalAPI)
class UTransformInputsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Show transform gizmo in the viewport to allow changing translation, rotation and scale of input meshes. */
	UPROPERTY(EditAnywhere, Category = Transform, meta = (DisplayName = "Show Gizmo"))
	bool bShowTransformGizmo = true;
};


/**
 * UBaseCreateFromSelectedTool is a base Tool (must be subclassed) 
 * that provides support for common functionality in tools that create a new mesh from a selection of one or more existing meshes
 */
UCLASS(MinimalAPI)
class UBaseCreateFromSelectedTool : public UMultiSelectionMeshEditingTool, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()
protected:
	using FFrame3d = UE::Geometry::FFrame3d;
public:
	UBaseCreateFromSelectedTool() = default;

	//
	// InteractiveTool API - generally does not need to be modified by subclasses
	//

	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	

protected:

	//
	// UBaseCreateFromSelectedTool API - subclasses typically implement these functions
	//

	/**
	 * After preview is created, this is called to convert inputs and set preview materials
	 * (grouped together because materials may come from inputs)
	 * Subclasses should always implement this.
	 * @param bSetPreviewMesh If true, function may try to set an initial "early" preview mesh to have some initial surface on tool start.  (Not all tools will actually create this.)
	 *						  This boolean is here in case a subclass needs to call this setup function again later (e.g. to change the materials used), when it won't need or want the preview surface to be created
	 */
	virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh = true) { check(false);  }

	/** overload to initialize any added properties in subclasses; called during setup */
	virtual void SetupProperties() {}

	/** overload to save any added properties in the subclasses; called on shutdown */
	virtual void SaveProperties() {}

	/** optional overload to set callbacks on preview, e.g. to visualize results; called after preview is created. */
	virtual void SetPreviewCallbacks() {}

	/** Return the name to be used for generated assets.  Note: Asset name will be prefixed by source actor name if only actor was selected. */
	virtual FString GetCreatedAssetName() const { return TEXT("Generated"); }

	/** Return the name of the action to be used in the Undo stack */
	UE_API virtual FText GetActionName() const;

	/** Return the materials to be used on the output mesh on tool accept; defaults to the materials set on the preview */
	UE_API virtual TArray<UMaterialInterface*> GetOutputMaterials() const;

	// Override this to control whether the Transfer Collision setting is available
	virtual bool SupportsCollisionTransfer() const
	{
		return true;
	}

	// Override this to control which inputs should transfer collision to the output (if collision transfer is enabled)
	virtual bool KeepCollisionFrom(int32 TargetIndex) const
	{
		return true;
	}


	/**
	 * IDynamicMeshOperatorFactory implementation that subclass must override and implement
	 */
	virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override
	{
		check(false);
		return TUniquePtr<UE::Geometry::FDynamicMeshOperator>();
	}

protected:

	/** Helper to build asset names */
	UE_API FString PrefixWithSourceNameIfSingleSelection(const FString& AssetName) const;

	// Helpers for managing transform gizoms; typically do not need to be overloaded
	UE_API virtual void UpdateGizmoVisibility();
	UE_API virtual void SetTransformGizmos();
	UE_API virtual void TransformChanged(UTransformProxy* Proxy, FTransform Transform);

	// Helper to generate assets when a result is accepted; typically does not need to be overloaded
	UE_API virtual void GenerateAsset(const FDynamicMeshOpResult& Result);

	// Helper to generate assets when a result is accepted; typically does not need to be overloaded
	UE_API virtual void UpdateAsset(const FDynamicMeshOpResult& Result, UToolTarget* Target);

	// Which of the transform gizmos to hide, or -1 if all gizmos can be shown
	UE_API virtual int32 GetHiddenGizmoIndex() const;

protected:

	UPROPERTY()
	TObjectPtr<UTransformInputsToolProperties> TransformProperties;

	UPROPERTY()
	TObjectPtr<UCreateMeshObjectTypeProperties> OutputTypeProperties;

	UPROPERTY()
	TObjectPtr<UBaseCreateFromSelectedHandleSourceProperties> HandleSourcesProperties;

	UPROPERTY()
	TObjectPtr<UBaseCreateFromSelectedCollisionProperties> CollisionProperties;

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview;

	UPROPERTY()
	TArray<TObjectPtr<UTransformProxy>> TransformProxies;

	UPROPERTY()
	TArray<TObjectPtr<UCombinedTransformGizmo>> TransformGizmos;
};

#undef UE_API
