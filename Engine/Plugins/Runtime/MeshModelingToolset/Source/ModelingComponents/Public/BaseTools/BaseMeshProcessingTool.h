// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SingleTargetWithSelectionTool.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "WeightMapTypes.h"
#include "MeshOpPreviewHelpers.h"
#include "PropertySets/WeightMapSetProperties.h"
#include "BaseMeshProcessingTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

// predeclarations
struct FMeshDescription;
class UDynamicMeshComponent;
class UPreviewMesh;
class UBaseMeshProcessingTool;
PREDECLARE_USE_GEOMETRY_CLASS(FMeshBoundaryLoops);
PREDECLARE_USE_GEOMETRY_CLASS(FMeshNormals);
using UE::Geometry::FDynamicMesh3;
using UE::Geometry::FIndexedWeightMap1f;
/**
 * ToolBuilder for UBaseMeshProcessingTool
 */
UCLASS(MinimalAPI)
class UBaseMeshProcessingToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	virtual bool RequiresInputSelection() const override { return false; }
	UE_API virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const override;


public:
	// subclass must override!
	UE_DEPRECATED(5.5, "MakeNewToolInstance is deprecated, please use CreateNewTool instead")
	virtual UBaseMeshProcessingTool* MakeNewToolInstance(UObject* Outer) const { check(false); return nullptr; }

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};




/**
 * UBaseMeshProcessingTool is a base Tool (ie has no functionality of it's own and must be subclassed) 
 * that provides the following structure:
 *   - a Background-Compute-With-Preview Temp Actor/Component is created based on the input mesh
 *   - The Subclass provides FDynamicMeshOperator instances (via IDynamicMeshOperatorFactory) that process/modify and update this Preview
 *   - PropertySets with custom visibility can be registered, and on change will invalidate the current computation
 *   
 * Optional support for a WeightMap property set and tracking of active weight map can be enabled by calling
 * AddWeightMapPropertySet(), GetActiveWeightMap() will then return the active WeightMap, and changes to the 
 * WeightMap selection will invalidate the computation.
 *
 * Most subclasses will only need to define their PropertySets and implement MakeNewOperator(), see eg SmoothMeshTool for a minimal example
 *
 * Other functions:
 *   - GetInitialMesh() : return reference to copy of initial mesh, used to initialize FDynamicMeshOperator
 *   - GetUPreviewMesh() : return the UPreviewMesh inside the background compute (for configuration/etc - should not directly touch the mesh!)
 *   - GetPreviewTransform() : return active FTransform on the Preview mesh, should be passed to FDynamicMeshOperator unless it is outputting world position
 *   - InvalidateResult() : subclasses call this to notify the base class that current result/computation has been invalidated
 *
 * The Base tool will do various optional precomputations or changes to the input mesh, which can be configured by
 * overriding various functions below.
 * 
 *   RequiresInitialVtxNormals() : return true (default=false) to calculate per-vertex normals on the input mesh, returned by GetInitialVtxNormals()
 *
 *   RequiresInitialBoundaryLoops() : return true (default=false) to calculate boundary loops on the input mesh, returned by GetInitialBoundaryLoops()
 *
 *   RequiresScaleNormalization() : return true (default=true) to apply an initial scale to the input mesh so that it has consistent size
 *     before being sent into the computation. Scaling factor (eg for scaling UI constants) can be accessed via GetScaleNormalizationFactor()
 *
 */
UCLASS(MinimalAPI)
class UBaseMeshProcessingTool : public USingleTargetWithSelectionTool, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()
protected:
	using FFrame3d = UE::Geometry::FFrame3d;
public:
	UBaseMeshProcessingTool() = default;

	//
	// InteractiveTool API - generally does not need to be modified by subclasses
	//

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void OnTick(float DeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	UE_API virtual bool HasAccept() const override;
	UE_API virtual bool CanAccept() const override;


protected:

	//
	// UBaseMeshProcessingTool REQUIRED API - subclasses must implement these functions
	//

	/**
	 * IDynamicMeshOperatorFactory implementation that subclass must override and implement
	 */
	virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override
	{
		check(false);
		return TUniquePtr<UE::Geometry::FDynamicMeshOperator>();
	}


	/**
	 * called when Tool is Accepted to determine whether it is safe to only update vertex positions, or if entire target mesh must be replaced.
	 */
	virtual bool HasMeshTopologyChanged() const
	{
		unimplemented();
		return true;
	}

	//
	// UBaseMeshProcessingTool OPTIONAL API - subclasses may implement these functions
	//

	/** @return text string shown to user for Accept Transaction that updates input mesh. Subclass should override. */
	UE_API virtual FText GetToolMessageString() const;

	/** @return text string shown to user for Accept Transaction that updates input mesh. Subclass should override. */
	UE_API virtual FText GetAcceptTransactionName() const;

	/** This function is called during ::Setup() to allow subclass to register property sets, before kicking off initial computation */
	virtual void InitializeProperties() {}

	/** This function is called during ::Shutdown so that subclass may perform final processing and save property sets */
	virtual void OnShutdown(EToolShutdownType ShutdownType) {}


	//
	// Optional Property Set API - subclasses can use this to manage property sets with configurable visibility that invalidate the precompute
	//

	/**
	 * Register an optional property set with the given VisibilityFunc
	 */
	template<class PropSetType>
	PropSetType* AddOptionalPropertySet(TUniqueFunction<bool()> VisibilityFunc, bool bChangeInvalidatesResult = true)
	{
		return AddOptionalPropertySet<PropSetType>(MoveTemp(VisibilityFunc), []() {}, bChangeInvalidatesResult);
	}

	/**
	 * Register an optional property set with the given VisibilityFunc, and call OnModifiedFunc if any of the properties change
	 */
	template<class PropSetType>
	PropSetType* AddOptionalPropertySet( TUniqueFunction<bool()> VisibilityFunc, TUniqueFunction<void()> OnModifiedFunc, bool bChangeInvalidatesResult)
	{
		PropSetType* PropSet = NewObject<PropSetType>(this);
		AddOptionalPropertySet(PropSet, MoveTemp(VisibilityFunc), MoveTemp(OnModifiedFunc), bChangeInvalidatesResult);
		return PropSet;
	}

	/** Call this function to update optional property sets visibility. Should call base implementation */
	UE_API virtual void UpdateOptionalPropertyVisibility();

	UE_API virtual void AddOptionalPropertySet(UInteractiveToolPropertySet* PropSet, 
		TUniqueFunction<bool()> VisibilityFunc, 
		TUniqueFunction<void()> OnModifiedFunc,
		bool bChangeInvalidatesResult);

	struct FOptionalPropertySet
	{
		TUniqueFunction<bool()> IsVisible;
		TUniqueFunction<void()> OnModifiedFunc;
		bool bInvalidateOnModify;
		TWeakObjectPtr<UInteractiveToolPropertySet> PropertySet;
	};

	TArray<FOptionalPropertySet> OptionalProperties;
	UE_API virtual void OnOptionalPropSetModified(int32 Index);
	UE_API virtual void SavePropertySets();


	// Preview object holds temporary Actor with preview mesh component
	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPreviewMesh* GetUPreviewMesh() const { return Preview->PreviewMesh; }
	const FTransform& GetPreviewTransform() const { return OverrideTransform; }


private:
	bool bResultValid = false;
protected:
	UE_API virtual void InvalidateResult();
	UE_API virtual void UpdateResult();


	//
	// Initial data, used to initialize background compute / etc
	//

private:
	FDynamicMesh3 InitialMesh;
protected:
	/** @return duplciate of initial mesh (possibly with optional size normalization) */
	const FDynamicMesh3& GetInitialMesh() const { return InitialMesh;  }
	/** @return duplciate of initial mesh (possibly with optional size normalization) */
	FDynamicMesh3& GetInitialMesh() { return InitialMesh; }



	//
	// Optional base mesh per-vertex normals. Default enabled.
	// These are computed at Tool startup if required, and then not modified, so can be passed to multithreaded operators/etc
	//
protected:
	/** 
	 * If this function returns true, BaseNormals will be initialized in Tool ::Setup(). 
	 * This has some cost and should be disabled if not necessary. 
	 */
	virtual bool RequiresInitialVtxNormals() const { return false; }

	/** @return calculated base normals. This pointer does not change for the lifetime of the Tool. */
	UE_API TSharedPtr<FMeshNormals>& GetInitialVtxNormals();

private:
	TSharedPtr<FMeshNormals> InitialVtxNormals;



	//
	// Optional base mesh boundary loops. Default enabled.
	// These are computed at Tool startup if required, and then not modified, so can be passed to multithreaded operators/etc
	//
protected:
	/** 
	 * If this function returns true, InitialBoundaryLoops will be initialized in Tool ::Setup(). 
	 * This has some cost and should be disabled if not necessary. 
	 */
	virtual bool RequiresInitialBoundaryLoops() const { return false; }

	/** @return calculated base normals. This pointer does not change for the lifetime of the Tool. */
	UE_API TSharedPtr<FMeshBoundaryLoops>& GetInitialBoundaryLoops();

private:
	TSharedPtr<FMeshBoundaryLoops> InitialBoundaryLoops;





	//
	// Optional weight map support
	// Weight map will only be enabled if base class registers a weight map property set
	//
protected:
	template<class PropSetType>
	PropSetType* AddWeightMapPropertySet(TUniqueFunction<bool()> VisibilityFunc = []() {return true; } )
	{
		PropSetType* PropSet = NewObject<PropSetType>(this);
		SetupWeightMapPropertySet(PropSet);
		WeightMapPropertySetVisibleFunc = MoveTemp(VisibilityFunc);
		return PropSet;
	}

	UE_API virtual void SetupWeightMapPropertySet(UWeightMapSetProperties* Properties);

	UE_API bool HasActiveWeightMap() const;
	UE_API TSharedPtr<FIndexedWeightMap1f>& GetActiveWeightMap();

private:
	TWeakObjectPtr<UWeightMapSetProperties> WeightMapPropertySet;
	TUniqueFunction<bool()> WeightMapPropertySetVisibleFunc;
	TSharedPtr<FIndexedWeightMap1f> ActiveWeightMap;
	UE_API void OnSelectedWeightMapChanged(bool bInvalidate);




	//
	// Optional uniform scale applied to mesh. Enabled by default
	// 

protected:
	/** 
	 * If this function returns true, input mesh will be scaled to normalized dimension in ::Setup() before any processing begins.
	 * This scaling will be undone on Accept()
	 */
	virtual bool RequiresScaleNormalization() const { return true; }

	double GetScaleNormalizationFactor() const { return 1.0 / SrcScale; };

private:
	// scale/translate applied to input mesh to regularize it
	bool bIsScaleNormalizationApplied = false;
	FVector3d SrcTranslate;
	double SrcScale;
	// transform that does the opposite of scale/translate so that mesh stays in the right spot on screen
	FTransform OverrideTransform;
};

#undef UE_API
