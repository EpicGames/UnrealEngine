// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/BaseCreateFromSelectedTool.h"
#include "PropertySets/VoxelProperties.h"

#include "BaseVoxelTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * Base for Voxel tools
 */
UCLASS(MinimalAPI)
class UBaseVoxelTool : public UBaseCreateFromSelectedTool
{
	GENERATED_BODY()

protected:

	/** Sets up VoxProperties; typically need to overload and call "Super::SetupProperties();" */
	UE_API virtual void SetupProperties() override;

	/** Saves VoxProperties; typically need to overload and call "Super::SaveProperties();" */
	UE_API virtual void SaveProperties() override;

	/** Sets up the default preview and converted inputs for voxel tools; Typically do not need to overload */
	UE_API virtual void ConvertInputsAndSetPreviewMaterials(bool bSetPreviewMesh = true) override;

	/** Sets the output material to the default "world grid" material */
	UE_API virtual TArray<UMaterialInterface*> GetOutputMaterials() const override;

	// Test whether any of the OriginalDynamicMeshes has any open boundary edges
	UE_API virtual bool HasOpenBoundariesInMeshInputs();


	UPROPERTY()
	TObjectPtr<UVoxelProperties> VoxProperties;

	TArray<TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe>> OriginalDynamicMeshes;
};

#undef UE_API
