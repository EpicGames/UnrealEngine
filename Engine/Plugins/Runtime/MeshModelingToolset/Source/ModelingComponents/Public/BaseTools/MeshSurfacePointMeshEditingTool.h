// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MeshSurfacePointTool.h"
#include "MeshSurfacePointMeshEditingTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

/**
 * Base tool builder class for UMeshSurfacePointTools with mesh editing requirements.
 */
UCLASS(MinimalAPI)
class UMeshSurfacePointMeshEditingToolBuilder : public UMeshSurfacePointToolBuilder
{
	GENERATED_BODY()
public:
	/** @return new Tool instance. Override this in subclasses to build a different Tool class type */
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

#undef UE_API
