// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MultiSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "MultiSelectionMeshEditingTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UMultiSelectionMeshEditingTool;

/**
 * UMultiSelectionMeshEditingToolBuilder is a base tool builder for multi
 * selection tools that define a common set of ToolTarget interfaces required
 * for editing meshes.
 */
UCLASS(MinimalAPI, Transient, Abstract)
class UMultiSelectionMeshEditingToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	/** @return true if mesh sources can be found in the active selection */
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance initialized with selected mesh source(s) */
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance. Override this in subclasses to build a different Tool class type */
	virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const PURE_VIRTUAL(UMultiSelectionMeshEditingToolBuilder::CreateNewTool, return nullptr; );

	/** Called by BuildTool to configure the Tool with the input mesh source(s) based on the SceneState */
	UE_API virtual void InitializeNewTool(UMultiSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


/**
 * Multi Selection Mesh Editing tool base class.
 */
UCLASS(MinimalAPI)
class UMultiSelectionMeshEditingTool : public UMultiSelectionTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
    UE_API virtual void OnShutdown(EToolShutdownType ShutdownType);

	UE_API virtual void SetWorld(UWorld* World);
	UE_API virtual UWorld* GetTargetWorld();

	/**
	 * Helper to find which targets share source data.
	 * Requires UAssetBackedTarget as a tool target requirement.
	 *
	 * @return Array of indices, 1:1 with Targets, indicating the first index where a component target sharing the same source data appeared.
	 */
	UE_API bool GetMapToSharedSourceData(TArray<int32>& MapToFirstOccurrences);

protected:
	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;
};

#undef UE_API
