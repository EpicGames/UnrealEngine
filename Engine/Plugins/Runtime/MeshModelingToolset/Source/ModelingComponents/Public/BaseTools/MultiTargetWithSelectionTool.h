// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "InteractiveToolBuilder.h"
#include "MultiSelectionMeshEditingTool.h"
#include "Selections/GeometrySelection.h"
#include "MultiTargetWithSelectionTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UMultiTargetWithSelectionTool;
class UPreviewGeometry;
class UGeometrySelectionVisualizationProperties;

PREDECLARE_GEOMETRY(class FDynamicMesh3)
PREDECLARE_GEOMETRY(class FGroupTopology)


/**
 * UMultiTargetWithSelectionToolBuilder is a base tool builder for multi
 * selection tools with selections.
 * Currently, geometry selection across multiple meshes is not supported, restricting the effectiveness
 * of this class. If that support is built in the future, this will become more useful, and likely need to be expanded
 */
UCLASS(MinimalAPI, Transient, Abstract)
class UMultiTargetWithSelectionToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()
public:
	/** @return true if mesh sources can be found in the active selection */
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance initialized with selected mesh source(s) */
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance. Override this in subclasses to build a different Tool class type */
	virtual UMultiTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const PURE_VIRTUAL(UMultiTargetWithSelectionToolBuilder::CreateNewTool, return nullptr; );

	/** Called by BuildTool to configure the Tool with the input mesh source(s) based on the SceneState */
	UE_API virtual void InitializeNewTool(UMultiTargetWithSelectionTool* NewTool, const FToolBuilderState& SceneState) const;

	/** @return true if this Tool requires an input selection */
	virtual bool RequiresInputSelection() const { return false; }

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

/**
 * Multi Target with Selection tool base class.
 */

UCLASS(MinimalAPI)
class UMultiTargetWithSelectionTool : public UMultiSelectionTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType);

	UE_API virtual void OnTick(float DeltaTime) override;

	UE_API virtual void SetTargetWorld(UWorld* World);
	UE_API virtual UWorld* GetTargetWorld();

protected:
	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;


public:
	UE_API virtual void SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn, const int TargetIndex);
	UE_API virtual void SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn, const int TargetIndex);

	/** @return true if a Selection is available for the Target at the given index*/
	UE_API virtual bool HasGeometrySelection(const int TargetIndex) const;

	/** @return the input Selection for the Target at the given index*/
	UE_API virtual const UE::Geometry::FGeometrySelection& GetGeometrySelection(const int TargetIndex) const;

	/** @return if a Selection is available for ANY of the Targets */
	UE_API virtual bool HasAnyGeometrySelection() const;

	/** initialize the Geometry Selection array and the boolean arrays according to the number of targets */
	UE_API virtual void InitializeGeometrySelectionArrays(const int NumTargets);

protected:
	TArray<UE::Geometry::FGeometrySelection> GeometrySelectionArray;
	TArray<bool> GeometrySelectionBoolArray;

	UPROPERTY()
	TObjectPtr<UGeometrySelectionVisualizationProperties> GeometrySelectionVizProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> GeometrySelectionViz = nullptr;
};

#undef UE_API
