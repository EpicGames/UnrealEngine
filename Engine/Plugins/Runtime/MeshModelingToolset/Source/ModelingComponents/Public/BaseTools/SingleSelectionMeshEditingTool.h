// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "SingleSelectionMeshEditingTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class USingleSelectionMeshEditingTool;

/**
 * USingleSelectionMeshEditingToolBuilder is a base tool builder for single
 * selection tools that define a common set of ToolTarget interfaces required
 * for editing meshes.
 */
UCLASS(MinimalAPI, Transient, Abstract)
class USingleSelectionMeshEditingToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()

public:
	/** @return true if a single mesh source can be found in the active selection */
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance initialized with selected mesh source */
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance. Override this in subclasses to build a different Tool class type */
	virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const PURE_VIRTUAL(USingleSelectionMeshEditingToolBuilder::CreateNewTool, return nullptr; );

	/** Called by BuildTool to configure the Tool with the input MeshSource based on the SceneState */
	UE_API virtual void InitializeNewTool(USingleSelectionMeshEditingTool* Tool, const FToolBuilderState& SceneState) const;

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};


/**
 * Single Selection Mesh Editing tool base class.
 */
UCLASS(MinimalAPI)
class USingleSelectionMeshEditingTool : public USingleSelectionTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType);

	UE_API virtual void SetWorld(UWorld* World);
	UE_API virtual UWorld* GetTargetWorld();

protected:
	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;
};

#undef UE_API
