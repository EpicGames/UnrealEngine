// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SingleSelectionTool.h"
#include "InteractiveToolBuilder.h"
#include "Selections/GeometrySelection.h"
#include "SingleTargetWithSelectionTool.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UPreviewGeometry;
class UGeometrySelectionVisualizationProperties;

PREDECLARE_GEOMETRY(class FDynamicMesh3)
PREDECLARE_GEOMETRY(class FGroupTopology)

UCLASS(MinimalAPI, Transient, Abstract)
class USingleTargetWithSelectionToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()
public:
	/** @return true if a single mesh source can be found in the active selection */
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance initialized with selected mesh source */
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	/** @return new Tool instance. Override this in subclasses to build a different Tool class type */
	virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState& SceneState) const PURE_VIRTUAL(USingleTargetWithSelectionToolBuilder::CreateNewTool, return nullptr; );

	/** Called by BuildTool to configure the Tool with the input MeshSource based on the SceneState */
	UE_API virtual void InitializeNewTool(USingleTargetWithSelectionTool* Tool, const FToolBuilderState& SceneState) const;

	/** @return true if this Tool requires an input selection */
	virtual bool RequiresInputSelection() const { return false; }

protected:
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};



UCLASS(MinimalAPI)
class USingleTargetWithSelectionTool : public USingleSelectionTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void OnShutdown(EToolShutdownType ShutdownType);

	UE_API virtual void OnTick(float DeltaTime) override;

	UE_API virtual void SetTargetWorld(UWorld* World);
	UE_API virtual UWorld* GetTargetWorld();

protected:
	UPROPERTY()
	TWeakObjectPtr<UWorld> TargetWorld = nullptr;


public:
	UE_API virtual void SetGeometrySelection(const UE::Geometry::FGeometrySelection& SelectionIn);
	UE_API virtual void SetGeometrySelection(UE::Geometry::FGeometrySelection&& SelectionIn);

	/** @return true if a Selection is available */
	UE_API virtual bool HasGeometrySelection() const;

	/** @return the input Selection */
	UE_API virtual const UE::Geometry::FGeometrySelection& GetGeometrySelection() const;

protected:
	UE::Geometry::FGeometrySelection GeometrySelection;
	bool bGeometrySelectionInitialized = false;

	UPROPERTY()
	TObjectPtr<UGeometrySelectionVisualizationProperties> GeometrySelectionVizProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> GeometrySelectionViz = nullptr;
};

#undef UE_API
