// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolChange.h"


/**
 * TSimpleValueLambdaChange is a generic FToolCommandChange that swaps between two (template-type) Values.
 * The swap is applied via a lambda provided by the creator.
 */
template<typename ValueType>
class TSimpleValueLambdaChange : public FToolCommandChange
{
public:
	ValueType FromValue;
	ValueType ToValue;
	TUniqueFunction<void(UObject*, const ValueType& From, const ValueType& To, bool bIsRevert)> ValueChangeFunc;

	/** Makes the change to the object */
	virtual void Apply(UObject* Object) override
	{
		ValueChangeFunc(Object, FromValue, ToValue, false);
	}

	/** Reverts change to the object */
	virtual void Revert(UObject* Object) override
	{
		ValueChangeFunc(Object, ToValue, FromValue, true);
	}
};
