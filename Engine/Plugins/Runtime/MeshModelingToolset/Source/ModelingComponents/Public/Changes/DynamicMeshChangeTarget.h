// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Changes/MeshChange.h"
#include "Changes/MeshReplacementChange.h"

#include "DynamicMeshChangeTarget.generated.h"

#define UE_API MODELINGCOMPONENTS_API


PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * Bare bones wrapper of FDynamicMesh3 that supports MeshReplacementChange-based updates.  Shares the mesh ptr with the MeshReplacementChange objects, so it must not be changed directly
 * TODO: also support other MeshChange objects (by making a copy of the mesh when applying these changes)
 */
UCLASS(MinimalAPI, Transient)
class UDynamicMeshReplacementChangeTarget : public UObject, public IMeshReplacementCommandChangeTarget
{
	GENERATED_BODY()

protected:
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> Mesh;

public:
	UDynamicMeshReplacementChangeTarget() = default;
	UDynamicMeshReplacementChangeTarget(TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> Mesh) : Mesh(Mesh)
	{}
	virtual ~UDynamicMeshReplacementChangeTarget() = default;

	/**
	* Apply/Revert a general mesh change to the internal mesh   (implements IMeshReplacementCommandChangeTarget)
	*/
	UE_API virtual void ApplyChange(const FMeshReplacementChange* Change, bool bRevert) override;

	virtual void SetMesh(TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> MeshIn)
	{
		Mesh = MeshIn;
	}

	const TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>& GetMesh() const
	{
		return Mesh;
	}

	/**
	 * Update the mesh and return a corresponding FChange
	 * Does not emit the change.
	 * Does not fire OnMeshChanged.
	 */
	UE_API TUniquePtr<FMeshReplacementChange> ReplaceMesh(const TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>& UpdateMesh);

	/**
	* This delegate fires when a FCommandChange is applied to this component, so that
	* parent objects know the mesh has changed.
	*/
	FSimpleMulticastDelegate OnMeshChanged;

};

#undef UE_API
