// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolChange.h"


/**
 * TIndexedValuesChange stores a change of values at a set of indices.
 * For example if you were changing positions of vertices of a mesh, you could 
 * use this change to store the new and old positions.
 *
 * This is an abstract base class, you must provide suitable Apply/Revert implementations.
 * The TCustomIndexedValuesChange subclass will allow you to do this with lambdas.
 */
template<typename ValueType>
class TIndexedValuesChange : public FToolCommandChange
{
public:
	TArray<int32> Indices;
	TArray<ValueType> OldValues;
	TArray<ValueType> NewValues;

	virtual void Apply(UObject* Object) = 0;
	virtual void Revert(UObject* Object) = 0;

	virtual FString ToString() const override
	{
		return FString(TEXT("Indexed Values Change"));
	}
};


/**
 * TIndexedValuesChangeBuilder incrementally builds up a TIndexedValuesChange instance/subclass.
 * The ChangeClassType template parameter should be the name of the TIndexedValuesChange-derived class.
 */
template<typename ValueType, typename ChangeClassType>
class TIndexedValuesChangeBuilder
{
public:
	TUniquePtr<ChangeClassType> Change;
	TMap<int32, int32> SavedIndices;

	TIndexedValuesChangeBuilder()
	{
	}

	/** Allocate a new change and start tracking */
	void BeginNewChange()
	{
		Change = MakeUnique<ChangeClassType>();
		SavedIndices.Reset();
	}

	/** Modify the value at an Index. Changes will be tracked as necessary. */
	void UpdateValue(int32 Index, const ValueType& OldValue, const ValueType& NewValue)
	{
		check(Change.IsValid());
		const int32* FoundIndex = SavedIndices.Find(Index);
		if (FoundIndex == nullptr)
		{
			int32 NewIndex = Change->Indices.Num();
			SavedIndices.Add(Index, NewIndex);
			Change->Indices.Add(Index);
			Change->OldValues.Add(OldValue);
			Change->NewValues.Add(NewValue);
		}
		else
		{
			Change->NewValues[*FoundIndex] = NewValue;
		}
	}

	/** Modify a set of values */
	void UpdateValues(const TArray<int32>& Indices, const TArray<ValueType>& OldValues, const TArray<ValueType>& NewValues)
	{
		int32 NumIndices = Indices.Num();
		for (int32 k = 0; k < NumIndices; ++k)
		{
			UpdateValue(Indices[k], OldValues[k], NewValues[k]);
		}
	}

	/** Complete the change construction and return it */
	TUniquePtr<ChangeClassType> ExtractResult()
	{
		return MoveTemp(Change);
	}
};



/**
 * TCustomIndexedValuesChange is an implementation of TIndexedValuesChange that allows you to
 * provide the Apply() And Revert() implementations via lambdas, as well as specify a custom data
 * element to be stored with the Change. For example if this is a change to an attribute array,
 * you could store the attribute identifier in the CustomData.
 */
template<typename ValueType, typename CustomDataType>
class TCustomIndexedValuesChange : public TIndexedValuesChange<ValueType>
{
public:
	using BaseType = TIndexedValuesChange<ValueType>;

	CustomDataType CustomData;

	TUniqueFunction<void(UObject*, const CustomDataType& Data, const TArray<int32>&, const TArray<ValueType>&)> ApplyFunction;
	TUniqueFunction<void(UObject*, const CustomDataType& Data, const TArray<int32>&, const TArray<ValueType>&)> RevertFunction;

	virtual void Apply(UObject* Object) override
	{
		if (ApplyFunction)
		{
			ApplyFunction(Object, CustomData, BaseType::Indices, BaseType::NewValues);
		}
	}

	virtual void Revert(UObject* Object) override
	{
		if (RevertFunction)
		{
			RevertFunction(Object, CustomData, BaseType::Indices, BaseType::OldValues);
		}
	}
};

