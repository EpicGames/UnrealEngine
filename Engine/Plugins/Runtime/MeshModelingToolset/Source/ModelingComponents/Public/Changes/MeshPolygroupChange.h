// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "Changes/MeshChange.h"
#include "GeometryBase.h"
#include "Polygroups/PolygroupSet.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * FDynamicMeshGroupEdit stores a modification of polygroup IDs on a set of triangles
 */
class FDynamicMeshGroupEdit
{
public:
	/** GroupLayerIndex indicates which AttributeSet PolyGroup layer to apply the change to, or if -1, apply to the default on the Mesh */
	int32 GroupLayerIndex = -1;
	/** IDs of triangles that are modified */
	TArray<int32> Triangles;
	/** Old PolygroupID for each triangle */
	TArray<int32> OldGroups;
	/** New PolygroupID for each triangle */
	TArray<int32> NewGroups;

	virtual ~FDynamicMeshGroupEdit() = default;

	UE_API virtual void ApplyToMesh(FDynamicMesh3* Mesh, bool bRevert);
};



/**
 * FDynamicMeshGroupEditBuilder builds up a FDynamicMeshGroupEdit incrementally.
 */
class FDynamicMeshGroupEditBuilder
{
public:
	UE_API FDynamicMeshGroupEditBuilder(FDynamicMesh3* Mesh);
	UE_API FDynamicMeshGroupEditBuilder(FDynamicMesh3* Mesh, int32 GroupLayer);
	UE_API FDynamicMeshGroupEditBuilder(UE::Geometry::FPolygroupSet* PolygroupSet);

	TUniquePtr<FDynamicMeshGroupEdit> ExtractResult() { return MoveTemp(Edit); }

	UE_API void SaveTriangle(int32 TriangleID);
	UE_API void SaveTriangle(int32 TriangleID, int32 OldGroup, int32 NewGroup);

	template<typename EnumerableType>
	void SaveTriangles(EnumerableType Enumerable)
	{
		for (int32 tid : Enumerable)
		{
			SaveTriangle(tid);
		}
	}

protected:
	TPimplPtr<UE::Geometry::FPolygroupSet> PolygroupSet;
	TUniquePtr<FDynamicMeshGroupEdit> Edit;
	TMap<int32, int32> SavedIndexMap;
};





/**
 * FMeshPolygroupChange stores a change to Polygroup IDs on a set of triangles, as a FDynamicMeshGroupEdit.
 */
class FMeshPolygroupChange : public FMeshChange
{
public:
	UE_API FMeshPolygroupChange(TUniquePtr<FDynamicMeshGroupEdit>&& GroupEditIn);

	TUniquePtr<FDynamicMeshGroupEdit> GroupEdit;

	UE_API virtual void ApplyChangeToMesh(FDynamicMesh3* Mesh, bool bRevert) const override;
};

#undef UE_API
