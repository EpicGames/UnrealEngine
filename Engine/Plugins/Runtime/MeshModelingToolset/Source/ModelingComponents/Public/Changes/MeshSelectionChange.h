// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolChange.h"
#include "SelectionSet.h"

#define UE_API MODELINGCOMPONENTS_API




/**
 * FMeshSelectionChange represents an reversible change to a UMeshSelectionSet
 */
class FMeshSelectionChange : public FToolCommandChange
{
public:
	EMeshSelectionElementType ElementType = EMeshSelectionElementType::Vertex;
	TArray<int32> Indices;
	bool bAdded;

	/** Makes the change to the object */
	UE_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	UE_API virtual void Revert(UObject* Object) override;

	/** Describes this change (for debugging) */
	UE_API virtual FString ToString() const override;
};




/**
 * FMeshSelectionChangeBuilder can be used to construct a FMeshSelectionChange.
 */
class FMeshSelectionChangeBuilder
{
public:
	TUniquePtr<FMeshSelectionChange> Change;

	/**
	 * Initialize of a selection change of given type. 
	 * @param ElementType type of element (face/edge/vtx) being added/removed
	 * @param bAdding if true, output change has bAdded=true, otherwise bAdded=false
	 */
	UE_API FMeshSelectionChangeBuilder(EMeshSelectionElementType ElementType, bool bAdding);

	/**
	 * Add ElementID to list of changed elements  (which may then be added or removed, depending on bAdding parameter to constructor)
	 */
	UE_API void Add(int32 ElementID);

	/**
	 * Add Array of elements
	 */
	UE_API void Add(const TArray<int32>& Elements);

	/**
	 * Add Set of elements
	 */
	UE_API void Add(const TSet<int32>& Elements);
};

#undef UE_API
