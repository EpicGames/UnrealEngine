// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/PimplPtr.h"
#include "Changes/MeshChange.h"
#include "GeometryBase.h"
#include "Polygroups/GroupSetAdapter.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * FDynamicMeshTriangleLabelEdit stores a modification of labels on a set of triangles
 */
class FDynamicMeshTriangleLabelEdit
{
public:
	/** Layer name as registered in FDynamicMeshAttributeSet::TriangleLabelAttributes */
	FName LayerName;
	/** IDs of triangles that are modified */
	TArray<int32> Triangles;
	/** Old labels for each triangle */
	TArray<FName> OldLabels;
	/** New labels for each triangle */
	TArray<FName> NewLabels;
	/** Applies the changes to Mesh looking for a FDynamicMeshTriangleLabelAttribute named LayerName */
	UE_API void ApplyToMesh(FDynamicMesh3* Mesh, bool bRevert);
	
protected:
	/** Ensures that TriangleIndex is a valid index for OldLabels or NewLabels */
	bool IsTriangleIndexValid(const int32 TriangleIndex, const bool bRevert) const;
};



/**
 * FDynamicMeshTriangleLabelEditBuilder builds up a FDynamicMeshTriangleLabelEdit incrementally storing old and new triangle labels
 */
class FDynamicMeshTriangleLabelEditBuilder
{
public:
	
	/** Stores a copy of the incoming adapter to get data from, as well as it's name, and starts a new edit */
	UE_API FDynamicMeshTriangleLabelEditBuilder(UE::Geometry::FTriangleLabelAdapter* InAdapter);

	/** Extracts the label edits */
	TUniquePtr<FDynamicMeshTriangleLabelEdit> ExtractResult()
	{
		return MoveTemp(Edit);
	}

	/** Stores old and new label for this triangle using the adapter */
	UE_API void SaveTriangle(int32 TriangleID);
	
	/** Stores old and new labels for the triangles using the adapter */
	template<typename EnumerableType>
	void SaveTriangles(EnumerableType Enumerable)
	{
		for (int32 tid : Enumerable)
		{
			SaveTriangle(tid);
		}
	}
	
	/** Stores old and new label for this triangle */
	UE_API void SaveTriangle(int32 TriangleID, FName OldLabel, FName NewLabel);

protected:
	/** Adapter to get labels from */
	TPimplPtr<UE::Geometry::FTriangleLabelAdapter> Adapter;
	/** Edit storing old and new triangle labels */
	TUniquePtr<FDynamicMeshTriangleLabelEdit> Edit;
	/** Data table storing a triangle id - triangle edit mapping */
	TMap<int32, int32> SavedIndexMap;
};

/**
 * FMeshTriangleLabelChange stores some label changes a set of triangles, as a FDynamicMeshTriangleLabelEdit.
 */
class FMeshTriangleLabelChange : public FMeshChange
{
public:
	UE_API FMeshTriangleLabelChange(TUniquePtr<FDynamicMeshTriangleLabelEdit>&& LabelEditIn);

	/** Applies the underlying triangle label edit on the mesh */
	UE_API virtual void ApplyChangeToMesh(FDynamicMesh3* Mesh, bool bRevert) const override;
	
	TUniquePtr<FDynamicMeshTriangleLabelEdit> LabelEdit;
};

#undef UE_API
