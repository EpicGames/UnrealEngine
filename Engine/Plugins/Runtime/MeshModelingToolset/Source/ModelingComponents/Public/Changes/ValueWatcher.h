// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"


/**
 * TValueWatcher is used to implement a common Tool pattern where it is necessary to
 * essentially poll a value to see if it has changed, and if it has, call a function.
 * For example UObject UProperties are frequently directly modified by (eg) DetailsView/etc,
 * this results in PropertyChange events that can be used in the editor to respond to changes.
 * However at Runtime no such events are generated and so the Tool has no way of knowing
 * when UProperties change except by polling and comparing with a cached value.
 * The purpose of this class is simply to encapsulate this common pattern/logic.
 */
template<typename ValueType>
class TValueWatcher
{
	ValueType CachedValue;
	TFunction<ValueType(void)> GetValueFunc;
	TFunction<void(ValueType)> OnValueChangedFunc;

public:
	void Initialize(
		TFunction<ValueType(void)> GetValueFuncIn,
		TFunction<void(ValueType)> OnValueChangedFuncIn,
		ValueType InitialValue)
	{
		GetValueFunc = GetValueFuncIn;
		OnValueChangedFunc = OnValueChangedFuncIn;
		CachedValue = InitialValue;
	}

	void CheckAndUpdate()
	{
		ValueType CurValue = GetValueFunc();
		if (CurValue != CachedValue)
		{
			CachedValue = GetValueFunc();
			OnValueChangedFunc(CachedValue);
		}
	}

	/**
	 * Update known value without calling OnValueChangedFunc. Sometimes necessary during undo/redo.
	 */
	void SilentUpdate()
	{
		CachedValue = GetValueFunc();
	}
};
