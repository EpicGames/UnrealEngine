// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/BaseDynamicMeshComponent.h"
#include "UDynamicMesh.h"
#include "DynamicMesh/DynamicMeshOctree3.h"
#include "Util/IndexSetDecompositions.h"

#include "OctreeDynamicMeshComponent.generated.h"

#define UE_API MODELINGCOMPONENTS_API

// predecl
struct FMeshDescription;

/** internal FPrimitiveSceneProxy defined in OctreeDynamicMeshSceneProxy.h */
class FOctreeDynamicMeshSceneProxy;
class FBaseDynamicMeshSceneProxy;

/** 
 * UOctreeDynamicMeshComponent is a mesh component similar to UProceduralMeshComponent,
 * except it bases the renderable geometry off an internal FDynamicMesh3 instance.
 * The class generally has the same capabilities as UDynamicMeshComponent.
 * 
 * A FDynamicMeshOctree3 is available to dynamically track the triangles of the mesh
 * (however the client is responsible for updating this octree).
 * Based on the Octree, the mesh is partitioned into chunks that are stored in separate
 * RenderBuffers in the FOctreeDynamicMeshSceneProxy.
 * Calling NotifyMeshUpdated() will result in only the "dirty" chunks being updated,
 * rather than the entire mesh.
 * 
 * (So, if you don't need this capability, and don't want to update an Octree, use UDynamicMeshComponent!)
 */
UCLASS(MinimalAPI, hidecategories = (LOD, Physics, Collision), editinlinenew, ClassGroup = Rendering)
class UOctreeDynamicMeshComponent : public UBaseDynamicMeshComponent
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * initialize the internal mesh from a DynamicMesh
	 */
	UE_API virtual void SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh) override;

	/**
	 * @return pointer to internal mesh
	 */
	virtual FDynamicMesh3* GetMesh() override { return MeshObject->GetMeshPtr(); }

	/**
	 * @return pointer to internal mesh
	 */
	virtual const FDynamicMesh3* GetMesh() const override { return MeshObject->GetMeshPtr(); }


	virtual void ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const
	{
		MeshObject->ProcessMesh(ProcessFunc);
	}


	UE::Geometry::FDynamicMeshOctree3* GetOctree() { return Octree.Get(); }



	/**
	 * Apply transform to internal mesh. Invalidates RenderProxy.
	 * @param bInvert if true, inverse tranform is applied instead of forward transform
	 */
	UE_API virtual void ApplyTransform(const FTransform3d& Transform, bool bInvert) override;

	//
	// change tracking/etc
	//

	/**
	 * Call this if you update the mesh via GetMesh()
	 * @todo should provide a function that calls a lambda to modify the mesh, and only return const mesh pointer
	 */
	UE_API virtual void NotifyMeshUpdated() override;

	/**
	 * Apply a vertex deformation change to the internal mesh
	 */
	UE_API virtual void ApplyChange(const FMeshVertexChange* Change, bool bRevert) override;

	/**
	 * Apply a general mesh change to the internal mesh
	 */
	UE_API virtual void ApplyChange(const FMeshChange* Change, bool bRevert) override;

	/**
	* Apply a general mesh replacement change to the internal mesh
	*/
	UE_API virtual void ApplyChange(const FMeshReplacementChange* Change, bool bRevert) override;


	/**
	 * This delegate fires when a FCommandChange is applied to this component, so that
	 * parent objects know the mesh has changed.
	 */
	FSimpleMulticastDelegate OnMeshChanged;

	/**
	 * If this function is set, we will use these colors instead of vertex colors
	 */
	TFunction<FColor(const FDynamicMesh3*, int)> TriangleColorFunc = nullptr;

protected:
	/**
	 * This is called to tell our RenderProxy about modifications to the material set.
	 * We need to pass this on for things like material validation in the Editor.
	 */
	UE_API virtual void NotifyMaterialSetUpdated();

private:

	virtual FBaseDynamicMeshSceneProxy* GetBaseSceneProxy() override { return (FBaseDynamicMeshSceneProxy*)GetCurrentSceneProxy(); }
	FOctreeDynamicMeshSceneProxy* GetCurrentSceneProxy() { return (FOctreeDynamicMeshSceneProxy*)SceneProxy; }

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

	UPROPERTY()
	TObjectPtr<UDynamicMesh> MeshObject;

	FDelegateHandle MeshObjectChangedHandle;
	UE_API void OnMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo);

	FDelegateHandle PreMeshChangeHandle;
	UE_API void OnPreMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo);

	TUniquePtr<UE::Geometry::FDynamicMeshOctree3> Octree;
	TUniquePtr<UE::Geometry::FDynamicMeshOctree3::FTreeCutSet> OctreeCut;
	UE::Geometry::FArrayIndexSetsDecomposition TriangleDecomposition;
	struct FCutCellIndexSet
	{
		UE::Geometry::FDynamicMeshOctree3::FCellReference CellRef;
		int32 DecompSetID;
	};
	TArray<FCutCellIndexSet> CutCellSetMap;
	int32 SpillDecompSetID;


	UE_API FColor GetTriangleColor(int TriangleID);

	//friend class FCustomMeshSceneProxy;


public:
	//UFUNCTION(BlueprintCallable, Category = "DynamicMesh")
	virtual UDynamicMesh* GetDynamicMesh() override { return MeshObject; }

	UFUNCTION(BlueprintCallable, Category = "DynamicMesh")
	UE_API void SetDynamicMesh(UDynamicMesh* NewMesh);
};

#undef UE_API
