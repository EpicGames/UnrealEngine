// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

#include "GeometryBase.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);
class IMeshDescriptionCommitter;
class IMeshDescriptionProvider;
struct FMeshDescription;

namespace UE {
namespace Geometry {

	UE_DEPRECATED(5.5, "Use GetDynamicMeshViaMeshDescription which takes a FGetMeshParameters instead.")
	MODELINGCOMPONENTS_API FDynamicMesh3 GetDynamicMeshViaMeshDescription(
		IMeshDescriptionProvider& MeshDescriptionProvider, bool bRequestTangents);
	
	MODELINGCOMPONENTS_API FDynamicMesh3 GetDynamicMeshViaMeshDescription(
		IMeshDescriptionProvider& MeshDescriptionProvider,
		const FGetMeshParameters& InGetMeshParams = FGetMeshParameters());

	MODELINGCOMPONENTS_API void CommitDynamicMeshViaMeshDescription(
		FMeshDescription&& CurrentMeshDescription,
		IMeshDescriptionCommitter& MeshDescriptionCommitter,
		const UE::Geometry::FDynamicMesh3& Mesh, 
		const IDynamicMeshCommitter::FDynamicMeshCommitInfo& CommitInfo);

}}