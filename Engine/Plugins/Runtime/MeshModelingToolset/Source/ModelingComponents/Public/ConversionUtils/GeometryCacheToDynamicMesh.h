// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UGeometryCache;

namespace UE {
namespace Geometry{ class FDynamicMesh3; };

namespace Conversion {

	struct FGeometryCacheToDynamicMeshOptions
	{
		float Time = 0;
		bool bLooping = false;
		bool bReversed = false;
		bool bAllowInterpolation = true;
		bool bWantTangents = true;
	};

	/**
	 * Converts a Geometry Cache to a DynamicMesh.
	 *
	 * @param GeometryCache The input geometry cache
	 * @param MeshOut The result mesh
	 */
	bool MODELINGCOMPONENTS_API GeometryCacheToDynamicMesh(const UGeometryCache& GeometryCache, Geometry::FDynamicMesh3& MeshOut, const FGeometryCacheToDynamicMeshOptions& Options);

} // end namespace Geometry
} // end namespace UE