// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Materials/MaterialInterface.h"
#include "Math/Transform.h"
#include "Internationalization/Text.h"

class USceneComponent;

namespace UE {
namespace Geometry{ class FDynamicMesh3; };

namespace Conversion {

	/**
	 * The Type of LOD in a Mesh Asset. Note some options are only applicable to some asset types (e.g., only Static Meshes)
	 */
	enum class EMeshLODType : uint8
	{
		/** The Maximum-quality available SourceModel LOD (HiResSourceModel if it is available, otherwise SourceModel LOD0) */
		MaxAvailable,
		/** The HiRes SourceModel. LOD Index is ignored. HiResSourceModel is not available at Runtime. */
		HiResSourceModel,
		/**
		 * The SourceModel mesh at a given LOD Index. Note that a StaticMesh Asset with Auto-Generated LODs may not have a valid SourceModel for every LOD Index
		 * SourceModel meshes are not available at Runtime.
		 */
		SourceModel,
		/**
		 * The Render mesh at at given LOD Index.
		 * A StaticMesh Asset derives its RenderData LODs from it's SourceModel LODs. RenderData LODs always exist for every valid LOD Index.
		 * However the RenderData LODs are not identical to SourceModel LODs, in particular they will be split at UV seams, Hard Normal creases, etc.
		 * RenderData LODs in a StaticMesh Asset are only available at Runtime if the bAllowCPUAccess flag was enabled on the Asset at Cook time.
		 */
		RenderData
	};


	// General options for converting/extracting a mesh from an asset or scene component
	// Options may be ignored where not applicable (for example if the source type does not have LODs)
	struct FToMeshOptions
	{
		// default constructor -- requests max available LOD with normals and tangents
		FToMeshOptions() = default;
		// LOD index constructor -- requests specified source model LOD index
		FToMeshOptions(int32 LODIndex, bool bWantNormals = true, bool bWantTangents = true)
			: LODType(EMeshLODType::SourceModel), LODIndex(LODIndex), bWantNormals(bWantNormals), bWantTangents(bWantTangents)
		{}

		EMeshLODType LODType = EMeshLODType::MaxAvailable;
		// Which LOD to use. Ignored if the LODType is MaxAvailable or HiResSourceModel
		int32 LODIndex = 0;
		// Whether to fall back to the most similar LOD if the requested LOD is not available.
		bool bUseClosestLOD = true;

		bool bWantNormals = true;
		bool bWantTangents = true;
		// Whether to request per-instance vertex colors rather than asset colors
		// Note this is only supported for RenderData LODs of Static Mesh components
		bool bWantInstanceColors = false;
	};

	/**
	 * Helper to find the Polygon Group ID to Material Index remapping for source LOD MeshDescription data.
	 * Returns an empty array for runtime uses, render LODs, and other cases where an identity mapping can be used.
	 * 
	 * @return a mapping from polygon groups to material indices for the given LODType and LODIndex, or an empty array if the identity map should be used
	 */
	TArray<int32> MODELINGCOMPONENTS_API GetPolygonGroupToMaterialIndexMap(const UStaticMesh* StaticMesh, EMeshLODType LODType, int32 LODIndex);

	/**
	* Converts geometry from various types of Scene Component to Dynamic Mesh, or reports that it cannot.
	* 
	* @param SceneComponent			Component to attempt to convert to a mesh
	* @param Options				Options controlling the conversion
	* @param OutMesh				Stores the converted mesh
	* @param OutTransform			Stores the world transform of the mesh
	* @param OutErrorMessage		Stores any error message, if the conversion failed
	* @param OutComponentMaterials	Stores materials used on component, if non-null and applicable
	* @param OutAssetMaterials		Stores materials used on asset, if non-null and applicable
	* @return true if the component could be converted, false otherwise
	*/
	bool MODELINGCOMPONENTS_API SceneComponentToDynamicMesh(
		USceneComponent* SceneComponent, const FToMeshOptions& ConversionOptions, bool bTransformToWorld, 
		Geometry::FDynamicMesh3& OutMesh, FTransform& OutTransform, FText& OutErrorMessage, 
		TArray<UMaterialInterface*>* OutComponentMaterials = nullptr, TArray<UMaterialInterface*>* OutAssetMaterials = nullptr);

	// Options for converting Static Mesh assets
	struct FStaticMeshConversionOptions
	{
		// Whether to apply Build Settings during the mesh copy.
		bool bApplyBuildSettings = true;

		// Whether to request tangents on the copied mesh. If tangents are not requested, tangent-related build settings will also be ignored.
		bool bRequestTangents = true;

		// Whether to ignore the 'remove degenerates' option from Build Settings. Note: Only applies if 'Apply Build Settings' is enabled.
		bool bIgnoreRemoveDegenerates = true;

		// Whether to scale the copied mesh by the Build Setting's 'Build Scale'. Note: This is considered separately from the 'Apply Build Settings' option.
		bool bUseBuildScale = true;

		// Whether to directly use the section material indices as output material IDs. Otherwise, material IDs will be indices in to the materials array.
		bool bUseSectionMaterialIndices = false;

		// Whether in the case of non-manifold source geometry to include a vertex layer to map vertices back to src VertexIDs
		bool bIncludeNonManifoldSrcInfo = false;
	};

	/**
	 * Converts geometry from Static Mesh to Dynamic Mesh
	 * 
	 * @param InMesh			Static Mesh to convert
	 * @param OutMesh			Dynamic Mesh to contain static mesh geometry
	 * @param OutErrorMessage	If method returns false, contains error message
	 * @param ConversionOptions	Static mesh conversion settings
	 * @param LODType			Source of data to convert (e.g., render data, source data, or hi res source data)
	 * @param LODIndex			LOD index to convert
	 * @param bUseClosestLOD	Whether to remap to the most similar available LOD rather than fail, if the requested LOD type is not available
	 */
	bool MODELINGCOMPONENTS_API StaticMeshToDynamicMesh(
		UStaticMesh* InMesh, Geometry::FDynamicMesh3& OutMesh, FText& OutErrorMessage, const FStaticMeshConversionOptions& ConversionOptions,
		EMeshLODType LODType = EMeshLODType::SourceModel, int32 LODIndex = 0, bool bUseClosestLOD = true);
	
	/**
	 * Helper to quickly test if we expect to be able to convert SceneComponent to a dynamic mesh
	 * 
	 * @return true if we expect SceneComponentToDynamicMesh to be able to convert this component to a dynamic mesh
	 */
	bool MODELINGCOMPONENTS_API CanConvertSceneComponentToDynamicMesh(USceneComponent* SceneComponent);

} // end namespace Conversion
} // end namespace UE