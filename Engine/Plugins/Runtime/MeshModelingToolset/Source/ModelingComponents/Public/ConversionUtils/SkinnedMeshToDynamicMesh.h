// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USkinnedMeshComponent;

namespace UE {
namespace Geometry{ class FDynamicMesh3; };

namespace Conversion {

	/**
	* Converts a SkinnedMeshComponent to a DynamicMesh.  In doing so, it will create attribute layers for the vertex normals, tangents, UVs and colors defined on the SkinnedMeshComponent, but 
	* the resulting mesh will have mesh (edges) seams along any attribute seam.  You may want to weld the dynamic mesh after this conversion.
	* Per-triangle Material IDs are transfered directly from the SkinnedMeshComponent, and per-triangle Group IDs are assigned to these regions ( specifically the Group ID for a triangle will be the 
	* Material ID + 1).
	* 
	* If the SkinnedComponent is hidden then the result mesh will be empty, likewise material sections of the SkinnedMeshCompnent that are set to hidden will be ignored during conversion.
	*
	* @param SkinnedMeshComponent - the input component. Note, the constiness.  Internally this calls some const methods that really aren't const, and some non-const functions that should be..
	* @param MeshOut       - the result mesh.  Note, this function does not append the new mesh - the MeshOut is cleared by the function before populating it.
	* @param RequestedLOD  - the LOD to be converted, if a non-existent LOD is requested the result mesh will be empty.
	* @param bWantTangents - controls if tangents are transfered
	*/
	bool MODELINGCOMPONENTS_API SkinnedMeshComponentToDynamicMesh(USkinnedMeshComponent& SkinnedMeshComponent, Geometry::FDynamicMesh3& MeshOut, int32 RequestedLOD,  bool bWantTangents);

} // end namespace Geometry
} // end namespace UE