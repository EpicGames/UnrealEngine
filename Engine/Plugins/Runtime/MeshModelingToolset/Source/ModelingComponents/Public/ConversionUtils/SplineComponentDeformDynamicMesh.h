// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USplineMeshComponent;

namespace UE {
namespace Geometry {


class FDynamicMesh3;

/**
*  Applies a deformation as defined by the SplineMeshComponent to the dynamic mesh
*  The normal, tangent, and bi-tangent will be updated if bUpdateTangentSpace is selected.  
*  NB: Superior tangents will be obtained by recomputing them after this deformation; 
*      while the tangent space update this function does is not ideal, it should be consistent with the SplineMeshComponent GPU computation
*/ 
void MODELINGCOMPONENTS_API SplineDeformDynamicMesh(USplineMeshComponent& SplineMeshComponent, FDynamicMesh3& Mesh, bool bUpdateTangentSpace = true);

}; // end namespace Geometry
}; // end namespace UE
