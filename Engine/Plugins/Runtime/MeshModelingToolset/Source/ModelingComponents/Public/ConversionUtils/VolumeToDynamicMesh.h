// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FrameTypes.h"

class AVolume;
class UBrushComponent;
class UModel;
PREDECLARE_GEOMETRY(class FDynamicMesh3);

namespace UE {
namespace Conversion {

struct FVolumeToMeshOptions
{
	bool bInWorldSpace = false;
	bool bSetGroups = true;
	bool bMergeVertices = true;
	bool bAutoRepairMesh = true;
	bool bOptimizeMesh = true;
	bool bGenerateNormals = true;
};

/**
 * Converts an AVolume to a FDynamicMesh3. Does not initialize normals and does not delete the volume.
 */
void MODELINGCOMPONENTS_API VolumeToDynamicMesh(AVolume* Volume, UE::Geometry::FDynamicMesh3& Mesh, const FVolumeToMeshOptions& Options);

/**
* Converts a UBrushComponent to a FDynamicMesh3. Does not initialize normals and does not delete the volume.
*/
void MODELINGCOMPONENTS_API BrushComponentToDynamicMesh(UBrushComponent* Component, UE::Geometry::FDynamicMesh3& Mesh, const FVolumeToMeshOptions& Options);

/**
* Converts a UModel to a FDynamicMesh3. Does not initialize normals.
*/
void MODELINGCOMPONENTS_API BrushToDynamicMesh(UModel& BrushModel, UE::Geometry::FDynamicMesh3& Mesh, const FVolumeToMeshOptions& Options);


}}//end namespace UE::Conversion