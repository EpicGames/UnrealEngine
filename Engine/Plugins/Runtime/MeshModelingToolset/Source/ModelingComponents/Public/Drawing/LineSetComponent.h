// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "Containers/SparseArray.h"
#include "Materials/MaterialInterface.h"

#include "LineSetComponent.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class FPrimitiveSceneProxy;

struct FRenderableLine
{
	FRenderableLine()
		: Start(ForceInitToZero)
		, End(ForceInitToZero)
		, Color(ForceInitToZero)
		, Thickness(0.0f)
		, DepthBias(0.0f)
	{}

	FRenderableLine(const FVector& InStart, const FVector& InEnd, const FColor& InColor, const float InThickness, const float InDepthBias = 0.0f)
		: Start(InStart)
		, End(InEnd)
		, Color(InColor)
		, Thickness(InThickness)
		, DepthBias(InDepthBias)
	{}

	FVector Start;
	FVector End;
	FColor Color;
	float Thickness;
	float DepthBias;
};

UCLASS(MinimalAPI, meta=(BlueprintSpawnableComponent))
class ULineSetComponent : public UMeshComponent
{
	GENERATED_BODY()

public:

	UE_API ULineSetComponent();

	/** Specify material which handles lines */
	UFUNCTION(BlueprintCallable, Category="Line Set")
	UE_API void SetLineMaterial(UMaterialInterface* InLineMaterial);

	/** Clear the line set */
	UFUNCTION(BlueprintCallable, Category="Line Set")
	UE_API void Clear();

	/** Reserve enough memory for up to the given ID (for inserting via ID) */
	UE_API void ReserveLines(const int32 MaxID);

	/** Add a line to be rendered using the component. */
	UE_API int32 AddLine(const FRenderableLine& OverlayLine);

	/** Create and add a line to be rendered using the component. */
	inline int32 AddLine(const FVector& InStart, const FVector& InEnd, const FColor& InColor, const float InThickness, const float InDepthBias = 0.0f)
	{
		// This is just a convenience function to avoid client code having to know about FRenderableLine.
		return AddLine(FRenderableLine(InStart, InEnd, InColor, InThickness, InDepthBias));
	}

	UE_API int32 AddLines(const TArray<FRenderableLine>& InLines);

	/** Add lines to be rendered using the component. */
	UFUNCTION(BlueprintCallable, Category="Line Set", meta = (AutoCreateRefTerm = "InColor"))
	UE_API int32 AddLines(
		const TArray<FVector>& InStart,
		const TArray<FVector>& InEnd,
		const FColor& InColor = FColor::Black,
		const float InThickness = 1.0f,
		const float InDepthBias = 0.0f);

	/** Insert a line with the given ID to the overlay */
	UE_API void InsertLine(const int32 ID, const FRenderableLine& OverlayLine);

	/** Changes the start coordinates of a line */
	UE_API void SetLineStart(const int32 ID, const FVector& NewPostion);

	/** Changes the end coordinates of a line */
	UE_API void SetLineEnd(const int32 ID, const FVector& NewPostion);

	/** Sets the color of a line */
	UE_API void SetLineColor(const int32 ID, const FColor& NewColor);

	/** Sets the thickness of a line */
	UE_API void SetLineThickness(const int32 ID, const float NewThickness);


	/** Sets the color of all existing lines */
	UE_API void SetAllLinesColor(const FColor& NewColor);

	/** Sets the thickness of all existing lines */
	UE_API void SetAllLinesThickness(const float NewThickness);

	/** Sets the depth bias of all existing lines */
	UE_API void SetAllLinesDepthBias(const float NewDepthBias);

	/** Rescales each line assuming that vertex 0 is the origin */
	UE_API void SetAllLinesLength(const float NewLength, bool bUpdateBounds = false);

	/** Remove a line from the set */
	UE_API void RemoveLine(const int32 ID);

	/** Queries whether a line with the given ID exists */
	UE_API bool IsLineValid(const int32 ID) const;


	// utility construction functions

	/**
	 * Add a set of lines for each index in a sequence
	 * @param NumIndices iterate from 0...NumIndices and call LineGenFunc() for each value
	 * @param LineGenFunc called to fetch the lines for an index, callee filles LinesOut array (reset before each call)
	 * @param LinesPerIndexHint if > 0, will reserve space for NumIndices*LinesPerIndexHint new lines
	 */
	UE_API void AddLines(
		int32 NumIndices,
		TFunctionRef<void(int32 Index, TArray<FRenderableLine>& LinesOut)> LineGenFunc,
		int32 LinesPerIndexHint = -1,
		bool bDeferRenderStateDirty = false);

private:

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface.

	UPROPERTY()
	TObjectPtr<const UMaterialInterface> LineMaterial;

	UPROPERTY()
	mutable FBoxSphereBounds Bounds;

	UPROPERTY()
	mutable bool bBoundsDirty;

	TSparseArray<FRenderableLine> Lines;

	UE_API int32 AddLineInternal(const FRenderableLine& Line);

	friend class FLineSetSceneProxy;
};

#undef UE_API
