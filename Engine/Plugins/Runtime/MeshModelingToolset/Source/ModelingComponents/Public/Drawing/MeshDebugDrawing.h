// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "FrameTypes.h"
#include "ToolContextInterfaces.h"  // for FViewCameraState

class FPrimitiveDrawInterface;
using UE::Geometry::FDynamicMesh3;
using UE::Geometry::FDynamicMeshNormalOverlay;

/**
 * drawing utility functions useful for debugging. These are generally not performant.
 */
namespace MeshDebugDraw
{
	using namespace UE::Geometry;

	/**
	 * Draw normals of mesh overlay as lines
	 * @param Overlay The overlay that provides the normals (and mesh positions)
	 * @param Length length of the lines in world space
	 * @param Color color of the lines
	 * @param Thickness thickness of the lines
	 * @param bScreenSpace is the thickness in pixel or world space
	 * @param PDI drawing interface
	 * @param Transform transform applied to the line endpoints
	 */
	void MODELINGCOMPONENTS_API DrawNormals(
		const FDynamicMeshNormalOverlay* Overlay,
		float Length, FColor Color, float Thickness, bool bScreenSpace,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);


	/**
	 * Draw vertices of mesh as points
	 * @param Mesh The Mesh that provides the vertices
	 * @param Indices the list of indices
	 * @param PointSize the size of the points in screen space
	 * @param Color color of the lines
	 * @param PDI drawing interface
	 * @param Transform transform applied to the vertex positions
	 */
	void MODELINGCOMPONENTS_API DrawVertices(
		const FDynamicMesh3* Mesh, const TArray<int>& Indices,
		float PointSize, FColor Color,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);
	void MODELINGCOMPONENTS_API DrawVertices(
		const FDynamicMesh3* Mesh, const TSet<int>& Indices,
		float PointSize, FColor Color,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);


	/**
	 * Draw mesh triangle centroids as points
	 * @param Mesh The Mesh that provides the vertices
	 * @param Indices the list of triangle indices
	 * @param PointSize the size of the points in screen space
	 * @param Color color of the lines
	 * @param PDI drawing interface
	 * @param Transform transform applied to the centroid positions
	 */
	void MODELINGCOMPONENTS_API DrawTriCentroids(
		const FDynamicMesh3* Mesh, const TArray<int>& Indices,
		float PointSize, FColor Color,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);


	/**
	 * Draw a basic 2D grid with a number of lines at given spacing
	 * @param LocalFrame Pre-transform frame of grid (grid lies in XY plane). 
	 * @param GridLines number of grid lines. If odd, there is a center-line, if even then frame center is at center of a grid square
	 * @param GridLineSpacing spacing size between grid lines
	 * @param LineWidth thickness of the lines in screen space
	 * @param Color color of the lines
	 * @param bDepthTested drawing depth priority
	 * @param PDI drawing interface
	 * @param Transform transform applied to LocalFrame. Pass as Identity() if you have world frame.
	 */
	void MODELINGCOMPONENTS_API DrawSimpleGrid(
		const FFrame3d& LocalFrame, int GridLines, double GridLineSpacing,
		float LineWidth, FColor Color, bool bDepthTested,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);


	/**
	 * Draw a basic 2D grid with a given number of lines that covers roughly a constant area of the screen
	 * @param CameraState camera state, required to size grid appropriately
	 * @param LocalFrame Pre-transform frame of grid (grid lies in XY plane).
	 * @param NumGridLines number of grid lines. If odd, there is a center-line, if even then frame center is at center of a grid square
	 * @param VisualAngleSpan visual angle that grid should cover (in degrees, relative to default 90-degree FOV)
	 * @param LineWidth thickness of the lines in screen space
	 * @param Color color of the lines
	 * @param bDepthTested drawing depth priority
	 * @param PDI drawing interface
	 * @param Transform transform applied to LocalFrame. Pass as Identity() if you have world frame.
	 */
	void MODELINGCOMPONENTS_API DrawSimpleFixedScreenAreaGrid(
		const FViewCameraState& CameraState, 
		const FFrame3d& LocalFrame, int32 NumGridLines, double VisualAngleSpan,
		float LineWidth, FColor Color, bool bDepthTested,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);


	/**
	 * Draw a zoomable, hierarchical grid with configurable depth and density terms
	 * @param BaseGridScale At 1.0 zoom, what is the distance between the top most (level 0) grid lines 
	 * @param GridZoomFactor A pseudo zoom factor, where a value of 1.0 represents the baseline grid scaling with BaseScale the distance between coarsest grid lines.
	 * @param MaxLevelDensity The maximum number of lines to draw at any given grid level
	 * @param Levels Number of hierarchy levels to draw. 1 level is equivalent to a simple grid
	 * @param Subdivisions Number of gridlines inserted between lines of a higher level
	 * @param Colors Array containing colors to use for the grid levels. Must include at least one color. If fewer colors than levels are provided, last color will be repeated as needed.
	 * @param LocalFrame Pre-transform frame of grid (grid lies in XY plane).
	 * @param LineWidth thickness of the lines in screen space
	 * @param bDepthTested drawing depth priority
	 * @param PDI drawing interface
	 * @param Transform transform applied to LocalFrame. Pass as Identity() if you have world frame.		
	*/
	void MODELINGCOMPONENTS_API DrawHierarchicalGrid(
		double BaseGridScale, double GridZoomFactor, int32 MaxLevelDensity,
		const FVector& WorldMaxBounds, const FVector& WorldMinBounds,
		int32 Levels, int32 Subdivisions, TArray<FColor>& Colors,
		const FFrame3d& LocalFrame, float LineWidth, bool bDepthTested,
		FPrimitiveDrawInterface* PDI, const FTransform& Transform);
}