// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Drawing/PreviewGeometryActor.h"
#include "InteractiveTool.h"
#include "MeshWireframeComponent.h"
#include "GeometryBase.h"
#include "MeshElementsVisualizer.generated.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_GEOMETRY(class FDynamicMesh3);

/**
 * Visualization settings UMeshElementsVisualizer
 */
UCLASS(MinimalAPI)
class UMeshElementsVisualizerProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Should any mesh elements be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization)
	bool bVisible = true;

	/** Should mesh wireframe be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible" ))
	bool bShowWireframe = false;

	/** Should mesh boundary edges be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible"))
	bool bShowBorders = true;

	/** Should mesh uv seam edges be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible"))
	bool bShowUVSeams = true;

	/** Should mesh normal seam edges be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible"))
	bool bShowNormalSeams = true;

	/** Should mesh tangent seam edges be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible"))
	bool bShowTangentSeams = true;

	/** Should mesh color seam edges be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible"))
	bool bShowColorSeams = true;

	/** Should polygroup boundary edges be shown */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, meta = (EditCondition = "bVisible"))
	bool bShowGroupBoundaries = false;

	/** Helper function to set all the bShow... booleans to the same value. */
	UE_API void ShowAllElements(bool bShow);

	/** Multiplier on edge thicknesses */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (UIMin = 0.1, UIMax = 10.0, EditCondition = "bVisible"))
	float ThicknessScale = 1.0;

	/** Color of mesh wireframe */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor WireframeColor = FColor(128, 128, 128);

	/** Color of mesh boundary edges */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor BoundaryEdgeColor = FColor(245, 15, 15);

	/** Color of mesh UV seam edges */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor UVSeamColor = FColor(240, 160, 15);

	/** Color of mesh normal seam edges */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor NormalSeamColor = FColor(128, 128, 240);

	/** Color of mesh tangent seam edges */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor TangentSeamColor = FColor(64, 240, 240);

	/** Color of mesh color seam edges */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor ColorSeamColor = FColor(46, 204, 113);

	/** Color of polygroup boundary edges */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (EditCondition = "bVisible"))
	FColor GroupBoundaryColor = FColor(220, 100, 220);

	/** Depth bias used to slightly shift depth of lines */
	UPROPERTY(EditAnywhere, Category = MeshElementVisualization, AdvancedDisplay, meta = (UIMin = -2.0, UIMax = 2.0, EditCondition = "bVisible"))
	float DepthBias = 0.2f;

	//~ Will be removed- generally not desirable with percentage-wise depth offset
	UPROPERTY()
	bool bAdjustDepthBiasUsingMeshSize = false;
};


/**
 * UMeshElementsVisualizer is a subclass of UPreviewGeometry that displays mesh elements.
 * Currently supports wireframe, boundary edges, UV seams, Normal seams and Color seams.
 *
 * UMeshElementsVisualizer initializes an instance of UMeshElementsVisualizerProperties
 * as its .Settings value, and will watch for changes in these properties.
 *
 * Mesh is accessed via lambda callback provided by creator/client. See SetMeshAccessFunction() comments
 */
UCLASS(MinimalAPI, Transient)
class UMeshElementsVisualizer : public UPreviewGeometry
{
	GENERATED_BODY()

public:

	/**
	 * A reference to a function that can be called with a const FDynamicMesh3 to do some computation on it.
	 */
	typedef TFunctionRef<void(const UE::Geometry::FDynamicMesh3& Mesh)> ProcessDynamicMeshFunc;

	/**
	 * UMeshElementsVisualizer needs access to a mesh to generate it's data structures. However
	 * we may not be able to (or shouldn't) directly pass a pointer to UMeshElementsVisualizer.
	 * For example a UDynamicMeshComponent can expose it's internal mesh via a function ProcessMesh( TFunctionRef<void(const FDynamicMesh3&)> Func ).
	 * UMeshElementsVisualizer cannot directly call that function without knowing about the UDynamicMeshComponent,
	 * which we would like to avoid. So instead the client of UMeshElementsVisualizer provides a 
	 * MeshAccessFunction that we can call with an internal function/lambda that UMeshElementsVisualizer
	 * internals will produce, that can be passed a const FDynamicMesh3& for processing (this is a ProcessDynamicMeshFunc).
	 * The job of the MeshAccessFunction is to call this ProcessDynamicMeshFunc on a FDynamicMesh3,
	 * which it may have direct access to, or it may get by calling (eg) UDynamicMeshComponent::ProcessMesh() internally. 
	 * 
	 * So for example usage with a Tool that has a DynamicMeshComponent could be as follows:
	 *   MeshElementsVisualizer->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
	 *       DynamicMeshComponent->ProcessMesh(ProcessFunc);
	 *   });
	 * And for usage with a mesh that a Tool class owns:
	 *   MeshElementsVisualizer->SetMeshAccessFunction([this](UMeshElementsVisualizer::ProcessDynamicMeshFunc ProcessFunc) {
	 *       ProcessFunc(MyInternalMesh);
	 *   });
	 */
	UE_API void SetMeshAccessFunction(TUniqueFunction<void(ProcessDynamicMeshFunc)>&& MeshAccessFunction);

	/**
	 * Call if mesh provided by MeshAccessFunction has been modified, will cause a full recomputation
	 * of all rendering data structures.
	 */
	UE_API void NotifyMeshChanged();


	/**
	 * Client must call this every frame for changes to .Settings to be reflected in rendered result.
	 */
	UE_API void OnTick(float DeltaTime);


public:
	/** Visualization settings */
	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizerProperties> Settings;

	/** Mesh Wireframe component, draws wireframe, boundaries, UV seams, normal seams, color seams */
	UPROPERTY()
	TObjectPtr<UMeshWireframeComponent> WireframeComponent;


protected:
	UE_API virtual void OnCreated() override;

	bool bSettingsModified = false;

	UE_API void UpdateVisibility();
	UE_API void UpdateLineDepthBiasScale();

	TSharedPtr<IMeshWireframeSourceProvider> WireframeSourceProvider;
};

#undef UE_API
