// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "Materials/MaterialInterface.h"

#include "MeshWireframeComponent.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class FPrimitiveSceneProxy;


/**
 * IMeshWireframeSource is an abstract interface to a class that provides
 * a set of edges. Edges are identified by an integer ID, which may be invalid
 * for some edges in the ID range, and each edge has two vertex indices.
 *
 * The class is not specific to mesh wireframes, and could be used to draw any edge set, 
 * but the EdgeType is based on triangle/polygon mesh edges.
 */
class IMeshWireframeSource
{
public:
	enum class EMeshEdgeType
	{
		Regular = 1,
		MeshBoundary = 1 << 2,
		UVSeam = 1 << 3,
		NormalSeam = 1 << 4,
		GroupBoundary = 1 << 5,
		ColorSeam = 1 << 6,
		TangentSeam = 1 << 7
	};

public:
	virtual ~IMeshWireframeSource() = default;

	/** @return true if this source is valid */
	virtual bool IsValid() const = 0;

	/** @return bounds of all possible edges */
	virtual FBoxSphereBounds GetBounds() const = 0;
	/** @return position of vertex at given index */
	virtual FVector GetVertex(int32 Index) const = 0;

	/** @return number of valid edge indices */
	virtual int32 GetEdgeCount() const = 0;
	/** @return maximum edge index (may be greater than count, if some indices are invalid) */
	virtual int32 GetMaxEdgeIndex() const = 0;
	/** @return true if edge index is valid, ie refers to valid edge */
	virtual bool IsEdge(int32 Index) const = 0;
	/** @return vertex indices and type for a given edge index */
	virtual void GetEdge(int32 Index, int32& VertexAOut, int32& VertexBOut, EMeshEdgeType& TypeOut) const = 0;
};

/**
 * IMeshWireframeSourceProvider is an abstract interface to some implementation that
 * can provide an IMeshWireframeSource implementation on demand. This is used
 * to populate/update a UMeshWireframeComponent when necessary.
 */
class IMeshWireframeSourceProvider
{
public:
	virtual ~IMeshWireframeSourceProvider() = default;

	/**
	 * Request that the given function ProcessingFunc be called with an IMeshWireframeSource implementation.
	 * The IMeshWireframeSource implementation may be invalid, ie IsValid() == false, ProcessingFunc must handle this case.
	 * The AccessMesh implementation may choose to not call the lambda in such cases, as well
	 */
	virtual void AccessMesh(TFunctionRef<void(const IMeshWireframeSource&)> ProcessingFunc) = 0;
};


/**
 * UMeshWireframeComponent draws a mesh wireframe as lines, with line color/thickness
 * varying depending on line type and the configuration settings.
 * Currently can draw:
 *    - all mesh edges
 *    - boundary edges
 *    - UV seam edges
 *    - Normal seam edges
 *    - Color seam edges
 *
 * Client must provide a IMeshWireframeSourceProvider implementation that provides the 
 * edge set, vertices, and edge type
 */
UCLASS(MinimalAPI)
class UMeshWireframeComponent : public UMeshComponent
{
	GENERATED_BODY()

public:
	UE_API UMeshWireframeComponent();

	/** Specify material which handles lines */
	UE_API void SetLineMaterial(UMaterialInterface* InLineMaterial);

	/**
	 * Set the wireframe source
	 */
	UE_API void SetWireframeSourceProvider(TSharedPtr<IMeshWireframeSourceProvider> Provider);

	/** Causes the rendered wireframe to be updated from its source. */
	UE_API void UpdateWireframe();
public:
	
	/**
	 * Depth bias of the lines, used to offset in depth to avoid z-fighting
	 */
	UPROPERTY(EditAnywhere, Category=MeshWireframe)
	float LineDepthBias = 0.1f;

	/**
	 * Target-size depth bias scale. This is multiplied by LineDepthBias.
	 * Client of UMeshWireframeComponent can set this if desired, eg to fraction of target object bounding box size, etc.
	 */
	UPROPERTY(EditAnywhere, Category=MeshWireframe)
	float LineDepthBiasSizeScale = 1.0f;

	/**
	 * Scaling factor applied to the per-edge-type thicknesses defined below
	 */
	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float ThicknessScale = 1.0f;


	// Wireframe properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableWireframe = true;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor WireframeColor = FColor(128, 128, 128);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float WireframeThickness = 1.0f;


	// Boundary Edge properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableBoundaryEdges = true;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor BoundaryEdgeColor = FColor(245, 15, 15);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float BoundaryEdgeThickness = 4.0f;


	// UV seam properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableUVSeams = true;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor UVSeamColor = FColor(240, 160, 15);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float UVSeamThickness = 2.0f;


	// normal seam properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableNormalSeams = true;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor NormalSeamColor = FColor(128, 128, 240);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float NormalSeamThickness = 2.0f;


	// tangent seam properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableTangentSeams = true;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor TangentSeamColor = FColor(64, 240, 240);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float TangentSeamThickness = 2.0f;


	// color seam properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableColorSeams = true;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor ColorSeamColor = FColor(46, 204, 113);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float ColorSeamThickness = 2.0f;


	// group boundary properties

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	bool bEnableGroupBoundaries = false;

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	FColor GroupBoundaryColor = FColor(220, 100, 220);

	UPROPERTY(EditAnywhere, Category = MeshWireframe)
	float GroupBoundaryThickness = 1.5f;

private:

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface.

	UPROPERTY()
	TObjectPtr<const UMaterialInterface> LineMaterial;

	UPROPERTY()
	mutable FBoxSphereBounds LocalBounds;

	TSharedPtr<IMeshWireframeSourceProvider> SourceProvider;

	friend class FMeshWireframeSceneProxy;
};

#undef UE_API
