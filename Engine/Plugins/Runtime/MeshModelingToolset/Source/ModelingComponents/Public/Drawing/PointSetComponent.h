// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "Containers/SparseArray.h"
#include "PointSetComponent.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class FPrimitiveSceneProxy;


struct FRenderablePoint
{
	FRenderablePoint()
		: Position(ForceInitToZero)
		, Color(ForceInitToZero)
		, Size(0.f)
		, DepthBias(0.f)
	{}

	FRenderablePoint(const FVector& InPosition, const FColor& InColor, const float InSize, const float InDepthBias = 0.0f)
		: Position(InPosition)
		, Color(InColor)
		, Size(InSize)
		, DepthBias(InDepthBias)
	{}

	FVector Position;
	FColor Color;
	float Size;
	float DepthBias;
};


/**
 * UPointSetComponent is a Component that draws a set of points, as small squares.
 * Per-point Color and (view-space) Size is supported. Normals are not supported.
 * 
 * Points are inserted with an externally-defined ID, internally this is done via
 * a TSparseArray. This class allocates a contiguous TArray large enugh to hold the 
 * largest ID. Using ReservePoints() may be beneficial for huge arrays.
 *
 * The points are drawn as two triangles (ie a square) orthogonal to the view direction. 
 * The actual point size is calculated in the shader, and so a custom material must be used.
 */
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class UPointSetComponent : public UMeshComponent
{
	GENERATED_BODY()

public:
	UE_API UPointSetComponent();

	/** Specify material which handles points */
	UFUNCTION(BlueprintCallable, Category = "Point Set")
	UE_API void SetPointMaterial(UMaterialInterface* InPointMaterial);

	/** Clear all primitives */
	UFUNCTION(BlueprintCallable, Category = "Point Set")
	UE_API void Clear();

	/** Reserve enough memory for up to the given ID */
	UE_API void ReservePoints(const int32 MaxID);

	/** Add a point to be rendered using the component. */
	UE_API int32 AddPoint(const FRenderablePoint& OverlayPoint);

	/** Create and add a point to be rendered using the component. */
	int32 AddPoint(const FVector& InPosition, const FColor& InColor, const float InSize, const float InDepthBias = 0.0f)
	{
		// This is just a convenience function to avoid client code having to know about FRenderablePoint.
		return AddPoint(FRenderablePoint(InPosition, InColor, InSize, InDepthBias));
	}

	UE_API int32 AddPoints(const TArray<FRenderablePoint>& InPoints);

	/** 
	 * Add points to be rendered using the component. 
	 * @return Number of added points
	 */
	UFUNCTION(BlueprintCallable, Category = "Point Set", meta = (AutoCreateRefTerm = "InColor"))
	UE_API int32 AddPoints(
		const TArray<FVector>& Positions,
		const FColor& InColor = FColor::Black,
		const float InSize = 2.0f,
		const float InDepthBias = 0.0f
	);

	/** Insert a point with the given ID into the set. */
	UE_API void InsertPoint(const int32 ID, const FRenderablePoint& OverlayPoint);

	/** Retrieve a point with the given id. */
	UE_API const FRenderablePoint& GetPoint(const int32 ID);

	/** Sets the position of a point (assumes its existence). */
	UE_API void SetPointPosition(const int32 ID, const FVector& NewPosition);

	/** Sets the color of a point */
	UE_API void SetPointColor(const int32 ID, const FColor& NewColor);

	/** Sets the size of a point */
	UE_API void SetPointSize(const int32 ID, const float NewSize);

	/** Sets the color of all points currently in the set. */
	UE_API void SetAllPointsColor(const FColor& NewColor);

	/** Sets the size of all points currently in the set. */
	UE_API void SetAllPointsSize(float NewSize);

	/** Sets the depth bias of all points currently in the set. */
	UE_API void SetAllPointsDepthBias(float NewDepthBias);

	/** Remove a point from the set. */
	UE_API void RemovePoint(const int32 ID);

	/** Queries whether a point with the given ID exists */
	UE_API bool IsPointValid(const int32 ID) const;

	// utility construction functions
	
	/**
	 * Add a set of points for each index in a sequence
	 * @param NumIndices iterate from 0...NumIndices and call PointGenFunc() for each value
	 * @param PointGenFunc called to fetch the points for an index, callee filles PointsOut array (reset before each call)
	 * @param PointsPerIndexHint if > 0, will reserve space for NumIndices*PointsPerIndexHint new points
	 */
	UE_API void AddPoints(
		int32 NumIndices,
		TFunctionRef<void(int32 Index, TArray<FRenderablePoint>& PointsOut)> PointGenFunc,
		int32 PointsPerIndexHint = -1,
		bool bDeferRenderStateDirty = false);

private:

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ Begin USceneComponent Interface.

	UPROPERTY()
	TObjectPtr<const UMaterialInterface> PointMaterial;

	UPROPERTY()
	mutable FBoxSphereBounds Bounds;

	UPROPERTY()
	mutable bool bBoundsDirty;

	TSparseArray<FRenderablePoint> Points;

	UE_API int32 AddPointInternal(const FRenderablePoint& Point);

	friend class FPointSetSceneProxy;
};

#undef UE_API
