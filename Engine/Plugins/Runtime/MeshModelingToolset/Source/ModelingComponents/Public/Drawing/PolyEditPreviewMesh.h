// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreviewMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicSubmesh3.h"
#include "PolyEditPreviewMesh.generated.h"

#define UE_API MODELINGCOMPONENTS_API

/**
 * UPolyEditPreviewMesh is a variant of UPreviewMesh intended for use as a 'live preview' of
 * a mesh creation/editing operation. The class supports initializing the preview mesh in various
 * ways, generally as a submesh of a base mesh.
 */
UCLASS(MinimalAPI, Transient)
class UPolyEditPreviewMesh : public UPreviewMesh
{
	GENERATED_BODY()
public:

	//
	// "Static Type" is just a static mesh
	//

	UE_API void InitializeStaticType(const FDynamicMesh3* SourceMesh, const TArray<int32>& Triangles,
		const FTransform3d* MeshTransform = nullptr);
	UE_API void UpdateStaticType(TFunctionRef<void(FDynamicMesh3&)> UpdateMeshFunc, bool bFullRecalculate);
	UE_API void MakeStaticTypeTargetMesh(FDynamicMesh3& TargetMesh);

	//
	// "Extrude Type" duplicates the input faces, offsets them using FExtrudeMesh, and stitches them together
	//

	UE_API void InitializeExtrudeType(const FDynamicMesh3* SourceMesh, const TArray<int32>& Triangles,
		const FVector3d& TransformedOffsetDirection,
		const FTransform3d* MeshTransform = nullptr,
		bool bDeleteExtrudeBaseFaces = true);
	UE_API void InitializeExtrudeType(FDynamicMesh3&& BaseMesh,
		const FVector3d& TransformedOffsetDirection,
		const FTransform3d* MeshTransform = nullptr,
		bool bDeleteExtrudeBaseFaces = true);
	/** Update extrude-type preview mesh by moving existing offset vertices */
	UE_API void UpdateExtrudeType(double NewOffset, bool bUseNormalDirection = false);
	/** Update extrude-type preview mesh by moving existing offset vertices along their connected triangle normals and then averaging positions  */
	UE_API void UpdateExtrudeType_FaceNormalAvg(double NewOffset);
	/** Update extrude-type preview mesh using external function. if bFullRecalculate, mesh is re-initialized w/ initial extrusion patch SourceMesh before calling function */
	UE_API void UpdateExtrudeType(TFunctionRef<void(FDynamicMesh3&)> UpdateMeshFunc, bool bFullRecalculate);
	/** Make a hit-target mesh that is an infinite extrusion along extrude direction. If bUseNormalDirection, use per-vertex normals instead */
	UE_API void MakeExtrudeTypeHitTargetMesh(FDynamicMesh3& TargetMesh, bool bUseNormalDirection = false);


	//
	// "Inset Type" duplicates the input faces and insets them using FInsetMeshRegion
	//

	UE_API void InitializeInsetType(const FDynamicMesh3* SourceMesh, const TArray<int32>& Triangles,
		const FTransform3d* MeshTransform = nullptr);
	UE_API void UpdateInsetType(double NewOffset, bool bReproject = false, double Softness = 0.0, double AreaScaleT = 1.0, bool bBoundaryOnly = false);
	
	/// @param TargetMeshOut Will be populated with the initial edit patch mesh transformed to world space via the preview mesh transform
	UE_API void MakeInsetTypeTargetMesh(FDynamicMesh3& TargetMeshOut);
	
	void ApplyTranslationToPreview(const FTransform& TransformIn, FTransform& TranslationOut, FTransform& RotateScaleOut)
	{
		TranslationOut = FTransform(TransformIn.GetTranslation());
		SetTransform(TranslationOut);
		RotateScaleOut = TransformIn;
		RotateScaleOut.SetTranslation(FVector::ZeroVector);
	}

protected:
	TUniquePtr<UE::Geometry::FDynamicSubmesh3> ActiveSubmesh;
	FDynamicMesh3 InitialEditPatch;

	TArray<int32> EditVertices;
	TArray<FVector3d> InitialPositions;
	TArray<FVector3d> InitialNormals;
	TArray<FVector3d> InitialTriNormals;
	TMap<int32,int32> ExtrudeToInitialVerts;

	FVector3d InputDirection;

	FTransform3d MeshTransform; // Transform that has been baked into the FDynamicMesh3
	bool bHaveMeshTransform;
};

#undef UE_API
