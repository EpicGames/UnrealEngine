// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolObjects.h"
#include "Drawing/TriangleSetComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"

#include "PreviewGeometryActor.generated.h"

#define UE_API MODELINGCOMPONENTS_API

/**
 * An actor suitable for attaching components used to draw preview elements, such as LineSetComponent and TriangleSetComponent.
 */
UCLASS(MinimalAPI, Transient, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType)
class APreviewGeometryActor : public AInternalToolFrameworkActor
{
	GENERATED_BODY()
private:
	APreviewGeometryActor()
	{
#if WITH_EDITORONLY_DATA
		// hide this actor in the scene outliner
		bListedInSceneOutliner = false;
#endif
	}

public:
};


/**
 * UPreviewGeometry creates and manages an APreviewGeometryActor and a set of preview geometry Components.
 * Preview geometry Components are identified by strings.
 */
UCLASS(MinimalAPI, Transient)
class UPreviewGeometry : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual ~UPreviewGeometry();

	/**
	 * Create preview mesh in the World with the given transform
	 */
	UFUNCTION()
	UE_API void CreateInWorld(UWorld* World, const FTransform& WithTransform);

	/**
	 * Remove and destroy preview mesh
	 */
	UFUNCTION()
	UE_API void Disconnect();

	/** @return the preview geometry actor created by this class */
	UFUNCTION()
	APreviewGeometryActor* GetActor() const { return ParentActor;  }



	/**
	 * Get the current transform on the preview 
	 */
	UE_API FTransform GetTransform() const;

	/**
	 * Set the transform on the preview mesh
	 */
	UE_API void SetTransform(const FTransform& UseTransform);


	/**
	 * Set visibility state of the preview mesh
	 */
	UE_API void SetAllVisible(bool bVisible);


	//
	// Triangle Sets
	//

	/** Create a new triangle set with the given TriangleSetIdentifier and return it */
	UFUNCTION()
	UE_API UTriangleSetComponent* AddTriangleSet(const FString& TriangleSetIdentifier);

	/** @return the TriangleSetComponent with the given TriangleSetIdentifier, or nullptr if not found */
	UFUNCTION()
	UE_API UTriangleSetComponent* FindTriangleSet(const FString& TriangleSetIdentifier);


	//
	// Triangle Set Utilities
	//

	/**
	 * Find the identified triangle set and call UpdateFuncType(UTriangleSetComponent*)
	 */
	template<typename UpdateFuncType>
	void UpdateTriangleSet(const FString& TriangleSetIdentifier, UpdateFuncType UpdateFunc)
	{
		UTriangleSetComponent* TriangleSet = FindTriangleSet(TriangleSetIdentifier);
		if (TriangleSet)
		{
			UpdateFunc(TriangleSet);
		}
	}

	/**
	 * call UpdateFunc for all existing Triangle Sets
	 */
	template<typename UpdateFuncType>
	void UpdateAllTriangleSets(UpdateFuncType UpdateFunc)
	{
		for (TPair<FString, TObjectPtr<UTriangleSetComponent>> Entry : TriangleSets)
		{
			UpdateFunc(Entry.Value);
		}
	}

	/**
	 * Add a set of triangles produced by calling TriangleGenFunc for each index in range [0,NumIndices)
	 * @return the created or updated triangle set
	 */
	UE_API UTriangleSetComponent* CreateOrUpdateTriangleSet(const FString& TriangleSetIdentifier, int32 NumIndices,
		TFunctionRef<void(int32 Index, TArray<FRenderableTriangle>& TrianglesOut)> TriangleGenFunc,
		int32 TrianglesPerIndexHint = -1);

	/**
	 * Remove the TriangleSetComponent with the given TriangleSetIdentifier
	 * @param bDestroy if true, component will unregistered and destroyed.
	 * @return true if the TriangleSetComponent was found and removed
	 */
	UFUNCTION()
	UE_API bool RemoveTriangleSet(const FString& TriangleSetIdentifier, bool bDestroy = true);

	/**
	 * Remove all TriangleSetComponents
	 * @param bDestroy if true, the components will unregistered and destroyed.
	 */
	UFUNCTION()
	UE_API void RemoveAllTriangleSets(bool bDestroy = true);


	//
	// Line Sets
	//

	/** Create a new line set with the given LineSetIdentifier and return it */
	UFUNCTION()
	UE_API ULineSetComponent* AddLineSet(const FString& LineSetIdentifier);

	/** @return the LineSetComponent with the given LineSetIdentifier, or nullptr if not found */
	UFUNCTION()
	UE_API ULineSetComponent* FindLineSet(const FString& LineSetIdentifier);

	/** 
	 * Remove the LineSetComponent with the given LineSetIdentifier
	 * @param bDestroy if true, component will unregistered and destroyed. 
	 * @return true if the LineSetComponent was found and removed
	 */
	UFUNCTION()
	UE_API bool RemoveLineSet(const FString& LineSetIdentifier, bool bDestroy = true);

	/**
	 * Remove all LineSetComponents
	 * @param bDestroy if true, the components will unregistered and destroyed.
	 */
	UFUNCTION()
	UE_API void RemoveAllLineSets(bool bDestroy = true);

	/**
	 * Set the visibility of the LineSetComponent with the given LineSetIdentifier
	 * @return true if the LineSetComponent was found and updated
	 */
	UFUNCTION()
	UE_API bool SetLineSetVisibility(const FString& LineSetIdentifier, bool bVisible);

	/**
	 * Set the Material of the LineSetComponent with the given LineSetIdentifier
	 * @return true if the LineSetComponent was found and updated
	 */
	UFUNCTION()
	UE_API bool SetLineSetMaterial(const FString& LineSetIdentifier, UMaterialInterface* NewMaterial);

	/**
	 * Set the Material of all LineSetComponents
	 */
	UFUNCTION()
	UE_API void SetAllLineSetsMaterial(UMaterialInterface* Material);



	//
	// Line Set Utilities
	//

	/**
	 * Find the identified line set and call UpdateFuncType(ULineSetComponent*)
	 */
	template<typename UpdateFuncType>
	void UpdateLineSet(const FString& LineSetIdentifier, UpdateFuncType UpdateFunc)
	{
		ULineSetComponent* LineSet = FindLineSet(LineSetIdentifier);
		if (LineSet)
		{
			UpdateFunc(LineSet);
		}
	}

	/**
	 * call UpdateFuncType(ULineSetComponent*) for all existing Line Sets
	 */
	template<typename UpdateFuncType>
	void UpdateAllLineSets(UpdateFuncType UpdateFunc)
	{
		for (TPair<FString, TObjectPtr<ULineSetComponent>> Entry : LineSets)
		{
			UpdateFunc(Entry.Value);
		}
	}

	/**
	 * Add a set of lines produced by calling LineGenFunc for each index in range [0,NumIndices)
	 * @return the created or updated line set
	 */
	UE_API ULineSetComponent* CreateOrUpdateLineSet(const FString& LineSetIdentifier, int32 NumIndices,
		TFunctionRef<void(int32 Index, TArray<FRenderableLine>& LinesOut)> LineGenFunc,
		int32 LinesPerIndexHint = -1);

	//
	// Point Sets
	//

	/** Create a new point set with the given PointSetIdentifier and return it */
	UFUNCTION()
	UE_API UPointSetComponent* AddPointSet(const FString& PointSetIdentifier);

	/** @return the PointSetComponent with the given PointSetIdentifier, or nullptr if not found */
	UFUNCTION()
	UE_API UPointSetComponent* FindPointSet(const FString& PointSetIdentifier);

	/** 
	 * Remove the PointSetComponent with the given PointSetIdentifier
	 * @param bDestroy if true, component will unregistered and destroyed. 
	 * @return true if the PointSetComponent was found and removed
	 */
	UFUNCTION()
	UE_API bool RemovePointSet(const FString& PointSetIdentifier, bool bDestroy = true);

	/**
	 * Remove all PointSetComponents
	 * @param bDestroy if true, the components will unregistered and destroyed.
	 */
	UFUNCTION()
	UE_API void RemoveAllPointSets(bool bDestroy = true);

	/**
	 * Set the visibility of the PointSetComponent with the given PointSetIdentifier
	 * @return true if the PointSetComponent was found and updated
	 */
	UFUNCTION()
	UE_API bool SetPointSetVisibility(const FString& PointSetIdentifier, bool bVisible);

	/**
	 * Set the Material of the PointSetComponent with the given PointSetIdentifier
	 * @return true if the PointSetComponent was found and updated
	 */
	UFUNCTION()
	UE_API bool SetPointSetMaterial(const FString& PointSetIdentifier, UMaterialInterface* NewMaterial);

	/**
	 * Set the Material of all PointSetComponents
	 */
	UFUNCTION()
	UE_API void SetAllPointSetsMaterial(UMaterialInterface* Material);



	//
	// Point Set Utilities
	//

	/**
	 * Find the identified point set and call UpdateFuncType(UPointSetComponent*)
	 */
	template<typename UpdateFuncType>
	void UpdatePointSet(const FString& PointSetIdentifier, UpdateFuncType UpdateFunc)
	{
		UPointSetComponent* PointSet = FindPointSet(PointSetIdentifier);
		if (PointSet)
		{
			UpdateFunc(PointSet);
		}
	}

	/**
	 * call UpdateFuncType(UPointSetComponent*) for all existing Point Sets
	 */
	template<typename UpdateFuncType>
	void UpdateAllPointSets(UpdateFuncType UpdateFunc)
	{
		for (TPair<FString, TObjectPtr<UPointSetComponent>> Entry : PointSets)
		{
			UpdateFunc(Entry.Value);
		}
	}

	/**
	 * Add a set of points produced by calling PointGenFunc for each index in range [0,NumIndices)
	 */
	UE_API void CreateOrUpdatePointSet(const FString& PointSetIdentifier, int32 NumIndices,
		TFunctionRef<void(int32 Index, TArray<FRenderablePoint>& PointsOut)> PointGenFunc,
		int32 PointsPerIndexHint = -1);


public:

	/**
	 * Actor created and managed by the UPreviewGeometry
	 */
	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> ParentActor = nullptr;

	/**
	 * TriangleSetComponents created and owned by the UPreviewGeometry, and added as child components of the ParentActor
	 */
	UPROPERTY()
	TMap<FString, TObjectPtr<UTriangleSetComponent>> TriangleSets;

	/**
	 * LineSetComponents created and owned by the UPreviewGeometry, and added as child components of the ParentActor
	 */
	UPROPERTY()
	TMap<FString, TObjectPtr<ULineSetComponent>> LineSets;

	/**
	 * PointSetComponents created and owned by the UPreviewGeometry, and added as child components of the ParentActor
	 */
	UPROPERTY()
	TMap<FString, TObjectPtr<UPointSetComponent>> PointSets;

protected:

	// called at the end of CreateInWorld() to allow subclasses to add additional setup
	virtual void OnCreated() {}

	// called at the beginning of Disconnect() to allow subclasses to perform additional cleanup
	virtual void OnDisconnected() {}
};

#undef UE_API
