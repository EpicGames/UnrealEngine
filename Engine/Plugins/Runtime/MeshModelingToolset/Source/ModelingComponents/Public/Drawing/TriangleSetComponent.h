// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/MeshComponent.h"
#include "Materials/MaterialInterface.h"
#include "IndexTypes.h"

#include "TriangleSetComponent.generated.h"

#define UE_API MODELINGCOMPONENTS_API

USTRUCT()
struct FRenderableTriangleVertex
{
	GENERATED_BODY()

	FRenderableTriangleVertex()
		: Position(ForceInitToZero)
		, UV(ForceInitToZero)
		, Normal(ForceInitToZero)
		, Color(ForceInitToZero)
	{}

	FRenderableTriangleVertex( const FVector& InPosition, const FVector2D& InUV, const FVector& InNormal, const FColor& InColor )
		: Position( InPosition ),
		  UV( InUV ),
		  Normal( InNormal ),
		  Color( InColor )
	{
	}

	UPROPERTY()
	FVector Position;

	UPROPERTY()
	FVector2D UV;

	UPROPERTY()
	FVector Normal;

	UPROPERTY()
	FColor Color;
};


USTRUCT()
struct FRenderableTriangle
{
	GENERATED_BODY()

	FRenderableTriangle()
		: Material(nullptr)
		, Vertex0()
		, Vertex1()
		, Vertex2()
	{}

	FRenderableTriangle( UMaterialInterface* InMaterial, const FRenderableTriangleVertex& InVertex0, 
		const FRenderableTriangleVertex& InVertex1, const FRenderableTriangleVertex& InVertex2 )
		: Material( InMaterial ),
		  Vertex0( InVertex0 ),
		  Vertex1( InVertex1 ),
		  Vertex2( InVertex2 )
	{
	}

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material;

	UPROPERTY()
	FRenderableTriangleVertex Vertex0;

	UPROPERTY()
	FRenderableTriangleVertex Vertex1;

	UPROPERTY()
	FRenderableTriangleVertex Vertex2;
};

/**
* A component for rendering an arbitrary assortment of triangles. Suitable, for instance, for rendering highlighted faces.
*/
UCLASS(MinimalAPI)
class UTriangleSetComponent : public UMeshComponent
{
	GENERATED_BODY()

public:

	UE_API UTriangleSetComponent();

	/** Clear the triangle set */
	UE_API void Clear();

	/** Reserve enough memory for up to the given ID (for inserting via ID) */
	UE_API void ReserveTriangles(const int32 MaxID);

	/** Add a triangle to be rendered using the component. */
	UE_API int32 AddTriangle(const FRenderableTriangle& OverlayTriangle);

	/** Insert a triangle with the given ID to be rendered using the component. */
	UE_API void InsertTriangle(const int32 ID, const FRenderableTriangle& OverlayTriangle);

	/** Remove a triangle from the component. */
	UE_API void RemoveTriangle(const int32 ID);

	/** Queries whether a triangle with the given ID exists in the component. */
	UE_API bool IsTriangleValid(const int32 ID) const;

	/** Sets the color of all existing triangles */
	UE_API void SetAllTrianglesColor(const FColor& NewColor);

	/** Sets the materials of all existing triangles */
	UE_API void SetAllTrianglesMaterial(UMaterialInterface* Material);

	/**
	 * Add a triangle with the given vertices, normal, Color, and Material
	 * @return ID of the triangle created
	 */
	UE_API int32 AddTriangle(const FVector& A, const FVector& B, const FVector& C, const FVector& Normal, const FColor& Color, UMaterialInterface* Material);

	/**
	 * Add a Quad (two triangles) with the given vertices, normal, Color, and Material
	 * @return ID of the two triangles created
	 */
	UE_API UE::Geometry::FIndex2i AddQuad(const FVector& A, const FVector& B, const FVector& C, const FVector& D, const FVector& Normal, const FColor& Color, UMaterialInterface* Material);

	// utility construction functions

	/**
	 * Add a set of triangles for each index in a sequence
	 * @param NumIndices iterate from 0...NumIndices and call TriangleGenFunc() for each value
	 * @param TriangleGenFunc called to fetch the triangles for an index, callee filles TrianglesOut array (reset before each call)
	 * @param TrianglesPerIndexHint if > 0, will reserve space for NumIndices*TrianglesPerIndexHint new triangles
	 */
	UE_API void AddTriangles(
		int32 NumIndices,
		TFunctionRef<void(int32 Index, TArray<FRenderableTriangle>& TrianglesOut)> TriangleGenFunc,
		int32 TrianglesPerIndexHint = -1,
		bool bDeferRenderStateDirty = false);

private:

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface.

	UE_API int32 FindOrAddMaterialIndex(UMaterialInterface* Material);

	UPROPERTY()
	mutable FBoxSphereBounds Bounds;

	UPROPERTY()
	mutable bool bBoundsDirty;

	TSparseArray<TTuple<int32, int32>> Triangles;
	TSparseArray<TSparseArray<FRenderableTriangle>> TrianglesByMaterial;
	TMap<UMaterialInterface*, int32> MaterialToIndex;

	UE_API int32 AddTriangleInternal(const FRenderableTriangle& OverlayTriangle);

	friend class FTriangleSetSceneProxy;
};

#undef UE_API
