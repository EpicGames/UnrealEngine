// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PreviewMesh.h"
#include "TriangleSetComponent.h"
#include "InteractiveTool.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "UVLayoutPreview.generated.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class UMeshElementsVisualizer;

/**
 * Where should in-viewport UVLayoutPreview be shown, relative to target object
 */
UENUM()
enum class EUVLayoutPreviewSide
{
	/** On the left side of the object center relative to the camera */
	Left,

	/** On the right side of the object center relative to the camera */
	Right
};


/**
 * Visualization settings for UV layout preview
 */
UCLASS(MinimalAPI)
class UUVLayoutPreviewProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Enable the preview UV layout */
	UPROPERTY(EditAnywhere, Category = "Preview UV Layout")
	bool bEnabled = true;

	/** Which side of the selected object the preview UV layout is shown */
	UPROPERTY(EditAnywhere, Category = "Preview UV Layout", meta = (EditCondition = "bEnabled"))
	EUVLayoutPreviewSide Side = EUVLayoutPreviewSide::Right;

	/** Uniform scaling factor */
	UPROPERTY(EditAnywhere, Category = "Preview UV Layout",
		meta = (UIMin = "0.1", UIMax = "10.0", ClampMin = "0.0001", ClampMax = "1000", EditCondition = "bEnabled"))
	float Scale = 1.0;

	/** Offset relative to the center of the selected object */
	UPROPERTY(EditAnywhere, Category = "Preview UV Layout", meta = (EditCondition = "bEnabled", Delta = 0.5, LinearDeltaSensitivity = 1))
	FVector2D Offset = FVector2D(1.0, 0.5);

	/** Show wireframe mesh in the preview UV layout */
	UPROPERTY(EditAnywhere, Category = "Preview UV Layout", meta = (EditCondition = "bEnabled"))
	bool bShowWireframe = true;
};




/**
 * UUVLayoutPreview is a utility object that creates and manages a 3D plane on which a UV layout
 * for a 3D mesh is rendered. The UV layout
 */
UCLASS(MinimalAPI, Transient)
class UUVLayoutPreview : public UObject
{
	GENERATED_BODY()

public:
	UE_API virtual ~UUVLayoutPreview() override;


	/**
	 * Create preview mesh in the World with the given transform
	 */
	UE_API void CreateInWorld(UWorld* World);

	/**
	 * Remove and destroy preview mesh
	 */
	UE_API void Disconnect();


	/**
	 * Configure material set for UV-space preview mesh
	 */
	UE_API void SetSourceMaterials(const FComponentMaterialSet& MaterialSet);

	/**
	 * Specify the current world transform/bounds for the target object. 
	 * UV layout preview is positioned relative to this box
	 */
	UE_API void SetSourceWorldPosition(FTransform WorldTransform, FBox WorldBounds);

	/**
	 * Update the current camera state, used to auto-position the UV layout preview
	 */
	UE_API void SetCurrentCameraState(const FViewCameraState& CameraState);

	/**
	 * Notify the UV Layout Preview that the source UVs have been modified
	 */
	UE_API void UpdateUVMesh(const FDynamicMesh3* SourceMesh, int32 SourceUVLayer = 0);

	/**
	 * Tick the UV Layout Preview, allowing it to update various settings
	 */
	UE_API void OnTick(float DeltaTime);

	/**
	 * Render the UV Layout Preview, allowing it to update various settings
	 */
	UE_API void Render(IToolsContextRenderAPI* RenderAPI);


	/**
	 * Set the transform on the UV Layout preview mesh
	 */
	UE_API void SetTransform(const FTransform& UseTransform);

	/**
	 * Set the visibility of the UV Layout  preview mesh
	 */
	UE_API void SetVisible(bool bVisible);

	/** Visualization settings */
	UPROPERTY()
	TObjectPtr<UUVLayoutPreviewProperties> Settings;

	/** PreviewMesh is initialized with a copy of an input mesh with UVs mapped to position, ie such that (X,Y,Z) = (U,V,0) */
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh;

	/** MeshElementsVisualizer references the PreviewMesh to handle drawing the wireframes for UVs */
	UPROPERTY()
	TObjectPtr<UMeshElementsVisualizer> MeshElementsVisualizer;

	/** Set of additional triangles to draw, eg for backing rectangle, etc */
	UPROPERTY()
	TObjectPtr<UTriangleSetComponent> TriangleComponent;

	/** Configure whether the backing rectangle should be shown */
	UPROPERTY()
	bool bShowBackingRectangle = true;

	/** Configure the backing rectangle material */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> BackingRectangleMaterial = nullptr;


protected:
	UE_API void RecalculatePosition();

	FComponentMaterialSet SourceMaterials;

	UE::Geometry::FFrame3d SourceObjectFrame;
	UE::Geometry::FAxisAlignedBox3d SourceObjectWorldBounds;

	UE::Geometry::FFrame3d CurrentWorldFrame;

	FViewCameraState CameraState;

	bool bSettingsModified = false;

	UE_API float GetCurrentScale();
};

#undef UE_API
