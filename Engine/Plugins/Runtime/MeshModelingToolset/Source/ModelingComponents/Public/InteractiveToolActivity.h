// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractionMechanic.h"
#include "InteractiveTool.h" //EToolShutdownType

#include "InteractiveToolActivity.generated.h"

class IToolsContextRenderAPI;

enum class EToolActivityStartResult
{
	Running,
	FailedStart,

	// It's not clear whether we want this option. It's certainly the case that depending
	// on settings, an activity may want to end immediately, or perhaps we may want to
	// put single-action subtools on the same system. But the activity can just end on the
	// next tick, and if we allow a third start option, the tool may need to be aware of
	// it as a possibility, plus we have to decide whether or not NotifyActivitySelfEnded
	// applies... Seems messy to have it, so let's keep it out until we really want it.
	//ImmediatelyCompleted,
};

enum class EToolActivityEndResult
{
	Completed,
	Cancelled,
	ErrorDuringEnd,
};


/**
 * A tool activity is a sort of "sub-tool" used to break apart functionality in tools that
 * provide support for different multi-interaction subtasks. It is meant to limit the sprawl
 * of switch statements in the base tool, to allow subtasks to be designed similar to how a
 * tool might be designed, and to ease extendability.
 *
 * An activity has the following expectations:
 * - Setup() is called in host tool setup and Shutdown() is called in host tool Shutdown()
 * - Start() is called to start the activity (such as when user clicks a button).
 * - If the activity returns a result of EStartResult::ActivityRunning on Start(), it will expect
 *   Render() and Tick() calls from the host until either (a)- the host calls End() on the activity,
 *   or (b)- the activity reaches a stopping point itself and calls NotifyActivitySelfEnded() on the
 *   host. The activity should not require Render() and Tick() if it is not running.
 * 
 *  Compared to a UInteractionMechanic, a tool activity:
 * - Expects that it is the main consumer of input, i.e. takes over the tool. Mechanics, by contrast,
 *   are currently often used together with other mechanics, or to support main tool functionality.
 * - Should not require the hosting tool to have specific knowledge about it or be heavily involved.
 *   Mechanics, by contrast, currently often require tools to use various mechanic-specific getters/setters
 *   during the tool.
 * 
 * Passing data back and forth can be done either by letting a tool activity use a specific
 * context object that the tool can prep in the context store, or by requiring the host to
 * implement specific interfaces (that the activity can check for in Setup()).
 */
UCLASS(MinimalAPI)
class UInteractiveToolActivity : public UInteractionMechanic
{
	GENERATED_BODY()

public:
	// Note about the below: we didn't really need to repeat Setup(), Tick(), and Render() here
	// since these are staying the same as UInteractionMechanic and get inherited. But it is 
	// helpful to have the entire API that we need to implement described in one place

	virtual ~UInteractiveToolActivity() = default;

	/**
	 * Should be called during a tool's Setup(). Does not start the activity.
	 */
	virtual void Setup(UInteractiveTool* ParentToolIn) override
	{
		Super::Setup(ParentToolIn);
	}

	/**
	 * Should be called during a tool's Shutdown()
	 */
	virtual void Shutdown(EToolShutdownType ShutdownType)
	{
		Super::Shutdown();
	}

	/**
	 * Check whether a Start() call will result in a success.
	 */
	virtual bool CanStart() const { return false; }

	/**
	 * Attempt to start the activity.
	 */
	virtual EToolActivityStartResult Start() { return EToolActivityStartResult::FailedStart; }

	/**
	 * Check whether the activity is running (though the tool can just check the result of
	 * Start() itself).
	 */
	virtual bool IsRunning() const 
	{
		// We could keep a boolean here in the base class and update it in the base class Start() and End()
		// functions so that derived classes don't have to remember to implement this. But it is probably better
		// to have the inconvenience of having to implement IsRunning in the derived class than to force
		// derived classes to remember to call base Start() and End() or risk giving an incorrect result here...
		return ensure(false); 
	}

	/**
	 * Similar to a tool, this determines whether the activity prefers an accept/cancel option to be
	 * displayed (if true) vs a "complete".
	 */
	virtual bool HasAccept() const { return false; };

	/**
	 * If true, calling End with EToolShutdownType::Accept will result in a valid completion of
	 * the activity. It's up to the tool whether to respect this or not, since End() should still
	 * end the activity.
	 */
	virtual bool CanAccept() const { return false; };

	/**
	 * Force an end to the activity.
	 */
	virtual EToolActivityEndResult End(EToolShutdownType) { return EToolActivityEndResult::ErrorDuringEnd; };

	/**
	 * If the activity is running, should be called from the tool's OnTick()
	 */
	virtual void Tick(float DeltaTime) override {};

	/**
	 * If the activity is running, should be called from the tool's Render()
	 */
	virtual void Render(IToolsContextRenderAPI* RenderAPI) override {};

protected:


private:
	// We don't want to use this overload of Shutdown because an activity may need to know
	// the ShutdownType if it is still running when Shutdown is called (to End properly).
	virtual void Shutdown() override final
	{
		ensure(false);
		Shutdown(EToolShutdownType::Cancel);
	}
};

UINTERFACE(MinimalAPI)
class UToolActivityHost : public UInterface
{
	GENERATED_BODY()
};

class IToolActivityHost
{
	GENERATED_BODY()
public:

	/**
	 * Notify the tool that the activity has ended.
	 */
	virtual void NotifyActivitySelfEnded(UInteractiveToolActivity* Activity) = 0;
};
