// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "GeometryBase.h"
#include "BoxTypes.h"
#include "IntVectorTypes.h"
#include "LatticeManager.generated.h"

UENUM()
enum class ELatticeInterpolationType : uint8
{
	/** Use trilinear interpolation to get new mesh vertex positions from the lattice */
	Linear UMETA(DisplayName = "Linear"),

	/** Use tricubic interpolation to get new mesh vertex positions from the lattice */
	Cubic UMETA(DisplayName = "Cubic")
};

UINTERFACE(MinimalAPI)
class ULatticeStateStorage : public UInterface
{
	GENERATED_BODY()
};

class ILatticeStateStorage
{
	GENERATED_BODY()
public:

	virtual void StoreLatticePoints(const TArray<FVector3d>& LatticePoints) = 0;
	virtual void ReadLatticePoints(TArray<FVector3d>& LatticePoints) const = 0;
	
	virtual void StoreInterpolationType(ELatticeInterpolationType Type) = 0;
	virtual ELatticeInterpolationType ReadInterpolationType() const = 0;

	virtual UE::Geometry::FAxisAlignedBox3d GetInitialBounds() const = 0;
	virtual UE::Geometry::FTransformSRT3d GetTransform() const = 0;
	virtual UE::Geometry::FVector3i GetResolution() const = 0;

	virtual void InteractiveToolStarted() = 0;
	virtual void InteractiveToolShutDown() = 0;
};
