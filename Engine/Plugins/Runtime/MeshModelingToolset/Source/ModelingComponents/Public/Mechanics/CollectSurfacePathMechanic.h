// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractionMechanic.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "ToolDataVisualizer.h"
#include "CollectSurfacePathMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

using UE::Geometry::FDynamicMesh3;

enum class ECollectSurfacePathDoneMode
{
	SnapCloseLoop,
	SnapDoubleClick,
	SnapDoubleClickOrCloseLoop,
	ExternalLambda,
	FixedNumPoints
};



/**
 */
UCLASS(MinimalAPI)
class UCollectSurfacePathMechanic : public UInteractionMechanic
{
	GENERATED_BODY()
public:
	using FFrame3d = UE::Geometry::FFrame3d;

	TUniqueFunction<bool()> IsDoneFunc = nullptr;

	double ConstantSnapDistance = 10.0f;
	TUniqueFunction<bool(FVector3d, FVector3d)> SpatialSnapPointsFunc;

	bool bSnapToTargetMeshVertices = false;
	bool bSnapToWorldGrid = false;

	// tfunc to emit changes to...

	TArray<FFrame3d> HitPath;

	FFrame3d PreviewPathPoint;
	bool bPreviewPathPointValid = false;

	FToolDataVisualizer PathDrawer;
	FLinearColor PathColor;
	FLinearColor PreviewColor;
	FLinearColor PathCompleteColor;
	bool bDrawPath = true;


public:
	UE_API UCollectSurfacePathMechanic();

	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * Set the hit target mesh.
	 */
	UE_API virtual void InitializeMeshSurface(FDynamicMesh3&& TargetSurfaceMesh);
	UE_API virtual void InitializePlaneSurface(const FFrame3d& TargetPlane);

	UE_API virtual void SetFixedNumPointsMode(int32 NumPoints);
	UE_API virtual void SetDrawClosedLoopMode();
	UE_API virtual void SetCloseWithLambdaMode();
	UE_API virtual void SetDoubleClickOrCloseLoopMode();


	UE_API virtual bool IsHitByRay(const FRay3d& Ray, FFrame3d& HitPoint);
	UE_API virtual bool UpdatePreviewPoint(const FRay3d& Ray);
	UE_API virtual bool TryAddPointFromRay(const FRay3d& Ray);

	UE_API virtual bool PopLastPoint();

	UE_API virtual bool IsDone() const;

	/** Whether the path was finished by the user clicking on the first point */
	bool LoopWasClosed() const
	{
		return bLoopWasClosed;
	}

protected:
	FDynamicMesh3 TargetSurface;
	UE::Geometry::FDynamicMeshAABBTree3 TargetSurfaceAABB;

	FFrame3d TargetPlane;
	bool bHaveTargetPlane;

	UE_API bool RayToPathPoint(const FRay3d& Ray, FFrame3d& PointOut, bool bEnableSnapping);

	ECollectSurfacePathDoneMode DoneMode = ECollectSurfacePathDoneMode::SnapDoubleClick;
	int32 FixedPointTargetCount = 0;
	bool bCurrentPreviewWillComplete = false;
	bool bGeometricCloseOccurred = false;
	UE_API bool CheckGeometricClosure(const FFrame3d& Point, bool* bLoopWasClosedOut = nullptr);

	bool bLoopWasClosed = false;
};

#undef UE_API
