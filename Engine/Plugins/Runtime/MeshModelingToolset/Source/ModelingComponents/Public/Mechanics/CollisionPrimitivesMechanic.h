// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "RectangleMarqueeMechanic.h"
#include "InteractiveToolChange.h"
#include "Spatial/GeometrySet3.h"
#include "ToolContextInterfaces.h" //FViewCameraState
#include "IntVectorTypes.h"
#include "BoxTypes.h"
#include "TransactionUtil.h"

#include "CollisionPrimitivesMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UPreviewGeometry;
class ULineSetComponent;
class UMouseHoverBehavior;
class USingleClickInputBehavior;
class UCombinedTransformGizmo;
class UIntervalGizmo;
class UGizmoLocalFloatParameterSource;
class UTransformProxy;
class FPhysicsDataCollection;
struct FKAggregateGeom;
struct FKBoxElem;
struct FKSphylElem;

UCLASS(MinimalAPI)
class UCollisionPrimitivesMechanic : 
	public UInteractionMechanic, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()
	using FVector2i = UE::Geometry::FVector2i;

public:

	// TODO: Snapping

	// This delegate is called every time the collision geometry is changed or moved.
	DECLARE_MULTICAST_DELEGATE(OnCollisionGeometryChangedEvent);
	OnCollisionGeometryChangedEvent OnCollisionGeometryChanged;

	// This delegate is called every time the collision geometry selection changes.
	DECLARE_MULTICAST_DELEGATE(OnSelectionChangedEvent);
	OnSelectionChangedEvent OnSelectionChanged;

	UE_API virtual void Initialize(TSharedPtr<FPhysicsDataCollection>, const UE::Geometry::FAxisAlignedBox3d& MeshBoundsIn, const FTransform3d& LocalToWorldTransform );
	UE_API void SetWorld(UWorld* World);
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	// UInteractionMechanic
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// IClickBehaviorTarget implementation
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	UE_API void AddSphere();
	UE_API void AddBox();
	UE_API void AddCapsule();
	UE_API void DuplicateSelectedPrimitive();
	UE_API void DeleteSelectedPrimitive();
	UE_API void DeleteAllPrimitives();
	UE_API void UpdateDrawables();

	DECLARE_DELEGATE_RetVal(bool, FShouldHideGizmo);
	FShouldHideGizmo ShouldHideGizmo;

protected:

	TSharedPtr<FPhysicsDataCollection> PhysicsData;
	TSharedPtr<FKAggregateGeom> PrimitivePreTransform;
	FTransform3d LocalToWorldTransform;
	UE::Geometry::FAxisAlignedBox3d MeshBounds;

	// Used for spatial queries
	UE::Geometry::FGeometrySet3 GeometrySet;
	FViewCameraState CachedCameraState;

	/** Used for displaying Primitives/segments */
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry;
	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnPrimitiveEdges;

	TMap< int32, TArray<int32> > PrimitiveToCurveLookup;
	TMap< int32, int32 > CurveToPrimitiveLookup;

	struct FPrimitiveRenderData
	{
		int32 ShapeType;
		int32 PrimIndex;
		int32 LineIndexStart;
		int32 LineIndexEnd;		
		FColor RenderColor;
		FTransform Transform;
	};

	TArray<FPrimitiveRenderData> PrimitiveRenderData;

	// Variables for drawing
	FColor NormalSegmentColor;
	float SegmentsThickness;
	float DepthBias = 0.0;
	FColor HoverColor;
	FColor SelectedColor;

	// Cache previous color while temporarily changing the color of a hovered-over curve
	FColor PreHoverPrimitiveColor;

	// Support for Shift and Ctrl toggle
	bool bShiftToggle = false;
	bool bCtrlToggle = false;
	static const int32 ShiftModifierID = 1;
	static const int32 CtrlModifierID = 2;

	// Default modifier key behavior is consistent with PolygonSelectionMechanic
	TFunction<bool(void)> ShouldAddToSelectionFunc = [this]() {return bShiftToggle; };
	TFunction<bool(void)> ShouldRemoveFromSelectionFunc = [this]() {return bCtrlToggle; };

	// Support for gizmos. Since the primitives aren't individual components, we don't actually use UTransformProxy
	// for the transform forwarding- we just use it for the callbacks.
	UPROPERTY()
	TObjectPtr<UTransformProxy> TranslateTransformProxy = nullptr;
	UPROPERTY()
	TObjectPtr<UTransformProxy> SphereTransformProxy = nullptr;
	UPROPERTY()
	TObjectPtr<UTransformProxy> BoxTransformProxy = nullptr;
	UPROPERTY()
	TObjectPtr<UTransformProxy> CapsuleTransformProxy = nullptr;
	UPROPERTY()
	TObjectPtr<UTransformProxy> FullTransformProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UTransformProxy> CurrentActiveProxy = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TranslateTransformGizmo = nullptr;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> SphereTransformGizmo = nullptr;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> BoxTransformGizmo = nullptr;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> CapsuleTransformGizmo = nullptr;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> FullTransformGizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UIntervalGizmo> BoxIntervalGizmo = nullptr;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> BoxXIntervalSource;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> BoxYIntervalSource;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> BoxZIntervalSource;

	UPROPERTY()
	TObjectPtr<UIntervalGizmo> CapsuleIntervalGizmo = nullptr;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> CapsuleRadiusIntervalSource;
	UPROPERTY()
	TObjectPtr<UGizmoLocalFloatParameterSource> CapsuleLengthIntervalSource;

	// Used to make it easy to tell whether the gizmo was moved by the user or by undo/redo or
	// some other change that we shouldn't respond to. Basing our movement undo/redo on the
	// gizmo turns out to be quite a pain, though may someday be easier if the transform proxy
	// is able to manage arbitrary objects.
	bool bGizmoBeingDragged = false;

	// Callbacks we'll receive from the gizmo proxies
	UE_API void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void GizmoTransformStarted(UTransformProxy* Proxy);
	UE_API void GizmoTransformEnded(UTransformProxy* Proxy);

	// Callbacks we'll receive from the interval gizmo
	UE_API void IntervalGizmoValueChanged(UIntervalGizmo* IntervalGizmo, const FVector& Direction, float Value);

	// Support for hovering
	TFunction<bool(const FVector3d&, const FVector3d&)> GeometrySetToleranceTest;
	int32 HoveredPrimitiveID = -1;
	UE_API void ClearHover();

	// Support for selection
	UPROPERTY()
	TObjectPtr<URectangleMarqueeMechanic> MarqueeMechanic;
	bool bIsDraggingRectangle = false;
	TSet<int32> SelectedPrimitiveIDs;
	TSet<int32> PreDragSelection;
	TArray<int32> CurrentDragSelection;
	UE_API void OnDragRectangleStarted();
	UE_API void OnDragRectangleChanged(const FCameraRectangle& Rectangle);
	UE_API void OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled);

	// The starting point of the gizmo is needed to determine the offset by which to move the points.
	// TODO: Replace with single FTransform?
	FVector GizmoStartPosition;
	FQuat	GizmoStartRotation;
	FVector GizmoStartScale;

	// All of the following do not issue undo/redo change objects.
	UE_API bool HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut);
	UE_API void SelectPrimitive(int32 PrimitiveID);
	UE_API bool DeselectPrimitive(int32 PrimitiveID);
	UE_API void UpdateGizmoLocation();
	UE_API void UpdateGizmoVisibility();

	UE_API void UpdateCollisionGeometry(const FKAggregateGeom& NewGeometryIn);
	UE_API void RebuildDrawables( bool bRegenerateCurveLists = true );

	// Used for expiring undo/redo changes, which compare this to their stored value and expire themselves if they do not match.
	int32 CurrentChangeStamp = 0;


	// Undo/redo support:

	// Primitive selection has changed
	class FCollisionPrimitivesMechanicSelectionChange : public FToolCommandChange
	{
	public:
		FCollisionPrimitivesMechanicSelectionChange(int32 PrimitiveIDIn, bool bAddedIn, const FTransform& PreviousTransformIn,
													 const FTransform& NewTransformIn, int32 ChangeStampIn);

		FCollisionPrimitivesMechanicSelectionChange(const TSet<int32>& PrimitiveIDsIn, bool bAddedIn, const FTransform& PreviousTransformIn,
													 const FTransform& NewTransformIn, int32 ChangeStampIn);

		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;
		virtual bool HasExpired(UObject* Object) const override
		{
			return Cast<UCollisionPrimitivesMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
		}
		virtual FString ToString() const override;

	protected:
		TSet<int32> PrimitiveIDs;
		bool bAdded;

		const FTransform PreviousTransform;
		const FTransform NewTransform;

		int32 ChangeStamp;
	};


	// Primitives have moved/changed
	class FCollisionPrimitivesMechanicGeometryChange : public FToolCommandChange
	{
	public:

		FCollisionPrimitivesMechanicGeometryChange(
			TSharedPtr<FKAggregateGeom> GeometryPreviousIn,
			TSharedPtr<FKAggregateGeom> GeometryNewIn,
			int32 ChangeStampIn);

		virtual void Apply(UObject* Object) override;
		virtual void Revert(UObject* Object) override;
		virtual bool HasExpired(UObject* Object) const override
		{
			return Cast<UCollisionPrimitivesMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
		}
		virtual FString ToString() const override;

	protected:

		TSharedPtr<FKAggregateGeom> GeometryPrevious;
		TSharedPtr<FKAggregateGeom> GeometryNew;
		int32 ChangeStamp;	
	};

private:
	UE::TransactionUtil::FLongTransactionTracker LongTransactions;

	// Helper to get a 'safe' copy of scale in which no elements are zero, so we can divide by each dimension
	UE_API FVector3d GetSafeAbsScale(FVector3d Scale3D) const;
	
	UE_API void SetBoxShapeFromIntervals(FKBoxElem* BoxElem) const;
	UE_API void SetCapsuleShapeFromIntervals(FKSphylElem* CapsuleElem) const;
};

#undef UE_API
