// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractionMechanic.h"
#include "ToolDataVisualizer.h"
#include "FrameTypes.h"
#include "Selection/SelectClickedAction.h"

#include "ConstructionPlaneMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class ULocalSingleClickInputBehavior;
class UCombinedTransformGizmo;
class UTransformProxy;
class IClickBehaviorTarget;
class USingleClickInputBehavior;

/**
 * UConstructionPlaneMechanic implements an interaction in which a 3D plane can be
 * positioned using the standard 3D Gizmo, or placed at hit-locations in the existing scene.
 * A grid in the plane can optionally be rendered.
 */
UCLASS(MinimalAPI)
class UConstructionPlaneMechanic : public UInteractionMechanic
{
	GENERATED_BODY()
public:
	
	/** Replace this to externally control if plane can be updated */
	TUniqueFunction<bool()> CanUpdatePlaneFunc = []() { return true; };

	bool bShowGrid = true;

	UE::Geometry::FFrame3d Plane;

	DECLARE_MULTICAST_DELEGATE(OnConstructionPlaneChangedEvent);
	OnConstructionPlaneChangedEvent OnPlaneChanged;


public:

	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	UE_API void Initialize(UWorld* TargetWorld, const UE::Geometry::FFrame3d& InitialPlane);

	UE_API void SetDrawPlaneFromWorldPos(const FVector3d& Position, const FVector3d& Normal, bool bIgnoreNormal);

	/** 
	 * Sets the plane without broadcasting OnPlaneChanged. Useful when the user of the tool wants to change
	 * the plane through some other means. Better than setting the Plane field directly because this function
	 * properly deals with the gizmo.
	 */
	UE_API void SetPlaneWithoutBroadcast(const UE::Geometry::FFrame3d& Plane);

	UE_API void UpdateClickPriority(FInputCapturePriority NewPriority);

public:
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> PlaneTransformGizmo;
	
	UPROPERTY()
	TObjectPtr<UTransformProxy> PlaneTransformProxy;

	/** 
	 * This is the behavior target used for the Ctrl+click behavior that sets the plane
	 * in the world, exposed here so that the user can modify it after Setup() if needed.
	 * By default, Setup() will have it call SetDrawPlaneFromWorldPos.
	 */
	TUniquePtr<FSelectClickedAction> SetPlaneCtrlClickBehaviorTarget;

protected:
	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickToSetPlaneBehavior;
	
	/** This is the behavior and behavior target used for the Shift+click behavior that sets the gizmo's position in the world. */
	UPROPERTY()
	TObjectPtr<ULocalSingleClickInputBehavior> MiddleClickToSetGizmoBehavior;

	UE_API void TransformChanged(UTransformProxy* Proxy, FTransform Transform);
};

#undef UE_API
