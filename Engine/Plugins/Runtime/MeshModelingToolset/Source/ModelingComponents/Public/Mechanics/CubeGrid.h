// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "VectorTypes.h"
#include "FrameTypes.h"
#include "InteractionMechanic.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE {
namespace Geometry {

/**
 * Class representing an adjustable and resizable power-of-two grid in which faces can be
 * selected. 
 */
class FCubeGrid
{
public:
	virtual ~FCubeGrid() = default;

	/** 
	 * Direction that a face is facing. Values are set up like in IndexUtil, such that
	 * abs(value)-1 is the nonzero normal direction, and sign(value) is the sign of the
	 * normal.
	 */
	enum class EFaceDirection : int8
	{
		NegativeX = -1,
		PositiveX = 1,
		NegativeY = -2,
		PositiveY = 2,
		NegativeZ = -3,
		PositiveZ = 3
	};

	enum class EPowerMode : int8
	{
		PowerOfTwo,
		FiveAndTen
	};

	/**
	 * Convert face direction to the index of the nonzero normal dimension, or equivalently,
	 * the dimension in which that face is flat.
	 */
	static int8 DirToFlatDim(EFaceDirection DirectionIn)
	{
		return FMath::Abs(static_cast<int8>(DirectionIn)) - 1;
	}

	 /** Convert face direction to its normal. */
	static FVector3d DirToNormal(EFaceDirection DirectionIn)
	{
		FVector3d Normal = FVector3d::Zero();
		Normal[DirToFlatDim(DirectionIn)] = FMath::Sign(static_cast<int8>(DirectionIn));
		return Normal;
	}

	/** Get flipped version of a face direction. */
	static EFaceDirection FlipDir(EFaceDirection DirectionIn)
	{
		return static_cast<EFaceDirection>(-static_cast<int8>(DirectionIn));
	}

	/** 
	 * A face always refers to coordinates in the grid space. It is always
	 * on grid, and tied to a particular grid power.
	 */
	class FCubeFace
	{
	public:
		UE_API FCubeFace(const FVector3d& PointOnFace, EFaceDirection Direction, uint8 SourceCubeGridPower);

		FCubeFace()
			: FCubeFace(FVector3d::Zero(), 
				EFaceDirection::PositiveZ,
				0)
		{}

		const FVector3d& GetCenter() const { return Center; }
		EFaceDirection GetDirection() const { return Direction; }

		FVector3d GetNormal() const { 
			return DirToNormal(Direction);
		}

		UE_API FVector3d GetMinCorner() const;
		UE_API FVector3d GetMaxCorner() const;
		uint8 GetSourceCubeGridPower() const { return CubeGridPower; }

		bool operator==(const FCubeFace& Other) const
		{
			return Center == Other.Center && Direction == Other.Direction;
		}
		bool operator!=(const FCubeFace& Other) const
		{
			return !(*this == Other);
		}
	private:
		// One of the components will always be an integer, and the other two will always be offset
		// from an integer by 0.5, since this is the center of a cube grid face.
		FVector3d Center;

		EFaceDirection Direction;
		uint8 CubeGridPower;
	};

	FCubeGrid()
		: BaseGridCellSize(3.125)
		, CurrentGridPower(5)
	{
		UpdateCellSize();
	}

	UE_API virtual bool GetHitGridFaceBasedOnRay(const FVector3d& WorldHitPoint, 
		const FVector3d& NormalOrTowardCameraRay,
		FCubeGrid::FCubeFace& FaceOut, bool bPierceToBack, double PlaneTolerance) const;

	uint8 GetGridPower() const { return CurrentGridPower; }

	void SetGridPower(uint8 PowerIn)
	{
		ensureMsgf(PowerIn <= GetMaxGridPower(), TEXT("FCubeGrid: Grid power set above MaxGridPower."));
		CurrentGridPower = PowerIn;
		UpdateCellSize();
	}
	uint8 GetMaxGridPower() const
	{
		return sizeof(uint32) * 8 - 1;
	}
	void SetGridPowerMode(EPowerMode PowerModeIn)
	{
		GridPowerMode = PowerModeIn;
		UpdateCellSize();
	}
	EPowerMode GetGridPowerMode() const { return GridPowerMode; }

	double GetBaseGridCellSize() const { return BaseGridCellSize; }
	void SetBaseGridCellSize(double SizeIn)
	{
		if (!ensure(SizeIn > 0))
		{
			SizeIn = KINDA_SMALL_NUMBER;
		}
		BaseGridCellSize = SizeIn;
		UpdateCellSize();
	}

	double GetCurrentGridCellSize() const { return CurrentCellSize; }
	UE_API void SetCurrentGridCellSize(double SizeIn);

	UE_API double GetCellSize(uint8 GridPower) const;

	FVector3d ToWorldPoint(FVector3d GridPoint) const
	{
		return GridFrame.FromFramePoint(GridPoint * CurrentCellSize);
	}

	FVector3d ToWorldPoint(FVector3d GridPoint, uint8 GridPower) const
	{
		return GridFrame.FromFramePoint(GridPoint * GetCellSize(GridPower));
	}

	FVector3d ToGridPoint(FVector3d WorldPoint) const
	{
		return GridFrame.ToFramePoint(WorldPoint) / CurrentCellSize;
	}

	FTransform GetFTransform() const
	{
		FTransform ToReturn = GridFrame.ToFTransform();
		ToReturn.SetScale3D(FVector(GetCurrentGridCellSize()));
		return ToReturn;
	}

	const FFrame3d& GetFrame() const { return GridFrame; }
	void SetGridFrame(const FFrame3d& FrameIn) { GridFrame = FrameIn; }
protected:

	void UpdateCellSize() { CurrentCellSize = GetCellSize(CurrentGridPower); }

	FFrame3d GridFrame;
	double BaseGridCellSize;
	uint8 CurrentGridPower;
	EPowerMode GridPowerMode = EPowerMode::PowerOfTwo;
	// This should always be BaseGridCellSize * 2^CurrentGridPower
	double CurrentCellSize;
};

}}//end namespace UE::Geometry

#undef UE_API
