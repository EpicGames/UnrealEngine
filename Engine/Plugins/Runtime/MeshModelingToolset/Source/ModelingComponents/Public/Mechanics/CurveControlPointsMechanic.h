// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "FrameTypes.h"
#include "InteractionMechanic.h"
#include "InteractiveToolChange.h"
#include "Snapping/PointPlanarSnapSolver.h"
#include "Spatial/GeometrySet3.h"
#include "ToolContextInterfaces.h" //FViewCameraState
#include "VectorTypes.h"

#include "CurveControlPointsMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class APreviewGeometryActor;
class ULineSetComponent;
class UMouseHoverBehavior;
class UPointSetComponent;
class USingleClickInputBehavior;
class UCombinedTransformGizmo;
class UTransformProxy;

/**
 * A mechanic for displaying a sequence of control points and moving them about. Has an interactive initialization mode for
 * first setting the points.
 *
 * When editing, hold shift to select multiple points. Hold Ctrl to add an extra point along an edge. To add points to either end of
 * the sequence, first select either the first or last point and then hold Ctrl.
 * Backspace deletes currently selected points. In edit mode, holding Shift generally toggles the snapping behavior (makes it opposite
 * of the current SnappingEnabled setting), though this is not yet implemented while the gizmo is being dragged.
 *
 * TODO:
 * - Make it possible to open/close loop in edit mode
 * - Improve display of occluded control points (checkerboard the material)
 * - Allow deselection of vertices by clicking away?
 * - Lump the point/line set components into PreviewGeometryActor.
 */
UCLASS(MinimalAPI)
class UCurveControlPointsMechanic : public UInteractionMechanic, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

protected:

	// We want some way to store the control point sequence that lets us easily associate points with their renderable and hit-testable
	// representations, since we need to alter all of these together as points get moved or added. We use FOrderedPoints for this, until
	// we decide on how we want to store sequences of points in general.
	// FOrderedPoints maintains a sequence of point ID's, the positions associated with each ID, and a mapping back from ID to position.
	// The ID's can then be used to match to renderable points, hit-testable points, and segments going to the next point.
	
	/**
	 * A sequence of 3-component vectors that can be used to represent a polyline in 3d space, or
	 * some other sequence of control points in 3d space.
	 *
	 * The sequence is determined by a sequence of point IDs which can be used to look up the point
	 * coordinates.
	 */
	class FOrderedPoints
	{
	public:

		FOrderedPoints() {};
		FOrderedPoints(const FOrderedPoints& ToCopy);
		FOrderedPoints(const TArray<FVector3d>& PointSequence);

		/** @return number of points in the sequence. */
		int32 Num()
		{
			return Sequence.Num();
		}

		/** @return last point ID in the sequence. */
		int32 Last()
		{
			return Sequence.Last();
		}

		/** @return first point ID in the sequence. */
		int32 First()
		{
			return Sequence[0];
		}

		/**
		 * Appends a point with the given coordinates to the end of the sequence.
		 *
		 * @return the new point's ID.
		 */
		int32 AppendPoint(const FVector3d& PointCoordinates);

		/**
		 * Inserts a point with the given coordinates at the given position in the sequence.
		 *
		 * @param KnownPointID If not null, this parameter stores the PointID we want the new point
		 *  to have. This is useful for undo/redo operations, where we want to make sure that the
		 *  we don't end up giving a point a different ID than we did last time. If null, the
		 *  class generates an ID.
		 * @return the new point's ID.
		 */
		int32 InsertPointAt(int32 SequencePosition, const FVector3d& PointCoordinates, const int32* KnownPointID = nullptr);

		/**
		 * Removes the point at a particular position in the sequence.
		 *
		 * @return point ID of the removed point.
		 */
		int32 RemovePointAt(int32 SequencePosition);

		/**
		 * @return index in the sequence of a given Point ID.
		 */
		int32 GetSequencePosition(int32 PointID)
		{
			return PointIDToSequencePosition[PointID];
		}

		/**
		 * @return point ID at the given position in the sequence.
		 */
		int32 GetPointIDAt(int32 SequencePosition)
		{
			return Sequence[SequencePosition];
		}

		/**
		 * @return coordinates of the point with the given point ID.
		 */
		FVector3d GetPointCoordinates(int32 PointID) const
		{
			check(IsValidPoint(PointID));
			return Vertices[PointID];
		}

		/**
		 * @return coordinates of the point at the given position in the sequence.
		 */
		FVector3d GetPointCoordinatesAt(int32 SequencePosition) const
		{
			return Vertices[Sequence[SequencePosition]];
		}

		/**
		 * Checks whether given point ID exists in the sequence.
		 */
		bool IsValidPoint(int32 PointID) const
		{
			return Vertices.IsValidIndex(PointID);
		}

		/**
		 * Change the coordinates associated with a given point ID.
		 */
		void SetPointCoordinates(int32 PointID, const FVector3d& NewCoordinates)
		{
			checkSlow(UE::Geometry::VectorUtil::IsFinite(NewCoordinates));
			check(IsValidPoint(PointID));

			Vertices[PointID] = NewCoordinates;
		}

		/**
		 * Delete all points in the sequence.
		 */
		void Empty();

		// TODO: We should have a proper iterable to iterate over point ID's (and maybe one to iterate over point coordinates
		// too. We're temporarily taking a shortcut by using the actual sequence array as our iterable, but only until 
		// we've decided on the specifics of the class.
		typedef const TArray<int32>& PointIDEnumerable;

		/**
		 * This function should only be used to iterate across the point id's in sequence in a range-based for-loop
		 * as in "for (int32 PointID : PointSequence->PointIDItr()) { ... }"
		 * The return type of this function is likely to change, but it will continue to work in range-based for-loops.
		 */
		PointIDEnumerable PointIDItr()
		{
			return Sequence;
		}

	protected:
		TSparseArray<FVector3d> Vertices;
		TArray<int32> Sequence;
		TMap<int32, int32> PointIDToSequencePosition;

		void ReInitialize(const TArray<FVector3d>& PointSequence);
	};
	//end FOrderedPoints

public:

	// Behaviors used for moving points around and hovering them
	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickBehavior = nullptr;
	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior = nullptr;

	// This delegate is called every time the control point sequence is altered.
	DECLARE_MULTICAST_DELEGATE(OnPointsChangedEvent);
	OnPointsChangedEvent OnPointsChanged;

	// This delegate is called when the mode of the mechanic changes (i.e., we leave or re-enter interactive initialization)
	DECLARE_MULTICAST_DELEGATE(OnModeChangedEvent);
	OnModeChangedEvent OnModeChanged;

	// Functions used for initializing the mechanic
	UE_API virtual void Initialize(const TArray<FVector3d>& Points, bool bIsLoop);
	UE_API int32 AppendPoint(const FVector3d& PointCoordinates);
	
	// Interactive initialization mode allows the user to click multiple times to initialize
	// the curve (without having to hold Ctrl), and to transition to edit mode by clicking the
	// last or first points (provided the minimal numbers of points have been met)
	UE_API void SetInteractiveInitialization(bool bOn);
	bool IsInInteractiveIntialization()
	{
		return bInteractiveInitializationMode;
	}

	// In interactive intialization mode, these minimums determine how many points must
	// exist before initialization mode can be left.
	void SetMinPointsToLeaveInteractiveInitialization(int32 MinForLoop, int32 MinForNonLoop)
	{
		MinPointsForLoop = MinForLoop;
		MinPointsForNonLoop = MinForNonLoop;
	}

	// When true, if the number of control points falls below the mins required (through
	// deletion by the user), the mechanic automatically falls back into interactive 
	// intialization mode.
	void SetAutoRevertToInteractiveInitialization(bool bOn)
	{
		bAutoRevertToInteractiveInitialization = bOn;
	}


	UE_API void SetIsLoop(bool bIsLoop);

	/** Returns whether the underlying sequence of control points is a loop. */
	bool GetIsLoop()
	{
		return bIsLoop;
	}

	/** Sets the plane on which new points are added on the ends and in which the points are moved. */
	UE_API void SetPlane(const UE::Geometry::FFrame3d& DrawPlaneIn);
	// TODO: It is simple to allow the points to be moved arbitrarily, not just inside the plane, if we ever
	// want to use the mechanic somewhere where that is desirable. However, we'd need to do a little more work
	// to allow new points to be added in arbitrary locations.

	UE_API void SetSnappingEnabled(bool bOn);

	// Adds additional line to snap points to. Useful, for instance, if the curve is a revolution profile curve
	// and needs to be able to snap to the revolution axis.
	UE_API void AddSnapLine(int32 LineID, const UE::Geometry::FLine3d& Line);

	UE_API void RemoveSnapLine(int32 LineID);

	/** Clears all points in the mechanic. */
	UE_API void ClearPoints();

	/**
	 * Deletes currently selected points- can be called on a key press from the parent tool.
	 *
	 * Ideally, the mechanic would catch key presses itself, without the tool having to worry 
	 * about it. However for now, we are limited to having to register the key handler in the
	 * tool.
	 */
	UE_API void DeleteSelectedPoints();

	/** Expires any changes currently associated with the mechanic in the undo/redo stack. */
	void ExpireChanges()
	{
		++CurrentChangeStamp;
	}

	/** Outputs the positions of the points in the control point sequence. Does not clear PositionsOut before use. */
	UE_API void ExtractPointPositions(TArray<FVector3d> &PositionsOut);

	/** Gives number of points currently managed by the mechanic. */
	int32 GetNumPoints()
	{
		return ControlPoints.Num();
	}


	// Some other standard functions
	UE_API virtual ~UCurveControlPointsMechanic();
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;
	UE_API void SetWorld(UWorld* World);
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// IClickBehaviorTarget implementation
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	// IModifierToggleBehaviorTarget implementation, inherited through IClickBehaviorTarget
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

protected:

	// This actually stores the sequence of point IDs, and their coordinates.
	FOrderedPoints ControlPoints;

	bool bIsLoop;
	bool bInteractiveInitializationMode = false;

	int32 MinPointsForLoop = 3;
	int32 MinPointsForNonLoop = 2;
	bool bAutoRevertToInteractiveInitialization = false;

	bool bSnappingEnabled = true;
	UE::Geometry::FPointPlanarSnapSolver SnapEngine;

	// Used for snapping to the start/end of the curve to get out of initialization mode
	int32 FirstPointSnapID;
	int32 LastPointSnapID;
	int32 EndpointSnapPriority;

	// When storing user-defined lines to snap to, we add this to the user-provided id to avoid
	// conflicting with any lines generated by the snap engine.
	int32 LineSnapIDMin;

	int32 LineSnapPriority;
	
	// Used for spatial queries
	UE::Geometry::FGeometrySet3 GeometrySet;

	/** Used for displaying points/segments */
	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> PreviewGeometryActor;
	UPROPERTY()
	TObjectPtr<UPointSetComponent> DrawnControlPoints;
	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnControlSegments;

	// These get drawn separately because the other components have to be 1:1 with the control
	// points structure, which would make it complicated to keep track of special id's.
	UPROPERTY()
	TObjectPtr<UPointSetComponent> PreviewPoint;
	UPROPERTY()
	TObjectPtr<ULineSetComponent> PreviewSegment;

	// Variables for drawing
	FColor InitializationCurveColor;
	FColor NormalCurveColor;
	FColor CurrentSegmentsColor;
	FColor CurrentPointsColor;
	float SegmentsThickness;
	float PointsSize;
	float DepthBias;
	FColor PreviewColor;
	FColor HoverColor;
	FColor SelectedColor;
	FColor SnapLineColor;
	FColor HighlightColor;

	// Used for adding new points on the ends and for limiting point movement
	UE::Geometry::FFrame3d DrawPlane;

	// Support for Shift and Ctrl toggles
	bool bAddToSelectionToggle = false;
	bool bSnapToggle = false;
	int32 ShiftModifierId = 1;
	bool bInsertPointToggle = false;
	int32 CtrlModifierId = 2;

	// Support for gizmo. Since the points aren't individual components, we don't actually use UTransformProxy
	// for the transform forwarding- we just use it for the callbacks.
	UPROPERTY()
	TObjectPtr<UTransformProxy> PointTransformProxy;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> PointTransformGizmo;

	// Used to make it easy to tell whether the gizmo was moved by the user or by undo/redo or
	// some other change that we shoulnd't respond to. Basing our movement undo/redo on the
	// gizmo turns out to be quite a pain, though may someday be easier if the transform proxy
	// is able to manage arbitrary objects.
	bool bGizmoBeingDragged = false;

	// Callbacks we'll receive from the gizmo proxy
	UE_API void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void GizmoTransformStarted(UTransformProxy* Proxy);
	UE_API void GizmoTransformEnded(UTransformProxy* Proxy);

	// Support for hovering
	FViewCameraState CameraState;
	TFunction<bool(const FVector3d&, const FVector3d&)> GeometrySetToleranceTest;
	int32 HoveredPointID = -1;
	UE_API void ClearHover();
	// Used to unhover a point, since this will differ depending on whether the point is selected.
	FColor PreHoverPointColor;
	UE_API void UpdateSnapTargetsForHover();
	UE_API void UpdateSnapHistoryPoint(int32 Index, FVector3d NewPosition);

	// Support for selection
	TArray<int32> SelectedPointIDs;
	// We need the selected point start positions so we can move multiple points appropriately.
	TArray<FVector3d> SelectedPointStartPositions;
	// The starting point of the gizmo is needed to determine the offset by which to move the points.
	FVector GizmoStartPosition;

	// These issue undo/redo change objects, and must therefore not be called in undo/redo code.
	UE_API void ChangeSelection(int32 NewPointID, bool AddToSelection);
	UE_API void ClearSelection();

	// All of the following do not issue undo/redo change objects.
	UE_API int32 InsertPointAt(int32 SequencePosition, const FVector3d& NewPointCoordinates, const int32* KnownPointID = nullptr);
	UE_API int32 DeletePoint(int32 SequencePosition);
	UE_API bool HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut);
	UE_API void SelectPoint(int32 PointID);
	UE_API bool DeselectPoint(int32 PointID);
	UE_API void UpdateGizmoVisibility();
	UE_API void UpdateGizmoLocation();
	UE_API void UpdatePointLocation(int32 PointID, const FVector3d& NewLocation);

	// Used for expiring undo/redo changes, which compare this to their stored value and expire themselves if they do not match.
	int32 CurrentChangeStamp = 0;

	friend class FCurveControlPointsMechanicSelectionChange;
	friend class FCurveControlPointsMechanicInsertionChange;
	friend class FCurveControlPointsMechanicMovementChange;
	friend class FCurveControlPointsMechanicModeChange;
};


// Undo/redo support:

class FCurveControlPointsMechanicSelectionChange : public FToolCommandChange
{
public:
	UE_API FCurveControlPointsMechanicSelectionChange(int32 SequencePositionIn, bool AddedIn, int32 ChangeStampIn);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<UCurveControlPointsMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
	}
	UE_API virtual FString ToString() const override;

protected:
	int32 PointID;
	bool Added;
	int32 ChangeStamp;
};

class FCurveControlPointsMechanicInsertionChange : public FToolCommandChange
{
public:
	UE_API FCurveControlPointsMechanicInsertionChange(int32 SequencePositionIn, int32 PointID, 
		const FVector3d& CoordinatesIn, bool AddedIn, int32 ChangeStampIn);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<UCurveControlPointsMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
	}
	UE_API virtual FString ToString() const override;

protected:
	int32 SequencePosition;
	int32 PointID;
	FVector3d Coordinates;
	bool Added;
	int32 ChangeStamp;
};

class FCurveControlPointsMechanicModeChange : public FToolCommandChange
{
public:
	UE_API FCurveControlPointsMechanicModeChange(bool bDoneWithInitializationIn, bool bIsLoopIn, int32 ChangeStampIn);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<UCurveControlPointsMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
	}
	UE_API virtual FString ToString() const override;

protected:
	bool bDoneWithInitialization;
	bool bIsLoop;
	int32 ChangeStamp;
};

class FCurveControlPointsMechanicMovementChange : public FToolCommandChange
{
public:
	UE_API FCurveControlPointsMechanicMovementChange(int32 PointIDIn, const FVector3d& OriginalPositionIn, 
		const FVector3d& NewPositionIn, int32 ChangeStampIn);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<UCurveControlPointsMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
	}
	UE_API virtual FString ToString() const override;

protected:
	int32 PointID;
	FVector3d OriginalPosition;
	FVector3d NewPosition;
	int32 ChangeStamp;
};

#undef UE_API
