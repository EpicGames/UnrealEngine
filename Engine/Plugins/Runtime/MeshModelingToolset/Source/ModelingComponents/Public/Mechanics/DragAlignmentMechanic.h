// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "ToolContextInterfaces.h" // FViewCameraState
#include "Engine/HitResult.h"
#include "InteractionMechanic.h"
#include "TransformTypes.h"
#include "VectorTypes.h"

#include "DragAlignmentMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE::Geometry { class FGroupTopologyDeformer; }
class UPrimitiveComponent;
class UIntervalGizmo;
class UCombinedTransformGizmo;
class USceneSnappingManager;
class UKeyAsModifierInputBehavior;


/**
 * FDragAlignmentBase is a base class for shared functionality of UDragAlignmentMechanic and UDragAlignmentInteraction.
 * This interaction can be attached to (potentially multiple) UCombinedTransformGizmo object to allow them to snap to
 * scene geometry in rotation and translation when a modifier key is pressed. 
 */
class FDragAlignmentBase
{
public:
	virtual ~FDragAlignmentBase() = default;

	//
	// Subclasses must implement these various API functions for the interaction to function
	//
	virtual USceneSnappingManager* GetSnappingManager() = 0;
	virtual FViewCameraState GetCameraState() = 0;
	virtual bool GetAlignmentModeEnabled() const = 0;
	virtual UObject* GetUObjectContainer() = 0;

public:


	/**
	 * Adds this mechanic to the given gizmo. Can be called on multiple gizmos, and works both for UCombinedTransformGizmo
	 * and URepositionableTransformGizmo.
	 * For repositioning, the gizmo will not ignore anything in its alignment (so that the gizmo can be aligned to the
	 * target object(s)). For movement, the gizmo will ignore ComponentsToIgnore, as well as modified triangles if used
	 * for deformation.
	 */
	UE_API void AddToGizmo(UCombinedTransformGizmo* TransformGizmo, const TArray<const UPrimitiveComponent*>* ComponentsToIgnore = nullptr,
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);

	/**
	 * Like the other AddToGizmo, but acts on interval gizmos.
	 */
	UE_API void AddToGizmo(UIntervalGizmo* IntervalGizmo, const TArray<const UPrimitiveComponent*>* ComponentsToIgnoreInAlignment = nullptr,
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);

	/**
	 * Optional- allows the use of the mechanic in mesh deformation. Specifically, alignment will attempt to align
	 * to the dynamic mesh underlying the given FDynamicMeshAABBTree3, and ignore deformed triangles when the gizmo
	 * is moved.
	 * This works with only one deformed mesh, so it probably wouldn't be used with multiple gizmos.
	 */
	UE_API void InitializeDeformedMeshRayCast(
		TFunction<UE::Geometry::FDynamicMeshAABBTree3* ()> GetSpatialIn,
		const FTransform3d& TargetTransform, const UE::Geometry::FGroupTopologyDeformer* LinearDeformer);


protected:

	/**
	 * Subclasses must call this to initialize the internal functions/states/etc
	 */
	UE_API virtual void SetupInternal();

	/**
	 * This will draw a line from the last input position to the last output position. This
	 * gets cleared when the transform is ended. 
	 */
	UE_API virtual void RenderInternal(IToolsContextRenderAPI* RenderAPI);

protected:
	// These are modifiable in case we someday want to make it easier to customize them further.
	TUniqueFunction<bool(const FRay& InputRay, FHitResult& OutputHit, 
		const TArray<const UPrimitiveComponent*>* ComponentsToIgnore,
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude)> WorldRayCast = nullptr;
	TUniqueFunction<bool(const FRay&, FHitResult&, bool bUseFilter)> MeshRayCast = nullptr;
	TUniqueFunction<bool(int32 Tid)> MeshRayCastTriangleFilter = [](int32 Tid) { return true; };

	// Used to identify the type of object we hit, so we can choose different renderings.
	static const int32 TRIANGLE_HIT_TYPE_ID = 1;
	static const int32 VERT_HIT_TYPE_ID = 2;
	static const int32 EDGE_HIT_TYPE_ID = 3;

	// Used for snapping to verts
	FViewCameraState CachedCameraState;
	float VisualAngleSnapThreshold = 1.0;

	// We keep track of the last position we gave the user and the last position that
	// they moved their gizmo. This lets us render lines connecting the two.
	bool bPreviewEndpointsValid = false;
	bool bWaitingOnProjectedResult = false;
	FHitResult LastHitResult;
	FVector3d LastProjectedResult;

	UE_API virtual bool CastRay(const FRay& WorldRay, FVector& OutputPoint,
		const TArray<const UPrimitiveComponent*>* ComponentsToIgnore, 
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude,
		bool bUseFilter);

	UE_API void OnGizmoTransformChanged(FTransform NewTransform);
};


/**
 * Mechanic that can be added to (potentially multiple) UCombinedTransformGizmo object to allow them to snap to
 * scene geometry in rotation and translation when the Ctrl key is pressed.
 */
UCLASS(MinimalAPI)
class UDragAlignmentMechanic : public UInteractionMechanic, public FDragAlignmentBase, public IModifierToggleBehaviorTarget
{
	GENERATED_BODY()

public:

	UE_API virtual USceneSnappingManager* GetSnappingManager();
	UE_API virtual FViewCameraState GetCameraState();
	virtual bool GetAlignmentModeEnabled() const { return bAlignmentToggle;}
	virtual UObject* GetUObjectContainer() { return this; }

public:

	UE_API virtual void Setup(UInteractiveTool* ParentToolIn) override;
	UE_API virtual void Shutdown() override;


	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// IModifierToggleBehaviorTarget implementation
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

protected:
	bool bAlignmentToggle = false;
	int32 AlignmentModifierID = 1;

};


/**
 * Interaction that can be added to (potentially multiple) UCombinedTransformGizmo object to allow them to snap to
 * scene geometry in rotation and translation. Generally driven by an externally-provided UKeyAsModifierInputBehavior,
 * or alternately can be directly updated by calling ::OnUpdateModifierState()
 */
UCLASS(MinimalAPI)
class UDragAlignmentInteraction : public UObject, public FDragAlignmentBase, public IModifierToggleBehaviorTarget
{
	GENERATED_BODY()

public:
	virtual USceneSnappingManager* GetSnappingManager() { return SceneSnappingManager.Get(); }
	virtual FViewCameraState GetCameraState() { return LastRenderCameraState; }
	virtual bool GetAlignmentModeEnabled() const { return bAlignmentToggle;}
	virtual UObject* GetUObjectContainer() { return this; }


public:
	UE_API virtual void Setup(USceneSnappingManager* SnappingManager);

	// Client must call this every frame to do alignment visualization rendering
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI);


public:
	// Client can call this on setup to connect the interaction to an external Key behavior.
	// This function is hardcoded to register the the Ctrl key as the modifier that drives the snapping toggle
	UE_API virtual void RegisterAsBehaviorTarget(UKeyAsModifierInputBehavior* KeyModifierBehavior);

	// IModifierToggleBehaviorTarget implementation
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

protected:
	TWeakObjectPtr<USceneSnappingManager> SceneSnappingManager;

	FViewCameraState LastRenderCameraState;		// could maybe move this to FDragAlignmentBase and get rid of GetCameraState() API function...

	bool bAlignmentToggle = false;
	int32 AlignmentModifierID = 1;
};

#undef UE_API
