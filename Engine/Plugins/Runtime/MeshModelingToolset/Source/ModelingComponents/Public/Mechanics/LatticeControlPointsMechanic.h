// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "RectangleMarqueeMechanic.h"
#include "InteractiveToolChange.h"
#include "Spatial/GeometrySet3.h"
#include "ToolContextInterfaces.h" //FViewCameraState
#include "IntVectorTypes.h"
#include "TransactionUtil.h"

#include "LatticeControlPointsMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class APreviewGeometryActor;
class ULineSetComponent;
class UMouseHoverBehavior;
class UPointSetComponent;
class USingleClickInputBehavior;
class UCombinedTransformGizmo;
class UTransformProxy;

UCLASS(MinimalAPI)
class ULatticeControlPointsMechanic : 
	public UInteractionMechanic, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()
	using FVector2i = UE::Geometry::FVector2i;

public:

	// TODO: Snapping

	// This delegate is called every time the control points are moved.
	DECLARE_MULTICAST_DELEGATE(OnPointsChangedEvent);
	OnPointsChangedEvent OnPointsChanged;

	// This delegate is called every time the control point selection changes.
	DECLARE_MULTICAST_DELEGATE(OnSelectionChangedEvent);
	OnSelectionChangedEvent OnSelectionChanged;

	UE_API virtual void Initialize(const TArray<FVector3d>& Points, 
							const TArray<FVector2i>& Edges,
							const FTransform3d& LocalToWorldTransform );

	UE_API void SetWorld(UWorld* World);
	UE_API const TArray<FVector3d>& GetControlPoints() const;
	UE_API void UpdateControlPointPositions(const TArray<FVector3d>& NewPoints);

	UE_API void SetCoordinateSystem(EToolContextCoordinateSystem InCoordinateSystem);
	UE_API EToolContextCoordinateSystem GetCoordinateSystem() const;

	UE_API void UpdateSetPivotMode(bool bInSetPivotMode);

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	// UInteractionMechanic
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	// IClickBehaviorTarget implementation
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

	bool bHasChanged = false;

	bool ControlPointIsSelected(int32 Index)
	{
		return SelectedPointIDs.Contains(Index);
	}

	const TSet<int32>& GetSelectedPointIDs() const
	{
		return SelectedPointIDs;
	}

	void SetPointColorOverride(int32 Index, const FColor& NewColor)
	{
		ColorOverrides.FindOrAdd(Index) = NewColor;
	}

	void ClearPointColorOverride(int32 Index)
	{
		ColorOverrides.Remove(Index);
	}

	void ClearAllPointColorOverrides()
	{
		ColorOverrides.Reset();
	}

	UE_API void UpdateDrawables();

	UE_API void UpdatePointLocations(const TMap<int32, FVector3d>& NewLocations);

	/// Set all lattice control points
	UE_API void UpdatePointLocations(const TArray<FVector3d>& NewLocations);

	bool IsGizmoBeingDragged() const
	{
		return bGizmoBeingDragged;
	}

	DECLARE_DELEGATE_RetVal(bool, FShouldHideGizmo);
	FShouldHideGizmo ShouldHideGizmo;

protected:

	TArray<FVector3d> ControlPoints;
	TArray<FVector2i> LatticeEdges;
	FTransform3d LocalToWorldTransform;

	// Used for spatial queries
	UE::Geometry::FGeometrySet3 GeometrySet;
	FViewCameraState CachedCameraState;

	/** Used for displaying points/segments */
	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> PreviewGeometryActor;
	UPROPERTY()
	TObjectPtr<UPointSetComponent> DrawnControlPoints;
	UPROPERTY()
	TObjectPtr<ULineSetComponent> DrawnLatticeEdges;

	// Variables for drawing
	FColor NormalSegmentColor;
	FColor NormalPointColor;
	float SegmentsThickness;
	float PointsSize;
	FColor HoverColor;
	FColor SelectedColor;

	// Cache previous color while temporarily changing the color of a hovered-over point
	FColor PreHoverPointColor;

	// Support for Shift and Ctrl toggle
	bool bShiftToggle = false;
	bool bCtrlToggle = false;
	static const int32 ShiftModifierID = 1;
	static const int32 CtrlModifierID = 2;

	// Default modifier key behavior is consistent with PolygonSelectionMechanic
	TFunction<bool(void)> ShouldAddToSelectionFunc = [this]() {return bShiftToggle; };
	TFunction<bool(void)> ShouldRemoveFromSelectionFunc = [this]() {return bCtrlToggle; };

	// Support for gizmo. Since the points aren't individual components, we don't actually use UTransformProxy
	// for the transform forwarding- we just use it for the callbacks.
	UPROPERTY()
	TObjectPtr<UTransformProxy> PointTransformProxy;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> PointTransformGizmo;

	// Used to make it easy to tell whether the gizmo was moved by the user or by undo/redo or
	// some other change that we shouldn't respond to. Basing our movement undo/redo on the
	// gizmo turns out to be quite a pain, though may someday be easier if the transform proxy
	// is able to manage arbitrary objects.
	bool bGizmoBeingDragged = false;

	// Callbacks we'll receive from the gizmo proxy
	UE_API void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void GizmoTransformStarted(UTransformProxy* Proxy);
	UE_API void GizmoTransformEnded(UTransformProxy* Proxy);

	// Support for hovering
	TFunction<bool(const FVector3d&, const FVector3d&)> GeometrySetToleranceTest;
	int32 HoveredPointID = -1;
	UE_API void ClearHover();

	// Support for selection
	UPROPERTY()
	TObjectPtr<URectangleMarqueeMechanic> MarqueeMechanic;
	bool bIsDraggingRectangle = false;
	TSet<int32> SelectedPointIDs;
	TSet<int32> PreDragSelection;
	TArray<int32> CurrentDragSelection;
	UE_API void OnDragRectangleStarted();
	UE_API void OnDragRectangleChanged(const FCameraRectangle& Rectangle);
	UE_API void OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled);

	// We need the selected point start positions so we can move multiple points appropriately.
	TMap<int32, FVector3d> SelectedPointStartPositions;

	// The starting point of the gizmo is needed to determine the offset by which to move the points.
	// TODO: Replace with single FTransform?
	FVector GizmoStartPosition;
	FQuat	GizmoStartRotation;
	FVector GizmoStartScale;

	// All of the following do not issue undo/redo change objects.
	UE_API bool HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut);
	UE_API void SelectPoint(int32 PointID);
	UE_API bool DeselectPoint(int32 PointID);
	UE_API void UpdateGizmoLocation();
	UE_API void UpdateGizmoVisibility();

	UE_API void RebuildDrawables();

	// Used for expiring undo/redo changes, which compare this to their stored value and expire themselves if they do not match.
	int32 CurrentChangeStamp = 0;

	TMap<int32, FColor> ColorOverrides;

	friend class FLatticeControlPointsMechanicSelectionChange;
	friend class FLatticeControlPointsMechanicMovementChange;

private:
	UE::TransactionUtil::FLongTransactionTracker LongTransactions;
};


// Undo/redo support:

// Control point selection has changed
class FLatticeControlPointsMechanicSelectionChange : public FToolCommandChange
{
public:
	UE_API FLatticeControlPointsMechanicSelectionChange(int32 PointIDIn, bool bAddedIn, const FTransform& PreviousTransformIn,
												 const FTransform& NewTransformIn, int32 ChangeStampIn);

	UE_API FLatticeControlPointsMechanicSelectionChange(const TSet<int32>& PointIDsIn, bool bAddedIn, const FTransform& PreviousTransformIn,
												 const FTransform& NewTransformIn, int32 ChangeStampIn);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<ULatticeControlPointsMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
	}
	UE_API virtual FString ToString() const override;

protected:
	TSet<int32> PointIDs;
	bool bAdded;

	const FTransform PreviousTransform;
	const FTransform NewTransform;

	int32 ChangeStamp;
};


// Control points have moved
class FLatticeControlPointsMechanicMovementChange : public FToolCommandChange
{
public:

	UE_API FLatticeControlPointsMechanicMovementChange(
		const TMap<int32, FVector3d>& OriginalPositionsIn,
		const TMap<int32, FVector3d>& NewPositionsIn,
		int32 ChangeStampIn,
		bool bFirstMovementIn);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	virtual bool HasExpired(UObject* Object) const override
	{
		return Cast<ULatticeControlPointsMechanic>(Object)->CurrentChangeStamp != ChangeStamp;
	}
	UE_API virtual FString ToString() const override;

protected:

	TMap<int32, FVector3d> OriginalPositions;
	TMap<int32, FVector3d> NewPositions;
	int32 ChangeStamp;
	bool bFirstMovement;
};

#undef UE_API
