// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/HitResult.h"
#include "InteractionMechanic.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "PlaneDistanceFromHitMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

/**
 * UPlaneDistanceFromHitMechanic implements an interaction where a Height/Distance from a plane
 * is defined by intersecting a ray with a target mesh, and then using that hit point to define the distance.
 * Optionally the hit point can be snapped (eg to a world grid), and also the ray can hit other objects to define the height.
 *
 */
UCLASS(MinimalAPI)
class UPlaneDistanceFromHitMechanic : public UInteractionMechanic
{
	GENERATED_BODY()
	using FFrame3d = UE::Geometry::FFrame3d;
public:
	/** If this function is set, we will check it for a ray intersection if the target mesh is not hit */
	TUniqueFunction<bool(const FRay&, FHitResult&)> WorldHitQueryFunc = nullptr;

	/** If this function is set, the hit point will be passed in to this function for snapping. Return false to indicate no snapping occurred. */
	TUniqueFunction<bool(const FVector3d&, FVector3d&)> WorldPointSnapFunc = nullptr;

	/** Height at last hit point */
	float CurrentHeight = 0.0f;

	/** true if the current hit/height was determined via WorldHitQueryFunc result */
	bool bCurrentHitIsWorldHit = false;

	/** last active hit point */
	FHitResult LastActiveWorldHit;

	/** World frame at last hit point */
	UE::Geometry::FFrame3d CurrentHitPosFrameWorld;

	/** If true, then if we don't find any intersection point, just use nearest point on plane normal to ray */
	bool bFallbackToLineAxisPoint = true;

	const FFrame3d& GetPlane() const { return PreviewHeightFrame; }

public:

	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * Set the hit target mesh and the plane frame. Distance is measured along Z axis.
	 * If bMeshInWorldCoords is true, then HitTargetMesh is in world coords.
	 * Otherwise we assume it is in local coords of PlaneFrameWorld
	 */
	UE_API virtual void Initialize(UE::Geometry::FDynamicMesh3&& HitTargetMesh, const FFrame3d& PlaneFrameWorld, bool bMeshInWorldCoords);

	/**
	 * Set the hit target mesh and the plane frame. Distance is measured along Z axis.
	 * Applies MeshToPlaneFrame to transform the HitTargetMesh to the local space of PlaneFrameWorld
	 */
	UE_API virtual void Initialize(UE::Geometry::FDynamicMesh3&& HitTargetMesh, const FFrame3d& PlaneFrameWorld, const FTransform& MeshToPlaneFrame);

	/**
	 * Update the current distance/height based on the input world ray
	 */
	UE_API virtual void UpdateCurrentDistance(const FRay& WorldRay);

protected:
	UE::Geometry::FDynamicMesh3 PreviewHeightTarget;
	UE::Geometry::FDynamicMeshAABBTree3 PreviewHeightTargetAABB;
	FFrame3d PreviewHeightFrame;

};

#undef UE_API
