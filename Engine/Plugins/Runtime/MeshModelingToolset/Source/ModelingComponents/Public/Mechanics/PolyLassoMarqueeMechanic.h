// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InteractionMechanic.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "VectorTypes.h"

#include "PolyLassoMarqueeMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UClickDragInputBehavior;
class UMouseHoverBehavior;

/// Struct containing:
///		- camera information, 
///		- a 3D plane just in front of the camera,
///		- a 2D basis for coordinates in this plane, and
///		- the vertices of a PolyLasso contained in this plane, in this 2D basis
/// 
struct FCameraPolyLasso
{
	FVector CameraOrigin;
	bool bCameraIsOrthographic = false;
	FPlane CameraPlane;
	FVector UBasisVector;
	FVector VBasisVector;
	TArray<FVector2D> Polyline;

	FCameraPolyLasso() {};
	UE_API FCameraPolyLasso(const FViewCameraState& CachedCameraState);

	/** Append a point to the PolyLasso polyine using a world-space ray */
	UE_API void AddPoint(const FRay& WorldRay);

	/** Project a world-space point into the same space as the 2D PolyLine */
	UE_API FVector2D GetProjectedPoint(const FVector& Point) const;

	FVector2D PlaneCoordinates(const FVector& Point) const
	{
		float U = FVector::DotProduct(Point - CameraPlane.GetOrigin(), UBasisVector);
		float V = FVector::DotProduct(Point - CameraPlane.GetOrigin(), VBasisVector);
		return FVector2D{ U,V };
	}
};


/*
 * Mechanic for a PolyLasso "marquee" selection. It creates and maintains the 2D PolyLasso associated with a mouse drag. 
 * It does not test against any scene geometry, nor does it maintain any sort of list of selected objects.
 *
 * The PolyLasso has two potential modes, a freehand polyline and a multi-click polygon. By default both are enabled
 * but this can be selectively controlled with flags below. If a click-and-release is within a small distance tolerance,
 * then a multi-click polygon is entered, and must be exited by clicking again at the start point. The freehand polyline
 * is drawn by click-dragging, and exited by releasing the mouse.
 *
 * When using this mechanic, you should call DrawHUD() in the tool's DrawHUD() call so that it can draw the box.
 *
 * Attach to the mechanic's delegates and use the passed PolyLasso to test against your geometry. 
 */
UCLASS(MinimalAPI)
class UPolyLassoMarqueeMechanic : public UInteractionMechanic, public IClickDragBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()

public:

	// UInteractionMechanic
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	/** @return true if PolyLasso mechanic is enabled */
	UE_API bool IsEnabled() const;

	/** enable/disable the PolyLasso mechanic (effectively controls whether the mechanic's behaviors will be allowed to capture the mouse) */
	UE_API void SetIsEnabled(bool bOn);

	/**
	 * Sets the base priority so that users can make sure that their own behaviors are higher
	 * priority. The mechanic will not use any priority value higher than this.
	 * Mechanics could use lower priorities (and their range could be inspected with
	 * GetPriorityRange), but marquee mechanic doesn't.
	 *
	 * Can be called before or after Setup().
	 */
	UE_API void SetBasePriority(const FInputCapturePriority& Priority);

	/**
	 * Gets the current priority range used by behaviors in the mechanic, higher
	 * priority to lower.
	 *
	 * For marquee mechanic, the range will be [BasePriority, BasePriority] since
	 * it only uses one priority.
	 */
	UE_API TPair<FInputCapturePriority, FInputCapturePriority> GetPriorityRange() const;

	/**
	 * Called when user starts dragging a new PolyLasso.
	 */
	FSimpleMulticastDelegate OnDrawPolyLassoStarted;

	/**
	 * Called as the user drags the other corner of the PolyLasso around.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(OnDrawPolyLassoChangedEvent, const FCameraPolyLasso&);
	OnDrawPolyLassoChangedEvent OnDrawPolyLassoChanged;

	/**
	 * Called once the user lets go of the mouse button after dragging out a PolyLasso.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(OnDrawPolyLassoFinishedEvent, const FCameraPolyLasso&, bool bCanceled);
	OnDrawPolyLassoFinishedEvent OnDrawPolyLassoFinished;


	/** Tolerance for PolyLasso points and closure test */
	UPROPERTY()
	float SpacingTolerance = 2.0f;

	/** Thickness of the 2D PolyLasso drawing path */
	UPROPERTY()
	float LineThickness = 2.0f;

	/** PolyLasso path is drawn in this color */
	UPROPERTY()
	FLinearColor LineColor = FLinearColor::Red;

	/** PolyLasso path is drawn in this color if the cursor is at a point that would close the loop */
	UPROPERTY()
	FLinearColor ClosedColor = FLinearColor(1.0f, 0.549019607843f, 0.0f);

	/**
	 * If true, freehand polygons can be drawn by click-dragging the mouse
	 */
	UPROPERTY()
	bool bEnableFreehandPolygons = true;

	/**
	 * If true, if click and release are within SpacingTolerance, mechanic enters a multi-click mode,
	 * where each click adds a vertex to a polygon. Polygon must be closed by clicking within 2*SpacingTolerance
	 * of the initial point to exit this mode. However, rotating the camera will also cancel out of the interaction.
	 */
	UPROPERTY()
	bool bEnableMultiClickPolygons = true;


protected:
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> ClickDragBehavior = nullptr;

	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior = nullptr;

	// camera state for currently active PolyLasso
	FViewCameraState CachedCameraState;

	FInputCapturePriority BasePriority = FInputCapturePriority(FInputCapturePriority::DEFAULT_TOOL_PRIORITY);

private:

	// IClickDragBehaviorTarget implementation
	FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) final;
	void OnClickPress(const FInputDeviceRay& PressPos) final;
	void OnClickDrag(const FInputDeviceRay& DragPos) final;
	void OnClickRelease(const FInputDeviceRay& ReleasePos) final;
	void OnTerminateDragSequence() final;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override {}
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	virtual void OnEndHover() override {}

	bool bIsEnabled = false;
	bool bIsDragging = false;
	bool bIsInMultiClickPolygon = false;
	bool bIsMultiClickPolygonClosed = false;
	TArray<FInputDeviceRay> PolyPathPoints;
	FVector2D DragCurrentScreenPosition;

	FCameraPolyLasso CurrentPolyLasso;
	bool WillCloseCurrentLasso(const FInputDeviceRay& DevicePos) const;


public:
	// utility function clients can use to clean up poly-lasso if it is not closed (perhaps should be folded into mechanic behavior)
	static UE_API bool ApproximateSelfClipPolyline(TArray<FVector2D>& Polyline);
};

#undef UE_API
