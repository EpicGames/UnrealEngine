// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputBehavior.h"
#include "InteractionMechanic.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "GeometryBase.h"
#include "BoxTypes.h"
#include "PlaneTypes.h"
#include "SegmentTypes.h"
#include "VectorTypes.h"

#include "RectangleMarqueeMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UClickDragInputBehavior;
struct FConvexVolume;

PREDECLARE_USE_GEOMETRY_CLASS(FGeometrySet3);

/// Struct containing:
///		- camera information, 
///		- input device rays used to define the corners of a rectangle contained in a selection plane, the device ray
///		  screen positions could be used to change the selection behavior when dragging from the top right to bottom
///		  left or vice-versa
/// 
struct FCameraRectangle
{
	using FGeometrySet3 = UE::Geometry::FGeometrySet3;
	using FAxisAlignedBox2 = UE::Geometry::TAxisAlignedBox2<FVector::FReal>;
	using FPlane3 = UE::Geometry::TPlane3<FVector::FReal>;
	using FSegment2 = UE::Geometry::TSegment2<FVector::FReal>;
	using FVector2 = UE::Math::TVector2<FVector::FReal>;

	FViewCameraState CameraState;
	FInputDeviceRay RectangleStartRay = FInputDeviceRay(FRay());
	FInputDeviceRay RectangleEndRay = FInputDeviceRay(FRay());
	
	struct FRectangleInPlane
	{
		// Rectangle is defined in the UV-coordinate system of Plane. See PlaneCoordinates
		FPlane3 Plane;
		FAxisAlignedBox2 Rectangle; // Region of marquee selection
	};

	FRectangleInPlane SelectionDomain;

	// This function must be called before other member functions whenever camera state or start/end rays are updated
	UE_API void Initialize();
	
	bool bIsInitialized = false;

	// Returns the SelectionDomain projected onto a plane offset from SelectionDomain.Plane by the given distance in the
	// SelectionDomain.Plane.Normal direction.
	//
	// Note: This function is in the public API to optimize selection in the UV editor where meshes are known to lie in
	// the XY plane in front of the camera (so point projection/camera plane clipping is unnecessary)
	UE_API FRectangleInPlane ProjectSelectionDomain(double OffsetFromCameraPlane = 0.) const;

	// Return true if the given 3D geometry projected to the camera plane is inside or intersecting the rectangle, and
	// false otherwise
	UE_API bool IsProjectedPointInRectangle(const FVector& Point) const;
	UE_API bool IsProjectedSegmentIntersectingRectangle(const FVector& Endpoint1, const FVector& Endpoint2) const;
	// bool IsProjectedTriangleIntersectingRectangle(const FVector& A, const FVector& B, const FVector& C) const;

	UE_API FConvexVolume FrustumAsConvexVolume() const;

	// Return the 3D point obtained by projecting the given 3D Point onto the given projection plane
	// Note: Assumes an orthographic camera
	static FVector OrthographicProjection(const FPlane3& Plane, const FVector& Point)
	{
		return FVector::PointPlaneProject(Point, (FPlane)Plane);
	}

	// Return the 3D point obtained by projecting the given 3D Point onto the given projection plane
	// Note: Assumes a perspective camera
	FVector PerspectiveProjection(const FPlane3& Plane, const FVector& Point) const
	{
		ensure(bIsInitialized);
		checkSlow(!CameraState.bIsOrthographic);
		return FMath::RayPlaneIntersection(CameraState.Position, Point - CameraState.Position, (FPlane)Plane);
	}

	// Given a 3D point lying in the given Plane, return the UV coordinates of the point expressed in the following a
	// two dimensional parameterization of the given Plane:
	// - the 2D origin (0,0) is located at the foot of the projection of cartesian point (0,0,0) onto the plane
	// - the U basis vector is the camera right vector
	// - the V basis vector is the camera up vector
	FVector2 Point3DToPointUV(const FPlane3& Plane, const FVector& Point3DInPlane) const
	{
		ensure(bIsInitialized);
		FVector2 PointUV;
		PointUV[0] = FVector::DotProduct(Point3DInPlane - ((FPlane)Plane).GetOrigin(), CameraState.Right());
		PointUV[1] = FVector::DotProduct(Point3DInPlane - ((FPlane)Plane).GetOrigin(), CameraState.Up());
		return PointUV;
	}
	
	// Given a 2D point in the UV space of the given Plane, return the coordinates of the 3D point lying in the Plane.
	// This function is the inverse of Point3DToPointUV.
	FVector PointUVToPoint3D(const FPlane3& Plane, const FVector2& PointUV) const
	{
		ensure(bIsInitialized);
		FVector Point3DInPlane = ((FPlane)Plane).GetOrigin();
		Point3DInPlane += PointUV[0] * CameraState.Right();
		Point3DInPlane += PointUV[1] * CameraState.Up();
		return Point3DInPlane;
	}
};


/*
 * Mechanic for a rectangle "marquee" selection. It creates and maintains the 2D rectangle associated with a mouse drag. 
 * It does not test against any scene geometry, nor does it maintain any sort of list of selected objects.
 *
 * When using this mechanic, you should call Render() on it in the tool's Render() call so that it can cache
 * necessary camera state, and DrawHUD() in the tool's DrawHUD() call so that it can draw the box.
 *
 * Attach to the mechanic's delegates and use the passed rectangle to test against your geometry. 
 */
UCLASS(MinimalAPI)
class URectangleMarqueeMechanic : public UInteractionMechanic, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:
	/** 
	 * If true, then the URectangleMarqueeMechanic will not create an internal UClickDragInputBehavior in ::Setup(), allowing
	 * the client to control the marquee with an external InputBehavior that uses the marquee mechanic as it's IClickDragBehaviorTarget.
	 * For instance, this allows the mechanic to be hooked in as the drag component of a USingleClickOrDragInputBehavior.
	 * Must be configured before calling ::Setup()
	 */
	UPROPERTY()
	bool bUseExternalClickDragBehavior = false;

	/**
	 * If true, then the URectangleMarqueeMechanic will not query the tool manager for the current camera view state,
	 but instead call the custom provided function, UpdateCameraStateFunc, to recieve the correct camera state for the 
	 CameraRectangle.
	 */
	UPROPERTY()
	bool bUseExternalUpdateCameraState = false;

	TUniqueFunction< FViewCameraState() > UpdateCameraStateFunc;

	/**
	 * If the computation time for a single call to OnDragRectangleChanged ever exceeds this threshold then future calls
	 * to this function (in the current drag sequence) will be deferred until the mouse button is released. This will
	 * improve the responsiveness of the UI. The default value is set so this optimization is never triggered, if you
	 * want it you can set this to a small value e.g., 1./60 (time elapsed by 1 frame at 60 fps)
	 */
	UPROPERTY()
	double OnDragRectangleChangedDeferredThreshold = TNumericLimits<double>::Max(); // In seconds

public:

	// UInteractionMechanic
	UE_API void Setup(UInteractiveTool* ParentTool) override;
	UE_API void Render(IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	UE_API bool IsEnabled();
	UE_API void SetIsEnabled(bool bOn);

	/**
	 * Sets the base priority so that users can make sure that their own behaviors are higher
	 * priority. The mechanic will not use any priority value higher than this.
	 * Mechanics could use lower priorities (and their range could be inspected with
	 * GetPriorityRange), but marquee mechanic doesn't. 
	 *
	 * Can be called before or after Setup(). Only relevant if bUseExternalClickDragBehavior is false
	 */
	UE_API void SetBasePriority(const FInputCapturePriority& Priority);

	/**
	 * Gets the current priority range used by behaviors in the mechanic, higher
	 * priority to lower.
	 *
	 * For marquee mechanic, the range will be [BasePriority, BasePriority] since
	 * it only uses one priority.
	 */
	UE_API TPair<FInputCapturePriority, FInputCapturePriority> GetPriorityRange() const;

	/**
	 * Called when user starts dragging a new rectangle.
	 */
	FSimpleMulticastDelegate OnDragRectangleStarted;

	/**
	 * Called as the user drags the other corner of the rectangle around.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(OnDragRectangleChangedEvent, const FCameraRectangle&);
	OnDragRectangleChangedEvent OnDragRectangleChanged;

	/**
	 * Called once the user lets go of the mouse button after dragging out a rectangle.
	 * The last dragged rectangle is passed here so that clients can choose to just implement this function in simple cases. 
	 * bCancelled flag is true when the drag finishes due to a disabling of the mechanic or due to a TerminateDragSequence call, rather than a normal drag completion.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(OnDragRectangleFinishedEvent, const FCameraRectangle&, bool bCancelled);
	OnDragRectangleFinishedEvent OnDragRectangleFinished;
	
protected:
	UPROPERTY()
	TObjectPtr<UClickDragInputBehavior> ClickDragBehavior = nullptr;

	FCameraRectangle CameraRectangle;

	FInputCapturePriority BasePriority = FInputCapturePriority(FInputCapturePriority::DEFAULT_TOOL_PRIORITY);

private:

	FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) final;
	void OnClickPress(const FInputDeviceRay& PressPos) final;
	void OnClickDrag(const FInputDeviceRay& DragPos) final;
	void OnClickRelease(const FInputDeviceRay& ReleasePos) final;
	void OnTerminateDragSequence() final;

	bool bIsEnabled;
	bool bIsDragging;
	bool bIsOnDragRectangleChangedDeferred;
};





/*
 * URectangleMarqueeInteraction is a simplified version of URectangleMarqueeMechanic that is not a UInteractionMechanic,
 * which requires a base Tool/ToolManager. This variant does not create it's own InputBehavior, but is still a 
 * ClickDragBehaviorTarget. 
 * 
 * DrawHUD() must be called by the owning code. 
 */
UCLASS(MinimalAPI)
class URectangleMarqueeInteraction : public UObject, public IClickDragBehaviorTarget
{
	GENERATED_BODY()

public:

	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	/**
	 * OnDragRectangleStarted is called when user starts dragging a new rectangle.
	 */
	FSimpleMulticastDelegate OnDragRectangleStarted;

	/**
	 * OnDragRectangleChanged is called as the user drags the other corner of the rectangle around.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(OnDragRectangleChangedEvent, const FCameraRectangle&);
	OnDragRectangleChangedEvent OnDragRectangleChanged;

	/**
	 * OnDragRectangleFinished is called once the user lets go of the mouse button after dragging out a rectangle.
	 * bCancelled flag is true when the drag finishes due to a disabling of the mechanic or due to a TerminateDragSequence call, rather than a normal drag completion.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(OnDragRectangleFinishedEvent, const FCameraRectangle&, bool bCancelled);
	OnDragRectangleFinishedEvent OnDragRectangleFinished;
	
protected:
	FViewCameraState CurrentCameraState;
	FCameraRectangle CameraRectangle;

	UE_API virtual FInputRayHit CanBeginClickDragSequence(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickPress(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnClickDrag(const FInputDeviceRay& DragPos) override;
	UE_API virtual void OnClickRelease(const FInputDeviceRay& ReleasePos) override;
	UE_API virtual void OnTerminateDragSequence() override;

	bool bIsDragging = false;
};

#undef UE_API
