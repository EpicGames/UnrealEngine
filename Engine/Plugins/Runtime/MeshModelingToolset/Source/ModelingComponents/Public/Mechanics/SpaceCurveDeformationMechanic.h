// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "FrameTypes.h"
#include "InteractionMechanic.h"
#include "InteractiveToolChange.h"
#include "Spatial/GeometrySet3.h"
#include "ToolContextInterfaces.h" //FViewCameraState
#include "VectorTypes.h"

#include "SpaceCurveDeformationMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class APreviewGeometryActor;
class ULineSetComponent;
class UMouseHoverBehavior;
class UPointSetComponent;
class USingleClickInputBehavior;
class UCombinedTransformGizmo;
class UTransformProxy;




class FSpaceCurveSource
{
public:
	TUniqueFunction<int32()> GetPointCount;
	TUniqueFunction<UE::Geometry::FFrame3d(int32)> GetPoint;
	TUniqueFunction<bool()> IsLoop;
};





UENUM()
enum class ESpaceCurveControlPointTransformMode
{
	Shared,
	PerVertex
};


UENUM()
enum class ESpaceCurveControlPointOriginMode
{
	Shared,
	First,
	Last
};

UENUM()
enum class ESpaceCurveControlPointFalloffType
{
	Linear,
	Smooth
};



UCLASS(MinimalAPI)
class USpaceCurveDeformationMechanicPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = Options)
	ESpaceCurveControlPointTransformMode TransformMode = ESpaceCurveControlPointTransformMode::PerVertex;

	UPROPERTY(EditAnywhere, Category = Options)
	ESpaceCurveControlPointOriginMode TransformOrigin = ESpaceCurveControlPointOriginMode::First;

	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "1"))
	float Softness = 0.5;

	UPROPERTY(EditAnywhere, Category = Options)
	ESpaceCurveControlPointFalloffType SoftFalloff = ESpaceCurveControlPointFalloffType::Smooth;
};





/**

 */
UCLASS(MinimalAPI)
class USpaceCurveDeformationMechanic : public UInteractionMechanic, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()
public:

	// Behaviors used for moving points around and hovering them
	UPROPERTY()
	TObjectPtr<USingleClickInputBehavior> ClickBehavior = nullptr;
	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior = nullptr;

	// This delegate is called every time the control point sequence is altered.
	DECLARE_MULTICAST_DELEGATE(OnPointsChangedEvent);
	OnPointsChangedEvent OnPointsChanged;


	UE_API void SetCurveSource(TSharedPtr<FSpaceCurveSource> CurveSource);
	UE_API void ClearCurveSource();

	UE_API void ClearSelection();
	UE_API void SelectionGrowToNext();
	UE_API void SelectionGrowToPrev();
	UE_API void SelectionGrowToEnd();
	UE_API void SelectionGrowToStart();
	UE_API void SelectionFill();
	UE_API void SelectionClear();

	void GetCurrentCurvePoints(TArray<UE::Geometry::FFrame3d>& PointsOut) { PointsOut = CurvePoints; }


	// Some other standard functions
	UE_API virtual ~USpaceCurveDeformationMechanic();
	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;
	UE_API void SetWorld(UWorld* World);
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void Tick(float DeltaTime) override;

	// IClickBehaviorTarget implementation
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	// IModifierToggleBehaviorTarget implementation, inherited through IClickBehaviorTarget
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

public:
	UPROPERTY()
	TObjectPtr<USpaceCurveDeformationMechanicPropertySet> TransformProperties;

protected:

	TSharedPtr<FSpaceCurveSource> CurveSource;
	TSharedPtr<FSpaceCurveSource> EmptyCurveSource;

	TArray<UE::Geometry::FFrame3d> CurvePoints;

	// Used for spatial queries
	UE::Geometry::FGeometrySet3 GeometrySet;

	bool bSpatialValid = false;
	UE_API void UpdateSpatial();

	/** Used for displaying points/segments */
	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> PreviewGeometryActor;
	UPROPERTY()
	TObjectPtr<UPointSetComponent> RenderPoints;
	UPROPERTY()
	TObjectPtr<ULineSetComponent> RenderSegments;

	bool bRenderGeometryValid = false;
	UE_API void UpdateRenderGeometry();

	// Variables for drawing
	FColor NormalCurveColor;
	FColor CurrentSegmentsColor;
	FColor CurrentPointsColor;
	float SegmentsThickness;
	float PointsSize;
	float DepthBias;
	FColor PreviewColor;
	FColor HoverColor;
	FColor SelectedColor;

	// Support for Shift and Ctrl toggles
	bool bAddToSelectionToggle = false;
	int32 ShiftModifierId = 1;
	int32 CtrlModifierId = 2;

	// Support for gizmo. Since the points aren't individual components, we don't actually use UTransformProxy
	// for the transform forwarding- we just use it for the callbacks.
	UPROPERTY()
	TObjectPtr<UTransformProxy> PointTransformProxy;
	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> PointTransformGizmo;

	// Used to make it easy to tell whether the gizmo was moved by the user or by undo/redo or
	// some other change that we shoulnd't respond to. Basing our movement undo/redo on the
	// gizmo turns out to be quite a pain, though may someday be easier if the transform proxy
	// is able to manage arbitrary objects.
	bool bGizmoBeingDragged = false;

	// Callbacks we'll receive from the gizmo proxy
	UE_API void GizmoTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void GizmoTransformStarted(UTransformProxy* Proxy);
	UE_API void GizmoTransformEnded(UTransformProxy* Proxy);

	// Support for hovering
	FViewCameraState CameraState;
	TFunction<bool(const FVector3d&, const FVector3d&)> GeometrySetToleranceTest;
	int32 HoveredPointID = -1;
	UE_API void ClearHover();

	// Support for selection
	TArray<int32> SelectedPointIDs;
	// We need the selected point start positions so we can move multiple points appropriately.
	TArray<UE::Geometry::FFrame3d> CurveStartPositions;
	// The starting point of the gizmo is needed to determine the offset by which to move the points.
	UE::Geometry::FFrame3d GizmoStartPosition;

	// These issue undo/redo change objects, and must therefore not be called in undo/redo code.
	UE_API void ChangeSelection(int32 NewPointID, bool AddToSelection);

	// All of the following do not issue undo/redo change objects.
	UE_API bool HitTest(const FInputDeviceRay& ClickPos, FInputRayHit& ResultOut);
	UE_API void SelectPoint(int32 PointID);
	UE_API bool DeselectPoint(int32 PointID);
	UE_API void UpdateSelection(const TArray<int32>& NewSelection);
	UE_API void UpdateGizmoLocation();
	UE_API void UpdateCurve(const TArray<UE::Geometry::FFrame3d>& NewPositions);

	// Used for expiring undo/redo changes, which compare this to their stored value and expire themselves if they do not match.
	int32 CurrentChangeStamp = 0;

	friend class FSpaceCurveDeformationMechanicSelectionChange;
	friend class FSpaceCurveDeformationMechanicMovementChange;
};


// Undo/redo support:

class FSpaceCurveDeformationMechanicSelectionChange : public FToolCommandChange
{
public:
	UE_API FSpaceCurveDeformationMechanicSelectionChange(const TArray<int32>& FromIDs, const TArray<int32>& ToIDs);
	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;

protected:
	TArray<int32> From, To;
};


class FSpaceCurveDeformationMechanicMovementChange : public FToolCommandChange
{
public:
	UE_API FSpaceCurveDeformationMechanicMovementChange(const TArray<UE::Geometry::FFrame3d>& FromPositions, const TArray<UE::Geometry::FFrame3d>& ToPositions);

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;

protected:
	TArray<UE::Geometry::FFrame3d> From;
	TArray<UE::Geometry::FFrame3d> To;
};

#undef UE_API
