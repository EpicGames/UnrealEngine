// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractionMechanic.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "SpatialCurveDistanceMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

/**
 *
 */
UCLASS(MinimalAPI)
class USpatialCurveDistanceMechanic : public UInteractionMechanic
{
	GENERATED_BODY()
public:
	/** If this function is set, the hit point will be passed in to this function for snapping. Return false to indicate no snapping occurred. */
	TUniqueFunction<bool(const FVector3d&, FVector3d&)> WorldPointSnapFunc = nullptr;

	/** Current distance */
	double CurrentDistance = 0.0f;

	FVector3d CurrentCurvePoint;
	FVector3d CurrentSpacePoint;

public:

	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	/**
	 */
	UE_API virtual void InitializePolyCurve(const TArray<FVector3d>& CurvePoints, const FTransform3d& Transform);
	UE_API virtual void InitializePolyLoop(const TArray<FVector3d>& CurvePoints, const FTransform3d& Transform);

	/**
	 * Update the current distance/height based on the input world ray
	 */
	UE_API virtual void UpdateCurrentDistance(const FRay& WorldRay);

protected:
	TArray<FVector3d> Curve;
	UE::Geometry::FDynamicMesh3 TargetHitMesh;
	UE::Geometry::FDynamicMeshAABBTree3 TargetHitMeshAABB;
	FTransform3d Transform;
};

#undef UE_API
