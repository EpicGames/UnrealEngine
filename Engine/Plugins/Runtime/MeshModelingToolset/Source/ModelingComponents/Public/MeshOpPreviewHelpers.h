// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "PreviewMesh.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "BackgroundModelingComputeSource.h"
#include "MeshOpPreviewHelpers.generated.h"

#define UE_API MODELINGCOMPONENTS_API


/**
 * FBackgroundDynamicMeshComputeSource is an instantiation of the TBackgroundModelingComputeSource
 * template for FDynamicMeshOperator / IDynamicMeshOperatorFactory
 */
using FBackgroundDynamicMeshComputeSource = UE::Geometry::TBackgroundModelingComputeSource<UE::Geometry::FDynamicMeshOperator, UE::Geometry::IDynamicMeshOperatorFactory>;


/**
 * FDynamicMeshOpResult is a container for a computed Mesh and Transform
 */
struct FDynamicMeshOpResult
{
	TUniquePtr<UE::Geometry::FDynamicMesh3> Mesh;
	UE::Geometry::FTransformSRT3d Transform;
};


/**
 * UMeshOpPreviewWithBackgroundCompute is an infrastructure object that implements a common UI
 * pattern in interactive 3D tools, where we want to run an expensive computation on a mesh that
 * is based on user-specified parameters, and show a preview of the result. The expensive computation 
 * (a MeshOperator) must run in a background thread so as to not block the UI. If the user
 * changes parameters while the Operator is running, it should be canceled and restarted. 
 * When it completes, the Preview will be updated. When the user is happy, the current Mesh is
 * returned to the owner of this object.
 * 
 * The MeshOperators are provided by the owner via a IDynamicMeshOperatorFactory implementation.
 * The owner must also Tick() this object regularly to allow the Preview to update when the
 * background computations complete.
 * 
 * If an InProgress Material is set (via ConfigureMaterials) then when a background computation
 * is active, this material will be used to draw the previous Preview result, to give the 
 * user a visual indication that work is happening.
 */
UCLASS(MinimalAPI, Transient)
class UMeshOpPreviewWithBackgroundCompute : public UObject
{
	GENERATED_BODY()
public:

	//
	// required calls to setup/update/shutdown this object
	// 

	/**
	 * @param InWorld the Preview mesh actor will be created in this UWorld
	 * @param OpGenerator This factory is called to create new MeshOperators on-demand
	 */
	UE_API void Setup(UWorld* InWorld, UE::Geometry::IDynamicMeshOperatorFactory* OpGenerator);

	UE_API void Setup(UWorld* InWorld);

	/**
	 * Terminate any active computation and return the current Preview Mesh/Transform
	 */
	UE_API FDynamicMeshOpResult Shutdown();

	/**
	 * Stops any running computes and swaps in a different op generator. Does not 
	 * update the preview mesh or start a new compute.
	 */
	UE_API void ChangeOpFactory(UE::Geometry::IDynamicMeshOperatorFactory* OpGenerator);


	UE_API void ClearOpFactory();

	/**
	 * Cancel the active computation without returning anything. Doesn't destroy the mesh.
	 */
	UE_API void CancelCompute();

	/**
	* Terminate any active computation without returning anything. Destroys the preview
	* mesh.
	*/
	UE_API void Cancel();

	/**
	 * Tick the background computation and Preview update. 
	 * @warning this must be called regularly for the class to function properly
	 */
	UE_API void Tick(float DeltaTime);


	//
	// Control flow
	// 


	/**
	 * Request that the current computation be canceled and a new one started
	 */
	UE_API void InvalidateResult();

	/**
	 * @return true if the current PreviewMesh result is valid, ie no update being actively computed
	 */
	bool HaveValidResult() const { return bResultValid; }

	double GetValidResultComputeTime() const
	{
		if (HaveValidResult())
		{
			return ValidResultComputeTimeSeconds;
		}
		return -1;
	}


	/**
	 * @return true if current PreviewMesh result is valid (no update actively being computed) and that mesh has at least one triangle
	 */
	bool HaveValidNonEmptyResult() const { return bResultValid && PreviewMesh && PreviewMesh->GetMesh() && PreviewMesh->GetMesh()->TriangleCount() > 0; }

	/**
	 * @return true if current PreviewMesh result is valid (no update actively being computed) but that mesh has no triangles
	 */
	bool HaveEmptyResult() const { return bResultValid && PreviewMesh && PreviewMesh->GetMesh() && PreviewMesh->GetMesh()->TriangleCount() == 0; }


	/**
	 * Read back a copy of current preview mesh.
	 * @param bOnlyIfValid if true, then only create mesh copy if HaveValidResult() == true
	 * @return true if MeshOut was initialized
	 */
	UE_API bool GetCurrentResultCopy(FDynamicMesh3& MeshOut, bool bOnlyIfValid = true);


	/**
	 * Allow an external function to safely access the PreviewMesh's mesh 
	 * @param bOnlyIfValid if true, then only call ProcessFunc if current result is valid, ie HaveValidResult() == true. Default false.
	 * @return true if ProcessFunc was called
	 */
	UE_API bool ProcessCurrentMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc, bool bOnlyIfValid = false);

	/** @return UWorld that the created PreviewMesh exist in */
	virtual UWorld* GetWorld() const override { return PreviewWorld.Get(); }

	//
	// Optional configuration
	// 


	/**
	 * Configure the Standard and In-Progress materials
	 */
	UE_API void ConfigureMaterials(UMaterialInterface* StandardMaterial, UMaterialInterface* InProgressMaterial);

	/**
	 * Configure the Standard and In-Progress materials
	 * @param StandardMaterials list of materials corresponding to MaterialIDs on the Mesh
	 * @param InProgressMaterial material displayed if the background compute takes longer than WorkingMaterialDelay
	 * @param SecondaryMaterial optional secondary material passed to mesh component, call PreviewMesh->EnableSecondaryTriangleBuffers to enable
	 */
	UE_API void ConfigureMaterials(
		TArray<UMaterialInterface*> StandardMaterials, 
		UMaterialInterface* InProgressMaterial, 
		UMaterialInterface* SecondaryMaterial = nullptr);

	/** 
	 * Configure the In-Progress and Secondary Materials
	 * @param InProgressMaterial material displayed if the background compute takes longer than WorkingMaterialDelay
	 * @param SecondaryMaterial optional secondary material passed to mesh component, call PreviewMesh->EnableSecondaryTriangleBuffers to enable
	 */
	UE_API void ConfigurePreviewMaterials(
		UMaterialInterface* InProgressMaterial, 
		UMaterialInterface* SecondaryMaterial);

	/** Disable the In Progress and Secondary Materials */
	UE_API void DisablePreviewMaterials();

	/**
	 * Set the visibility of the Preview mesh
	 */
	UE_API void SetVisibility(bool bVisible);

	/**
	 * If set to true, then it will be assumed that the mesh topology (i.e. triangle/edge connectivity) remains constant,
	 * which will allow updates after the first one to modify existing render proxy buffers rather than creating entirely
	 * new ones. This will give a significant speedup to tools that do not add/remove vertices or affect their connectivity.
	 *
	 * @param bOn
	 * @param ChangingAttributes If bOn is set to true, determines which attributes need updating. For instance, if a tool 
	 *  moves verts without changing their UV's, then one would probably pass "EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals".
	 *  Has no effect if bOn was set to false.
	 */
	UE_API void SetIsMeshTopologyConstant(bool bOn, EMeshRenderAttributeFlags ChangingAttributes = EMeshRenderAttributeFlags::AllVertexAttribs);

	/**
	 * Set time that Preview will wait before showing working material
	 */
	void SetWorkingMaterialDelay(float TimeInSeconds) { SecondsBeforeWorkingMaterial = TimeInSeconds; }

	/**
	 * @return true if currently using the 'in progress' working material
	 */
	UE_API bool IsUsingWorkingMaterial();

	// Configure the maximum allowed number of background tasks
	UE_API void SetMaxActiveBackgroundTasks(int32 NewMaxTasks);

	// Configure maximum allowed number of background tasks heuristically from the input triangle count
	void SetMaxActiveBackgroundTasksFromMeshSizeHeuristic(int32 InputMeshTriangleCount, int32 MaxSimultaneousTrianglesToProcess = 10000000, int32 MaxShouldNotExceed = 20)
	{
		SetMaxActiveBackgroundTasks(FMath::Clamp(MaxSimultaneousTrianglesToProcess / FMath::Max(1, InputMeshTriangleCount), 1, MaxShouldNotExceed));
	}


	//
	// Change notification
	//
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMeshUpdated, UMeshOpPreviewWithBackgroundCompute*);
	/** This delegate is broadcast whenever the embedded preview mesh is updated */
	FOnMeshUpdated OnMeshUpdated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOpCompleted, const UE::Geometry::FDynamicMeshOperator*);
	FOnOpCompleted OnOpCompleted;

public:
	// preview of MeshOperator result
	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	// input set of materials to assign to PreviewMesh
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> StandardMaterials;

	// override material to forward to PreviewMesh if set
	UPROPERTY()
	TObjectPtr<UMaterialInterface> OverrideMaterial = nullptr;

	// if non-null, this material is swapped in when a background compute is active
	UPROPERTY()
	TObjectPtr<UMaterialInterface> WorkingMaterial = nullptr;
	
	// secondary render material to forward to PreviewMesh if set
	UPROPERTY()
	TObjectPtr<UMaterialInterface> SecondaryMaterial = nullptr;

	UPROPERTY()
	TWeakObjectPtr<UWorld> PreviewWorld = nullptr;

	/**
	 * When true, the preview mesh is allowed to be temporarily updated using results that we know
	 * are dirty (i.e., the preview was invalidated, but a result became available before the operation
	 * was restarted, so we can at least show that while we wait for the new result). The change 
	 * notifications will be fired as normal for these results, but HasValidResult will return false.
	 */
	bool bAllowDirtyResultUpdates = true;
protected:
	// state flag, if true then we have valid result
	bool bResultValid = false;
	double ValidResultComputeTimeSeconds = -1;

	// Stored status of last compute, mainly so that we know when we should
	// show the "busy" material.
	UE::Geometry::EBackgroundComputeTaskStatus LastComputeStatus = 
		UE::Geometry::EBackgroundComputeTaskStatus::NotComputing;

	bool bVisible = true;

	// Used for partial/fast updates of the preview mesh render proxy.
	bool bMeshTopologyIsConstant = false;
	EMeshRenderAttributeFlags ChangingAttributeFlags = EMeshRenderAttributeFlags::AllVertexAttribs;
	bool bMeshInitialized = false;

	float SecondsBeforeWorkingMaterial = 2.0;

	// this object manages the background computes
	TUniquePtr<FBackgroundDynamicMeshComputeSource> BackgroundCompute;

	// update the PreviewMesh if a new result is available from BackgroundCompute
	UE_API void UpdateResults();

private:

	int32 MaxActiveBackgroundTasks = 5;
	bool bWaitingForBackgroundTasks = false;
};







/**
 * TGenericDataBackgroundCompute is an infrastructure object that implements a common UI
 * pattern in interactive 3D tools, where we want to run an expensive parameterized computation 
 * (via a TGenericDataOperator) in a background thread so as to not block the UI. If the user changes 
 * parameters while the Operator is running, it should be canceled and restarted.
 *
 * The TGenericDataOperator are provided by the owner via a IGenericDataOperatorFactory implementation.
 * The owner must also Tick() this object regularly to allow results to be extracted from the
 * background thread and appropriate delegates fired when that occurs.
 */
template<typename ResultDataType>
class TGenericDataBackgroundCompute
{
public:
	using OperatorType = UE::Geometry::TGenericDataOperator<ResultDataType>;
	using FactoryType = UE::Geometry::IGenericDataOperatorFactory<ResultDataType>;
	using ComputeSourceType = UE::Geometry::TBackgroundModelingComputeSource<OperatorType, FactoryType>;

	//
	// required calls to setup/update/shutdown this object
	// 

	/**
	 * @param OpGenerator This factory is called to create new Operators on-demand
	 */
	void Setup(FactoryType* OpGenerator)
	{
		BackgroundCompute = MakeUnique<ComputeSourceType>(OpGenerator);
		bResultValid = false;
	}

	/**
	 * Terminate any active computation and return the current Result
	 */
	TUniquePtr<ResultDataType> Shutdown()
	{
		BackgroundCompute->CancelActiveCompute();
		return MoveTemp(CurrentResult);
	}

	/**
	* Terminate any active computation without returning anything
	*/
	void Cancel()
	{
		BackgroundCompute->CancelActiveCompute();
	}

	/**
	 * Tick the background computation to check for updated results
	 * @warning this must be called regularly for the class to function properly
	 */
	void Tick(float DeltaTime)
	{
		if (BackgroundCompute)
		{
			BackgroundCompute->Tick(DeltaTime);
		}
		UpdateResults();
	}

	//
	// Control flow
	// 

	/**
	 * Request that the current computation be canceled and a new one started
	 */
	void InvalidateResult()
	{
		check(BackgroundCompute);
		if (BackgroundCompute)
		{
			BackgroundCompute->NotifyActiveComputeInvalidated();
			bResultValid = false;
		}
	}

	/**
	 * @return true if the current Result is valid, ie no update being actively computed
	 */
	bool HaveValidResult() const { return bResultValid; }

	/**
	 * @return the elapsed compute time
	 */
	float GetElapsedComputeTime() const
	{
		check(BackgroundCompute);
		if (BackgroundCompute)
		{
			return (float)BackgroundCompute->GetElapsedComputeTime();
		}
		return 0.f;
	}


	//
	// Change notification
	//

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnOpCompleted, const OperatorType*);
	/** OnOpCompleted is fired via Tick() when an Operator finishes, with the operator pointer as argument  */
	FOnOpCompleted OnOpCompleted;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnResultUpdated, const TUniquePtr<ResultDataType>& );
	/** OnResultUpdated is fired via Tick() when an Operator finishes, with the computed result as argument  */
	FOnResultUpdated OnResultUpdated;

protected:
	// state flag, if true then we have valid result
	bool bResultValid = false;

	// current result value
	TUniquePtr<ResultDataType> CurrentResult;

	// this object manages the background computes
	TUniquePtr<ComputeSourceType> BackgroundCompute;

	// update CurrentResult if a new result is available from BackgroundCompute, and fires relevant signals
	void UpdateResults()
	{
		check(BackgroundCompute);
		if (BackgroundCompute)
		{
			UE::Geometry::EBackgroundComputeTaskStatus Status = BackgroundCompute->CheckStatus().TaskStatus;
			if (Status == UE::Geometry::EBackgroundComputeTaskStatus::ValidResultAvailable)
			{
				TUniquePtr<OperatorType> ResultOp = BackgroundCompute->ExtractResult();
				OnOpCompleted.Broadcast(ResultOp.Get());

				CurrentResult = ResultOp->ExtractResult();
				bResultValid = true;

				OnResultUpdated.Broadcast(CurrentResult);
			}
		}
	}

};

#undef UE_API
