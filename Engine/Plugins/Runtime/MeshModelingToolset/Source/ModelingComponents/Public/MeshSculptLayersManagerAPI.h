// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "GeometryBase.h"

#include "MeshSculptLayersManagerAPI.generated.h"


UINTERFACE(MinimalAPI)
class UMeshSculptLayersManager : public UInterface
{
	GENERATED_BODY()
};

// API to provide control over mesh sculpt layer support
class IMeshSculptLayersManager
{
	GENERATED_BODY()

public:

	virtual bool HasSculptLayers() const
	{
		return false;
	}

	// @return A number of base sculpt layers which should not be edited, or 0 if all layers can be edited
	virtual int32 NumLockedBaseSculptLayers() const
	{
		return 0;
	}
	
};
