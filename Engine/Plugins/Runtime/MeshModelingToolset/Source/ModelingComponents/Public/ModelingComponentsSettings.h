// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "BodySetupEnums.h"
#include "ModelingComponentsSettings.generated.h"

#define UE_API MODELINGCOMPONENTS_API

struct FCreateMeshObjectParams;

/**
 * Settings for the Modeling Components plug-in. These settings are primarily used to configure two things:
 *   - Behavior of things like optional Rendering features inside Modeling Tools, eg for edit-preview rendering
 *   - Setup of New Mesh Objects emitted by Modeling Tools (eg their default collision settings, etc)
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UModelingComponentsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	// UDeveloperSettings overrides
	virtual FName GetContainerName() const { return FName("Project"); }
	virtual FName GetCategoryName() const { return FName("Plugins"); }
	virtual FName GetSectionName() const { return FName("Modeling Mode Tools"); }
#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
#endif

public:

	//
	// Rendering settings applied to Preview Components used during live mesh editing
	//

	/** Enable Realtime Raytracing support for Mesh Editing Tools. This will impact performance of Tools with Real-Time feedback like 3D Sculpting. */
	UPROPERTY(config, EditAnywhere, Category = "Modeling Tools|Rendering")
	bool bEnableRayTracingWhileEditing = false;


	//
	// New Mesh Object settings, for new meshes/assets emitted by Modeling Tools
	//

	/** Enable Raytracing Support for new Mesh Objects created by Modeling Tools, if support is optional (eg DynamicMeshActors) */
	UPROPERTY(config, EditAnywhere, Category = "Modeling Tools|New Mesh Objects")
	bool bEnableRayTracing = false;

	/** Enable auto-generated Lightmap UVs for new Mesh Objects created by Modeling Tools, where supported */
	UPROPERTY(config, EditAnywhere, Category = "Modeling Tools|New Mesh Objects")
	bool bGenerateLightmapUVs = false;

	/** Enable Collision Support for new Mesh Objects created by Modeling Tools */
	UPROPERTY(config, EditAnywhere, Category = "Modeling Tools|New Mesh Objects")
	bool bEnableCollision = true;

	/** Default Collision Mode set on new Mesh Objects created by Modeling Tools */
	UPROPERTY(config, EditAnywhere, Category = "Modeling Tools|New Mesh Objects")
	TEnumAsByte<enum ECollisionTraceFlag> CollisionMode = ECollisionTraceFlag::CTF_UseSimpleAndComplex;

	static UE_API void ApplyDefaultsToCreateMeshObjectParams(FCreateMeshObjectParams& Params);


};



/** Modeling Components plugin-wide plane visualization modes */
UENUM()
enum class EModelingComponentsPlaneVisualizationMode : uint8
{
	/** Draw a grid with a fixed size in world space */
	SimpleGrid,
	
	/** Draw a hierarchical grid */
	HierarchicalGrid,
	
	/** Draw a grid with a fixed size in screen space */
	FixedScreenAreaGrid,
};

/**
 * Editor preferences for the Modeling Components plug-in.
 */
UCLASS(MinimalAPI, config=EditorSettings)
class UModelingComponentsEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// UDeveloperSettings overrides
	virtual FName GetContainerName() const { return FName("Editor"); }
	virtual FName GetCategoryName() const { return FName("Plugins"); }
	virtual FName GetSectionName() const { return FName("Modeling Mode Tools"); }

#if WITH_EDITOR
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;
#endif

public:
	/** The type of grid to draw in the viewport for modeling mode tools */
	UPROPERTY(EditAnywhere, NonTransactional, Category = "Modeling Tools|Work Plane Configuration")
	EModelingComponentsPlaneVisualizationMode GridMode = EModelingComponentsPlaneVisualizationMode::SimpleGrid;

	/** The number of lines to be drawn for the plane */
	UPROPERTY(EditAnywhere, NonTransactional, Category = "Modeling Tools|Work Plane Configuration", meta = (ClampMin = 2, EditCondition = "GridMode != EModelingComponentsPlaneVisualizationMode::HierarchicalGrid", EditConditionHides))
	int NumGridLines = 21;

	/** The space between grid lines in world space */
	UPROPERTY(EditAnywhere, NonTransactional, Category = "Modeling Tools|Work Plane Configuration", meta = (ClampMin = 0, EditCondition = "GridMode == EModelingComponentsPlaneVisualizationMode::SimpleGrid", EditConditionHides))
	float GridSpacing = 100.0f;

	/** The base scale used to determine the size of the hierarchical plane */
	UPROPERTY(EditAnywhere, NonTransactional, Category = "Modeling Tools|Work Plane Configuration", meta = (ClampMin = 1, EditCondition = "GridMode == EModelingComponentsPlaneVisualizationMode::HierarchicalGrid", EditConditionHides))
	float GridScale = 1.0f;

	/** The fraction of the viewport that the grid should occupy if looking at the plane's center */
	UPROPERTY(EditAnywhere, NonTransactional, Category = "Modeling Tools|Work Plane Configuration", meta = (ClampMin = 0, ClampMax = 2, EditCondition = "GridMode == EModelingComponentsPlaneVisualizationMode::FixedScreenAreaGrid", EditConditionHides))
	float GridSize = 0.5;

};

#undef UE_API
