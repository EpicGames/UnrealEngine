// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreMinimal.h"
#include "BodySetupEnums.h"
#include "Engine/EngineTypes.h"   // FMeshNaniteSettings
#include "MeshDescription.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "Misc/Optional.h"

#include "ModelingObjectsCreationAPI.generated.h"

#define UE_API MODELINGCOMPONENTS_API


class UInteractiveToolManager;
class AActor;
class UPrimitiveComponent;
class UActorComponent;
class UMaterialInterface;
class UTexture2D;


/**
 * Result code returned by UModelingObjectsCreationAPI functions
 */
UENUM(BlueprintType)
enum class ECreateModelingObjectResult : uint8
{
	Ok,
	Cancelled,
	Failed_Unknown,
	Failed_NoAPIFound,
	Failed_InvalidWorld,
	Failed_InvalidMesh,
	Failed_InvalidTexture,
	Failed_AssetCreationFailed,
	Failed_ActorCreationFailed,
	Failed_InvalidMaterial,
	Failed_InvalidActor
};

/**
 * Types of possible source meshes stored in FCreateMeshObjectParams
 */
UENUM(BlueprintType)
enum class ECreateMeshObjectSourceMeshType : uint8
{
	MeshDescription,
	DynamicMesh
};

/**
 * Hint for the type of mesh object a UModelingObjectsCreationAPI might create based
 * on FCreateMeshObjectParams data. This can be used by clients to try to specify
 * the type of object to emit, however there is no guarantee that an API implementation
 * supports creating all types.
 */
UENUM(BlueprintType)
enum class ECreateObjectTypeHint : uint8
{
	Undefined = 0,
	StaticMesh = 1,
	Volume = 2,
	DynamicMeshActor = 3
};


/**
 * FCreateMeshObjectParams is a collection of input data intended to be passed to
 * UModelingObjectsCreationAPI::CreateMeshObject(). Not all data necessarily needs
 * to be specified, this will depend on the particular implementation. The comments
 * below are representative of how this data structure is used in the Tools and
 * API implementation(s) provided with Unreal Engine, but end-user implementors
 * could abuse these fields as necessary.
 * 
 * The definition of a "mesh object" is implementation-specific.
 */
USTRUCT(Blueprintable)
struct FCreateMeshObjectParams
{
	// @param bConstructWithDefaultModelingComponentSettings	Whether to initialize with the default project settings.
	// Note the modeling component settings will not be used if the CVar "modeling.CreateMesh.IgnoreProjectSettings" is enabled
	UE_API FCreateMeshObjectParams(bool bConstructWithDefaultModelingComponentSettings = true);

	GENERATED_BODY()

	//
	// Base data
	//

	/** A Source Component the new mesh is based on, if such a Component exists */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	TObjectPtr<UPrimitiveComponent> SourceComponent = nullptr;

	/** A suggested type for the newly-created Mesh (possibly ignored) */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	ECreateObjectTypeHint TypeHint = ECreateObjectTypeHint::Undefined;

	/** A suggested UClass type for the newly-created Object (possibly ignored) */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	TObjectPtr<UClass> TypeHintClass = nullptr;

	/** An arbitrary integer that can be used to pass data to an API implementation */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	int32 TypeHintExtended = 0;

	/** The World/Level the new mesh object should be created in (if known) */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	TObjectPtr<UWorld> TargetWorld = nullptr;

	/** The 3D local-to-world transform for the new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	FTransform Transform;

	/** The base name of the new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	FString BaseName;

	//
	// Materials settings
	//

	/** Materials for the new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** Optional Materials for a newly-created Mesh Asset, if this is applicable for the created mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	TArray<TObjectPtr<UMaterialInterface>> AssetMaterials;

	//
	// Collision settings, if applicable for the given mesh object
	//

	/** Specify whether the new mesh object should have collision support/data */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	bool bEnableCollision = true;

	/** Which Collision mode to enable on the new mesh object, if supported */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	TEnumAsByte<enum ECollisionTraceFlag> CollisionMode = ECollisionTraceFlag::CTF_UseComplexAsSimple;

	/** Collision Shapes */
	TOptional<UE::Geometry::FSimpleShapeSet3d> CollisionShapeSet;


	//
	// Rendering Configuration Options, if this is applicable for the given mesh object
	//

	/** Specify whether normals should be automatically recomputed for this new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	bool bEnableRaytracingSupport = true;

	/** Specify whether to auto-generate Lightmap UVs (if applicable for the output mesh type) */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	bool bGenerateLightmapUVs = false;

	//
	// Mesh Build Options, if this is applicable for the given mesh object
	// (Currently somewhat specific to Assets in the Editor)
	//

	/** Specify whether normals should be automatically recomputed for this new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	bool bEnableRecomputeNormals = false;

	/** Specify whether tangents should be automatically recomputed for this new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	bool bEnableRecomputeTangents = false;

	/** Specify whether Nanite should be enabled on this new mesh object */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	bool bEnableNanite = false;

	/** Specify the Nanite proxy triangle percentage for this new mesh object */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Replaced NaniteProxyTrianglePercent with usage of Engine FMeshNaniteSettings"))
	float NaniteProxyTrianglePercent_DEPRECATED = 100.0f;

	/** Specify the Nanite Settings for this new mesh object, only used if bEnableNanite=true */
	UPROPERTY(Category = "CreateMeshObjectParams", EditAnywhere)
	FMeshNaniteSettings NaniteSettings = FMeshNaniteSettings();

	//
	// The Mesh Object should be created based on the mesh data structures below.
	// The assumption is that only one of these mesh data structures will be initialized.
	// Not currently exposed to BP because the types are not BP types.
	//

	ECreateMeshObjectSourceMeshType MeshType = ECreateMeshObjectSourceMeshType::MeshDescription;
	TOptional<FMeshDescription> MeshDescription;
	TOptional<UE::Geometry::FDynamicMesh3> DynamicMesh;

	UE_API void SetMesh(FMeshDescription&& MeshDescriptionIn);
	UE_API void SetMesh(const UE::Geometry::FDynamicMesh3* DynamicMeshIn);
	UE_API void SetMesh(UE::Geometry::FDynamicMesh3&& DynamicMeshIn);

};


/**
 * FCreateMeshObjectResult is returned by UModelingObjectsCreationAPI::CreateMeshObject()
 * to indicate success/failure and provide information about created mesh objects
 */
USTRUCT(BlueprintType)
struct FCreateMeshObjectResult
{
	GENERATED_BODY()

	/** Success/Failure status for the requested operation */
	UPROPERTY(Category = "CreateMeshObjectResult", VisibleAnywhere)
	ECreateModelingObjectResult ResultCode = ECreateModelingObjectResult::Ok;

	/** A pointer to a newly-created Actor for the mesh object, if applicable (eg StaticMeshActor) */
	UPROPERTY(Category="CreateMeshObjectResult", VisibleAnywhere)
	TObjectPtr<AActor> NewActor = nullptr;

	/** A pointer to a newly-created PrimitiveComponent for the mesh object, if applicable (eg StaticMeshComponent) */
	UPROPERTY(Category = "CreateMeshObjectResult", VisibleAnywhere)
	TObjectPtr<UPrimitiveComponent> NewComponent = nullptr;

	/** A pointer to a newly-created Asset for the mesh object, if applicable (eg StaticMeshAsset) */
	UPROPERTY(Category = "CreateMeshObjectResult", VisibleAnywhere)
	TObjectPtr<UObject> NewAsset = nullptr;


	bool IsOK() const { return ResultCode == ECreateModelingObjectResult::Ok; }
};





/**
 * FCreateTextureObjectParams is a collection of input data intended to be passed to
 * UModelingObjectsCreationAPI::CreateTextureObject(). Not all data necessarily needs
 * to be specified, this will depend on the particular implementation. The comments
 * below are representative of how this data structure is used in the Tools and
 * API implementation(s) provided with Unreal Engine, but end-user implementors
 * could abuse these fields as necessary.
 *
 * The definition of a "texture object" is implementation-specific. 
 * In the UE Editor this is generally a UTexture2D
 */
USTRUCT(Blueprintable)
struct FCreateTextureObjectParams
{
	GENERATED_BODY()

	//
	// Base data
	//

	/** An arbitrary integer that can be used to pass data to an API implementation */
	UPROPERTY(Category = "CreateTextureObjectParams", EditAnywhere)
	int32 TypeHintExtended = 0;

	/** 
	 * The World/Level the new texture object should be created in (if known).
	 * Note that Textures generally do not exist as objects in a Level. 
	 * However, it may be necessary to store the texture in a path relative to the
	 * level (for example if the level is in a Plugin, this would be necessary in-Editor)
	 */
	UPROPERTY(Category = "CreateTextureObjectParams", EditAnywhere)
	TObjectPtr<UWorld> TargetWorld = nullptr;

	/** An object to store the Texture relative to. For example the texture could be stored at the same path. */
	UPROPERTY(Category = "CreateTextureObjectParams", EditAnywhere)
	TObjectPtr<UObject> StoreRelativeToObject = nullptr;

	/** The base name of the new mesh object */
	UPROPERTY(Category = "CreateTextureObjectParams", EditAnywhere)
	FString BaseName;

	//
	// input data
	//

	/** 
	 * Texture source data. Generally assumed that this is a Texture created in the Transient package
	 * that is intended to be saved in a permanent package.
	 */
	UPROPERTY(Category = "CreateTextureObjectParams", EditAnywhere)
	TObjectPtr<UTexture2D> GeneratedTransientTexture = nullptr;

	/**
	 * A full path location to save the asset out to. If this parameter is not null, it overrides other work done to find a path
	 */
	UPROPERTY(Category= "CreateTextureObjectParams", EditAnywhere)
	FString FullAssetPath = "";
};


/**
 * FCreateTextureObjectResult is returned by UModelingObjectsCreationAPI::CreateTextureObject()
 * to indicate success/failure and provide information about created texture objects
 */
USTRUCT(BlueprintType)
struct FCreateTextureObjectResult
{
	GENERATED_BODY()

	/** Success/Failure status for the requested operation */
	UPROPERTY(Category = "CreateTextureObjectResult", VisibleAnywhere)
	ECreateModelingObjectResult ResultCode = ECreateModelingObjectResult::Ok;

	/** A pointer to a newly-created Asset for the texture object */
	UPROPERTY(Category = "CreateTextureObjectResult", VisibleAnywhere)
	TObjectPtr<UObject> NewAsset = nullptr;


	bool IsOK() const { return ResultCode == ECreateModelingObjectResult::Ok; }
};






/**
 * FCreateMaterialObjectParams is a collection of input data intended to be passed to
 * UModelingObjectsCreationAPI::CreateMaterialObject().
 */
USTRUCT(Blueprintable)
struct FCreateMaterialObjectParams
{
	GENERATED_BODY()

	//
	// Base data
	//

	/** 
	 * The World/Level the new Material object should be created in (if known).
	 * Note that Material generally do not exist as objects in a Level. 
	 * However, it may be necessary to store the texture in a path relative to the
	 * level (for example if the level is in a Plugin, this would be necessary in-Editor)
	 */
	UPROPERTY(Category = "CreateMaterialObjectParams", EditAnywhere)
	TObjectPtr<UWorld> TargetWorld = nullptr;

	/** An object to store the Material relative to. */
	UPROPERTY(Category = "CreateMaterialObjectParams", EditAnywhere)
	TObjectPtr<UObject> StoreRelativeToObject = nullptr;

	/** The base name of the new Material object */
	UPROPERTY(Category = "CreateMaterialObjectParams", EditAnywhere)
	FString BaseName;

	//
	// input data
	//

	/** 
	 * The parent UMaterial of this material will be duplicated to create the new UMaterial Asset.
	 */
	UPROPERTY(Category = "CreateMaterialObjectParams", EditAnywhere)
	TObjectPtr<UMaterialInterface> MaterialToDuplicate = nullptr;
};


/**
 * FCreateMaterialObjectResult is returned by UModelingObjectsCreationAPI::CreateMaterialObject()
 * to indicate success/failure and provide information about created texture objects
 */
USTRUCT(BlueprintType)
struct FCreateMaterialObjectResult
{
	GENERATED_BODY()

	/** Success/Failure status for the requested operation */
	UPROPERTY(Category = "CreateMaterialObjectResult", VisibleAnywhere)
	ECreateModelingObjectResult ResultCode = ECreateModelingObjectResult::Ok;

	/** A pointer to a newly-created Asset for the material object */
	UPROPERTY(Category = "CreateTextureObjectResult", VisibleAnywhere)
	TObjectPtr<UObject> NewAsset = nullptr;


	bool IsOK() const { return ResultCode == ECreateModelingObjectResult::Ok; }
};




/**
 * FCreateActorParams is a collection of input data intended to be passed to
 * UModelingObjectsCreationAPI::CreateNewActor().
 */
USTRUCT(Blueprintable)
struct FCreateActorParams
{
	GENERATED_BODY()
	

	
	//
	// Base data
	//

	/** 
	 * The World/Level the new Actor should be created in (if known).
	 */
	UPROPERTY(Category = "CreateActorParams", EditAnywhere)
	TObjectPtr<UWorld> TargetWorld = nullptr;

	/** The base name of the new Actor */
	UPROPERTY(Category = "CreateActorParams", EditAnywhere)
	FString BaseName;

	/** The 3D local-to-world transform for the new actor */
	UPROPERTY(Category = "CreateActorParams", EditAnywhere)
	FTransform Transform;

	//
	// input data
	//

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "TemplateActor is being deprecated. Please use TemplateAsset instead."))
	TObjectPtr<AActor> TemplateActor_DEPRECATED;

	/** A template Asset used to determine the type of Actor to spawn. */
	UPROPERTY(Category = "CreateActorParams", EditAnywhere)
	TObjectPtr<UObject> TemplateAsset = nullptr;
};


/**
 * FCreateActorResult is returned by UModelingObjectsCreationAPI::FCreateActorParams()
 * to indicate success/failure and provide information about created actors
 */
USTRUCT(BlueprintType)
struct FCreateActorResult
{
	GENERATED_BODY()

	/** Success/Failure status for the requested operation */
	UPROPERTY(Category = "CreateMaterialObjectResult", VisibleAnywhere)
	ECreateModelingObjectResult ResultCode = ECreateModelingObjectResult::Ok;

	/** A pointer to a newly-created Actor */
	UPROPERTY(Category="CreateMeshObjectResult", VisibleAnywhere)
	TObjectPtr<AActor> NewActor = nullptr;


	bool IsOK() const { return ResultCode == ECreateModelingObjectResult::Ok; }
};

/**
 * FCreateComponentParams is a collection of input data intended to be passed to
 * UModelingObjectsCreationAPI::CreateNewComponentOnActor().
 */
USTRUCT(BlueprintType)
struct FCreateComponentParams
{
	GENERATED_BODY()

	/** An actor to host the new component */
	UPROPERTY(Category = "CreateComponentParams", EditAnywhere)
	TObjectPtr<AActor> HostActor = nullptr;

	/** A component class to instantiate a new component instance of. */
	UPROPERTY(Category = "CreateComponentParams", EditAnywhere)
	TObjectPtr<UClass> ComponentClass;

	/** The base name of the new Component */
	UPROPERTY(Category = "CreateComponentParams", EditAnywhere)
	FString BaseName;

	/** If true, and if the component class is derived from USceneComponent, set this component as the root of the Actor. */
	UPROPERTY(Category = "CreateComponentParams", EditAnywhere)
	bool bSetAsRoot = false;

	/** If true, wrap the creation operation in a transaction, and create the component with the RF_Transactional object flag set. */
	UPROPERTY(Category = "CreateComponentParams", EditAnywhere)
	bool bTransact = false;
};

/**
 * FCreateComponentResult is returned by UModelingObjectsCreationAPI::CreateNewComponentOnActor()
 * to indicate success/failure and provide information about created components
 */
USTRUCT(BlueprintType)
struct FCreateComponentResult
{
	GENERATED_BODY()

	/** Success/Failure status for the requested operation */
	UPROPERTY(Category = "CreateComponentResult", VisibleAnywhere)
	ECreateModelingObjectResult ResultCode = ECreateModelingObjectResult::Ok;

	/** A pointer to a newly-created Component */
	UPROPERTY(Category = "CreateComponentResult", VisibleAnywhere)
	TObjectPtr<UActorComponent> NewComponent = nullptr;

	bool IsOK() const { return ResultCode == ECreateModelingObjectResult::Ok; }
};


/**
 * UModelingObjectsCreationAPI is a base interface for functions that can be used to
 * create various types of objects from Modeling Tools, or other sources. The "type" is
 * very generic here - "Mesh", "Texture", etc - because this API is meant to provide
 * an abstraction for Tools to emit different types of objects in different situations.
 * For example an Tool might emit StaticMesh Asset/Actors in-Editor, but ProceduralMeshComponents at Runtime.
 * 
 * The creation inputs are specified via the structs above (eg FCreateMeshObjectParams, FCreateTextureObjectParams),
 * which are very extensive, kitchen-sink sort of structs. Generally "New Mesh Object"
 * creation behavior will be very complex and so this API is really just a way to route
 * the data, and very few guarantees can be made about any specific implementation.
 * 
 * The assumed (but not really required) usage of instances of this type are that they
 * will be registered with an InteractiveToolsContext's ContextObjectStore, and then 
 * fetched from there by Tools/Algorithms/etc that need to use these capabilities can
 * use the UE::Modeling::CreateXObject() helper functions below. However the interface
 * does not have any dependencies on this usage model. 
 * 
 * See UEditorModelingObjectsCreationAPI for an example implementation suitable for in-Editor use.
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UModelingObjectsCreationAPI : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Create a new mesh object based on the data in CreateMeshParams
	 * @return a results data structure, containing a result code and information about any new objects created
	 */
	UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	virtual FCreateMeshObjectResult CreateMeshObject(const FCreateMeshObjectParams& CreateMeshParams) { return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_Unknown }; }

	/**
	 * Create a new texture object based on the data in CreateTexParams
	 * @return a results data structure, containing a result code and information about any new objects created
	 */
	UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	virtual FCreateTextureObjectResult CreateTextureObject(const FCreateTextureObjectParams& CreateTexParams) { return FCreateTextureObjectResult{ ECreateModelingObjectResult::Failed_Unknown }; }

	/**
	 * Create a new material object based on the data in CreateMaterialParams
	 * @return a results data structure, containing a result code and information about any new objects created
	 */
	UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	virtual FCreateMaterialObjectResult CreateMaterialObject(const FCreateMaterialObjectParams& CreateMaterialParams) { return FCreateMaterialObjectResult{ ECreateModelingObjectResult::Failed_Unknown }; }

	/**
	 * Create a new material object based on the data in CreateMaterialParams
	 * @return a results data structure, containing a result code and information about any new objects created
	 */
	UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	virtual FCreateActorResult CreateNewActor(const FCreateActorParams& CreateActorParams) { return FCreateActorResult{ ECreateModelingObjectResult::Failed_Unknown }; }

	/**
	 * Create a new component on the specified actor of the requested class.
	 * @return a results data structure, containing a result code and information about the new component created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	virtual FCreateComponentResult CreateNewComponentOnActor(const FCreateComponentParams& CreateComponentParams) { return FCreateComponentResult{ ECreateModelingObjectResult::Failed_Unknown }; }

	//
	// Non-UFunction variants that support std::move operators
	//

	/**
	 * If this function returns true, then the CreateMeshObject() and CreateTextureObject() that take && types can be called.
	 * This can reduce data copying, eg if the mesh data can be directly moved into the output mesh object.
	 */
	virtual bool HasMoveVariants() const { return false; }

	virtual FCreateMeshObjectResult CreateMeshObject(FCreateMeshObjectParams&& CreateMeshParams) { return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_Unknown }; }
	virtual FCreateTextureObjectResult CreateTextureObject(FCreateTextureObjectParams&& CreateTexParams) { return FCreateTextureObjectResult{ ECreateModelingObjectResult::Failed_Unknown }; }
	virtual FCreateMaterialObjectResult CreateMaterialObject(FCreateMaterialObjectParams&& CreateMaterialParams) { return FCreateMaterialObjectResult{ ECreateModelingObjectResult::Failed_Unknown }; }
	virtual FCreateActorResult CreateNewActor(FCreateActorParams&& CreateActorParams) { return FCreateActorResult{ ECreateModelingObjectResult::Failed_Unknown }; }
	virtual FCreateComponentResult CreateNewComponentOnActor(FCreateComponentParams&& CreateComponentParams) { return FCreateComponentResult{ ECreateModelingObjectResult::Failed_Unknown }; }

};




namespace UE
{
namespace Modeling
{


/**
 * Create a new mesh object based on the data in CreateMeshParams.
 * This is a convenience function that will try to locate a UModelingObjectsCreationAPI instance in the ToolManager's ContextObjectStore,
 * and then call UModelingObjectsCreationAPI::CreateMeshObject()
 * @return a results data structure, containing a result code and information about any new objects created
 */
MODELINGCOMPONENTS_API FCreateMeshObjectResult CreateMeshObject(UInteractiveToolManager* ToolManager, FCreateMeshObjectParams&& CreateMeshParams);


/**
 * Create a new texture object based on the data in CreateTexParams.
 * This is a convenience function that will try to locate a UModelingObjectsCreationAPI instance in the ToolManager's ContextObjectStore,
 * and then call UModelingObjectsCreationAPI::CreateTextureObject()
 * @return a results data structure, containing a result code and information about any new objects created
 */
MODELINGCOMPONENTS_API FCreateTextureObjectResult CreateTextureObject(UInteractiveToolManager* ToolManager, FCreateTextureObjectParams&& CreateTexParams);


/**
 * Create a new material object based on the data in CreateMaterialParams.
 * This is a convenience function that will try to locate a UModelingObjectsCreationAPI instance in the ToolManager's ContextObjectStore,
 * and then call UModelingObjectsCreationAPI::CreateMaterialObject()
 * @return a results data structure, containing a result code and information about any new objects created
 */
MODELINGCOMPONENTS_API FCreateMaterialObjectResult CreateMaterialObject(UInteractiveToolManager* ToolManager, FCreateMaterialObjectParams&& CreateMaterialParams);


/**
 * Create a new actor based on the data in CreateActorParams.
 * This is a convenience function that will try to locate a UModelingObjectsCreationAPI instance in the ToolManager's ContextObjectStore,
 * and then call UModelingObjectsCreationAPI::CreateNewActor()
 * @return a results data structure, containing a result code and information about any new objects created
 */
MODELINGCOMPONENTS_API FCreateActorResult CreateNewActor(UInteractiveToolManager* ToolManager, FCreateActorParams&& CreateActorParams);

/**
 * Create a new component on an actor based on the data in CreateComponentParams.
 * This is a convenience function that will try to locate a UModelingObjectsCreationAPI instance in the ToolManager's ContextObjectStore,
 * and then call UModelingObjectsCreationAPI::CreateNewComponentOnActor()
 * @return a results data structure, containing a result code and information about any new objects created
 */
MODELINGCOMPONENTS_API FCreateComponentResult CreateNewComponentOnActor(UInteractiveToolManager* ToolManager, FCreateComponentParams&& CreateComponentParams);

/**
 * Strip the appended auto-generated _UUID suffix on the given string, if we can detect one
 * @param InputName input string that may have _UUID suffix
 * @return InputName without _UUID suffix
 */
MODELINGCOMPONENTS_API FString StripGeneratedAssetSuffixFromName( FString InputName );


/**
* Generate a N-letter GUID string that contains only hex digits,
* and contains at least one letter and one number. Used to create _UUID suffixes
* for making asset names unique, etc.
*/
MODELINGCOMPONENTS_API FString GenerateRandomShortHexString(int32 NumChars = 8);


/**
 * Look up the name for the Asset underlying the given Component, if there is one.
 * Optionally Strip off the appended auto-generated UUID string if we can detect one.
 * If there is not a known underlying asset, returns the Component's name
 * @param Component the Component we want the Asset Base Name for
 * @param bRemoveAutoGeneratedSuffixes strip off generated UUID suffix if one is detected (default true)
 * @return the Name for the Component
 */
MODELINGCOMPONENTS_API FString GetComponentAssetBaseName(
	UActorComponent* Component,
	bool bRemoveAutoGeneratedSuffixes = true);

}
}

#undef UE_API
