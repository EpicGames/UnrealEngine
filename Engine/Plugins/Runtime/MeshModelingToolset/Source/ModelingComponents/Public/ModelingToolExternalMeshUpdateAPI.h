// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Changes/MeshRegionChange.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "UObject/Interface.h"

#include "ModelingToolExternalMeshUpdateAPI.generated.h"


class UInteractiveToolManager;

// UInterface for IModelingToolExternalDynamicMeshUpdateAPI
UINTERFACE(MinimalAPI)
class UModelingToolExternalDynamicMeshUpdateAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * Provides an API to allow external methods to update the mesh or meshes managed by a tool
 */
class IModelingToolExternalDynamicMeshUpdateAPI
{
	GENERATED_BODY()
public:

	// @return true if the tool will currently allow an external mesh update to run
	virtual bool AllowToolMeshUpdates() const { return false; }

	// Update the tool meshes with the provided method, then emit any resulting change transaction(s) and do associated book-keeping required by the tool (updating any spatial data-structures, rendering, etc)
	virtual void UpdateToolMeshes(TFunctionRef<TUniquePtr<FMeshRegionChangeBase>(UE::Geometry::FDynamicMesh3&, int32 MeshIdx)> UpdateMesh) {}

	// Read the current tool meshes
	virtual void ProcessToolMeshes(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&, int32 MeshIdx)> UpdateMesh) const {}

	// @return the number of meshes managed by the tool
	virtual int32 NumToolMeshes() const
	{
		return 0;
	}
};