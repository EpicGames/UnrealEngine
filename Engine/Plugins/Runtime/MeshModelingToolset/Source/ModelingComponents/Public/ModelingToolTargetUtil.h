// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"
#include "TargetInterfaces/MaterialProvider.h" // FComponentMaterialSet
#include "MeshConversionOptions.h"

class UStaticMesh;
class USkeletalMesh;
class UToolTarget;
class UPrimitiveComponent;
class USceneComponent;
class AActor;
class UBodySetup;
class IInterface_CollisionDataProvider;
struct FMeshDescription;
struct FCreateMeshObjectParams;
class IPersistentDynamicMeshSource;

//
// UE::ToolTarget:: namespace contains utility/helper functions for interacting with UToolTargets.
// Generally these are meant to be used by UInteractiveTools to handle standard tasks that would
// otherwise require each Tool to figure out things like which ToolTargetInterface to cast to, etc.
// Using these functions ideally avoids all the boilerplate inherent in the ToolTarget system.
// 
// However, the cost is that it is not necessarily the most efficient, as each one of these functions
// may potentially do many repeated Cast<>'s internally. So, use sparingly, or cache the outputs.
//
namespace UE
{
namespace ToolTarget
{

/**
 * @return the AActor backing a ToolTarget, or nullptr if there is no such Actor
 */
MODELINGCOMPONENTS_API AActor* GetTargetActor(UToolTarget* Target);
	

/**
 * @return the UPrimitiveComponent backing a ToolTarget, or nullptr if there is no such Component
 */
MODELINGCOMPONENTS_API UPrimitiveComponent* GetTargetComponent(UToolTarget* Target);

/**
 * @return the UActorComponent backing a ToolTarget, or nullptr if there is no such Component
 */
MODELINGCOMPONENTS_API USceneComponent* GetTargetSceneComponent(UToolTarget* Target);

/**
 * @return a human readable identifier for a ToolTarget, based on the underlying nature of it.
 */
MODELINGCOMPONENTS_API FString GetHumanReadableName(UToolTarget* Target);

/**
 * Hide the "Source Object" (eg PrimitiveComponent, Actor, etc) backing a ToolTarget
 * @return true on success
 */
MODELINGCOMPONENTS_API bool HideSourceObject(UToolTarget* Target);

/**
 * Show the "Source Object" (eg PrimitiveComponent, Actor, etc) backing a ToolTarget
 * @return true on success
 */
MODELINGCOMPONENTS_API bool ShowSourceObject(UToolTarget* Target);

/**
 * Show or Hide the "Source Object" (eg PrimitiveComponent, Actor, etc) backing a ToolTarget
 * @return true on success
 */
MODELINGCOMPONENTS_API bool SetSourceObjectVisible(UToolTarget* Target, bool bVisible);

/**
 * @return the local-to-world Transform underlying a ToolTarget, eg the Component or Actor transform
 */
MODELINGCOMPONENTS_API FTransform3d GetLocalToWorldTransform(UToolTarget* Target);

/**
 * Fetch the Material Set on the object underlying a ToolTarget. In cases where there are (eg) 
 * separate Component and Asset material sets, prefers the Component material set
 * @param bPreferAssetMaterials if true, prefer an Asset material set, if available
 * @return a valid MaterialSet
 */
MODELINGCOMPONENTS_API FComponentMaterialSet GetMaterialSet(UToolTarget* Target, bool bPreferAssetMaterials = false);


/**
 * Update the material set of the Target
 * @param bApplyToAsset In situations where the Target has both Component-level and Asset-level materials, this specifies which should be updated (this flag is passed to the IMaterialProvider, which may or may not respect it)
 */
MODELINGCOMPONENTS_API bool CommitMaterialSetUpdate(
	UToolTarget* Target,
	const FComponentMaterialSet& UpdatedMaterials,
	bool bApplyToAsset = true);

/**
 * @param Target					The tool target to query for LODs
 * @param bOutTargetSupportsLODs	Will be set to 'false' if the target does not support querying MeshDescriptions with LODs
 * @param bOnlyReturnDefaultLOD		If true, always return an array of just the Default LOD
 * @param bExcludeAutoGeneratedLODs	If true, exclude any auto-generated LODs (which have no associated Mesh Description) from the LOD list
 * @return An array of valid LODs for the mesh data underlying the ToolTarget, or just the Default LOD if the target doesn't support LODs or bOnlyReturnDefaultLOD is true
 */
MODELINGCOMPONENTS_API TArray<EMeshLODIdentifier> GetMeshDescriptionLODs(
	UToolTarget* Target,
	bool& bOutTargetSupportsLODs,
	bool bOnlyReturnDefaultLOD = false,
	bool bExcludeAutoGeneratedLODs = true);

/**
 * Return identifier for the mesh LOD that a ToolTarget represents, if defined for the given ToolTarget
 * @param Target					The tool target to query for LODs
 * @param bOutTargetSupportsLODs	Will be set to 'false' if the target does not support querying MeshDescriptions with LODs
 * @return LODIdentifier for current LOD that would be provided/edited by the ToolTarget, or the Default LODIdentifier if the target doesn't support LODs or bOnlyReturnDefaultLOD is true
 */
MODELINGCOMPONENTS_API EMeshLODIdentifier GetTargetMeshDescriptionLOD(
	UToolTarget* Target,
	bool& bOutTargetSupportsLODs);


/**
 * @return the MeshDescription underlying a ToolTarget, if it has such a mesh. May be generated internally by the ToolTarget. May be nullptr if the Target does not have a mesh.
 */
MODELINGCOMPONENTS_API const FMeshDescription* GetMeshDescription(
	UToolTarget* Target,
	const FGetMeshParameters& GetMeshParams = FGetMeshParameters());

/**
 * @return an empty MeshDescription with attributes already registered appropriate to the target.
 */
MODELINGCOMPONENTS_API FMeshDescription GetEmptyMeshDescription(
	UToolTarget* Target);

/**
 * Return a copy of the MeshDescription underlying a ToolTarget
 * @return a new MeshDescription, which may be empty if the Target doesn't have a mesh  
 */
MODELINGCOMPONENTS_API FMeshDescription GetMeshDescriptionCopy(
	UToolTarget* Target, 
	const FGetMeshParameters& GetMeshParams = FGetMeshParameters());

/**
 * Fetch a DynamicMesh3 representing the given ToolTarget. This may be a conversion of the output of GetMeshDescription().
 * This function returns a copy, so the caller can take ownership of this Mesh.
 * @param bWantMeshTangents if true, tangents will be returned if the target has them available. This may require that they be auto-calculated in some cases (which may be expensive)
 * @return a created DynamicMesh3, which may be empty if the Target doesn't have a mesh 
 */
	UE_DEPRECATED(5.5, "Use GetDynamicMeshCopy which takes a FGetMeshParameters instead.")
	MODELINGCOMPONENTS_API UE::Geometry::FDynamicMesh3 GetDynamicMeshCopy(UToolTarget* Target, bool bWantMeshTangents);
	
/**
 * Fetch a DynamicMesh3 representing the given ToolTarget. This may be a conversion of the output of GetMeshDescription().
 * This function returns a copy, so the caller can take ownership of this Mesh.
 * @param InGetMeshParams to specify various options like specific LOD and/or tangents on the returned mesh.
 * if InGetMeshParams.bWantMeshTangents is true, tangents will be returned if the target has them available. This may require that they be auto-calculated in some cases (which may be expensive)
 * @return a created DynamicMesh3, which may be empty if the Target doesn't have a mesh 
 */
	MODELINGCOMPONENTS_API UE::Geometry::FDynamicMesh3 GetDynamicMeshCopy(
		UToolTarget* Target,
		const FGetMeshParameters& InGetMeshParams = FGetMeshParameters());

/** @return The triangle count of Target's persistent dynamic mesh or its mesh description, if available, or 0 otherwise */
MODELINGCOMPONENTS_API int32 GetTriangleCount(UToolTarget* Target);

/**
 * EDynamicMeshUpdateResult is returned by functions below that update a ToolTarget with a new Mesh
 */
enum class EDynamicMeshUpdateResult
{
	/** Update was successful */
	Ok = 0,
	/** Update failed */
	Failed = 1,
	/** Update was successful, but required that the entire target mesh be replaced, instead of a (requested) partial update */
	Ok_ForcedFullUpdate = 2
};


/**
 * Update the Mesh in a ToolTarget based on the provided MeshDescription, and optional material set
 * @return EDynamicMeshUpdateResult::Ok on success
 */
MODELINGCOMPONENTS_API EDynamicMeshUpdateResult CommitMeshDescriptionUpdate(
	UToolTarget* Target, 
	const FMeshDescription* UpdatedMesh, 
	const FComponentMaterialSet* UpdatedMaterials = nullptr,
	const FCommitMeshParameters& CommitParams = FCommitMeshParameters());

/**
* Update the Mesh in a ToolTarget based on the provided MeshDescription, and optional material set
* @return EDynamicMeshUpdateResult::Ok on success
*/
MODELINGCOMPONENTS_API EDynamicMeshUpdateResult CommitMeshDescriptionUpdate(
	UToolTarget* Target, 
	FMeshDescription&& UpdatedMesh,
	const FCommitMeshParameters& CommitParams = FCommitMeshParameters());

/**
* Update the Mesh in a ToolTarget based on the provided MeshDescription, and optional material set
* @return EDynamicMeshUpdateResult::Ok on success
*/
MODELINGCOMPONENTS_API EDynamicMeshUpdateResult CommitMeshDescriptionUpdateViaDynamicMesh(
	UToolTarget* Target, 
	const UE::Geometry::FDynamicMesh3& UpdatedMesh,
	bool bHaveModifiedTopology,
	const FCommitMeshParameters& CommitParams = FCommitMeshParameters());


/**
 * Update the Mesh in a ToolTarget based on the provided DynamicMesh, and optional material set
 * @param bHaveModifiedTopology If the update only changes vertex or attribute values (but not counts), then in some cases a more efficient and/or less destructive update can be applied to the Target
 * @param ConversionOptions if the commit to the Target involves conversion to MeshDescription, these options can configure that conversion
 * @param UpdatedMaterials optional new material set that will be applied to the updated Target, and the Target Asset if available. If more control is needed use CommitMaterialSetUpdate()
 * @return EDynamicMeshUpdateResult::Ok on success
 */
MODELINGCOMPONENTS_API EDynamicMeshUpdateResult CommitDynamicMeshUpdate(
	UToolTarget* Target,
	const UE::Geometry::FDynamicMesh3& UpdatedMesh,
	bool bHaveModifiedTopology,
	const FConversionToMeshDescriptionOptions& ConversionOptions = FConversionToMeshDescriptionOptions(),
	const FComponentMaterialSet* UpdatedMaterials = nullptr);

/**
 * Update the UV sets of the ToolTarget's mesh (assuming it has one) based on the provided UpdatedMesh.
 * @todo: support updating a specific UV set/index, rather than all sets
 * @return EDynamicMeshUpdateResult::Ok on success, or Ok_ForcedFullUpdate if any dependent mesh topology was modified
 */
MODELINGCOMPONENTS_API EDynamicMeshUpdateResult CommitDynamicMeshUVUpdate(UToolTarget* Target, const UE::Geometry::FDynamicMesh3* UpdatedMesh);

/**
* Update the Normals/Tangents of the ToolTarget's mesh (assuming it has one) based on the provided UpdatedMesh.
* @return EDynamicMeshUpdateResult::Ok on success, or Ok_ForcedFullUpdate if any dependent mesh topology was modified
*/
MODELINGCOMPONENTS_API EDynamicMeshUpdateResult CommitDynamicMeshNormalsUpdate(UToolTarget* Target, const UE::Geometry::FDynamicMesh3* UpdatedMesh, bool bUpdateTangents = false);

/**
 * @return true if ToolTarget can be directly updated via an incremental edits, ie if it is acceptable to call ApplyIncrementalMeshEditChange() on the target
 */
MODELINGCOMPONENTS_API bool SupportsIncrementalMeshChanges(UToolTarget* Target);


/**
 * Apply an incremental mesh edit (MeshEditingFunc) to the ToolTarget. Tools/etc can use this to directly
 * modify the ToolTarget's internal DynamicMesh, while (presumably) also emitting a FChange transaction that
 * represents that Edit. This allows for undoable edits directly to a DynamicMeshComponent/etc in the level. 
 * @return true if the incremental edit succeeds
 */
MODELINGCOMPONENTS_API bool ApplyIncrementalMeshEditChange(
	UToolTarget* Target,
	TFunctionRef<bool(UE::Geometry::FDynamicMesh3& EditMesh, UObject* TransactionTarget)> MeshEditingFunc );



/**
 * FCreateMeshObjectParams::TypeHint is used by the ModelingObjectsCreationAPI to suggest what type of mesh object to create
 * inside various Tools. This should often be derived from the input mesh object type (eg if you plane-cut a Volume, the output
 * should be Volumes). This function interrogates the ToolTarget to try to determine this information
 * @return true if a known type was detected and configured in FCreateMeshObjectParams::TypeHint (and possibly FCreateMeshObjectParams::TypeHintClass)
 */
MODELINGCOMPONENTS_API bool ConfigureCreateMeshObjectParams(UToolTarget* SourceTarget, FCreateMeshObjectParams& DerivedParamsOut);


namespace Internal
{
	/**
	 * Not intended for direct use by tools, just for use by tool target util functions and tool target
	 * implementations that may need to do this operation. Uses the IPersistentDynamicMeshSource interface
	 * to perform an update of the dynamic mesh.
	 * Currently ignores bHaveModifiedTopology.
	 */
	MODELINGCOMPONENTS_API void CommitDynamicMeshViaIPersistentDynamicMeshSource(
		IPersistentDynamicMeshSource& DynamicMeshSource,
		const UE::Geometry::FDynamicMesh3& UpdatedMesh, bool bHaveModifiedTopology);

#if WITH_EDITOR
	/**
	 * Internal path for use by tool targets to invoke PostEditChange with conditional capture for undo
	 * transactions. This capture is controlled by the CVar 'modeling.CapturePostEditChangeInTransactions'.
	 *
	 * PostEditChange can include arbitrary user code which can lead to a tool transaction capturing data
	 * beyond the modifications the tool has made, opening up the risk of various side effects. It is
	 * recommended that PostEditChange is not included in a transaction scope since PostEditUndo will
	 * reinvoke PostEditChange and provide the user code the opportunity to reapply its changes in response
	 * to the data modifications being made.
	 *
	 * That said, not all user code in PostEditChange is well behaved and deterministic so we provide the CVar
	 * to toggle this behavior.
	 */
	MODELINGCOMPONENTS_API void PostEditChangeWithConditionalUndo(UObject* Object);

	/** Scoped override that forces PostEditChangeWithConditionalUndo onto its safe branch. */
	class FScopedSuppressPostEditChangeCapture
	{
	public:
		MODELINGCOMPONENTS_API FScopedSuppressPostEditChangeCapture();
		MODELINGCOMPONENTS_API ~FScopedSuppressPostEditChangeCapture();

		FScopedSuppressPostEditChangeCapture(const FScopedSuppressPostEditChangeCapture&) = delete;
		FScopedSuppressPostEditChangeCapture& operator=(const FScopedSuppressPostEditChangeCapture&) = delete;

	private:
		bool bSavedValue;
	};
#endif
}

/**
 * @return the Physics UBodySetup for the given ToolTarget, or nullptr if it does not exist
 */
MODELINGCOMPONENTS_API UBodySetup* GetPhysicsBodySetup(UToolTarget* Target);

/**
* @return the Physics CollisionDataProvider (ie Complex Collision source) for the given ToolTarget, or nullptr if it does not exist
*/
MODELINGCOMPONENTS_API IInterface_CollisionDataProvider* GetPhysicsCollisionDataProvider(UToolTarget* Target);

/** @return StaticMesh from a tool target */ 
MODELINGCOMPONENTS_API UStaticMesh* GetStaticMeshFromTargetIfAvailable(UToolTarget* Target);

/** @return SkeletalMesh from a tool target */
MODELINGCOMPONENTS_API USkeletalMesh* GetSkeletalMeshFromTargetIfAvailable(UToolTarget* Target);

/**
 * Delete an AActor in the correct way for the editor, with a fallback for the non-editor case
 */
MODELINGCOMPONENTS_API void SafeDeleteActor(AActor* TargetActor);

}  // end namespace ToolTarget
}  // end namespace UE