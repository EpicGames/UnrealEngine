// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "OrientedBoxTypes.h"
#include "SegmentTypes.h"
#include "CapsuleTypes.h"
#include "SphereTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "Spatial/DenseGrid3.h"
#include "PhysicsEngine/AggregateGeom.h"


namespace UE
{
	namespace Geometry
	{
		/**
		 * Convert FSphere3d to FKSphereElem
		 */
		inline void GetFKElement(const FSphere3d& Sphere, FKSphereElem& ElemInOut)
		{
			ElemInOut.Center = (FVector)Sphere.Center;
			ElemInOut.Radius = (float)Sphere.Radius;
		}

		/**
		 * Convert FOrientedBox3d to FKBoxElem
		 */
		inline void GetFKElement(const FOrientedBox3d& Box, FKBoxElem& BoxInOut)
		{
			BoxInOut.X = 2 * (float)Box.Extents.X;
			BoxInOut.Y = 2 * (float)Box.Extents.Y;
			BoxInOut.Z = 2 * (float)Box.Extents.Z;

			BoxInOut.Center = (FVector)Box.Frame.Origin;
			BoxInOut.Rotation = FRotator((FQuat)Box.Frame.Rotation);
		}

		/**
		 * Convert FCapsule3d to FKSphylElem
		 */
		inline void GetFKElement(const FCapsule3d& Capsule, FKSphylElem& CapsuleInOut)
		{
			FFrame3d CapsuleFrame(Capsule.Center(), Capsule.Direction());

			CapsuleInOut.Center = (FVector)CapsuleFrame.Origin;
			CapsuleInOut.Rotation = FRotator((FQuat)CapsuleFrame.Rotation);
			CapsuleInOut.Length = (float)Capsule.Length();		// Sphyl length is full length
			CapsuleInOut.Radius = (float)Capsule.Radius;
		}

		/**
		 * Convert FDynamicMesh3 to FKConvexElem
		 */
		inline void GetFKElement(const FDynamicMesh3& Mesh, FKConvexElem& ConvexInOut)
		{
			ConvexInOut.VertexData.Reset();

			for (int32 vid : Mesh.VertexIndicesItr())
			{
				FVector3d Pos = Mesh.GetVertex(vid);
				ConvexInOut.VertexData.Add((FVector)Pos);
			}

			// despite the name this actually computes the convex hull of the point set...
			ConvexInOut.UpdateElemBox();
		}

		/**
		* Create FKLevelSet from the given grid information
		*/
		inline void GetFKElement(const FTransform3d& GridTransform, const FDenseGrid3f& Grid, float CellSize, FKLevelSetElem& LevelSetOut)
		{
			const FVector3i InGridDims = Grid.GetDimensions();
			const FIntVector OutGridDims(InGridDims[0], InGridDims[1], InGridDims[2]);

			TArray<double> OutGridValues;
			OutGridValues.Init(0.0, OutGridDims[0] * OutGridDims[1] * OutGridDims[2]);

			for (int I = 0; I < OutGridDims[0]; ++I)
			{
				for (int J = 0; J < OutGridDims[1]; ++J)
				{
					for (int K = 0; K < OutGridDims[2]; ++K)
					{
						const int InBufferIndex = I + InGridDims[0] * (J + InGridDims[1] * K);
						const int OutBufferIndex = K + OutGridDims[2] * (J + OutGridDims[1] * I);
						OutGridValues[OutBufferIndex] = Grid[InBufferIndex];
					}
				}
			}

			FTransform3d ChaosTransform = GridTransform;
			ChaosTransform.AddToTranslation(-0.5 * CellSize * FVector::One());
			LevelSetOut.BuildLevelSet(ChaosTransform, OutGridValues, OutGridDims, CellSize);
		}


		/**
		 * Convert FKSphereElem to FSphereShape3d
		 */
		inline void GetShape(const FKSphereElem& SphereElem, FSphereShape3d& SphereShapeOut)
		{
			SphereShapeOut.Sphere.Center = (FVector3d)SphereElem.Center;
			SphereShapeOut.Sphere.Radius = (double)SphereElem.Radius;
		}

		/**
		 * Convert FKBoxElem to FBoxShape3d
		 */
		inline void GetShape(const FKBoxElem& BoxElem, FBoxShape3d& BoxShapeOut)
		{
			BoxShapeOut.Box.Frame.Origin = (FVector3d)BoxElem.Center;
			BoxShapeOut.Box.Frame.Rotation = (FQuaterniond)BoxElem.Rotation.Quaternion();
			BoxShapeOut.Box.Extents = FVector3d(BoxElem.X, BoxElem.Y, BoxElem.Z) / 2.0;
		}

		/**
		 * Convert FKSphylElem to FCapsuleShape3d
		 */
		inline void GetShape(const FKSphylElem& CapsuleElem, UE::Geometry::FCapsuleShape3d& CapsuleShapeOut)
		{
			FQuaterniond CapsuleRotation(CapsuleElem.Rotation.Quaternion());
			CapsuleShapeOut.Capsule.Segment.Center = (FVector3d)CapsuleElem.Center;
			CapsuleShapeOut.Capsule.Segment.Direction = CapsuleRotation.AxisZ();
			CapsuleShapeOut.Capsule.Segment.Extent = (double)CapsuleElem.Length / 2.0;
			CapsuleShapeOut.Capsule.Radius = (double)CapsuleElem.Radius;
		}


		/**
		 * Convert FKConvexElem to FDynamicMesh3 
		 */
		inline void GetShape(const FKConvexElem& ConvexElem, FDynamicMesh3& MeshOut)
		{
			MeshOut.Clear();
			for (int32 k = 0; k < ConvexElem.VertexData.Num(); ++k)
			{
				MeshOut.AppendVertex((FVector3d)ConvexElem.VertexData[k]);
			}
			int32 NumTriangles = ConvexElem.IndexData.Num() / 3;
			for (int32 k = 0; k < NumTriangles; ++k)
			{
				MeshOut.AppendTriangle( ConvexElem.IndexData[3*k], ConvexElem.IndexData[3*k+1], ConvexElem.IndexData[3*k+2]);
			}
		}

		/**
		 * Convert FKConvexElem to FConvexShape3d
		 */
		inline void GetShape(const FKConvexElem& ConvexElem, FConvexShape3d& ConvexShapeOut)
		{
			GetShape(ConvexElem, ConvexShapeOut.Mesh);
		}

		/**
		* Convert FKLevelSetElem to FLevelSetShape3d
		*/
		inline void GetShape(const FKLevelSetElem& LevelSetElem, FLevelSetShape3d& LevelSetShapeOut)
		{
			FTransform InGridTransform;
			TArray<double> InGridValues;
			FIntVector GridDims;
			float InGridCellSize;

			LevelSetElem.GetLevelSetData(InGridTransform, InGridValues, GridDims, InGridCellSize);

			if (!ensure(InGridValues.Num() == GridDims[0] * GridDims[1] * GridDims[2]))
			{
				return;
			}

			LevelSetShapeOut.GridTransform = InGridTransform;
			LevelSetShapeOut.GridTransform.AddToTranslation(0.5 * InGridCellSize * FVector::One());
			LevelSetShapeOut.CellSize = InGridCellSize;
			LevelSetShapeOut.Grid.Resize(GridDims[0], GridDims[1], GridDims[2]);

			for (int I = 0; I < GridDims[0]; ++I)
			{
				for (int J = 0; J < GridDims[1]; ++J)
				{
					for (int K = 0; K < GridDims[2]; ++K)
					{
						const int InBufferIndex = K + GridDims[2] * (J + GridDims[1] * I);
						const int OutBufferIndex = I + GridDims[0] * (J + GridDims[1] * K);
						LevelSetShapeOut.Grid[OutBufferIndex] = InGridValues[InBufferIndex];
					}
				}
			}
		}

		/**
		 * Convert FKAggregateGeom to FSimpleShapeSet3d
		 */
		inline void GetShapeSet(const FKAggregateGeom& AggGeom, FSimpleShapeSet3d& ShapeSetOut)
		{
			for (const FKBoxElem& BoxElem : AggGeom.BoxElems)
			{
				FBoxShape3d BoxShape;
				GetShape(BoxElem, BoxShape);
				ShapeSetOut.Boxes.Add(BoxShape);
			}

			for (const FKSphereElem& SphereElem : AggGeom.SphereElems)
			{
				FSphereShape3d SphereShape;
				GetShape(SphereElem, SphereShape);
				ShapeSetOut.Spheres.Add(SphereShape);
			}

			for (const FKSphylElem& CapsuleElem : AggGeom.SphylElems)
			{
				UE::Geometry::FCapsuleShape3d CapsuleShape;
				GetShape(CapsuleElem, CapsuleShape);
				ShapeSetOut.Capsules.Add(CapsuleShape);
			}

			for (const FKConvexElem& ConvexElem : AggGeom.ConvexElems)
			{
				FConvexShape3d ConvexShape;
				GetShape(ConvexElem, ConvexShape);
				ShapeSetOut.Convexes.Add(ConvexShape);
			}

			for (const FKLevelSetElem& LevelSetElem : AggGeom.LevelSetElems)
			{
				FLevelSetShape3d LevelSetShape;
				GetShape(LevelSetElem, LevelSetShape);
				ShapeSetOut.LevelSets.Add(LevelSetShape);
			}

		}


	}
}