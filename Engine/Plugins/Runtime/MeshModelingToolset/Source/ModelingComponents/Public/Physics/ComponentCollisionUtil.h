// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "TransformTypes.h"
#include "TransformSequence.h"
#include "DynamicMesh/DynamicMesh3.h"

class UBodySetup;
class UStaticMesh;
struct FKAggregateGeom;
class IInterface_CollisionDataProvider;

namespace UE
{
namespace Geometry
{

struct FSimpleShapeSet3d;

/**
 * Component/BodySetup collision settings (eg StaticMeshComponent) we might need to pass through the functions below
 */
struct FComponentCollisionSettings
{
	int32 CollisionTypeFlag = 0;		// this is ECollisionTraceFlag
	bool bIsGeneratedCollision = true;
};


enum EComponentCollisionSupportLevel
{
	ReadOnly = 0,
	ReadWrite = 1
};

/**
 * @return true if the component type supports collision settings
 */
MODELINGCOMPONENTS_API bool ComponentTypeSupportsCollision(
	const UPrimitiveComponent* Component, EComponentCollisionSupportLevel SupportLevel = EComponentCollisionSupportLevel::ReadWrite);


/**
 * @return current Component collision settings
 */
MODELINGCOMPONENTS_API FComponentCollisionSettings GetCollisionSettings(
	const UPrimitiveComponent* Component);

/**
 * Get current Component collision shapes 
 */
MODELINGCOMPONENTS_API bool GetCollisionShapes(const UPrimitiveComponent* Component, FKAggregateGeom& AggGeomOut);

/**
 * Get current Component collision shapes 
 */
MODELINGCOMPONENTS_API bool GetCollisionShapes(const UPrimitiveComponent* Component, FSimpleShapeSet3d& ShapeSetOut);



/**
 * Apply Transform to any Simple Collision geometry owned by Component.
 * Note that Nonuniform scaling support is very limited and will generally enlarge collision volumes.
 */
MODELINGCOMPONENTS_API bool TransformSimpleCollision(
	UPrimitiveComponent* Component,
	const FTransform3d& Transform);

/**
 * Replace existing Simple Collision geometry in Component with that defined by ShapeSet,
 * and update the Component/BodySetup collision settings
 */
MODELINGCOMPONENTS_API bool SetSimpleCollision(
	UPrimitiveComponent* Component,
	const FSimpleShapeSet3d* ShapeSet,
	FComponentCollisionSettings CollisionSettings = FComponentCollisionSettings() );


/**
 * Apply Transform to the existing Simple Collision geometry in Component and then append to to ShapeSetOut
 */
MODELINGCOMPONENTS_API bool AppendSimpleCollision(
	const UPrimitiveComponent* SourceComponent,
	FSimpleShapeSet3d* ShapeSetOut,
	const FTransform3d& Transform);

/**
 * Apply TransformSequence (in-order) to the existing Simple Collision geometry in Component and then append to to ShapeSetOut
 */
MODELINGCOMPONENTS_API bool AppendSimpleCollision(
	const UPrimitiveComponent* SourceComponent,
	FSimpleShapeSet3d* ShapeSetOut,
	const TArray<FTransform3d>& TransformSeqeuence);

/**
 * Replace existing Simple Collision geometry in BodySetup with that defined by NewGeometry,
 * and update the Component/BodySetup collision settings. StaticMesh argument allows
 * for necessary updates to it and any active UStaticMeshComponent that reference it
 */
MODELINGCOMPONENTS_API void UpdateSimpleCollision(
	UBodySetup* BodySetup,
	const FKAggregateGeom* NewGeometry,
	UStaticMesh* StaticMesh,
	FComponentCollisionSettings NewCollisionSettings = FComponentCollisionSettings() );

/**
 * @return BodySetup on the given SourceComponent, or nullptr if no BodySetup was found
 */
MODELINGCOMPONENTS_API const UBodySetup* GetBodySetup(const UPrimitiveComponent* SourceComponent);
/**
 * @return BodySetup on the given SourceComponent, or nullptr if no BodySetup was found
 */
MODELINGCOMPONENTS_API UBodySetup* GetBodySetup(UPrimitiveComponent* SourceComponent);

/**
 * Extract the simple collision geometry from AggGeom as meshes (ie spheres and capsules are tessellated) and
 * accumulate into MeshOut. TransformSequence is applied to the meshes as they are accumulated, and not to the meshes passed to the PerElementMeshCallback.
 * @param bSetToPerTriangleNormals if true and the mesh has a normals attribute overlay,  the mesh is set to face normals, otherwise averaged vertex normals
 * @param bInitializeConvexUVs if true convex hulls have their UVs initialized to per-face planar projections, otherwise no UVs are set
 * @param PerElementMeshCallback if provided, called with each element mesh before transforming/appending to MeshOut. The int parameter is the shape type (EAggCollisionShape::Type), 0=Sphere, 1=Box, 2=Capsule, 3=Convex
 * @param bApproximateLevelSetWithCubes if true, level sets will be approximated with cubes; otherwise, marching cubes will be used to triangulate level set surfaces
 * @param ExternalScale if provided, apply external component scaling according to the simple collision shapes, in limited way that UE does so (e.g., so spheres w/ non-uniform scaling will still remain spheres)
 */
MODELINGCOMPONENTS_API void ConvertSimpleCollisionToMeshes(
	const FKAggregateGeom& AggGeom,
	UE::Geometry::FDynamicMesh3& MeshOut,
	const FTransformSequence3d& TransformSequence,
	int32 SphereResolution = 16,
	bool bSetToPerTriangleNormals = false,
	bool bInitializeConvexUVs = false,
	TFunction<void(int, const FDynamicMesh3&)> PerElementMeshCallback = nullptr,
	bool bApproximateLevelSetWithCubes = true,
	FVector ExternalScale = FVector::OneVector);

// Settings to define how simple collision shapes are triangulated
struct FSimpleCollisionTriangulationSettings
{
	// Steps to use per side if bUseBoxSphere is true
	int32 BoxSphereStepsPerSide = 5;
	// Steps to use along circumference if bUseBoxSphere is false
	int32 LatLongSphereSteps = 16;
	// Steps to use along the capsule hemisphere arc
	int32 CapsuleHemisphereSteps = 5;
	// Steps to use radially on the capsule
	int32 CapsuleCircleSteps = 16;

	// Whether to use a box-sphere triangulation for spheres (otherwise, uses a lat/long sphere triangulation)
	bool bUseBoxSphere = false;

	// Whether to cheaply approximate level set meshes with voxel cubes. Otherwise the level set will be triangulated using marching cubes.
	bool bApproximateLevelSetWithCubes = true;

	// Init sphere-related settings from a single resolution parameter
	void InitFromSphereResolution(int32 SphereResolution)
	{
		bUseBoxSphere = false;
		LatLongSphereSteps = SphereResolution;
		CapsuleHemisphereSteps = SphereResolution / 4 + 1;
		CapsuleCircleSteps = SphereResolution;
	}
};

// Settings to define how attributes such as UVs and normals are set when converting simple collision shapes to dynamic meshes
struct FSimpleCollisionToMeshAttributeSettings
{
	// Whether to enable any attributes on the constructed dynamic meshes. If false, settings for normals and UVs below will be ignored.
	bool bEnableAttributes = true;

	// Whether to create faceted meshes where each triangle has a separate normal
	bool bSetToPerTriangleNormals = false;

	// Whether to generate UVs to convex hulls and level sets
	bool bInitializeConvexAndLevelSetUVs = false;

	FSimpleCollisionToMeshAttributeSettings(bool bEnableAttributes = true, bool bSetToPerTriangleNormals = false, bool bInitializeConvexAndLevelSetUVs = false)
		: bEnableAttributes(bEnableAttributes), bSetToPerTriangleNormals(bSetToPerTriangleNormals), bInitializeConvexAndLevelSetUVs(bInitializeConvexAndLevelSetUVs)
	{}
};

// Similar to ConvertSimpleCollisionToMeshes but without aggregating to an output mesh.
// @param PerElementMeshCallback	Called with each shape element and the corresponding mesh
MODELINGCOMPONENTS_API void ConvertSimpleCollisionToDynamicMeshes(
	const FKAggregateGeom& AggGeom, FVector ExternalScale,
	TFunctionRef<void(int32 Index, const FKShapeElem&, FDynamicMesh3& Mesh)> PerElementMeshCallback,
	const FSimpleCollisionTriangulationSettings& TriangulationSettings,
	const FSimpleCollisionToMeshAttributeSettings& MeshAttributeSettings = FSimpleCollisionToMeshAttributeSettings()
);

// Similar to ConvertSimpleCollisionToMeshes but without aggregating to an output mesh, and with the option to filter by Shape Element
// @param PerElementMeshCallback	Called with each shape element and the corresponding mesh
// @param IncludeElement			Filters Shape Elements: The PerElementMeshCallback will only be called if this function returns true
MODELINGCOMPONENTS_API void ConvertSimpleCollisionToDynamicMeshes(
	const FKAggregateGeom& AggGeom, FVector ExternalScale,
	TFunctionRef<void(int32 Index, const FKShapeElem&, FDynamicMesh3& Mesh)> PerElementMeshCallback,
	TFunctionRef<bool(const FKShapeElem&)> IncludeElement,
	const FSimpleCollisionTriangulationSettings& TriangulationSettings,
	const FSimpleCollisionToMeshAttributeSettings& MeshAttributeSettings = FSimpleCollisionToMeshAttributeSettings()
);

/**
 * Extract the complex collision geometry from CollisionDataProvider as a dynamic mesh.
 * TransformSequence is applied to the mesh.
 * @param bFoundMeshErrorsOut will be returned as true if potential non-manifold geometry was detected during construction (in that case some triangles will be disconnected)
 * @param bWeldEdges if true the mesh edges are welded
 * @param bSetToPerTriangleNormals if true and the mesh has a normals attribute overlay, the mesh is set to face normals, otherwise averaged vertex normals
 */
MODELINGCOMPONENTS_API bool ConvertComplexCollisionToMeshes(
	IInterface_CollisionDataProvider* CollisionDataProvider,
	UE::Geometry::FDynamicMesh3& MeshOut,
	const FTransformSequence3d& TransformSeqeuence,
	bool& bFoundMeshErrorsOut,
	bool bWeldEdges = true,
	bool bSetToPerTriangleNormals = false);

}  // end namespace Geometry
}  // end namespace UE
