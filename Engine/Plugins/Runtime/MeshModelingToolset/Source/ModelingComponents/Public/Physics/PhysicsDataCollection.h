// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShapeApproximation/SimpleShapeSet3.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/AggregateGeom.h"

#define UE_API MODELINGCOMPONENTS_API

class UActorComponent;
class UBodySetup;

/**
 * FPhysicsDataCollection holds onto physics-system data that is needed for various interactive tools and algorithms.
 * Currently this is split into two parts, pointers to the owning physics state, and collision geometry data.
 * 
 * The collision geometry data is stored in both the editable GeometryProcessing format and the 
 * physics-system representations (currently FKAggregateGeom but more are expected).
 * This class provides a high-level API for transferring geometry between these two representations.
 */
class FPhysicsDataCollection
{
public:
	//
	// External State Data
	//

	/** The Component this physics data came from / is for */
	TWeakObjectPtr<const UActorComponent> SourceComponent;
	/** The StaticMesh this physics data came from / is for */
	TWeakObjectPtr<const UStaticMesh> SourceStaticMesh;
	/** The BodySetup in use by the SourceComponent */
	TWeakObjectPtr<const UBodySetup> BodySetup;
	/** Scaling factor applied to the SourceComponent, which should be transferred to Collision Geometry in some cases */
	FVector ExternalScale3D = FVector::OneVector;


	//
	// Geometry data
	//

	/**
	 * Stores representation of Collision geometry
	 */
	UE::Geometry::FSimpleShapeSet3d Geometry;

	/**
	 * Collision geometry in Physics-system format. Not necessarily in sync with Geometry member.
	 */
	FKAggregateGeom AggGeom;


	/**
	 * Initialize from the given Component, and optionally initialize internal geometry members
	 */
	UE_API void InitializeFromComponent(const UActorComponent* Component, bool bInitializeAggGeom);

	/**
	 * Initialize from the given StaticMesh, and optionally initialize internal geometry members
	 */
	UE_API void InitializeFromStaticMesh(const UStaticMesh* StaticMesh, bool bInitializeAggGeom);

	/**
	 * Initialize 
	 */
	UE_API void InitializeFromExisting(const FPhysicsDataCollection& Other);

	/**
	 * replace our geometry data with that from another FPhysicsDataCollection
	 */
	UE_API void CopyGeometryFromExisting(const FPhysicsDataCollection& Other);


	/**
	 * Empty out our FKAggregateGeom
	 */
	UE_API void ClearAggregate();

	/**
	 * Populate our FKAggregateGeom from our FSimpleShapeSet3d
	 */
	UE_API void CopyGeometryToAggregate();

};

#undef UE_API
