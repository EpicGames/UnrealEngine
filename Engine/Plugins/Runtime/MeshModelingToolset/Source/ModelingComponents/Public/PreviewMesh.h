// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "InteractiveToolObjects.h"
#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "PreviewMesh.generated.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
struct FMeshDescription;

// predeclare tangents template
PREDECLARE_GEOMETRY(template<typename RealType> class TMeshTangents);


/**
 * UPreviewMesh internally spawns a APreviewMeshActor to hold the preview mesh object.
 * We use this AInternalToolFrameworkActor subclass so that we can identify such objects
 * at higher levels (for example to prevent them from being deleted in the Editor)
 */
UCLASS(MinimalAPI, Transient, NotPlaceable, Hidden, NotBlueprintable, NotBlueprintType)
class APreviewMeshActor : public AInternalToolFrameworkActor
{
	GENERATED_BODY()
private:
	APreviewMeshActor();
public:

};



/** 
 * UPreviewMesh is a utility object that spawns and owns a transient mesh object in the World.
 * This can be used to show live preview geometry during modeling operations.
 * Call CreateInWorld() to set it up, and Disconnect() to shut it down.
 * 
 * Currently implemented via an internal Actor that has a UDynamicMeshComponent root component,
 * with an AABBTree created/updated if FProperty bBuildSpatialDataStructure=true.
 * The Actor is destroyed on Disconnect().
 * 
 * The intention with UPreviewMesh is to provide a higher-level interface than the Component.
 * In future the internal Component may be replaced with another class (eg OctreeDynamicMeshComponent),
 * or automatically swap between the two, etc.
 * 
 * As a result direct access to the Actor/Component, or a non-const FDynamicMesh3, is intentionally not provided.
 * Wrapper functions are provided (or should be added) for necessary Actor/Component parameters.
 * To edit the mesh either a copy is done, or EditMesh()/ApplyChange() must be used.
 * These functions automatically update necessary internal data structures.
 * 
 */
UCLASS(MinimalAPI, Transient)
class UPreviewMesh : public UObject, public IMeshVertexCommandChangeTarget, public IMeshCommandChangeTarget, public IMeshReplacementCommandChangeTarget
{
	GENERATED_BODY()
public:
	UE_API UPreviewMesh();
	UE_API virtual ~UPreviewMesh();

	//
	// construction / destruction
	// 

	/**
	 * Create preview mesh in the World with the given transform
	 */
	UE_API void CreateInWorld(UWorld* World, const FTransform& WithTransform);

	/**
	 * Remove and destroy preview mesh
	 */
	UE_API void Disconnect();


	/**
	 * @return internal Actor created by this UPreviewMesh
	 */
	AActor* GetActor() const { return TemporaryParentActor; }

	/**
	 * @return internal Root Component of internal Actor
	 */
	UPrimitiveComponent* GetRootComponent() { return DynamicMeshComponent; }

	//
	// visualization parameters
	// 

	/**
	 * Enable/disable wireframe overlay rendering
	 */
	UE_API void EnableWireframe(bool bEnable);

	/**
	 * Enable/disable shadow rendering
	 */
	UE_API void SetShadowsEnabled(bool bEnable);

	/**
	 * Set material on the preview mesh
	 */
	UE_API void SetMaterial(UMaterialInterface* Material);

	/**
	 * Set material on the preview mesh
	 */
	UE_API void SetMaterial(int MaterialIndex, UMaterialInterface* Material);

	/**
	 * Set the entire material set on the preview mesh
	 */
	UE_API void SetMaterials(const TArray<UMaterialInterface*>& Materials);

	/**
	* Get number of materials in the preview mesh (base materials, i.e., not including override material)
	*/
	UE_API int32 GetNumMaterials() const;

	/**
	* Get material from the preview mesh
	*/
	UE_API UMaterialInterface* GetMaterial(int MaterialIndex = 0) const;

	/**
	* Get the entire materials array from the preview mesh. Appends to OutMaterials without clearing it.
	*/
	UE_API void GetMaterials(TArray<UMaterialInterface*>& OutMaterials) const;

	/**
	 * Set an override material for the preview mesh. This material will override all the given materials.
	 */
	UE_API void SetOverrideRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear the override material for the preview mesh.
	 */
	UE_API void ClearOverrideRenderMaterial();

	/**
	 * @return the actual material that will be used for rendering for the given MaterialIndex. Will return override material if set.
	 * 
	 */
	UE_API UMaterialInterface* GetActiveMaterial(int MaterialIndex = 0) const;


	/**
	 * Set an secondary material for the preview mesh. This material will be applied to secondary triangle buffer if enabled.
	 */
	UE_API void SetSecondaryRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear the secondary material for the preview mesh.
	 */
	UE_API void ClearSecondaryRenderMaterial();

	/**
	 * Set a override wireframe material for the preview mesh. This material will be applied when wireframe is enabled.
	 */
	UE_API void SetOverrideWireframeRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear the wireframe override material for the preview mesh.
	 */
	UE_API void ClearOverrideWireframeRenderMaterial();

	/**
	 * Set an override secondary wireframe material for the preview mesh. This material will be applied to seccondary triangle buffer when wireframe is enabled.
	 */
	UE_API void SetOverrideSecondaryWireframeRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear the wireframe override material for the preview mesh.
	 */
	UE_API void ClearOverrideSecondaryWireframeRenderMaterial();

	/**
	 * Enable secondary triangle buffers. The Secondary material will be applied to any triangles that pass TriangleFilterFunc.
	 * @param TriangleFilterFunc predicate used to identify secondary triangles
	 */
	UE_API void EnableSecondaryTriangleBuffers(TUniqueFunction<bool(const FDynamicMesh3*, int32)> TriangleFilterFunc);

	/**
	 * Disable secondary triangle buffers
	 */
	UE_API void DisableSecondaryTriangleBuffers();

	/**
	 * Show/Hide the secondary triangle buffers.
	 */
	UE_API void SetSecondaryBuffersVisibility(bool bSecondaryVisibility);

	/** 
	 * Call this after updating the secondary triangle sorting.
	 * This function will update the existing buffers if possible, without rebuilding entire RenderProxy.
	 */
	UE_API void FastNotifySecondaryTrianglesChanged();

	/**
	 * Set the tangents mode for the underlying component, if available. 
	 * Note that this function may need to be called before the mesh is initialized.
	 */
	UE_API void SetTangentsMode(EDynamicMeshComponentTangentsMode TangentsType);

	/**
	 * Calculate tangents for the underlying component.
	 * This will calculate and assign tangents for the preview mesh independent of the tangents mode.
	 * But if the tangents mode is set to AutoCalculated then it will try to use the auto calculated tangents.
	 * @return true if tangents were successfully calculated and assigned to the underlying mesh
	 */
	UE_API bool CalculateTangents();

	/**
	 * Get the current transform on the preview mesh
	 */
	UE_API FTransform GetTransform() const;

	/**
	 * Set the transform on the preview mesh
	 */
	UE_API void SetTransform(const FTransform& UseTransform);

	/**
	 * @return true if the preview mesh is visible
	 */
	UE_API bool IsVisible() const;

	/**
	 * Set visibility state of the preview mesh
	 */
	UE_API void SetVisible(bool bVisible);


	/** Render data update hint (values mirror EDynamicMeshComponentRenderUpdateMode) */
	enum class ERenderUpdateMode
	{
		/** Do not update render data */
		NoUpdate = 0,	
		/** Invalidate overlay of internal component, rebuilding all render data */
		FullUpdate = 1,
		/** Attempt to do partial update of render data if possible */
		FastUpdate = 2
	};

	/**
	 * Set the triangle color function for rendering / render data construction
	 */
	UE_API void SetTriangleColorFunction(TFunction<FColor(const FDynamicMesh3*, int)> TriangleColorFunc, ERenderUpdateMode UpdateMode = ERenderUpdateMode::FullUpdate);

	/**
	 * Clear the triangle color function for rendering / render data construction
	 */
	UE_API void ClearTriangleColorFunction(ERenderUpdateMode UpdateMode = ERenderUpdateMode::FullUpdate);





	//
	// Queries
	// 

	/**
	 * Test for ray intersection with the preview mesh.
	 * Requires that bBuildSpatialDataStructure = true
	 * @param TriangleFilterF optional filter function for triangles
	 * @return true if ray intersections mesh
	 * @warning returns false if preview is not visible
	 */
	UE_API bool TestRayIntersection(const FRay3d& WorldRay, TFunction<bool(int32 Tid)> TriangleFilterF = nullptr);

	/**
	 * Find ray intersection with the preview mesh.
	 * Requires that bBuildSpatialDataStructure = true
	 * @param WorldRay ray in world space
	 * @param HitOut output hit data (only certain members are initialized, see implementation)
	 * @param TriangleFilterF optional filter function for triangles
	 * @return true if ray intersections mesh
	 * @warning returns false if preview is not visible
	 */
	UE_API bool FindRayIntersection(const FRay3d& WorldRay, FHitResult& HitOut, TFunction<bool(int32 Tid)> TriangleFilterF = nullptr);


	/**
	 * Find nearest point on current mesh to given WorldPoint
	 * Requires that bBuildSpatialDataStructure = true unless bLinearSearch = true
	 * @param WorldPoint point in world space
	 * @param bLinearSearch test every triangle. On a mesh where only a few queries will be run, this is faster than building the spatial data structure.
	 * @return nearest point in world space
	 */
	UE_API FVector3d FindNearestPoint(const FVector3d& WorldPoint, bool bLinearSearch = false);



	//
	// Read access to internal mesh
	// 

	/**
	 * Clear the preview mesh
	 */
	UE_API void ClearPreview();

	/**
	 * Update the internal mesh by copying the given Mesh
	 * @param Mesh to copy.
	 * @param UpdateMode Type of rendering update required. Should be FullUpdate if topology changes, otherwise can be FastUpdate.
	 * @param ModifiedAttribs Only relevant in case of FastUpdate- determines which attributes actually changed.
	 */
	UE_API void UpdatePreview(const FDynamicMesh3* Mesh, ERenderUpdateMode UpdateMode = ERenderUpdateMode::FullUpdate, 
		EMeshRenderAttributeFlags ModifiedAttribs = EMeshRenderAttributeFlags::AllVertexAttribs);

	/**
	 * Update the internal mesh by moving in the given Mesh
	 * @param Mesh to move.
	 * @param UpdateMode Type of rendering update required. Should be FullUpdate if topology changes, otherwise can be FastUpdate.
	 * @param ModifiedAttribs Only relevant in case of FastUpdate- determines which attributes actually changed.
	 */
	UE_API void UpdatePreview(FDynamicMesh3&& Mesh, ERenderUpdateMode UpdateMode = ERenderUpdateMode::FullUpdate,
		EMeshRenderAttributeFlags ModifiedAttribs = EMeshRenderAttributeFlags::AllVertexAttribs);

	/**
	* @return pointer to the current FDynamicMesh used for preview  @todo deprecate this function, use GetMesh() instead
	*/
	const FDynamicMesh3* GetPreviewDynamicMesh() const { return GetMesh(); }

	/**
	* Read access to the internal mesh. This function will be deprecated/removed, use ProcessMesh() instead.
	* @return pointer to the current FDynamicMesh used for preview
	*/
	UE_API const FDynamicMesh3* GetMesh() const;


	/**
	 * Give external code direct read access to the internal FDynamicMesh3. 
	 * This should be used preferentially over GetMesh() / GetPreviewDynamicMesh()
	 */
	UE_API virtual void ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const;

	/**
	 * @return point to the current AABBTree used for preview spatial mesh, or nullptr if not available
	 * @warning this has to return non-const because of current FDynamicMeshAABBTree3 API, but you should not modify this!
	 */
	UE_API UE::Geometry::FDynamicMeshAABBTree3* GetSpatial();

	/**
	 * @return the current preview FDynamicMesh, and replace with a new empty mesh
	 */
	UE_API TUniquePtr<FDynamicMesh3> ExtractPreviewMesh() const;




	//
	// Edit access to internal mesh, and change-tracking/notification
	// 

	/**
	 * Replace mesh with new mesh
	 */
	UE_API void ReplaceMesh(const FDynamicMesh3& NewMesh);

	/**
	 * Replace mesh with new mesh
	 */
	UE_API void ReplaceMesh(FDynamicMesh3&& NewMesh);

	/**
	 * Apply EditFunc to the internal mesh and update internal data structures as necessary
	 */
	UE_API void EditMesh(TFunctionRef<void(FDynamicMesh3&)> EditFunc);

	/**
	 * Apply EditFunc to the internal mesh, and update spatial data structure if requested,
	 * but do not update/rebuild rendering data structures. NotifyDeferredEditOcurred() must be
	 * called to complete a deferred edit, this will update the rendering mesh.
	 * DeferredEditMesh can be called multiple times before NotifyDeferredEditCompleted() is called.
	 * @param EditFunc function that is applied to the internal mesh
	 * @param bRebuildSpatial if true, and internal spatial data structure is enabled, rebuild it for updated mesh
	 */
	UE_API void DeferredEditMesh(TFunctionRef<void(FDynamicMesh3&)> EditFunc, bool bRebuildSpatial);

	/**
	 * Notify that a DeferredEditMesh sequence is complete and cause update of rendering data structures.
	 * @param UpdateMode type of rendering update required for the applied mesh edits
	 * @param ModifiedAttribs which mesh attributes have been modified and need to be updated
	 * @param bRebuildSpatial if true, and internal spatial data structure is enabled, rebuild it for updated mesh
	 */
	UE_API void NotifyDeferredEditCompleted(ERenderUpdateMode UpdateMode, EMeshRenderAttributeFlags ModifiedAttribs, bool bRebuildSpatial);

	/**
	 * Notify that a deferred edit is completed and cause update of rendering data structures for modified Triangles.
	 * This can reduce the cost of mesh updates, but only if SetEnableRenderMeshDecomposition(true) has been called
	 * @param ModifiedAttribs which mesh attributes have been modified and need to be updated
	 */
	UE_API void NotifyRegionDeferredEditCompleted(const TArray<int32>& Triangles, EMeshRenderAttributeFlags ModifiedAttribs);
	
	/**
	 * Notify that a deferred edit is completed and cause update of rendering data structures for modified Vertices.
	 * This can reduce the cost of mesh updates, but only if SetEnableRenderMeshDecomposition(true) has been called
	 * @param ModifiedAttribs which mesh attributes have been modified and need to be updated
	 */
	UE_API void NotifyVertexRegionDeferredEditCompleted(const TArray<int32>& Vertices, EMeshRenderAttributeFlags ModifiedAttribs);

	/**
	 * Notify that a deferred edit is completed and cause update of rendering data structures for modified Triangles.
	 * This can reduce the cost of mesh updates, but only if SetEnableRenderMeshDecomposition(true) has been called
	 * @param ModifiedAttribs which mesh attributes have been modified and need to be updated
	 */
	UE_API void NotifyRegionDeferredEditCompleted(const TSet<int32>& Triangles, EMeshRenderAttributeFlags ModifiedAttribs);



	/**
	 * Apply EditFunc to the internal mesh and update internal data structures as necessary.
	 * EditFunc is required to notify the given FDynamicMeshChangeTracker about all mesh changes
	 * @return FMeshChange extracted from FDynamicMeshChangeTracker that represents mesh edit
	 */
	UE_API TUniquePtr<FMeshChange> TrackedEditMesh(TFunctionRef<void(FDynamicMesh3&, UE::Geometry::FDynamicMeshChangeTracker&)> EditFunc);

	/**
	 * Apply/Revert a vertex deformation change to the internal mesh (implements IMeshVertexCommandChangeTarget)
	 */
	UE_API virtual void ApplyChange(const FMeshVertexChange* Change, bool bRevert) override;

	/**
	 * Apply/Revert a general mesh change to the internal mesh   (implements IMeshCommandChangeTarget)
	 */
	UE_API virtual void ApplyChange(const FMeshChange* Change, bool bRevert) override;

	/**
	* Apply/Revert a general mesh change to the internal mesh   (implements IMeshReplacementCommandChangeTarget)
	*/
	UE_API virtual void ApplyChange(const FMeshReplacementChange* Change, bool bRevert) override;

	/** @return delegate that is broadcast whenever the internal mesh component is changed */
	UE_API FSimpleMulticastDelegate& GetOnMeshChanged();


	/**
	 * Force rebuild of internal spatial data structure. Can be used in context of DeferredEditMesh to rebuild spatial data
	 * structure w/o rebuilding render data
	 */
	UE_API void ForceRebuildSpatial();


	/**
	 * Enable automatically-computed decomposition of internal mesh into subregions when rendering (ie inside the Component).
	 * This allows for faster local updates via NotifyRegionDeferredEditCompleted() functions above.
	 * Decomposition will be automatically recomputed as necessary when internal mesh is modified via changes, edits, etc
	 */
	UE_API virtual void SetEnableRenderMeshDecomposition(bool bEnable);

	/** @return true if SetEnableRenderMeshDecomposition(true) has been called. */
	bool GetIsRenderMeshDecompositionEnabled() const { return bDecompositionEnabled; }



public:
	/** If true, we build a spatial data structure internally for the preview mesh, which allows for hit-testing */
	UPROPERTY()
	bool bBuildSpatialDataStructure;

protected:
	/** The temporary actor we create internally to own the preview mesh component */
	APreviewMeshActor* TemporaryParentActor = nullptr;

	/** This component is set as the root component of TemporaryParentActor */
	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = nullptr;

	/** Spatial data structure that is initialized if bBuildSpatialDataStructure = true when UpdatePreview() is called */
	UE::Geometry::FDynamicMeshAABBTree3 MeshAABBTree;

	/** If true, mesh will be chunked into multiple render buffers inside the DynamicMeshComponent */
	bool bDecompositionEnabled = false;

	/** Update chunk decomposition */
	UE_API void UpdateRenderMeshDecomposition();

	// This function is called internally on some changes, to let the path tracer know that this mesha/actor
	// has been modified in a way that will require invalidating the current path tracing result
	UE_API void NotifyWorldPathTracedOutputInvalidated();
};


#undef UE_API
