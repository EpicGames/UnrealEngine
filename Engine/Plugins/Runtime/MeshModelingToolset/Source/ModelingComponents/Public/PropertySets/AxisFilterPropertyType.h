// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AxisFilterPropertyType.generated.h"


USTRUCT(BlueprintType)
struct FModelingToolsAxisFilter
{
	GENERATED_BODY()

	/** X Axis */
	UPROPERTY(EditAnywhere, Category = AxisFilters, meta = (DisplayName = "X"))
	bool bAxisX = true;

	/** Y Axis */
	UPROPERTY(EditAnywhere, Category = AxisFilters, meta = (DisplayName = "Y"))
	bool bAxisY = true;

	/** Z Axis */
	UPROPERTY(EditAnywhere, Category = AxisFilters, meta = (DisplayName = "Z"))
	bool bAxisZ = true;


	// helper functions

	bool AnyAxisFiltered() const { return bAxisX == false || bAxisY == false || bAxisZ == false; }
};
