// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IndexTypes.h"
#include "ColorChannelFilterPropertyType.generated.h"


USTRUCT(BlueprintType)
struct FModelingToolsColorChannelFilter
{
	GENERATED_BODY()

	/** Red Channel */
	UPROPERTY(EditAnywhere, Category = ChannelFilters, meta = (DisplayName = "R"))
	bool bRed = true;

	/** Green Channel */
	UPROPERTY(EditAnywhere, Category = ChannelFilters, meta = (DisplayName = "G"))
	bool bGreen = true;

	/** Blue Channel */
	UPROPERTY(EditAnywhere, Category = ChannelFilters, meta = (DisplayName = "B"))
	bool bBlue = true;

	/** Alpha Channel */
	UPROPERTY(EditAnywhere, Category = ChannelFilters, meta = (DisplayName = "A"))
	bool bAlpha = false;

	// helper functions

	bool operator!=(const FModelingToolsColorChannelFilter& Other) const
	{
		return (bRed != Other.bRed) || (bGreen != Other.bGreen) || (bBlue != Other.bBlue) || (bAlpha != Other.bAlpha);
	}

	bool AnyChannelFiltered() const { return bRed == false || bGreen == false || bBlue == false || bAlpha == false; }

	UE::Geometry::FIndex4i AsFlags() const
	{
		return UE::Geometry::FIndex4i((bRed) ? 1 : 0, (bGreen) ? 1 : 0, (bBlue) ? 1 : 0, (bAlpha) ? 1 : 0);
	}
};
