// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "Engine/BlockingVolume.h"
#include "ModelingObjectsCreationAPI.h"
#include "CreateMeshObjectTypeProperties.generated.h"

#define UE_API MODELINGCOMPONENTS_API

struct FCreateMeshObjectParams;

/**
 * UCreateMeshObjectTypeProperties is a InteractiveTool PropertySet used to select
 * what type of object to create, in creation tools (ie StaticMesh, Volume, etc).
 */
UCLASS(MinimalAPI)
class UCreateMeshObjectTypeProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UE_API void InitializeDefault();
	UE_API void InitializeDefaultWithAuto();
	UE_API void Initialize(bool bEnableStaticMeshes = true, bool bEnableVolumes = false, bool bEnableDynamicMeshActor = false);

	/** Type of object to create */
	UPROPERTY(EditAnywhere, Category = OutputType, meta = (DisplayName = "Output Type", GetOptions = GetOutputTypeNamesFunc, NoResetToDefault))
	FString OutputType;

	/** Type of volume to create */
	UPROPERTY(EditAnywhere, Category = OutputType, meta = (EditCondition = "bShowVolumeList == true", EditConditionHides, HideEditConditionToggle) )
	TSubclassOf<class AVolume> VolumeType = ABlockingVolume::StaticClass();

	// This function returns a list that is shown to select OutputType
	UFUNCTION()
	UE_API const TArray<FString>& GetOutputTypeNamesFunc();

	// The list returned by GetOutputTypeNamesFunc()
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> OutputTypeNamesList;

	// control whether the VolumeType field is enabled
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowVolumeList = false;

	/** @return true if we should show this property set on a Tool, ie if there are options the user can actually change (false if only one possible output type that has no options) */
	UFUNCTION()
	UE_API bool ShouldShowPropertySet() const;

	/** Convert current OutputType selection into a type that can be passed to UModelingObjectsCreationAPI */
	UFUNCTION()
	UE_API ECreateObjectTypeHint GetCurrentCreateMeshType() const;

	// update visibility of properties based on current OutputType selection
	UE_API virtual void UpdatePropertyVisibility();

	/**
	 * Utility function to configure a FCreateMeshObjectParams based on current settings
	 *    - for StaticMesh, set ParamsOut.TypeHint
	 *    - for Volume, if in UE Editor, set ParamsOut.TypeHint and ParamsOut.TypeHintClass
	 * @return true if a type this function can handle was configured
	 */
	UE_API virtual bool ConfigureCreateMeshObjectParams(FCreateMeshObjectParams& ParamsOut) const;

	// constants used for different known types
	static UE_API const FString AutoIdentifier;
	static UE_API const FString StaticMeshIdentifier;
	static UE_API const FString VolumeIdentifier;
	static UE_API const FString DynamicMeshActorIdentifier;


	//
	// Public static values used to configure behavior of this class, set (eg) from Editor settings
	//
	static UE_API bool bEnableDynamicMeshActorSupport;
	static UE_API FString DefaultObjectTypeIdentifier;
};

#undef UE_API
