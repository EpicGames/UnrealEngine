// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"

#include "GeometrySelectionVisualizationProperties.generated.h"

#define UE_API MODELINGCOMPONENTS_API

//~Keep this synchronised with EGeometryElementType
UENUM()
enum class EGeometrySelectionElementType : uint8
{
	Vertex = 1,
	Edge = 2,
	Face = 4
};

//~Keep this synchronised with EGeometryTopologyType
UENUM()
enum class EGeometrySelectionTopologyType : uint8
{
	Triangle = 1,
	Polygroup = 2
};

UCLASS(MinimalAPI)
class UGeometrySelectionVisualizationProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

	static inline FColor DefaultTriangleROIBorderColor = FColor(240, 15, 240);
	static inline FColor DefaultFaceColor = FLinearColor(.95f, .05f, .05f).ToFColor(true);
	static inline FColor DefaultLineColor = FLinearColor(.0f,  .3f,  .95f).ToFColor(true);
	static inline FColor DefaultPointColor = DefaultLineColor;

public:

	/**
	 * Sets property watchers and materials.
	 */
	UE_API void Initialize(UInteractiveTool* Tool);






	// EditCondition properties, not user visible.  Configure these in your tool Setup function.

	/**
	 * This option should be enabled for tools in which the triangle color is reserved for some tool function.  The
	 * Generate PolyGroups tool, for example, uses triangle color to indicate the polygroup id.
	 * 
	 * If true show a user option to visualize the geometry selection as the border of the triangles in the selection
	 * ROI (region of influence).
	 */
	UPROPERTY(Transient, meta=(TransientToolProperty))
	bool bEnableShowTriangleROIBorder = false;

	/**
	 * This option should be enabled for tools that treat edge selections as if they were vertex selections containing
	 * all incident vertices.  See EditNormalsTool for an example.  For such tools seeing the vertex selection
	 * corresponding to the edges selection can be useful for understanding the output.
	 * 
	 * If true  edge selections can be visualized as a line set or as a set of points touched by the selection
	 * If false edge selections are always visualized as a line set
	 */
	UPROPERTY(Transient, meta=(TransientToolProperty))
	bool bEnableShowEdgeSelectionVertices = false;

	/**
	 * Geometry type of the geometry selection
	 */
	UPROPERTY(Transient, meta=(TransientToolProperty))
	EGeometrySelectionElementType SelectionElementType;

	/**
	 * Topology type of the geometry selection
	 */
	UPROPERTY(Transient, meta=(TransientToolProperty))
	EGeometrySelectionTopologyType SelectionTopologyType;






	// User-exposed properties

	/** Render the geometry selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", meta = (NoResetToDefault))
	bool bShowSelection = false;

	/** Render the geometry selection as the border of the triangles in the ROI (region of influence) */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "bEnableShowTriangleROIBorder",
		DisplayName = "Show Selection ROI"))
	bool bShowTriangleROIBorder = true;

	/** Show occluded parts of the selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay)
	bool bShowHidden = false;

	/** This tool treats edge selections as vertex selections. Enable this to show the edited vertices */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "SelectionElementType == EGeometrySelectionElementType::Edge && bEnableShowEdgeSelectionVertices"))
	bool bShowEdgeSelectionVertices = false;

	/** Line thickness used to render the geometry selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "SelectionElementType == EGeometrySelectionElementType::Edge || (bEnableShowTriangleROIBorder && bShowTriangleROIBorder)"))
	float LineThickness = 3.0f; //~Note: Thinner than in the viewport since in the tool the selection is not editable

	/** Point size used to render the geometry selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "SelectionElementType == EGeometrySelectionElementType::Vertex || (SelectionElementType == EGeometrySelectionElementType::Edge && bShowEdgeSelectionVertices && bEnableShowEdgeSelectionVertices)"))
	float PointSize = 5.f; //~Note: Smaller than in the viewport since in the tool the selection is not editable

	/** Depth bias used to slightly shift depth of points/lines */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (UIMin = -2.0, UIMax = 2.0, HideEditConditionToggle, EditConditionHides,
		EditCondition = "(bEnableShowTriangleROIBorder && bShowTriangleROIBorder) || (bShowSelection && SelectionElementType != EGeometrySelectionElementType::Face)"))
	float DepthBias = 0.2f; //~TODO Enable this for face selections as well

	//~The following color options all have the same display name because only one will be enabled

	/** Color used to render the geometry selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "false && SelectionElementType == EGeometrySelectionElementType::Face"),
		DisplayName = "Selection Color")
	FColor FaceColor = DefaultFaceColor; //~TODO Remove the false && from the Edit condition when the color of the faces can be set from the vertex color

	/** Color used to render the geometry selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "SelectionElementType == EGeometrySelectionElementType::Edge"),
		DisplayName = "Selection Color")
	FColor LineColor = DefaultLineColor;

	/** Color used to render the geometry selection */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "SelectionElementType == EGeometrySelectionElementType::Vertex"),
		DisplayName = "Selection Color")
	FColor PointColor = DefaultPointColor;

	/** Color used to render the geometry selection ROI */
	UPROPERTY(EditAnywhere, Category = "Selection Visualization", AdvancedDisplay, meta = (HideEditConditionToggle, EditConditionHides,
		EditCondition = "bEnableShowTriangleROIBorder"),
		DisplayName = "Selection ROI Color")
	FColor TriangleROIBorderColor = DefaultTriangleROIBorderColor;






	// Internal data/transient properties

	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> TriangleMaterial;

	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> LineMaterial;

	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> PointMaterial;

	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> TriangleMaterialShowingHidden;

	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> LineMaterialShowingHidden;

	UPROPERTY(Transient, meta=(TransientToolProperty))
	TObjectPtr<UMaterialInterface> PointMaterialShowingHidden;

	bool bVisualizationDirty = false;






	// Helper functions

	UE_API UMaterialInterface* GetPointMaterial() const;
	UE_API UMaterialInterface* GetLineMaterial() const;
	UE_API UMaterialInterface* GetFaceMaterial() const;

	UE_API bool ShowVertexSelectionPointSet() const;
	UE_API bool ShowEdgeSelectionLineSet() const;
	UE_API bool ShowFaceSelectionTriangleSet() const;
	UE_API bool ShowEdgeSelectionVerticesPointSet() const;
	UE_API bool ShowTriangleROIBorderLineSet() const;
};

#undef UE_API
