// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"

#include "OnAcceptProperties.generated.h"

#define UE_API MODELINGCOMPONENTS_API


class UInteractiveToolManager;
class AActor;


/** Options to handle source meshes */
UENUM()
enum class EHandleSourcesMethod : uint8
{
	/** Delete all input objects */
	DeleteSources UMETA(DisplayName = "Delete Inputs"),

	/** Hide all input objects */
	HideSources UMETA(DisplayName = "Hide Inputs"),

	/** Keep all input objects */
	KeepSources UMETA(DisplayName = "Keep Inputs"),

	/** Keep only the first input object and delete all other input objects */
	KeepFirstSource UMETA(DisplayName = "Keep First Input"),

	/** Keep only the last input object and delete all other input objects */
	KeepLastSource UMETA(DisplayName = "Keep Last Input")
};

// Base class for property settings for tools that create a new actor and need to decide what to do with the input objects.
UCLASS(MinimalAPI)
class UOnAcceptHandleSourcesPropertiesBase : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UE_API void ApplyMethod(const TArray<AActor*>& Actors, UInteractiveToolManager* ToolManager, const AActor* MustKeepActor = nullptr);

protected:
	virtual EHandleSourcesMethod GetHandleInputs() const
	{
		return EHandleSourcesMethod::KeepSources;
	}
};

// Specialization for property settings for tools that create a new actor and need to decide what to do with multiple input objects.
UCLASS(MinimalAPI)
class UOnAcceptHandleSourcesProperties : public UOnAcceptHandleSourcesPropertiesBase
{
	GENERATED_BODY()
public:

	/** Defines what to do with the input objects when accepting the tool results. */
	UPROPERTY(EditAnywhere, Category = OnToolAccept)
	EHandleSourcesMethod HandleInputs;

protected:
	virtual EHandleSourcesMethod GetHandleInputs() const override
	{
		return HandleInputs;
	}
};

// Specialization for property settings for tools that create a new actor and need to decide what to do with a single input object.
UCLASS(MinimalAPI)
class UOnAcceptHandleSourcesPropertiesSingle : public UOnAcceptHandleSourcesPropertiesBase
{
	GENERATED_BODY()
public:

	/** Defines what to do with the input object when accepting the tool results. */
	UPROPERTY(EditAnywhere, Category = OnToolAccept, meta = (ValidEnumValues = "DeleteSources, HideSources, KeepSources"))
	EHandleSourcesMethod HandleInputs;

protected:
	virtual EHandleSourcesMethod GetHandleInputs() const override
	{
		return HandleInputs;
	}
};

#undef UE_API
