// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "GeometryBase.h"
#include "Polygroups/PolygroupSet.h"
#include "PolygroupLayersProperties.generated.h"

#define UE_API MODELINGCOMPONENTS_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * Basic Tool Property Set that allows for selecting from a list of FNames (that we assume are Polygroup Layers)
 */
UCLASS(MinimalAPI)
class UPolygroupLayersProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Select PolyGroup layer to use. */
	UPROPERTY(EditAnywhere, Category = "PolyGroup Layer", meta = (DisplayName = "Active PolyGroup", GetOptions = GetGroupLayersFunc))
	FName ActiveGroupLayer = "Default";

	// Provides set of available group layers
	UFUNCTION()
	TArray<FString> GetGroupLayersFunc() { return GroupLayersList; }

	// internal list used to implement above
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> GroupLayersList;

	UE_API void InitializeGroupLayers(const FDynamicMesh3* Mesh);

	UE_API void InitializeGroupLayers(const TSet<FName>& LayerNames);

	// return true if any option other than "Default" is selected
	UE_API bool HasSelectedPolygroup() const;

	UE_API void SetSelectedFromPolygroupIndex(int32 Index);

	UE_API UE::Geometry::FPolygroupLayer GetSelectedLayer(const FDynamicMesh3& FromMesh);
	
	// whether triangle label attributes should be added to the group layers list
	UPROPERTY(meta = (TransientToolProperty))
	bool bSupportLabels = false;
};

#undef UE_API
