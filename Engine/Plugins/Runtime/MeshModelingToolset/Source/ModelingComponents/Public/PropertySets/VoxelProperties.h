// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveTool.h"

#include "BaseOps/VoxelBaseOp.h"

#include "VoxelProperties.generated.h"



UCLASS(MinimalAPI)
class UVoxelProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** The size of the geometry bounding box major axis measured in voxels */
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = "8", UIMax = "1024", ClampMin = "8", ClampMax = "1024"))
	int32 VoxelCount = 128;

	/** Automatically simplify the result of voxel-based meshes.*/
	UPROPERTY(EditAnywhere, Category = VoxelSettings)
	bool bAutoSimplify = false;

	/** Remove internal, occluded geometry */
	UPROPERTY(EditAnywhere, Category = VoxelSettings)
	bool bRemoveInternalSurfaces = false;

	/** The max error (as a multiple of the voxel size) to accept when simplifying the output */
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = ".1", UIMax = "5", ClampMin = ".001", ClampMax = "10", EditCondition = "bAutoSimplify == true"))
	double SimplifyMaxErrorFactor = 1.0;

	/** Automatically remove components smaller than this (to clean up any isolated floating bits) */
	UPROPERTY(EditAnywhere, Category = VoxelSettings, meta = (UIMin = "0.0", UIMax = "100", ClampMin = "0.0", ClampMax = "1000"))
	double CubeRootMinComponentVolume = 0.0;

	void SetPropertiesOnOp(UE::Geometry::FVoxelBaseOp& Op)
	{
		Op.InputVoxelCount = VoxelCount;
		Op.OutputVoxelCount = VoxelCount;
		Op.bAutoSimplify = bAutoSimplify;
		Op.bRemoveInternalSurfaces = bRemoveInternalSurfaces;
		Op.SimplifyMaxErrorFactor = SimplifyMaxErrorFactor;
		Op.MinComponentVolume = CubeRootMinComponentVolume * CubeRootMinComponentVolume * CubeRootMinComponentVolume;
	}
};
