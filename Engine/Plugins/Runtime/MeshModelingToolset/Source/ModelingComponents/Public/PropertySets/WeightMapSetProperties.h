// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "WeightMapSetProperties.generated.h"

#define UE_API MODELINGCOMPONENTS_API

struct FMeshDescription;


/**
 * Basic Tool Property Set that allows for selecting from a list of FNames (that we assume are Weight Maps)
 */
UCLASS(MinimalAPI)
class UWeightMapSetProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	/** Select vertex weight map. If configured, the weight map value will be sampled to modulate displacement intensity. */
	UPROPERTY(EditAnywhere, Category = WeightMap, meta = (GetOptions = GetWeightMapsFunc))
	FName WeightMap;

	// this function is called provide set of available weight maps
	UFUNCTION()
	UE_API TArray<FString> GetWeightMapsFunc();

	// internal list used to implement above
	UPROPERTY(meta = (TransientToolProperty))
	TArray<FString> WeightMapsList;

	UPROPERTY(EditAnywhere, Category = WeightMap)
	bool bInvertWeightMap = false;

	// set list of weightmap FNames explicitly. Adds "None" as first option.
	UE_API void InitializeWeightMaps(const TArray<FName>& WeightMapNames);

	// set list of weightmap FNames based on per-vertex float attributes in MeshDescription. Adds "None" as first option.
	UE_API void InitializeFromMesh(const FMeshDescription* Mesh);

	// return true if any option other than "None" is selected
	UE_API bool HasSelectedWeightMap() const;

	// set selected weightmap from its position in the WeightMapsList
	UE_API void SetSelectedFromWeightMapIndex(int32 Index);
};

#undef UE_API
