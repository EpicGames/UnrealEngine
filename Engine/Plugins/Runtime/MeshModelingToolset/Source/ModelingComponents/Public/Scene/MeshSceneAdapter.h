// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "BoxTypes.h"
#include "TransformSequence.h"
#include "DynamicMesh/DynamicMesh3.h"

#define UE_API MODELINGCOMPONENTS_API

class AActor;
class UActorComponent;
class UStaticMesh;

namespace UE
{
namespace Geometry
{

class FSparseDynamicOctree3;

/**
 * ESceneMeshType is used to indicate which type of Mesh a FMeshTypeContainer contains.
 */
enum class ESceneMeshType
{
	StaticMeshAsset,
	Unknown
};


/**
 * FMeshTypeContainer is a wrapper for an object that contains a unique Mesh of some kind,
 * which is used by an FActorChildMesh to represent that unique mesh. For example this could be 
 * a UStaticMesh asset, a FMeshDescription, a FDynamicMesh3, and so on.
 * (Currently only UStaticMesh is supported)
 */
struct FMeshTypeContainer
{
	/** raw pointer to the Mesh, used as a key to identify this mesh in various maps/etc */
	void* MeshPointer = nullptr;
	/** type of unique Mesh object this container contains */
	ESceneMeshType MeshType = ESceneMeshType::Unknown;

	/** @return key for this mesh */
	void* GetMeshKey() const { return MeshPointer; }

	/** @return the UStaticMesh this container contains, if this is a StaticMeshAsset container (otherwise nullptr) */
	UStaticMesh* GetStaticMesh() const
	{
		if (ensure(MeshType == ESceneMeshType::StaticMeshAsset))
		{
			return (UStaticMesh*)MeshPointer;
		}
		return nullptr;
	}
};



/**
 * Configuration for FMeshSceneAdapter::Build()
 */
struct FMeshSceneAdapterBuildOptions
{
	/** If true, various build log messages and statistics will be written to LogGeometry */
	bool bPrintDebugMessages = false;

	/** If true, find approximately-planar meshes with a main dimension below DesiredMinThickness and thicken them to DesiredMinThickness  */
	bool bThickenThinMeshes = false;
	/** Thickness used for bThickenThinMeshes processing */
	double DesiredMinThickness = 0.1;

	/** If true, tiny objects will be discarded from the mesh scene. This can improve performance when (eg) voxelizing huge scenes */
	bool bFilterTinyObjects = false;
	/** If bFilterTinyObjects is enabled, then a tiny object is identified by having a maximum (transformed) bounding-box dimension below this size */
	double TinyObjectBoxMaxDimension = 0.001;

	/** If true, only mesh sections that are assigned a valid surface material (ie MaterialDomain::MD_Surface) will be included/processed in the Mesh Scene. This filters out Decals, for example. */
	bool bOnlySurfaceMaterials = false;

	/** If true, AABBTree and Fast Winding data structures will be built for unique scene meshes */
	bool bBuildSpatialDataStructures = true;

	/** If true, UV queries will be supported on mesh wrappers */
	bool bEnableUVQueries = true;

	/** If true, Normals queries at hit locations will be supported */
	bool bEnableNormalsQueries = true;


	/** 
	* If true, the source meshes in StaticMesh Assets are ignored in favor of the render meshes.
	* This can speed up processing and reduce memory usage on scenes with large Nanite-enabled meshes,
	* but potentially with lower-quality results for operations querying the mesh scene.
	*/
	bool bIgnoreStaticMeshSourceData = false;
};


class IMeshSpatialWrapper;


/**
 * FMeshSceneRayHit is returned by various ray-intersection functions below
 */
struct FMeshSceneRayHit
{
	// world ray that was cast, stored here for convenience
	FRay3d Ray;
	// distance along ray that intersection occurred at
	double RayDistance = -1.0;
	// Actor that was hit, if available
	AActor* HitActor = nullptr;
	// Component that was hit, if available
	UActorComponent* HitComponent = nullptr;
	// Element Index on hit Component, if available (eg Instance Index on an InstancedStaticMesh)
	int32 HitComponentElementIndex = -1;
	// Triangle Index/ID on mesh of hit Component
	int32 HitMeshTriIndex = -1;
	// Triangle barycentric coordinates of the hit location.
	FVector3d HitMeshBaryCoords = FVector3d::Zero();

	// SpatialWrapper for the mesh geometry that was hit, if available. This is a pointer to data owned by the FMeshSceneAdapter that was queried
	IMeshSpatialWrapper* HitMeshSpatialWrapper = nullptr;

	// LocalToWorld Transform on the IMeshSpatialWrapper (for convenience)
	FTransformSequence3d LocalToWorld;
};


/**
 * Abstract interface to a spatial data structure for a mesh
 */
class IMeshSpatialWrapper
{
public:
	virtual ~IMeshSpatialWrapper() = default;

	FMeshTypeContainer SourceContainer;

	/** If possible, spatial data structure should defer construction until this function, which will be called off the game thread (in ParallelFor) */
	virtual bool Build(const FMeshSceneAdapterBuildOptions& BuildOptions) = 0;

	/*** @return triangle count for this mesh */
	virtual int32 GetTriangleCount() const = 0;

	/** Calculate bounding box for this Mesh */
	virtual FAxisAlignedBox3d GetWorldBounds(TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) = 0;

	/** Calculate the mesh winding number at the given Position. Must be callable in parallel from any thread.  */
	virtual double FastWindingNumber(const FVector3d& P, const FTransformSequence3d& LocalToWorldTransform) = 0;

	/** Find the nearest ray-intersection with the mesh. Must be callable in parallel from any thread. */
	virtual bool RayIntersection(const FRay3d& WorldRay, const FTransformSequence3d& LocalToWorldTransform, FMeshSceneRayHit& WorldHitResultOut) = 0;

	/** Collect a set of seed points from this Mesh, mapped through LocalToWorldFunc to world space. Must be callable in parallel from any thread. */
	virtual void CollectSeedPoints(TArray<FVector3d>& WorldPoints, TFunctionRef<FVector3d(const FVector3d&)> LocalToWorldFunc) = 0;

	/** apply ProcessFunc to each vertex in world space. Return false from ProcessFunc to terminate iteration. Returns true if iteration completed. */
	virtual bool ProcessVerticesInWorld(TFunctionRef<bool(const FVector3d&)> ProcessFunc, const FTransformSequence3d& LocalToWorldTransform) = 0;

	/** Append the geometry represented by this wrapper to the accumulated AppendTo mesh, under the given world transform */
	virtual void AppendMesh(FDynamicMesh3& AppendTo, const FTransformSequence3d& TransformSeq) = 0;

	/** @return true if the given index is a valid triangle */
	virtual bool IsTriangle(int32 TriId) const = 0;

	/** @return vertex indices of the given triangle */
	virtual FIndex3i GetTriangle(int32 TriId) const = 0;

	/** @return true if the mesh has normals */
	virtual bool HasNormals() const = 0;

	/** @return true if the mesh has UVs */
	virtual bool HasUVs(int UVLayer = 0) const = 0;

	/** Compute the barycentric interpolated normal for the given tri */
	virtual FVector3d TriBaryInterpolatePoint(int32 TriId, const FVector3d& BaryCoords) const = 0;

	/** Compute the barycentric interpolated normal for the given tri */
	virtual bool TriBaryInterpolateNormal(int32 TriId, const FVector3d& BaryCoords, FVector3f& NormalOut) const = 0;

	/** Compute the barycentric interpolated UV for the given tri */
	virtual bool TriBaryInterpolateUV(int32 TriId, const FVector3d& BaryCoords, int UVLayer, FVector2f& UVOut) const = 0;
};


/**
 * EActorMeshComponentType enum is used to determine which type of Component 
 * an FActorChildMesh represents.
 */
enum class EActorMeshComponentType
{
	StaticMesh,
	InstancedStaticMesh,
	HierarchicalInstancedStaticMesh,
	SplineStaticMesh,

	InternallyGeneratedComponent,

	Unknown
};


/**
 * FActorChildMesh represents a 3D Mesh attached to an Actor. This generally comes from a Component,
 * however in some cases a Component generates multiple FActorChildMesh (eg an InstancedStaticMeshComponent),
 * and potentially some Actors may store/represent a Mesh directly (no examples currently).
 */
struct FActorChildMesh
{
public:
	FActorChildMesh() {}
	FActorChildMesh(const FActorChildMesh&) = delete;		// must delete to allow TArray<TUniquePtr> in FActorAdapter
	FActorChildMesh(FActorChildMesh&&) = delete;

	/** the Component this Mesh was generated from, if there is one. */
	UActorComponent* SourceComponent = nullptr;
	/** Type of SourceComponent, if known */
	EActorMeshComponentType ComponentType = EActorMeshComponentType::Unknown;
	/** Index of this Mesh in the SourceComponent, if such an index exists (eg Instance Index in InstancedStaticMeshComponent) */
	int32 ComponentIndex = 0;

	/** Wrapper around the Mesh this FActorChildMesh refers to (eg from a StaticMeshAsset, etc) */
	FMeshTypeContainer MeshContainer;
	/** Local-to-World transformation of the Mesh in the MeshContainer */
	FTransformSequence3d WorldTransform;
	bool bIsNonUniformScaled = false;

	/** Spatial data structure that represents the Mesh in MeshContainer - assumption is this is owned externally */
	IMeshSpatialWrapper* MeshSpatial = nullptr;
};

/**
 * FActorAdapter is used by FMeshSceneAdapter to represent all the child info for an AActor.
 * This is primarily a list of FActorChildMesh, which represent the spatially-positioned meshes
 * of any child StaticMeshComponents or other mesh Components that can be identified and represented.
 * Note that ChildActorComponents will be flatted into the parent Actor.
 */
struct FActorAdapter
{
public:
	FActorAdapter() {}
	FActorAdapter(const FActorAdapter&) = delete;		// must delete to allow TArray<TUniquePtr> in FMeshSceneAdapter
	FActorAdapter(FActorAdapter&&) = delete;

	// the AActor this Adapter represents
	AActor* SourceActor = nullptr;
	// set of child Meshes with transforms
	TArray<TUniquePtr<FActorChildMesh>> ChildMeshes;
	// World-space bounds of this Actor (meshes)
	FAxisAlignedBox3d WorldBounds;
};



/**
 * FMeshSceneAdapter creates an internal representation of an Actor/Component/Asset hierarchy,
 * so that a minimal set of Mesh data structures can be constructed for the unique Meshes (generally Assets).
 * This allows queries against the Actor set to be computed without requiring mesh copies or 
 * duplicates of the mesh data structures (ie, saving memory, at the cost of some computation overhead).
 * 
 * Currently this builds an AABBTree and FastWindingTree for each unique Mesh, and
 * supports various queries based on those data structures.
 */
class FMeshSceneAdapter
{
public:
	virtual ~FMeshSceneAdapter() = default;

	FMeshSceneAdapter() {}
	FMeshSceneAdapter(const FMeshSceneAdapter&) = delete;		// must delete this due to TArray<TUniquePtr> member
	FMeshSceneAdapter(FMeshSceneAdapter&&) = delete;

	//
	// Mesh Scene Setup etc
	//

	/** 
	 * Add the given Actors to the Mesh Scene 
	 */
	UE_API void AddActors(const TArray<AActor*>& ActorsSetIn);

	/** 
	 * Add the given Components to the Mesh Scene.
	 * If multiple Components from the same Actor are passed in, duplicate FActorAdapters will be created
	 */
	UE_API void AddComponents(const TArray<UActorComponent*>& ComponentSetIn);

	/**
	 * Generate a new mesh that "caps" the mesh scene on the bottom. This can be used in cases where
	 * the geometry is open on the base, to fill in the hole, which allows things like mesh-solidification
	 * to work better. The base mesh is a polygon which can be optionally extruded.
	 * Currently the closing mesh is a convex hull, todo: implement a better option
	 * @param BaseHeight the height in world units from the bounding-box MinZ to consider as part of the "base".
	 * @param ExtrudeHeight height in world units to extrude the generated base. Positive is in +Z direction. If zero, an open-boundary polygon is generated instead.
	 */
	UE_API void GenerateBaseClosingMesh(double BaseHeight = 1.0, double ExtrudeHeight = 0.0);

	/**
	 * Build the Mesh Scene. This must be called after the functions above, and before any queries below can be run.
	 */
	UE_API void Build(const FMeshSceneAdapterBuildOptions& BuildOptions);

	/** 
	 * Precompute data structures that accelerate spatial evaluation queries. Precondition for calling FastWindingNumber() and FindNearestRayIntersection().
	 * This can be called after Build() has completed
	 */
	UE_API virtual void BuildSpatialEvaluationCache();

	/**
	 * Update the transforms on all the Components in the current Mesh Scene. Requires that Build() has been called.
	 * This will invalidate any existing SpatialEvaluationCache
	 * @param bRebuildSpatialCache if true, rebuild the SpatialEvaluationCache
	 */
	UE_API void FastUpdateTransforms(bool bRebuildSpatialCache);


	//
	// Mesh Scene Queries 	  
	//

	/** @return bounding box for the Actor set */
	UE_API virtual FAxisAlignedBox3d GetBoundingBox();

	/**
	 * Statistics about the Mesh Scene returned by GetGeometryStatistics()
	 */
	struct FStatistics
	{
		int64 UniqueMeshCount = 0;
		int64 UniqueMeshTriangleCount = 0;

		int64 InstanceMeshCount = 0;
		int64 InstanceMeshTriangleCount = 0;
	};

	/** @return bounding box for the Actor set */
	UE_API virtual void GetGeometryStatistics(FStatistics& StatsOut);

	/** @return a set of points on the surface of the meshes, can be used to initialize the MarchingCubes mesher */
	UE_API virtual void CollectMeshSeedPoints(TArray<FVector3d>& PointsOut);

	/** 
	 * @return FastWindingNumber computed at WorldPoint across all mesh Actors/Components 
	 * @param bFastEarlyOutIfPossible if true, then if any Mesh Scene Element returns a Winding Number of 1, the function returns immediately
	 */
	UE_API virtual double FastWindingNumber(const FVector3d& WorldPoint, bool bFastEarlyOutIfPossible = true);

	/** 
	 * Intersect the given Ray with the MeshScene and find the nearest mesh hit
	 * @param HitResultOut ray-intersection information will be returned here
	 * @return true if hit was found and HitResultOut is initialized
	 */
	UE_API virtual bool FindNearestRayIntersection(const FRay3d& WorldRay, FMeshSceneRayHit& HitResultOut);

	/** Append all instance triangles to a single mesh. May be **very** large. */
	UE_API virtual void GetAccumulatedMesh(FDynamicMesh3& AccumMesh);

	/** Get all world-space bounding boxes for all Scene Meshes */
	UE_API virtual void GetMeshBoundingBoxes(TArray<FAxisAlignedBox3d>& Bounds);

	/** @return world-space bounding box for the Scene Mesh of the specified Component and ComponentIndex. Requires that SpatialEvaluationCache is available. */
	UE_API virtual FAxisAlignedBox3d GetMeshBoundingBox(UActorComponent* Component, int32 ComponentIndex = -1);

	/**
	 * Run a custom query across all world-transformed scene mesh vertices. Requires that SpatialEvaluationCache is available.
	 * The internal loop is run as:
	 *  ParallelFor(SceneMesh, {
	 *    if ( MeshFilterFunc(SceneMesh) ) {
	 *      for ( FVector3d Vtx in Scene Mesh ) { PerVertexFunc(Vtx) }
	 *    }
	 *  }
	 * 
	 * @param InitializeFunc called with the total number of scene meshes. MeshIndex in later queries will be < NumMeshes  Return false to terminate query.
	 * @param MeshFilterFunc called once for each scene mesh, with unique MeshIndex per scene mesh. Return false to skip this scene mesh.
	 * @param PerVertexFunc called for each worldspace-transformed scene mesh vertex. Return false to terminate enumeration over this scene mesh.
	 */
	UE_API virtual void ParallelMeshVertexEnumeration(
		TFunctionRef<bool(int32 NumMeshes)> InitializeFunc,
		TFunctionRef<bool(int32 MeshIndex, AActor* SourceActor, const FActorChildMesh* ChildMeshInfo, const FAxisAlignedBox3d& WorldBounds)> MeshFilterFunc,
		TFunctionRef<bool(int32 MeshIndex, AActor* SourceActor, const FActorChildMesh* ChildMeshInfo, const FVector3d& WorldPos)> PerVertexFunc,
		bool bForceSingleThreaded = false
	);

	/**
	 * Run a custom query across all scene actor child meshes.
	 */
	UE_API virtual void ProcessActorChildMeshes(TFunctionRef<void(const FActorAdapter* ActorAdapter, const FActorChildMesh* ChildMesh)> ProcessFunc);

	/** Whether valid actors/components were added to the scene representation */
	UE_API bool IsValid() const;

protected:
	// top-level list of ActorAdapters, which represent each Actor and set of Components
	TArray<TUniquePtr<FActorAdapter>> SceneActors;

	/*
	 * The data structures used for supporting spatial queries in FMeshSceneAdapter are somewhat complex.
	 * Externally, the user adds Actors/Components, which results in FActorAdapter objects, each with list of
	 * FActorChildMesh. Each FActorChildMesh is a /reference/ to a mesh, not a mesh itself, as well as a transform.
	 * This allows unique mesh data to be shared across many usages, eg similar to StaticMeshActor / ISMComponent / Asset.
	 * (However there is no required 1-1 correspondence between FActorChildMesh and an actual Component, and in the case
	 *  of an InstancedStaticMeshComponent, there will be an FActorChildMesh for each Component)
	 *
	 * Each FActorChildMesh has a pointer to an IMeshSpatialWrapper, this is where unique mesh data, and any spatial data
	 * structures, will be stored. Each unique IMeshSpatialWrapper instance is owned by a FSpatialWrapperInfo, which
	 * are stored in the SpatialAdapters map below. Each FSpatialWrapperInfo also knows which parent FActorChildMesh 
	 * objects own it.
	 * 
	 * The main action inside a FSpatialWrapperInfo (ie unique mesh) is inside the IMeshSpatialWrapper implementation,
	 * this is where mesh data is unpacked and things like an AABBTree will be built if required. The top-level Build() 
	 * will do this work.
	 * 
	 * *However* evaluating spatial queries across a large set of actors / scene meshes would require testing each
	 * mesh instance one-by-one. To speed this up, BuildSpatialEvaluationCache() can be called to build an Octree
	 * across mesh instances. FSpatialCacheInfo is a separate representation of each unique (actor, chlidmesh, spatial, boundingbox)
	 * tuple, a list of these is built and then inserted into an Octree.
	 */


	// FSpatialWrapperInfo is a "unique mesh" that knows which parent objects have references to it
	struct FSpatialWrapperInfo
	{
		// identifier for mesh, including source pointer, mesh type, etc
		FMeshTypeContainer SourceContainer;
		// list of scene mesh instances that reference this unique mehs
		TArray<FActorChildMesh*> ParentMeshes;
		// number of non-uniform scales applied to this mesh by parent instances
		int32 NonUniformScaleCount = 0;
		// implementation of IMeshSpatialWrapper for this mesh, that provides spatial and other mesh data queries
		// *NOTE* that each FActorChildMesh in ParentMeshes has a pointer to this object, so if SpatialWrapper is
		// replaced, the FActorChildMesh objects must also be updated
		TUniquePtr<IMeshSpatialWrapper> SpatialWrapper;
	};

	// Unique set of spatial data structure query interfaces, one for each Mesh object, which is identified by void* pointer
	TMap<void*, TSharedPtr<FSpatialWrapperInfo>> SpatialAdapters;

	// Spatial data structure query interfaces for spline meshes
	TArray<TSharedPtr<FSpatialWrapperInfo>> SplineMeshSpatialAdapters;

	UE_API void InitializeSpatialWrappers(const TArray<FActorAdapter*>& NewItemsToProcess);

	UE_API void ForEachSpatialWrapper(TFunctionRef<void(TSharedPtr<FSpatialWrapperInfo>)> Func) const;

	bool bSceneIsAllSolids = false;

	UE_API void UpdateActorBounds(FActorAdapter& Actor);

	UE_API void Build_FullDecompose(const FMeshSceneAdapterBuildOptions& BuildOptions);


	// FSpatialCacheInfo represents a unique scene mesh with spatial data structure
	struct FSpatialCacheInfo
	{
		FActorAdapter* Actor;
		FActorChildMesh* ChildMesh;
		IMeshSpatialWrapper* Spatial;
		FAxisAlignedBox3d Bounds;
	};
	// list of all unique scene meshes that have spatial data structure available
	TArray<FSpatialCacheInfo> SortedSpatials;
	// Octree of elements in SortedSpatials list
	TSharedPtr<FSparseDynamicOctree3> Octree;
	FAxisAlignedBox3d CachedWorldBounds;
	bool bHaveSpatialEvaluationCache = false;
};


}
}

#undef UE_API
