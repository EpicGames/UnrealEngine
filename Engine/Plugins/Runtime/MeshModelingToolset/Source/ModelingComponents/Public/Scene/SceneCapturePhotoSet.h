// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorTypes.h"
#include "Image/SpatialPhotoSet.h"
#include "Scene/WorldRenderCapture.h"

#define UE_API MODELINGCOMPONENTS_API

class UWorld;
class AActor;
class UActorComponent;
struct FScopedSlowTask;

namespace UE
{
namespace Geometry
{

/**
 * FSceneCapturePhotoSet creates a set of render captures for a given World and set of Actors,
 * stored as a SpatialPhotoSet for each desired render buffer type. Currently the set of buffers are
 * defined by ERenderCaptureType:
 * 
 *		BaseColor
 *		Roughness
 *		Metallic
 *		Specular
 *		PackedMRS   (Metallic / Roughness / Specular)
 *		Emissive
 *		Opacity
 *		SubsurfaceColor
 *		WorldNormal
 *		DeviceDepth
 *
 * There are various efficiences possible by doing these captures as a group, rather
 * than doing each one individually.
 * 
 * One the capture set is computed, the ComputeSample() function can be used to 
 * call SpatialPhotoSet::ComputeSample() on each photo set, ie to estimate the
 * value of the different channels at a given 3D position/normal by raycasting
 * against the photo set. Again, it can be more efficient to do this on the
 * group, rather than each individually.
 * 
 */
class FSceneCapturePhotoSet
{
public:

	UE_API FSceneCapturePhotoSet();

	/**
	 * Set the target World and set of Actors/Components
	 * If World != this->World or Actors != this->Actors all existing photo sets are cleared
	 * Note: The caller must ensure Actors does not contain nullptr
	 */
	UE_API void SetCaptureSceneActors(UWorld* World, const TArray<AActor*>& Actors);
	UE_API void SetCaptureSceneComponents(UWorld* World, const TArray<UActorComponent*>& Components);
	UE_API void SetCaptureSceneActorsAndComponents(UWorld* World, const TArray<AActor*>& Actors, const TArray<UActorComponent*>& Components);

	UE_API UWorld* GetCaptureTargetWorld();

	/**
	 * Set the parameters positioning the virtual cameras used to capture the photo sets
	 * If SpatialParams != this->PhotoSetParams all existing photo sets are cleared (TODO clear on a more granular level)
	 */
	UE_API void SetSpatialPhotoParams(const TArray<FSpatialPhotoParams>& SpatialParams);
	UE_API const TArray<FSpatialPhotoParams>& GetSpatialPhotoParams() const;

	/**
	 * Disable all capture types. By default a standard set of capture types is enabled but, for performance reasons,
	 * you can use this function to disable them all and then only enable the ones you need.
	 */
	UE_API void DisableAllCaptureTypes();

	/**
	 * Enable/Disable a particular capture type
	 * If bEnabled == false the corresponding photo set will be cleared
	 */
	UE_API void SetCaptureTypeEnabled(ERenderCaptureType CaptureType, bool bEnabled);

	/** Status of a given capture type */
	enum class ECaptureTypeStatus {
		/** The capture type is not enabled */
		Disabled = 0,

		/** The capture type is enabled and the photo set is computed for the current scene capture parameters */
		Computed = 1,

		/**
		 * The capture type is enabled but the photo set is not computed for the current scene capture parameters.
		 * Note: This state is encountered if 1) a scene capture parameter which invalidates the photo set is changed
		 * but Compute has not yet been called or 2) the Compute function was cancelled before the photo set was computed
		 */
		Pending = 2
	};

	/** Returns the current status of the given capture type */
	UE_API ECaptureTypeStatus GetCaptureTypeStatus(ERenderCaptureType CaptureType) const;

	/** Status of the overall scene capture i.e., the combined status of all capture types */
	using FStatus = TRenderCaptureTypeData<ECaptureTypeStatus>;

	/** Returns the current status of all capture types */
	UE_API FStatus GetSceneCaptureStatus() const;

	/**
	 * Configure the given capture type
	 * If Config != the existing capture config the corresponding photo set will be cleared
	 */
	UE_API void SetCaptureConfig(ERenderCaptureType CaptureType, const FRenderCaptureConfig& Config);
	UE_API FRenderCaptureConfig GetCaptureConfig(ERenderCaptureType CaptureType) const;

	/**
	 * Render the configured scene, for the configured capture types from the configured viewpoints.
	 * This function the does most of the work.
	 */
	UE_API void Compute();

	/**
	 * Post-process the various PhotoSets after capture, to reduce memory usage and sampling cost.
	 */
	UE_API void OptimizePhotoSets();


	/**
	 * FSceneSample stores a full sample of all possible channels, some
	 * values may be default-values though
	 */
	struct FSceneSample
	{
		FRenderCaptureTypeFlags HaveValues;		// defines which channels have non-default values
		FVector3f BaseColor;
		float Roughness;
		float Specular;
		float Metallic;
		FVector3f Emissive;
		FVector3f WorldNormal;
		float DeviceDepth;
		float Opacity;
		FVector3f SubsurfaceColor;

		UE_API FSceneSample();

		/** @return value for the given captured channel, or default value */
		UE_API FVector3f GetValue3f(ERenderCaptureType CaptureType) const;
		UE_API FVector4f GetValue4f(ERenderCaptureType CaptureType) const;
	};


	/**
	 * Sample the requested SampleChannels from the available PhotoSets to determine
	 * values at the given 3D Position/Normal. This calls TSpatialPhotoSet::ComputeSample()
	 * internally, see that function for more details.
	 * @param DefaultsInResultsOut this value is passed in by caller with suitable Default values, and returned with any available computed values updated
	 */
	UE_API bool ComputeSample(
		const FRenderCaptureTypeFlags& SampleChannels,
		const FVector3d& Position,
		const FVector3d& Normal,
		TFunctionRef<bool(const FVector3d&, const FVector3d&)> VisibilityFunction,
		FSceneSample& DefaultsInResultsOut) const;

	/**
	 * If ValidSampleDepthThreshold >  0 then the VisibilityFunction and the DeviceDepthPhotoSet will be used to determine
	 * sample validity. Using DeviceDepthPhotoSet will mitigate artefacts caused by samples which cannot be determined
	 * invalid by querying only the VisibilityFunction.
	 * If ValidSampleDepthThreshold <= 0 only VisibilityFunction will be used to determine sample validity
	 */
	UE_API bool ComputeSampleLocation(
		const FVector3d& Position,
		const FVector3d& Normal,
		float ValidSampleDepthThreshold,
		TFunctionRef<bool(const FVector3d&, const FVector3d&)> VisibilityFunction,
		int& PhotoIndex,
		FVector2d& PhotoCoords) const;

	/**
	 * @returns the value of the nearest pixel in the given photo for the given CaptureType
	 * Note: This function does not interpolate because this causes artefacts; two adjacent pixels on a photo can be
	 * from parts of the scene which are very far from each other so interpolating the values when makes no sense.
	 * Texture filtering is accomplished by the baking framework where it can use correctly localized information.
	 */
	template <ERenderCaptureType CaptureType>
	FVector4f ComputeSampleNearest(
		const int& PhotoIndex,
		const FVector2d& PhotoCoords,
		const FSceneSample& DefaultSample) const;

	/**
	 * FSceneSamples stores samples corresponding to pixels in the DeviceDepth photoset where the values are strictly
	 * within the viewing frustum, which corresponds to a camera ray intersecting an actor in VisibleActors. All non-null
	 * containers with Computed capture status will be filled by GetSceneSamples() and will have the same .Num() counts
	 */
	struct FSceneSamples
	{
		// These are world-space point, normal or oriented point samples
		// The points are computed from the DeviceDepth channel and normals from the WorldNormal channel
		// Coordinate frames are located at the world points with Z axis aligned with world normals and arbitrary X/Y axes
		TArray<FVector3f>* WorldPoint = nullptr;
		TArray<FVector3f>* WorldNormal = nullptr;
		TArray<FFrame3f>*  WorldOrientedPoints = nullptr;

		// These are samples of the corresponding capture channels
		TArray<float>* Metallic = nullptr;
		TArray<float>* Roughness = nullptr;
		TArray<float>* Specular = nullptr;
		TArray<FVector3f>* PackedMRS = nullptr;
		TArray<FVector3f>* Emissive = nullptr;
		TArray<FVector3f>* BaseColor = nullptr;
		TArray<FVector3f>* SubsurfaceColor = nullptr;
		TArray<float>* Opacity = nullptr;
	};

	/**
	 * Fills in the non-null arrays in OutSamples if the status of the needed captures is Computed
	 */
	UE_API void GetSceneSamples(FSceneSamples& OutSamples);

	const FSpatialPhotoSet3f& GetBaseColorPhotoSet() { return BaseColorPhotoSet; }
	const FSpatialPhotoSet1f& GetRoughnessPhotoSet() { return RoughnessPhotoSet; }
	const FSpatialPhotoSet1f& GetSpecularPhotoSet() { return SpecularPhotoSet; }
	const FSpatialPhotoSet1f& GetMetallicPhotoSet() { return MetallicPhotoSet; }
	const FSpatialPhotoSet3f& GetPackedMRSPhotoSet() { return PackedMRSPhotoSet; }
	const FSpatialPhotoSet3f& GetWorldNormalPhotoSet() { return WorldNormalPhotoSet; }
	const FSpatialPhotoSet3f& GetEmissivePhotoSet() { return EmissivePhotoSet; }
	const FSpatialPhotoSet1f& GetOpacityPhotoSet() { return OpacityPhotoSet; }
	const FSpatialPhotoSet3f& GetSubsurfaceColorPhotoSet() { return SubsurfaceColorPhotoSet; }
	const FSpatialPhotoSet1f& GetDeviceDepthPhotoSet() { return DeviceDepthPhotoSet; }

	/**
	 * Enable debug image writing. All captured images will be written to <Project>/Intermediate/<FolderName>.
	 * If FolderName is not specified, "SceneCapturePhotoSet" is used by default.
	 * See FWorldRenderCapture::SetEnableWriteDebugImage() for more details
	 */
	UE_API void SetEnableWriteDebugImages(bool bEnable, FString FolderName = FString());

	/**
	 * If enabled, any Component Scene Proxies in the level that are not meant to be included in
	 * the capture (ie not added via SetCaptureSceneActors), will be unregistered to hide them.
	 * This is generally not necessary, and disabled by default, but in some cases the Renderer
	 * may not be able to fully exclude the effects of an object via hidden/visible flags.
	 */
	UE_API void SetEnableVisibilityByUnregisterMode(bool bEnable);

	void SetAllowCancel(bool bAllowCancelIn)
	{
		bAllowCancel = bAllowCancelIn;
	}

	bool Cancelled() const
	{
		return bWasCancelled;
	}

protected:
	UWorld* TargetWorld = nullptr;
	TArray<AActor*> VisibleActors;
	TArray<UActorComponent*> VisibleComponents;

	bool bEnforceVisibilityViaUnregister = false;

	FStatus PhotoSetStatus;

	FSpatialPhotoSet3f BaseColorPhotoSet;
	FSpatialPhotoSet1f RoughnessPhotoSet;
	FSpatialPhotoSet1f SpecularPhotoSet;
	FSpatialPhotoSet1f MetallicPhotoSet;
	FSpatialPhotoSet3f PackedMRSPhotoSet;
	FSpatialPhotoSet3f WorldNormalPhotoSet;
	FSpatialPhotoSet3f EmissivePhotoSet;
	FSpatialPhotoSet1f OpacityPhotoSet;
	FSpatialPhotoSet3f SubsurfaceColorPhotoSet;
	FSpatialPhotoSet1f DeviceDepthPhotoSet;

	TRenderCaptureTypeData<FRenderCaptureConfig> RenderCaptureConfig;

	TArray<FSpatialPhotoParams> PhotoSetParams;

	bool bWriteDebugImages = false;
	FString DebugImagesFolderName = TEXT("SceneCapturePhotoSet");

	bool bAllowCancel = false;
	bool bWasCancelled = false;

private:

	UE_API FSpatialPhotoSet1f& GetPhotoSet1f(ERenderCaptureType CaptureType);

	UE_API FSpatialPhotoSet3f& GetPhotoSet3f(ERenderCaptureType CaptureType);

	UE_API void EmptyPhotoSet(ERenderCaptureType CaptureType);

	UE_API void EmptyAllPhotoSets();
};


template <ERenderCaptureType CaptureType>
FVector4f FSceneCapturePhotoSet::ComputeSampleNearest(
	const int& PhotoIndex,
	const FVector2d& PhotoCoords,
	const FSceneSample& DefaultSample) const
{
	if constexpr (CaptureType == ERenderCaptureType::BaseColor)
	{
		FVector3f BaseColor = BaseColorPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.BaseColor);
		return FVector4f(BaseColor, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::Roughness)
	{
		float Roughness = RoughnessPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.Roughness);
		return FVector4f(Roughness, Roughness, Roughness, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::Specular)
	{
		float Specular = SpecularPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.Specular);
		return FVector4f(Specular, Specular, Specular, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::Metallic)
	{
		float Metallic = MetallicPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.Metallic);
		return FVector4f(Metallic, Metallic, Metallic, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::CombinedMRS)
	{
		FVector3f MRSValue(DefaultSample.Metallic, DefaultSample.Roughness, DefaultSample.Specular);
		MRSValue = PackedMRSPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, MRSValue);
		return FVector4f(MRSValue, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::Emissive)
	{
		FVector3f Emissive = EmissivePhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.Emissive);
		return FVector4f(Emissive, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::Opacity)
	{
		float Opacity = OpacityPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.Opacity);
		return FVector4f(Opacity, Opacity, Opacity, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::SubsurfaceColor)
	{
		FVector3f SubsurfaceColor = SubsurfaceColorPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.SubsurfaceColor);
		return FVector4f(SubsurfaceColor, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::WorldNormal)
	{
		FVector3f WorldNormal = WorldNormalPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.WorldNormal);
		return FVector4f(WorldNormal, 1.f);
	}

	else if constexpr (CaptureType == ERenderCaptureType::DeviceDepth)
	{
		float Depth = DeviceDepthPhotoSet.ComputeSampleNearest(PhotoIndex, PhotoCoords, DefaultSample.DeviceDepth);
		return FVector4f(Depth, Depth, Depth, 1.f);
	}

	else
	{
		ensure(false);
		return FVector4f::Zero();
	}
}

MODELINGCOMPONENTS_API
TArray<FSpatialPhotoParams> ComputeStandardExteriorSpatialPhotoParameters(
	UWorld* Unused,
	const TArray<AActor*>& Actors,
	const TArray<UActorComponent*>& Components,
	FImageDimensions PhotoDimensions,
	double HorizontalFOVDegrees,
	double NearPlaneDist,
	bool bFaces,
	bool bUpperCorners,
	bool bLowerCorners,
	bool bUpperEdges,
	bool bSideEdges);

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
