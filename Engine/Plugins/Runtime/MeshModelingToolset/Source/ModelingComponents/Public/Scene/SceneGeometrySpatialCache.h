// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Spatial/SpatialInterfaces.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/ColliderMesh.h"
#include "Spatial/SparseDynamicOctree3.h"
#include "SceneQueries/SceneSnappingManager.h"		// only needed for FSceneQueryVisibilityFilter, maybe can wrap this?

// need these for TWeakObjectPtr
#include "Components/BrushComponent.h"
#include "Components/DynamicMeshComponent.h"

#define UE_API MODELINGCOMPONENTS_API

class AActor;
class UPrimitiveComponent;


namespace UE
{
namespace Geometry
{

enum class EGeometryPointType
{
	Triangle = 0,
	Edge = 1,
	Vertex = 2
};

struct FSceneGeometryPoint
{
	// world ray that was cast, stored here for convenience
	FRay3d Ray;
	// distance along ray that point occurs at
	double RayDistance = TNumericLimits<float>::Max();		// float so that square-distance doesn't overflow

	// Actor that was hit, if available
	AActor* Actor = nullptr;
	// Component that was hit, if available
	UPrimitiveComponent* Component = nullptr;
	// Element Index on hit Component, if available (eg Instance Index on an InstancedStaticMesh)
	int32 ComponentElementIndex = -1;
	// Index/ID on geometry of hit Component
	int32 GeometryIndex = -1;
	// Geometry type
	EGeometryPointType GeometryType = EGeometryPointType::Triangle;
	// Barycentric coordinates of the point, if that is applicable
	FVector3d PointBaryCoords = FVector3d::Zero();
	// Component-space point
	FVector3d LocalPoint = FVector3d::Zero();
	// World-space point
	FVector3d WorldPoint = FVector3d::Zero();

	// todo: normals...
};



class ISceneGeometrySpatial
{
public:
	ISceneGeometrySpatial();
	virtual ~ISceneGeometrySpatial() = default;

	int32 GetID() const { return SpatialID; }

	virtual bool IsValid() = 0;
	virtual bool IsVisible() = 0;

	virtual UPrimitiveComponent* GetComponent() = 0;

	virtual FAxisAlignedBox3d GetWorldBounds() = 0;
	virtual bool FindNearestHit(
		const FRay3d& WorldRay, 
		FSceneGeometryPoint& HitResultOut,
		double MaxDistance = TNumericLimits<double>::Max() ) = 0;
	virtual bool GetGeometry(EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C) = 0;

	virtual void OnGeometryModified(bool bDeferSpatialRebuild) = 0;
	virtual void OnTransformModified() = 0;
	
private:
	int32 SpatialID = 0;

	static std::atomic<int32> UniqueIDGenerator;
};




struct FSceneGeometryID
{
	int32 UniqueID;
	FSceneGeometryID() { UniqueID = 0; }
	FSceneGeometryID(int32 ID) { UniqueID = ID; }
	bool operator==(const FSceneGeometryID& Other) const { return UniqueID == Other.UniqueID; }
};
inline uint32 GetTypeHash(const FSceneGeometryID& Identifier)
{
	return ::GetTypeHash(Identifier.UniqueID);
}




class FSceneGeometrySpatialCache
{
public:

public:
	UE_API FSceneGeometrySpatialCache();
	UE_API virtual ~FSceneGeometrySpatialCache();

	UE_API virtual bool EnableComponentTracking(UPrimitiveComponent* Component, FSceneGeometryID& IdentifierOut);

	UE_API virtual void DisableComponentTracking(UPrimitiveComponent* Component);
	UE_API virtual void DisableGeometryTracking(FSceneGeometryID& IdentifierOut);

	UE_API virtual bool HaveCacheForComponent(UPrimitiveComponent* Component);


	UE_API virtual bool NotifyTransformUpdate(UPrimitiveComponent* Component);
	UE_API virtual bool NotifyGeometryUpdate(UPrimitiveComponent* Component, bool bDeferRebuild);

	UE_API virtual bool FindNearestHit(
		const FRay3d& WorldRay,
		FSceneGeometryPoint& HitResultOut,
		FSceneGeometryID& HitIdentifier,
		const FSceneQueryVisibilityFilter* VisibilityFilter = nullptr,
		double MaxDistance = TNumericLimits<double>::Max());

	UE_API virtual bool GetGeometry(FSceneGeometryID Identifier, EGeometryPointType GeometryType, int32 GeometryIndex, bool bWorldSpace, FVector3d& A, FVector3d& B, FVector3d& C);

protected:
	int32 UniqueIDGenerator = 0;

	TMap<UPrimitiveComponent*, FSceneGeometryID> ComponentMap;
	TMap<FSceneGeometryID, TSharedPtr<ISceneGeometrySpatial>> Spatials;
	FSparseDynamicOctree3 BoundsOctree;
};



}
}



#undef UE_API
