// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"
#include "SceneTypes.h"		// FPrimitiveComponentId
#include "Image/SpatialPhotoSet.h"

#define UE_API MODELINGCOMPONENTS_API

class UTextureRenderTarget2D;
class AActor;
class UPrimitiveComponent;

namespace UE
{
namespace Geometry
{

/**
 * ERenderCaptureType defines which type of render buffer should be captured.
 */
enum class ERenderCaptureType
{
	BaseColor = 1,
	Roughness = 2,
	Metallic = 4,
	Specular = 8,
	Emissive = 16,
	WorldNormal = 32,
	DeviceDepth = 64,
	CombinedMRS = 128,
	Opacity = 256,
	SubsurfaceColor = 512
};

// Call the given Function for each capture type/channel
template <typename FunctionType>
void ForEachCaptureType(const FunctionType& Function)
{
	Function(ERenderCaptureType::BaseColor);
	Function(ERenderCaptureType::Roughness);
	Function(ERenderCaptureType::Specular);
	Function(ERenderCaptureType::Metallic);
	Function(ERenderCaptureType::CombinedMRS);
	Function(ERenderCaptureType::WorldNormal);
	Function(ERenderCaptureType::Emissive);
	Function(ERenderCaptureType::Opacity);
	Function(ERenderCaptureType::SubsurfaceColor);
	Function(ERenderCaptureType::DeviceDepth);
}

struct FRenderCaptureConfig
{
	// You might want to disable this if you're using FMeshMapBaker because it supports multi-sampling in a way which
	// will avoid blending pixels at different depths. This option is ignored for ERenderCaptureType::DeviceDepth
	bool bAntiAliasing = true;

	bool operator==(const FRenderCaptureConfig& Other) const
	{
		return bAntiAliasing == Other.bAntiAliasing;
	}

	bool operator!=(const FRenderCaptureConfig& Other) const
	{
		return !operator==(Other);
	}
};

FRenderCaptureConfig MODELINGCOMPONENTS_API GetDefaultRenderCaptureConfig(ERenderCaptureType CaptureType);

/**
 * FRenderCaptureTypeFlags is a set of per-capture-type booleans
 */
struct FRenderCaptureTypeFlags
{
	bool bBaseColor = false;
	bool bRoughness = false;
	bool bMetallic = false;
	bool bSpecular = false;
	bool bEmissive = false;
	bool bWorldNormal = false;
	bool bCombinedMRS = false;
	bool bDeviceDepth = false;
	bool bOpacity = false;
	bool bSubsurfaceColor = false;

	/** @return FRenderCaptureTypeFlags with all types enabled/true */
	static UE_API FRenderCaptureTypeFlags All(bool bCombinedMRS = false);

	/** @return FRenderCaptureTypeFlags with all types disabled/false */
	static UE_API FRenderCaptureTypeFlags None();

	/** @return FRenderCaptureTypeFlags with only BaseColor enabled/true */
	static UE_API FRenderCaptureTypeFlags BaseColor();

	/** @return FRenderCaptureTypeFlags with only WorldNormal enabled/true */
	static UE_API FRenderCaptureTypeFlags WorldNormal();

	/** @return FRenderCaptureTypeFlags with a single capture type enabled/true */
	static UE_API FRenderCaptureTypeFlags Single(ERenderCaptureType CaptureType);

	/** Set the indicated CaptureType to enabled */
	UE_API void SetEnabled(ERenderCaptureType CaptureType, bool bEnabled);

	/** @return mutable flag corresponding to the given CaptureType */
	UE_API bool& operator[](ERenderCaptureType CaptureType);

	/** @return constant flag corresponding to the given CaptureType */
	UE_API const bool& operator[](ERenderCaptureType CaptureType) const;

	/** @return true if the flags for this object match the Other object and false otherwise */
	UE_API bool operator==(const FRenderCaptureTypeFlags& Other) const;
	UE_API bool operator!=(const FRenderCaptureTypeFlags& Other) const;
};

template <typename T>
struct TRenderCaptureTypeData
{
	// FRenderCaptureTypeFlags uses the correct member naming convention and has additional boolean helpers
	static_assert(std::is_same_v<T, bool> == false, "Use FRenderCaptureTypeFlags instead");

	T BaseColor;
	T Roughness;
	T Specular;
	T Metallic;
	T CombinedMRS;
	T WorldNormal;
	T Emissive;
	T Opacity;
	T SubsurfaceColor;
	T DeviceDepth;

	/** @return mutable data corresponding to the given CaptureType */
	inline T& operator[](ERenderCaptureType CaptureType)
	{
		switch (CaptureType)
		{
		case ERenderCaptureType::BaseColor:
			return BaseColor;
		case ERenderCaptureType::WorldNormal:
			return WorldNormal;
		case ERenderCaptureType::Roughness:
			return Roughness;
		case ERenderCaptureType::Metallic:
			return Metallic;
		case ERenderCaptureType::Specular:
			return Specular;
		case ERenderCaptureType::Emissive:
			return Emissive;
		case ERenderCaptureType::CombinedMRS:
			return CombinedMRS;
		case ERenderCaptureType::Opacity:
			return Opacity;
		case ERenderCaptureType::SubsurfaceColor:
			return SubsurfaceColor;
		case ERenderCaptureType::DeviceDepth:
			return DeviceDepth;
		default:
			ensure(false);
		}
		return BaseColor;
	}

	/** @return constant data corresponding to the given CaptureType */
	inline const T& operator[](ERenderCaptureType CaptureType) const
	{
		return const_cast<TRenderCaptureTypeData<T>*>(this)->operator[](CaptureType);
	}
};


/** 
 * Render capture images use the render target coordinate system, which is defined such that:
 * - The upper-left corner is (0,0) in pixel space, (0,0) in UV space and (-1,1) in NDC space
 * - The bottom-right corner is (Width,Height) in pixel space, (1,1) in UV Space and (1,-1) in NDC space
 * Pixel centers are offset by (0.5,0.5) from integer locations
 */
struct FRenderCaptureCoordinateConverter2D
{
	// Convert normalized device coordinates to render capture image UV coordinates
	static FVector2d DeviceToUV(FVector2d NDC)
	{
		FVector2d UV = NDC / 2. + FVector2d(.5, .5);
		UV.Y = 1. - UV.Y;
		return UV;
	}

	// Convert render capture image pixel coordinates to normalized device coordinates
	static FVector2d PixelToDevice(FVector2i Pixel, int32 Width, int32 Height)
	{
		FVector2d NDC;
		NDC.X = ((double)Pixel.X + .5) / Width;
		NDC.Y = ((double)Pixel.Y + .5) / Height;
		NDC.X = 2. * NDC.X - 1.;
		NDC.Y = 2. * NDC.Y - 1.;
		NDC.Y *= -1;
		return NDC;
	}
};

/**
 * Returns the view matrices corresponding to the given render capture cameara parameters
 * Useful to get the needed information to unproject the DeviceDepth render capture e.g., using the following
 *
 *   FViewMatrices ViewMatrices = GetRenderCaptureViewMatrices(...);
 *   FVector2d DeviceXY = FRenderCaptureCoordinateConverter2D::PixelToDevice(...);
 *   FVector4d WorldPoint = ViewMatrices.GetInvViewProjectionMatrix().TransformPosition(FVector3d(DeviceXY, DeviceZ));
 *   WorldPoint /= WorldPoint.W;
 */
FViewMatrices GetRenderCaptureViewMatrices(const FFrame3d& ViewFrame, double HorzFOVDegrees, double NearPlaneDist, FImageDimensions ViewRect);

/**
 * FWorldRenderCapture captures a rendering of a set of Actors in a World from a
 * specific viewpoint. Various types of rendering are supported, as defined by ERenderCaptureType.
 * Currently rendering an entire World, ie without an explicit list of Actors or Components, is not supported.
 */
class FWorldRenderCapture : public FGCObject
{
public:

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Silence deprecation warnings for deprecated LastCaptureViewMatrices member in implicit constructors
	FWorldRenderCapture() = default;
	FWorldRenderCapture(const FWorldRenderCapture&) = default;
	FWorldRenderCapture(FWorldRenderCapture&&) noexcept = default;
	FWorldRenderCapture& operator=(const FWorldRenderCapture&) = default;
	FWorldRenderCapture& operator=(FWorldRenderCapture&&) noexcept = default;
UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS

	~FWorldRenderCapture();

	/** Explicitly release any allocated textures or other data structres */
	UE_API void Shutdown();

	/** Set the target World */
	UE_API void SetWorld(UWorld* World);

	/** 
	 *  Set the set of Actors in the target World that should be included in the Rendering.
	 */
	UE_API void SetVisibleActors(const TArray<AActor*>& Actors);

	/**
	 *  Set the set of Components in the target World that should be included in the Rendering.
	 */
	UE_API void SetVisibleComponents(const TArray<UActorComponent*>& Components);

	/**
	 *  Set the set of Actors and Components in the target World that should be included in the Rendering.
	 */
	UE_API void SetVisibleActorsAndComponents(const TArray<AActor*>& VisibleActors, const TArray<UActorComponent*>& VisibleComponents);

	/** Get bounding-box of the Visible actors */
	FBoxSphereBounds GetVisibleActorBounds() const { return VisibleBounds; }

	/** 
	 * Compute a sphere where, if the camera is on the sphere pointed at the center, then the Visible Actors
	 * will be fully visible (ie a square capture will not have any clipping), for the given Field of View.
	 * SafetyBoundsScale multiplier is applied to the bounding box.
	 */
	UE_API FSphere ComputeContainingRenderSphere(float HorzFOVDegrees, float SafetyBoundsScale = 1.25f) const;

	/** Set desired pixel dimensions of the rendered target image */
	UE_API void SetDimensions(const FImageDimensions& Dimensions);

	/** @return pixel dimensions that target image will be rendered at */
	const FImageDimensions& GetDimensions() const { return Dimensions; }
	
	/**
	 * @return view matrices used in the last call to CaptureFromPosition
	 * Useful to get the needed information to unproject the DeviceDepth render capture
	 */
	UE_DEPRECATED(5.4, "Please use GetRenderCaptureViewMatrices instead")
	const FViewMatrices& GetLastCaptureViewMatrices() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return LastCaptureViewMatrices;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Capture the desired buffer type CaptureType with the given view/camera parameters. The returned image
	 * data is interpreted according to the comment in FRenderCaptureCoordinateConverter2D
	 * @param ResultImageOut output image of size GetDimensions() is stored here.
	 * @return true if capture could be rendered successfully
	 */
	UE_API bool CaptureFromPosition(
		ERenderCaptureType CaptureType,
		const FFrame3d& ViewFrame,
		double HorzFOVDegrees,
		double NearPlaneDist,
		FImageAdapter& ResultImageOut,
		const FRenderCaptureConfig& Config = {});

	/**
	 * Enable debug image write. The captured image will be written to <Project>/Intermediate/<FolderName>/<CaptureType>_<ImageCounter>.bmp
	 * If FolderName is not specified, "WorldRenderCapture" is used by default.
	 * If an ImageCounter is not specified, an internal static counter is used that increments on every write
	 */
	UE_API void SetEnableWriteDebugImage(bool bEnable, int32 ImageCounter = -1, FString FolderName = FString());

	// ~Begin FGCObject Interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	// ~End FGCObject Interface

protected:
	UWorld* World = nullptr;

	TSet<FPrimitiveComponentId> VisiblePrimitives;
	FBoxSphereBounds VisibleBounds;

	FImageDimensions Dimensions = FImageDimensions(128, 128);

	// Temporary textures used as render targets. We explicitly prevent this from being GC'd internally
	TObjectPtr<UTextureRenderTarget2D> LinearRenderTexture = nullptr;
	TObjectPtr<UTextureRenderTarget2D> DepthRenderTexture = nullptr;

	FImageDimensions RenderTextureDimensions;
	UE_API UTextureRenderTarget2D* GetRenderTexture();
	UE_API UTextureRenderTarget2D* GetDepthRenderTexture();

	UE_DEPRECATED(5.4, "Please use GetRenderCaptureViewMatrices instead")
	FViewMatrices LastCaptureViewMatrices;

	// temporary buffer used to read from texture
	TArray<FLinearColor> ReadImageBuffer;

	/** Combined Metallic/Roughness/Specular uses a custom postprocess material */
	UE_DEPRECATED(5.8, "Please use CaptureFromPosition with ERenderCaptureType::CombinedMRS instead")
	UE_API bool CaptureMRSFromPosition(
		const FFrame3d& Frame,
		double HorzFOVDegrees,
		double NearPlaneDist,
		FImageAdapter& ResultImageOut,
		const FRenderCaptureConfig& Config = {});

	/** Depth is a special case and uses different code than capture of color/property channels */
	UE_API bool CaptureDeviceDepthFromPosition(
		const FFrame3d& Frame,
		double HorzFOVDegrees,
		double NearPlaneDist,
		FImageAdapter& ResultImageOut,
		const FRenderCaptureConfig& Config = {});

	/**
	 * The following values for VisualizationMode are supported:
	 *   BaseColor, Metallic, Roughness, Specular, WorldNormal, Opacity, SubsurfaceColor, PreTonemapHDRColor
	 * 
	 * Its not clear where the valid VisualizationMode FNames are defined, it may be in the [Engine.BufferVisualizationMaterials] section of
	 * Config/BaseEngine.ini, which is read by FBufferVisualizationData::Initialize() and called when the engine initializes in UEngine::Init()
	 */
	UE_API bool CaptureBufferVisualizationFromPosition(
		const FName& VisualizationMode,
		const FFrame3d& Frame,
		double HorzFOVDegrees,
		double NearPlaneDist,
		FImageAdapter& ResultImageOut,
		const FRenderCaptureConfig& Config = {});

	/**
	 * Renders the given PostProcessMaterial and captures its output using BufferVisualizationPipes, which grabs the material output before
	 * any gamma correction is applied
	 */
	UE_API bool CaptureUsingPostProcessMaterialPipe(
		UMaterialInterface* PostProcessMaterial,
		const FFrame3d& Frame,
		double HorzFOVDegrees,
		double NearPlaneDist,
		FImageAdapter& ResultImageOut,
		const FRenderCaptureConfig& Config,
		bool bEnableLighting,
		const TCHAR* DebugImageName);

	bool bWriteDebugImage = false;
	int32 DebugImageCounter = -1;
	FString DebugImageFolderName = TEXT("WorldRenderCapture");
	UE_API void WriteDebugImage(const FImageAdapter& ResultImageOut, const FString& ImageTypeName);

	TArray<TObjectPtr<UPrimitiveComponent>> ComponentsRestoreMaxDrawDistance;
	void RestoreComponentsMaxDrawDistance();
};





} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
