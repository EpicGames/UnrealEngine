// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshTopologySelector.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE::Geometry
{
	class FMeshBoundaryLoops;
}

/**
* MeshTopologySelector for selecting edge spans -- similar to FBoundarySelector but operating on spans instead of loops
*/
class FBoundaryEdgeSpanSelector : public FMeshTopologySelector
{
public:

	/**
	 * Initialize the selector with the given Mesh and Topology.
	 * This does not create the internal data structures, this happens lazily on GetGeometrySet()
	 */
	UE_API FBoundaryEdgeSpanSelector(const UE::Geometry::FDynamicMesh3* Mesh, const UE::Geometry::FMeshBoundaryLoops* BoundaryLoops);

	UE_API virtual void DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle = ECornerDrawStyle::Point) override;
};

#undef UE_API
