// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PolygonSelectionMechanic.h"
#include "BoundarySelectionMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE::Geometry { class FMeshBoundaryLoops; }
class FBoundarySelector;

UCLASS(MinimalAPI)
class UBoundarySelectionMechanic : public UMeshTopologySelectionMechanic
{
	GENERATED_BODY()

public:

	enum class EBoundarySelectionType : uint8
	{
		Loops,
		Spans
		// TODO: Both?
	};

	UE_API void Initialize(
		const FDynamicMesh3* MeshIn,
		FTransform3d TargetTransformIn,
		UWorld* WorldIn,
		const UE::Geometry::FMeshBoundaryLoops* BoundaryLoopsIn,
		TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFuncIn,
		EBoundarySelectionType SelectionType = EBoundarySelectionType::Loops);

	UE_API virtual bool UpdateHighlight(const FRay& WorldRay) override;

	UE_API virtual bool UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut) override;

};

#undef UE_API
