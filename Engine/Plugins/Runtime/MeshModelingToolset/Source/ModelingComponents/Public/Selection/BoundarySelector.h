// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GroupTopologySelector.h"
#include "MeshBoundaryLoops.h"

#define UE_API MODELINGCOMPONENTS_API

using UE::Geometry::FMeshBoundaryLoops;
using UE::Geometry::FEdgeLoop;

class FBoundaryTopologyProvider : public FTopologyProvider
{
public:

	FBoundaryTopologyProvider(const FMeshBoundaryLoops* BoundaryLoops) :
		BoundaryLoops(BoundaryLoops)
	{}

	virtual ~FBoundaryTopologyProvider() = default;

	virtual int GetNumEdges() const override
	{
		return BoundaryLoops->Loops.Num();
	}

	virtual void GetEdgePolyline(int EdgeID, FPolyline3d& OutPolyline) const override
	{
		OutPolyline.Clear();
		const TArray<int>& Vertices = BoundaryLoops->Loops[EdgeID].Vertices;
		for (int i = 0; i < Vertices.Num(); ++i)
		{
			OutPolyline.AppendVertex(BoundaryLoops->Mesh->GetVertex(Vertices[i]));
		}
	}

	virtual int FindGroupEdgeID(int MeshEdgeID) const override
	{
		return BoundaryLoops->FindLoopContainingEdge(MeshEdgeID);
	}

	virtual const TArray<int>& GetGroupEdgeEdges(int GroupEdgeID) const override
	{
		return BoundaryLoops->Loops[GroupEdgeID].Edges;
	}

	virtual const TArray<int>& GetGroupEdgeVertices(int GroupEdgeID) const override
	{
		return BoundaryLoops->Loops[GroupEdgeID].Vertices;
	}

	virtual void ForGroupSetEdges(const TSet<int32>& GroupIDs, const TFunction<void(int EdgeID)>& EdgeFunc) const override
	{
		// We should only have at most one "group" for MeshBoundaryLoops
		check(GroupIDs.Num() < 2);
		for (int32 GroupID : GroupIDs)
		{
			for (const FEdgeLoop& Loop : BoundaryLoops->Loops)
			{
				for (const int32 EdgeID : Loop.Edges)
				{
					EdgeFunc(EdgeID);
				}
			}
		}
	}

	virtual FAxisAlignedBox3d GetSelectionBounds(const FGroupTopologySelection& Selection, TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const override;
	virtual FFrame3d GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame) const override;

public:

	//
	// Functions that don't make sense if we don't have Groups and Corners but are still part of the interface:
	//

	virtual int GetNumCorners() const override
	{
		return 0;
	}

	virtual int GetCornerVertexID(int CornerID) const override
	{
		return IndexConstants::InvalidID;
	}

	virtual int GetNumGroups() const override
	{
		return 1;
	}

	virtual int GetGroupIDAt(int GroupIndex) const override
	{
		return 0;
	}

	virtual int GetGroupIDForTriangle(int TriangleID) const override
	{
		return 0;
	}

private:

	bool GetLoopTangent(int LoopID, FVector3d& TangentOut) const;
	FVector3d GetLoopMidpoint(int32 LoopID, double* ArcLengthOut = nullptr, TArray<double>* PerVertexLengthsOut = nullptr) const;
	double GetLoopArcLength(int32 LoopID, TArray<double>* PerVertexLengthsOut = nullptr) const;

	const FMeshBoundaryLoops* BoundaryLoops;

};

/*
* This class allows selection of mesh boundary loops. It inherits from FGroupTopologySelector to leverage a lot of the selection functionality, but it uses
* a FMeshBoundaryLoops object rather than a FGroupTopology object to determine which loops are selectable.
*/
class FBoundarySelector : public FMeshTopologySelector
{
public:

	/**
	 * Initialize the selector with the given Mesh and Topology.
	 * This does not create the internal data structures, this happens lazily on GetGeometrySet()
	 */
	UE_API FBoundarySelector(const FDynamicMesh3* Mesh, const FMeshBoundaryLoops* BoundaryLoops);

	UE_API virtual void DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle = ECornerDrawStyle::Point) override;
};

#undef UE_API
