// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/PimplPtr.h"
#include "Selections/GeometrySelection.h"
#include "Selection/GeometrySelector.h"
#include "Selection/DynamicMeshSelector.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE::Geometry { class FGroupTopologyDeformer; }

/**
* IGeometrySelectionTransformer for Dynamic Mesh Polygroups, that does various types of soft deformation.
*/
class FDynamicMeshPolygroupTransformer : public FBasicDynamicMeshSelectionTransformer
{
public:

	UE_API virtual void BeginTransform(const UE::Geometry::FGeometrySelection& Selection) override;
	UE_API virtual void UpdateTransform( TFunctionRef<FVector3d(int32 VertexID, double Weight, const FVector3d& InitialPosition, const FTransform& WorldTransform)> PositionTransformFunc ) override;
	UE_API virtual void EndTransform(IToolsContextTransactionsAPI* TransactionsAPI) override;

	UE_API virtual void PreviewRender(IToolsContextRenderAPI* RenderAPI) override;

protected:
	bool bFallbackToSimpleTransform = false;

	TMap<int32, int32> ROIMap;
	TPimplPtr<UE::Geometry::FGroupTopologyDeformer> LinearDeformer;
};

#undef UE_API
