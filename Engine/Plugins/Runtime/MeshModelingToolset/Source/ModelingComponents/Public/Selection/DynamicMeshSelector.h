// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Change.h"
#include "Templates/PimplPtr.h"
#include "Selections/GeometrySelection.h"
#include "Selection/GeometrySelector.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "TransformTypes.h"
#include "Parameterization/MeshDijkstra.h"

#define UE_API MODELINGCOMPONENTS_API


class FBasicDynamicMeshSelectionTransformer;
class FMeshVertexChangeBuilder;
PREDECLARE_GEOMETRY(class FGroupTopology);
PREDECLARE_GEOMETRY(class FColliderMesh);
PREDECLARE_GEOMETRY(class FSegmentTree3);

/**
 * FBaseDynamicMeshSelector is an implementation of IGeometrySelector for a UDynamicMesh.
 * Note that the Selector itself does *not* require that the target object be a UDynamicMeshComponent,
 * and subclasses of FBaseDynamicMeshSelector are used for both Volumes and StaticMeshComponents.
 * Access to the World transform is provided by a TUniqueFunction set up in the Factory.
 */
class FBaseDynamicMeshSelector : public IGeometrySelector
{
public:
	UE_API virtual ~FBaseDynamicMeshSelector();

	/**
	 * Initialize the FBaseDynamicMeshSelector for a given source/target UDynamicMesh.
	 * @param SourceGeometryIdentifier identifier for the object that the TargetMesh came from (eg DynamicMeshComponent or other UDynamicMesh source)
	 * @param TargetMesh the target UDynamicMesh
	 * @param GetWorldTransformFunc function that provides the Local to World Transform
	 */
	UE_API virtual void Initialize(
		FGeometryIdentifier SourceGeometryIdentifier,
		UDynamicMesh* TargetMesh,
		TUniqueFunction<UE::Geometry::FTransformSRT3d()> GetWorldTransformFunc);

	/**
	 * @return FGeometryIdentifier for the parent of this Selector (eg a UDynamicMeshComponent in the common case)
	 */
	virtual FGeometryIdentifier GetSourceGeometryIdentifier() const
	{
		return SourceGeometryIdentifier;
	}

	//
	// IGeometrySelector API implementation
	//

	UE_API virtual void Shutdown() override;
	UE_API virtual bool SupportsSleep() const;
	UE_API virtual bool Sleep();
	UE_API virtual bool Restore();


	virtual FGeometryIdentifier GetIdentifier() const override
	{
		FGeometryIdentifier Identifier;
		Identifier.TargetType = FGeometryIdentifier::ETargetType::MeshContainer;
		Identifier.ObjectType = FGeometryIdentifier::EObjectType::DynamicMesh;
		Identifier.TargetObject = TargetMesh;
		return Identifier;
	}

	UE_API virtual void InitializeSelectionFromPredicate(	
		FGeometrySelection& SelectionInOut,
		TFunctionRef<bool(UE::Geometry::FGeoSelectionID)> SelectionIDPredicate,
		EInitializeSelectionMode InitializeMode = EInitializeSelectionMode::All,
		const FGeometrySelection* ReferenceSelection = nullptr) override;



	UE_API virtual void UpdateSelectionFromSelection(	
		const FGeometrySelection& FromSelection,
		bool bAllowConversion,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		UE::Geometry::FGeometrySelectionDelta* SelectionDelta = nullptr ) override;


	UE_API virtual bool RayHitTest(
		const FWorldRayQueryInfo& RayInfo,
		UE::Geometry::FGeometrySelectionHitQueryConfig QueryConfig,
		FInputRayHit& HitResultOut) override;


	UE_API virtual void UpdateSelectionViaRaycast(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut) override;

	UE_API virtual void GetSelectionPreviewForRaycast(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& PreviewEditor) override;



	UE_API virtual void UpdateSelectionViaShape(	
		const FWorldShapeQueryInfo& ShapeInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut 
	) override;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API virtual void UpdateSoftSelection(FGeometrySelection& Selection, const FUpdateSoftSelectionConfig& InSoftSelectionConfig) override;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	UE_API virtual bool ConvertSelection(const FGeometrySelection& FromSelectionIn, FGeometrySelection& ToSelectionOut, const UE::Geometry::EEnumerateSelectionConversionParams ConversionParams) override;

	virtual FTransform GetLocalToWorldTransform() const
	{
		return (FTransform)GetWorldTransformFunc();
	}

	UE_API virtual void GetSelectionFrame(const FGeometrySelection& Selection, UE::Geometry::FFrame3d& SelectionFrame, bool bTransformToWorld) override;
	UE_API virtual void GetTargetFrame(const FGeometrySelection& Selection, UE::Geometry::FFrame3d& SelectionFrame) override;
	UE_API virtual void AccumulateSelectionBounds(const FGeometrySelection& Selection, FGeometrySelectionBounds& BoundsInOut, bool bTransformToWorld) override;

	UE_DEPRECATED(5.5, "AccumulateSelectionElements which takes a bIsForPreview boolean is deprecated."
				"Please use the function of the same name which takes EEnumerateSelectionMapping flags instead")
	UE_API virtual void AccumulateSelectionElements(const FGeometrySelection& Selection, FGeometrySelectionElements& Elements, bool bTransformToWorld, bool bIsForPreview) override;

	/** Prefer AccumulateSelectionElements with Flags parameter. */
	UE_API virtual void AccumulateSelectionElements(const FGeometrySelection& Selection, FGeometrySelectionElements& Elements, bool bTransformToWorld, UE::Geometry::EEnumerateSelectionMapping Flags = UE::Geometry::EEnumerateSelectionMapping::Default) override;
	
	UE_API virtual void AccumulateElementsFromPredicate(
		FGeometrySelectionElements& Elements,
		bool bTransformToWorld,
		bool bIsForPreview,
		bool bUseGroupTopology,
		TFunctionRef<bool(UE::Geometry::EGeometryElementType, UE::Geometry::FGeoSelectionID)> Predicate
	) override;

	/**
	 * UpdateAfterGeometryEdit should be called after editing the UDynamicMesh owned by the Selector (TargetMesh). 
	 * This may be an external DynamicMesh in the case of a DynamicMeshComponent, or a temporary UDynamicMesh in
	 * the case of (eg) the StaticMeshSelector and VolumeSelector subclasses. In DynamicMeshSelector the MeshChange
	 * can just be emitted as a transaction, this is the default behavior. However in StaticMeshSelector, the StaticMesh 
	 * needs to be synchronized with the UDynamicMesh modification in the same transaction. And potentially the MeshChange 
	 * does not need to be emitted at all in that case. 
	 * 
	 * This is a bit ugly and might be possible to do more cleanly by having the StaticMeshSelector listen to the 
	 * UDynamicMesh for changes. However currently we do not have the granularity to have it /only/ listen for 
	 * external mesh edit changes, and not all changes (and since the UDynamicMesh changes in response to  
	 * StaticMesh changes, eg on undo or external edits, it creates a cycle). 
	 */
	UE_API virtual void UpdateAfterGeometryEdit(
		IToolsContextTransactionsAPI* TransactionsAPI, 
		bool bInTransaction, 
		TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChange, 
		FText GeometryEditTransactionString );



public:
	// these need to be public for the Transformers...can we do it another way?
	UDynamicMesh* GetDynamicMesh() const { return TargetMesh.Get(); }
	UE_API const UE::Geometry::FGroupTopology* GetGroupTopology();

protected:
	FGeometryIdentifier SourceGeometryIdentifier;
	TUniqueFunction<UE::Geometry::FTransformSRT3d()> GetWorldTransformFunc;

	TWeakObjectPtr<UDynamicMesh> TargetMesh;
	//UDynamicMesh* GetDynamicMesh() const { return TargetMesh.Get(); }		// temporarily public above

	FDelegateHandle TargetMesh_OnMeshChangedHandle;
	UE_API void RegisterMeshChangedHandler();
	UE_API void InvalidateOnMeshChange(FDynamicMeshChangeInfo ChangeInfo);

	//
	// FColliderMesh is used to store a hit-testable AABBTree independent of the UDynamicMesh 
	//
	TPimplPtr<UE::Geometry::FColliderMesh> ColliderMesh;
	UE_API void UpdateColliderMesh();
	UE_API const UE::Geometry::FColliderMesh* GetColliderMesh();

	//
	// GroupTopology will be built on-demand if polygroup selection queries are made
	//
	TPimplPtr<UE::Geometry::FGroupTopology> GroupTopology;
	UE_API void UpdateGroupTopology();
	//const UE::Geometry::FGroupTopology* GetGroupTopology();		// temporarily public above

	//
	// GroupEdgeSegmentTree stores a hit-testable AABBTree for the polygroup edges (depends on GroupTopology)
	//
	TPimplPtr<UE::Geometry::FSegmentTree3> GroupEdgeSegmentTree;
	UE_API void UpdateGroupEdgeSegmentTree();
	UE_API const UE::Geometry::FSegmentTree3* GetGroupEdgeSpatial();

	//
	// PathFinder will be built on-demand if surface-based soft selection is enabled
	//
	TPimplPtr<UE::Geometry::TMeshDijkstra<FDynamicMesh3>> PathFinder;

	//
	// CachedVolumeDistanceToSelection caches the Euclidean distance from each mesh vertex
	// to its nearest seed for the current selection. Index by VertexID; 
	//
	TArray<double> CachedVolumeDistanceToSelection;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API virtual void UpdateSoftSelection_Surface(FGeometrySelection& Selection, const FUpdateSoftSelectionConfig& InSoftSelectionConfig);
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API virtual void UpdateSoftSelection_Volume(FGeometrySelection& Selection, const FUpdateSoftSelectionConfig& InSoftSelectionConfig);
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	
	// support for sleep/restore
	TWeakObjectPtr<UDynamicMesh> SleepingTargetMesh = nullptr;


	UE_API virtual void UpdateSelectionViaRaycast_GroupEdges(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut);

	UE_API virtual void UpdateSelectionViaRaycast_MeshTopology(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut);

};


/**
 * FDynamicMeshSelector is an implementation of FBaseDynamicMeshSelector meant to be used
 * with UDynamicMeshComponents.
 */
class FDynamicMeshSelector : public FBaseDynamicMeshSelector
{
public:

	UE_API virtual IGeometrySelectionTransformer* InitializeTransformation(const FGeometrySelection& Selection) override;
	UE_API virtual void ShutdownTransformation(IGeometrySelectionTransformer* Transformer) override;

protected:
	TSharedPtr<FBasicDynamicMeshSelectionTransformer> ActiveTransformer;

	// give Transformer access to internals 
	friend class FBasicDynamicMeshSelectionTransformer;
};



/**
 * FDynamicMeshComponentSelectorFactory constructs FDynamicMeshSelector instances 
 * for UDynamicMeshComponents
 */
class FDynamicMeshComponentSelectorFactory : public IGeometrySelectorFactory
{
public:
	UE_API virtual bool CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const;

	UE_API virtual TUniquePtr<IGeometrySelector> BuildForTarget(FGeometryIdentifier TargetIdentifier) const;
};


/**
* BasicDynamicMeshSelectionTransformer is a basic Transformer implementation that can be
* used with a FBaseDynamicMeshSelector. This Transformer moves the selected vertices and 
* nothing else (ie no polygroup-based soft deformation)
*/
class FBasicDynamicMeshSelectionTransformer : public IGeometrySelectionTransformer
{
public:
	UE_API virtual void Initialize(FBaseDynamicMeshSelector* Selector);

	virtual IGeometrySelector* GetSelector() const override
	{
		return Selector;
	}

	UE_API virtual void BeginTransform(const UE::Geometry::FGeometrySelection& Selection) override;

	UE_API virtual void UpdateTransform( TFunctionRef<FVector3d(int32 VertexID, double Weight, const FVector3d& InitialPosition, const FTransform& WorldTransform)> PositionTransformFunc ) override;
	UE_API void UpdatePendingVertexChange(bool bFinal);

	/** Enable line drawing of selection during transform, this is necessary in some contexts where live mesh update is too slow */
	bool bEnableSelectionTransformDrawing = false;

	UE_API virtual void PreviewRender(IToolsContextRenderAPI* RenderAPI) override;

	UE_API virtual void EndTransform(IToolsContextTransactionsAPI* TransactionsAPI) override;

	TFunction<void(IToolsContextTransactionsAPI* TransactionsAPI)> OnEndTransformFunc;

protected:
	FBaseDynamicMeshSelector* Selector;

	TArray<int32> MeshVertices;
	
	TArray<double> SoftSelectedVertexWeights;
	
	// The number of vertices in MeshVertices = NumSelectedMeshVertices + SoftSelectedVertexWeights.Num()
	int32 NumSelectedMeshVertices = 0;
	
	TArray<FVector3d> InitialPositions;
	TSet<int32> TriangleROI;
	TSet<int32> OverlayNormalsSet;
	TArray<int32> OverlayNormalsArray;

	TArray<FVector3d> UpdatedPositions;

	TPimplPtr<FMeshVertexChangeBuilder> ActiveVertexChange;

	// used for preview rendering
	TArray<UE::Geometry::FIndex2i> ActiveSelectionEdges;
	TArray<int32> ActiveSelectionVertices;
	TArray<UE::Geometry::FIndex2i> ActiveROIEdges;

};




#undef UE_API
