// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolChange.h"
#include "Selections/GeometrySelection.h"
#include "Selection/GeometrySelector.h"		// for FGeometryIdentifier

#define UE_API MODELINGCOMPONENTS_API


/**
 * Base interface intended to be implemented by FToolCommandChange subclasses that represent 
 * changes to a FGeometrySelection. The point of the interface is to allow UGeometrySelectionManager
 * to execute the ApplyChange() and RevertChange() with the FGeometrySelectionEditor it provides.
 * So the call pattern is that the FToolCommandChange::Apply(), with the UGeometrySelectionManager
 * as the UObject target, will call UGeometrySelectionManager::ApplyChange(), which then in turn
 * will call IGeometrySelectionChange::ApplyChange()
 */
class IGeometrySelectionChange
{
public:
	using FGeometrySelection = UE::Geometry::FGeometrySelection;
	using FGeometrySelectionEditor = UE::Geometry::FGeometrySelectionEditor;
	using FGeometrySelectionDelta = UE::Geometry::FGeometrySelectionDelta;

	virtual ~IGeometrySelectionChange() = default;

	virtual FGeometryIdentifier GetIdentifier() const = 0;

	virtual void ApplyChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& ApplyDelta) = 0;
	virtual void RevertChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& RevertDelta) = 0;
};


/**
 * FGeometrySelectionDeltaChange stores a Remove-then-Add change in a FGeometrySelection.
 */
class FGeometrySelectionDeltaChange : public FToolCommandChange, public IGeometrySelectionChange
{
public:
	FGeometryIdentifier Identifier;
	FGeometrySelectionDelta Delta;

	/** Makes the change to the object */
	UE_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	UE_API virtual void Revert(UObject* Object) override;

	/** Describes this change (for debugging) */
	UE_API virtual FString ToString() const override;

	/** Change has expired if the SelectionManager no longer exists */
	UE_API virtual bool HasExpired(UObject* Object) const override;

	virtual FGeometryIdentifier GetIdentifier() const override { return Identifier;	}
	UE_API virtual void ApplyChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& ApplyDelta) override;
	UE_API virtual void RevertChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& RevertDelta) override;
};



/**
 * FGeometrySelectionReplaceChange stores a full replacement of a FGeometrySelection,
 * ie full copies of the selection set before and after the change
 */
class FGeometrySelectionReplaceChange : public FToolCommandChange, public IGeometrySelectionChange
{
public:
	FGeometryIdentifier Identifier;
	FGeometrySelection Before;
	FGeometrySelection After;

	/** Makes the change to the object */
	UE_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	UE_API virtual void Revert(UObject* Object) override;

	/** Describes this change (for debugging) */
	UE_API virtual FString ToString() const override;

	/** Change has expired if the SelectionManager no longer exists */
	UE_API virtual bool HasExpired(UObject* Object) const override;

	virtual FGeometryIdentifier GetIdentifier() const override { return Identifier;	}
	UE_API virtual void ApplyChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& ApplyDelta) override;
	UE_API virtual void RevertChange(FGeometrySelectionEditor* Editor, FGeometrySelectionDelta& RevertDelta) override;
};

#undef UE_API
