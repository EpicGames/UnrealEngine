// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConvexVolume.h"
#include "Selections/GeometrySelection.h"
#include "Selection/GeometrySelector.h"
#include "Selection/GeometrySelectionChanges.h"

#include "GeometrySelectionManager.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class UPreviewGeometry;
class IGeometrySelector;
class UInteractiveToolsContext;
class IToolsContextRenderAPI;
class IToolsContextTransactionsAPI;
class UGeometrySelectionEditCommand;
class UGeometrySelectionEditCommandArguments;

USTRUCT()
struct FMeshElementSelectionParams
{
	GENERATED_BODY()
	TStaticArray<FString, 3> Identifiers {};
	float DepthBias = 0.0f;
	float LineThickness = 0.0f;
	float PointSize = 0.0f;
	FColor Color = FColor(0,0,0);
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> SelectionFillColor = nullptr;
};

enum class EEnumerateRenderCachesDirtyFlags: uint8
{
	None = 0,

	SelectionCachesDirty = 1 << 0,
	UnselectedCachesDirty = 1 << 1, //SelectableRenderCaches
	PreviewCachesDirty = 1 << 2,
	SoftSelectionCachesDirty = 1 << 3,

	Default = SelectionCachesDirty | UnselectedCachesDirty | PreviewCachesDirty | SoftSelectionCachesDirty,
};
ENUM_CLASS_FLAGS(EEnumerateRenderCachesDirtyFlags);

/**
 * UGeometrySelectionManager provides the infrastructure for "Element Selection", ie 
 * geometric sub-elements of some geometry object like a Triangle Mesh. The Manager is
 * designed to work with a relatively vague concept of "element", so it doesn't explicitly
 * reference triangles/etc, and the selectable-elements and how-elements-are-selected
 * concepts are provided by abstract-interfaces that allow various implememtations.
 * 
 * The "Geometry Objects", eg like a DynamicMeshComponent, Gameplay Volume, etc, are
 * referred to as "Active Targets" in the Manager. External code provides and updates
 * the set of Active Targets, eg for example tracking the active Actor Selection in the Editor.
 * 
 * For a given Target, a tuple (Selector, Selection, SelectionEditor) is created and maintained.
 * The FGeometrySelection is ultimately a basic list of integers and does not have any knowledge
 * of what it is a selection *of*, and is not intended to be directly edited. Instead the
 * SelectionEditor provides that functionality. This separation allows "selection editing" to
 * be customized, eg to enforce invariants or constraints that might apply to certain kinds of selections.
 * 
 * The IGeometrySelector provides the core implementation of what "selection" means for a given
 * Target, eg like a mesh Component, or mesh object like a UDynamicMesh. The Selector is
 * created by a registered Factory, allowing client code to provide custom implementations for
 * different Target Types. Updates to the Selection are done via the Selector, as well as queries
 * about (eg) renderable selection geometry. 3D Transforms are also applied via the Selector,
 * as only it has the knowledge about what can be transformed and how it can be applied.
 * 
 * The GeometrySelectionManager provides high-level interfaces for this system, for example
 * external code (eg such as something that creates a Gizmo for the active selection) only
 * needs to interact with SelectionManager, calling functions like 
 * ::BeginTransformation() / ::UpdateTransformation() / ::EndTransformation().
 * The SelectionManager also handles Transactions/FChanges for the active Targets and Selections. 
 * 
 */
UCLASS(MinimalAPI)
class UGeometrySelectionManager : public UObject
{
	GENERATED_BODY()

public:

	using FGeometrySelection = UE::Geometry::FGeometrySelection;
	using FGeometrySelectionEditor = UE::Geometry::FGeometrySelectionEditor;
	using FGeometrySelectionBounds = UE::Geometry::FGeometrySelectionBounds;
	using FGeometrySelectionElements = UE::Geometry::FGeometrySelectionElements;
	using FGeometrySelectionUpdateConfig = UE::Geometry::FGeometrySelectionUpdateConfig;
	using FGeometrySelectionUpdateResult = UE::Geometry::FGeometrySelectionUpdateResult;
	using EGeometryElementType = UE::Geometry::EGeometryElementType;
	using EGeometryTopologyType = UE::Geometry::EGeometryTopologyType;

	//
	// Setup/Teardown
	//

	UE_API virtual void Initialize(UInteractiveToolsContext* ToolsContextIn, IToolsContextTransactionsAPI* TransactionsAPIIn);

	UE_API virtual void RegisterSelectorFactory(TUniquePtr<IGeometrySelectorFactory> Factory);

	UE_EXPERIMENTAL(5.7, "Selector factory unregistering is experimental while the API is under active development.")
	UE_API virtual void UnregisterSelectorFactories();

	UE_API virtual void Shutdown();
	UE_API virtual bool HasBeenShutDown() const;

	UInteractiveToolsContext* GetToolsContext() const { return ToolsContext; }
	IToolsContextTransactionsAPI* GetTransactionsAPI() const { return TransactionsAPI; }

	//
	// Configuration
	//

	/** EMeshTopologyMode determines what level of mesh element will be selected  */
	enum class EMeshTopologyMode
	{
		None = 0,
		/** Select mesh triangles, edges, and vertices */
		Triangle = 1,
		/** Select mesh polygroups, polygroup-borders, and polygroup-corners */
		Polygroup = 2
	};

	UE_API virtual void SetSelectionElementType(EGeometryElementType ElementType);
	virtual EGeometryElementType GetSelectionElementType() const { return SelectionElementType; }

	UE_API virtual void SetMeshTopologyMode(EMeshTopologyMode SelectionMode);
	virtual EMeshTopologyMode GetMeshTopologyMode() const { return MeshTopologyMode; }
	UE_API virtual EGeometryTopologyType GetSelectionTopologyType() const;

	// Switch the selection mode and type, optionally converting any existing selection to the new type and mode
	UE_API virtual void SetMeshSelectionTypeAndMode(EGeometryElementType NewElementType, EMeshTopologyMode NewSelectionMode, bool bConvertSelection);

	/**
	 * Removes Triangle, Line, and Point sets with the given SetIdentifier prefix
	 * Full Identifier strings are in the format: SetIdentifier_Triangles, SetIdentifier_Lines, and SetIdentifier_Points
	 */
	UE_API void RemoveSets(const TArrayView<FString>& SetIdentifiers) const;
	


	//
	// Target Management / Queries
	// TODO: be able to update active target set w/o losing current selections?
	//

	/**
	 * @return true if there are any active selection targets
	 */
	UE_API bool HasActiveTargets() const;

	/**
	 * Attempt to validate the current selection state; can be called to detect if e.g. selected objects have been deleted from under the selection manager.
	 * @return true if current active selection state appears to be valid (i.e., does not include stale / deleted objects)
	 */
	UE_API bool ValidateSelectionState() const;
	
	/**
	 * Empty the active selection target set
	 * @warning Active selection must be cleared (eg via ClearSelection()) before calling this function
	 */
	UE_API void ClearActiveTargets();

	/**
	 * Add a target to the active target set, if a valid IGeometrySelectorFactory can be found
	 * @return true on success
	 */
	UE_API bool AddActiveTarget(FGeometryIdentifier Target);


	UE_API bool GetAnyCurrentTargetsLockable() const;
	UE_API bool GetAnyCurrentTargetsLocked() const;
	UE_API void SetCurrentTargetsLockState(bool bLocked);


	/**
	 * Update the current active target set based on DesiredActiveSet, assuming that a valid IGeometrySelectorFactory
	 * can be found for each identifier. 
	 * This function will emit a transaction/change if the target set is modified.
	 * @param WillChangeActiveTargetsCallback this function will be called if the DesiredActiveSet is not the same as the current active set, allowing calling code to (eg) clear the active selection
	 */
	UE_API void SynchronizeActiveTargets(
		const TArray<FGeometryIdentifier>& DesiredActiveSet,
		TFunctionRef<void()> WillChangeActiveTargetsCallback );

	/**
	 * Test if a World-space ray "hits" the current active target set, which can be used to (eg)
	 * determine if a higher-level user interaction for selection should "Capture" the click
	 * @param HitResultOut hit information is returned here
	 * @return true on hit
	 */
	UE_API virtual bool RayHitTest(
		const FRay3d& WorldRay,
		FInputRayHit& HitResultOut
	);

	/**
	 * Invalidates all cached selection elements by default.
	 * When desired, can choose to not mark the Selectable render cache as dirty
	 */
	UE_API void MarkRenderCachesDirty(bool bMarkSelectableDirty = true);
	
	//
	// Selection Updates
	//
public:
	/**
	 * Clear any active element selections. 
	 * This function will emit a Transaction for the selection change.
	 * @param bSaveSelectionBeforeClear if true, calls SaveCurrentSelection() before the selection is cleared, so it can be restored later.
	 */
	UE_API virtual void ClearSelection(bool bSaveSelectionBeforeClear = false);

protected:
	/**
	 * Save the active selection. Overwrites any existing saved selections with the current selections. Typically used via ClearSelection(true)
	 */
	UE_API virtual void SaveCurrentSelection();

public:
	/**
	 * Attempt to restore (and then discard) the most recent saved selection.
	 * If there is no active saved selection, does nothing. On failure to restore, will still discard the saved selection.
	 * @return false if could not restore selection, which can happen if the restore was called while transacting (e.g., when a tool is exited via undo), or if the selection objects were not found or not valid
	 */
	UE_API virtual bool RestoreSavedSelection();
	
	/**
	 * Discard the saved selection, if there is one.
	 */
	UE_API virtual void DiscardSavedSelection();

	/**
	 * @return true if there is a non-empty saved selection, false otherwise.
	 */
	UE_API virtual bool HasSavedSelection();

	/**
	 * Use the given WorldRay to update the active element selection based on UpdateConfig.
	 * The intention is that this function is called by higher-level user interaction code after
	 * RayHitTest() has returned true.
	 * @param ResultOut information on any element selection modifications is returned here
	 */
	UE_API virtual void UpdateSelectionViaRaycast(
		const FRay3d& WorldRay,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut
	);


	/**
	 * Use the given ConvexVolume to update the active element selection based on UpdateConfig.
	 * @param ResultOut information on any element selection modifications is returned here
	 */
	UE_API virtual void UpdateSelectionViaConvex(
		const FConvexVolume& ConvexVolume,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut
	);



	//
	// Support for more complex selection changes that might (eg) occur over multiple frames, 
	// or be computed externally. The usage pattern is:
	//   - verify that CanBeginTrackedSelectionChange() returns true
	//   - call BeginTrackedSelectionChange(), this opens transacation
	//      - modify selection here, eg via multiple calls to AccumulateSelectionUpdate_Raycast
	//   - call EndTrackedSelectionChange() to emit changes and close transaction

	/** @return true if a tracked selection change can be initialized */
	UE_API virtual bool CanBeginTrackedSelectionChange() const;

	/** @return true if an active tracked selection change is in flight */
	virtual bool IsInTrackedSelectionChange() const { return bInTrackedSelectionChange; }

	/**
	 * Begin a tracked selection change. CanBeginTrackedSelectionChange() must return true to call this function.
	 * EndTrackedSelectionChange() must be called to close the selection change.
	 * @param UpdateConfig how the selection will be modified. Currently this must be consistent across the entire tracked change (ie, only add, only subtract, etc)
	 * @param bClearOnBegin if true, selection will be initially cleared (eg Replace-style)
	 */
	UE_API virtual bool BeginTrackedSelectionChange(FGeometrySelectionUpdateConfig UpdateConfig, bool bClearOnBegin);

	/**
	 * Update the tracked selection change via a single Raycast, using the active UpdateConfig mode passed to BeginTrackedSelectionChange
	 * @param ResultOut information on any element selection modifications is returned here* 
	 */
	UE_API virtual void AccumulateSelectionUpdate_Raycast(
		const FRay3d& WorldRay,
		FGeometrySelectionUpdateResult& ResultOut
	);	

	/**
	 * Close an active tracked selection change. 
	 * This will emit one or more FChanges for the selection modifications, and then close the open Transaction
	 */
	UE_API virtual void EndTrackedSelectionChange();


	/**
	 * Directly set the current Selection for the specified Component to NewSelection.
	 * This function allows external code to construct explicit selections, eg for a Tool or Command to emit a new Selection.
	 * Component must already be an active target, ie set via AddActiveTarget.
	 * If the selection of the Target would be modified, a selection-change transaction will be emitted.
	 */
	UE_API virtual bool SetSelectionForComponent(UPrimitiveComponent* Component, const FGeometrySelection& NewSelection);


	DECLARE_MULTICAST_DELEGATE(FModelingSelectionInteraction_SelectionModified);
	/**
	 * OnSelectionModified is broadcast if the selection is modified via the above functions.
	 * There are no arguments.
	 */
	FModelingSelectionInteraction_SelectionModified OnSelectionModified;



	//
	// Hover/Preview support
	//
public:

	UE_API virtual bool UpdateSelectionPreviewViaRaycast(
		const FRay3d& WorldRay
	);

	/**
	 * Resets the active preview selection and invalidates its associated cached render elements.
	 */
	UE_API virtual void ClearSelectionPreview();

	
	//
	// Selection queries
	//
public:
	/** @return true if there is an active element selection */
	UE_API virtual bool HasSelection() const;

	/** 
	 * Get available information about the active selection/state
	 */
	UE_API virtual void GetActiveSelectionInfo(EGeometryTopologyType& TopologyTypeOut, EGeometryElementType& ElementTypeOut, int& NumTargetsOut, bool& bIsEmpty) const;

	/** @return a world-space bounding box for the active element selection */
	UE_API virtual bool GetSelectionBounds(FGeometrySelectionBounds& BoundsOut) const;

	/** @return a 3D transformation frame suitable for use with the active element selection */
	UE_API virtual void GetSelectionWorldFrame(UE::Geometry::FFrame3d& SelectionFrame) const;

	/** @return a 3D transformation frame suitable for use with the set of active targets */
	UE_API virtual void GetTargetWorldFrame(UE::Geometry::FFrame3d& SelectionFrame) const;

	/** @return true if there is an active IGeometrySelector target for the given Component and it has a non-empty selection */
	UE_API virtual bool HasSelectionForComponent(UPrimitiveComponent* Component) const;
	/** 
	 * Get the active element selection for the given Component, if it exists
	 * @return true if a valid non-empty selection was returned in SelectionOut
	 */
	UE_API virtual bool GetSelectionForComponent(UPrimitiveComponent* Component, FGeometrySelection& SelectionOut) const;



	//
	// Transformations
	//
protected:
	// Set of existing transformer objects, collected from active IGeometrySelector::InitializeTransformation().
	// The SelectionManager does not own these objects, they must not be deleted. 
	// Instead they need to be returned via IGeometrySelector::ShutdownTransformation
	TArray<IGeometrySelectionTransformer*> ActiveTransformations;

public:
	/** @return true if SelectionManager is actively transforming element selections (ie during a mouse-drag) */
	virtual bool IsInActiveTransformation() const { return (ActiveTransformations.Num() > 0); }
	
	/**
	 * Begin a transformation of element selections in active Targets.
	 * @return true if at least one valid Transformer was initialized, ie the transformation will do something
	 */
	UE_API virtual bool BeginTransformation();

	/**
	 * Update the active transformations with the given PositionTransformFunc.
	 * See IGeometrySelectionTransformer::UpdateTransform for details on this callback
	 */
	UE_DEPRECATED(5.8, "Use the UpdateTransformation function that takes a lambda that includes per-vertex weight instead")
	UE_API virtual void UpdateTransformation( TFunctionRef<FVector3d(int32 VertexID, const FVector3d&, const FTransform&)> PositionTransformFunc );
	
	/**
	 * Update the active transformations with the given PositionTransformFunc.
	 * See IGeometrySelectionTransformer::UpdateTransform for details on this callback
	 */
	UE_EXPERIMENTAL(5.8, "Per-vertex weighted transformation is experiemental while the API is under active development.")
	UE_API virtual void UpdateTransformation( TFunctionRef<FVector3d(int32 VertexID, double Weight, const FVector3d&, const FTransform&)> PositionTransformFunc );

	/**
	 * End the current active transformation, and emit changes/transactions
	 */
	UE_API virtual void EndTransformation();

	/**
	 * Update the active transformations with the given transform deltas
	 */
	UE_EXPERIMENTAL(5.8, "This function is experiemental while the API is under active development.")
	UE_API void UpdateTransformation(
		const FTransform& StartTransform,
		const FVector& TranslationToApply,
		const FQuat& RotationToApply,
		const FVector& ScaleToApply,
		bool bWorldSpaceScale
		);

	//
	// Command Execution
	//
public:
	/**
	 * @return true if Command->CanExecuteCommand() returns true for *all* the current Selections
	 */
	UE_API bool CanExecuteSelectionCommand(UGeometrySelectionEditCommand* Command);

	/**
	 * execute the selection command for *all* the current selections
	 */
	UE_API void ExecuteSelectionCommand(UGeometrySelectionEditCommand* Command);


protected:
	// This is set to current selection during CanExecuteSelectionCommand/ExecuteSelectionCommand, to keep the UObject alive
	// Not expected to be used outside that context
	UPROPERTY()
	TObjectPtr<UGeometrySelectionEditCommandArguments> SelectionArguments;

	// apply ProcessFunc to active selections via handles, perhaps should be public?
	UE_API void ProcessActiveSelections(TFunctionRef<void(FGeometrySelectionHandle)> ProcessFunc);



	//
	// Undo/Redo
	//
public:
	UE_API virtual void ApplyChange(IGeometrySelectionChange* Change);
	UE_API virtual void RevertChange(IGeometrySelectionChange* Change);


	//
	// Debugging stuff
	//
public:
	/** Print information about the active selection using UE_LOG */
	UE_API virtual void DebugPrintSelection();
	/** Visualize the active selection using PDI drawing */
	UE_API virtual void DebugRender(IToolsContextRenderAPI* RenderAPI);
	
	/** Set the colors to be used during mesh element selection for:
	 * Unselected elements, Hover over selection, Hover over non-selection, Selected elements, and Soft selected element
	 */
	UE_API void SetSelectionColors(FLinearColor UnselectedCol, FLinearColor HoverOverSelectedCol, FLinearColor HoverOverUnselectedCol, FLinearColor GeometrySelectedCol, FLinearColor SoftSelectedCol);

	/** Disconnect and cleanup for PreviewGeometry object. */
	UE_API void DisconnectPreviewGeometry();
	
protected:

	// current selection mode settings

	EGeometryElementType SelectionElementType = EGeometryElementType::Face;
	UE_API void SetSelectionElementTypeInternal(EGeometryElementType NewElementType);

	EMeshTopologyMode MeshTopologyMode = EMeshTopologyMode::None;
	UE_API void SetMeshTopologyModeInternal(EMeshTopologyMode NewTopologyMode);

	UE_API UE::Geometry::FGeometrySelectionHitQueryConfig GetCurrentSelectionQueryConfig() const;

public:
	// Selection Filters
	UE_API void SetHitBackFaces(bool bInHitBackFaces);
	bool GetHitBackFaces() const { return bHitBackFaces; }

	// Soft Selection
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API void SetSoftSelectionEnabled(bool bEnable);

	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	bool IsSoftSelectionEnabled() const { return bEnableSoftSelection; };

	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API void SetSoftSelectionRadius(double Radius);

	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	double GetSoftSelectionRadius() const {return SoftSelectionRadius; }

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	UE_API void SetSoftSelectionDistanceMode(IGeometrySelector::ESoftSelectionDistanceMode InMode);

	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	IGeometrySelector::ESoftSelectionDistanceMode GetSoftSelectionDistanceMode() const { return SoftSelectionDistanceMode; }
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

protected:
	bool bHitBackFaces = true;
	
	// ITF references

	UPROPERTY()
	TObjectPtr<UInteractiveToolsContext> ToolsContext;

	IToolsContextTransactionsAPI* TransactionsAPI;

	// set of registered IGeometrySelector factories
	TArray<TUniquePtr<IGeometrySelectorFactory>> Factories;



	// FGeometrySelectionTarget is the set of information tracked for a given "Active Target",
	// which is (eg) a Mesh Component or other external object that "owns" selectable Geometry.
	// This includes the IGeometrySelector for that target, the SelectionEditor, and the active Selection
	struct FGeometrySelectionTarget
	{
	public:
		FGeometryIdentifier TargetIdentifier;		// identifier of target object used to initialize the selection (eg Component/etc)
		FGeometryIdentifier SelectionIdentifer;		// identifier of object that is being selected-on, eg UDynamicMesh/etc

		TUniquePtr<IGeometrySelector> Selector;					// active Selector

		FGeometrySelection Selection;				// current Selection
		TUniquePtr<FGeometrySelectionEditor> SelectionEditor;	// active Selection Editor

		FDelegateHandle OnGeometryModifiedHandle;	// hooked up to (eg) UDynamicMesh OnMeshChanged, etc
	};

	// Set of active Selection Targets updated by SynchronizeActiveTargets / etc
	TArray<TSharedPtr<FGeometrySelectionTarget>> ActiveTargetReferences;

	// map from external Identifiers to active Selection Targets
	TMap<FGeometryIdentifier, TSharedPtr<FGeometrySelectionTarget>> ActiveTargetMap;


	TArray<FGeometryIdentifier> UnlockedTargets;


	//
	// Support for cached FGeometrySelectionTarget / IGeometrySelectors. 
	// The intention here is to reduce the overhead on selection changes.
	// Functional, but needs to be smarter.
	//
	TMap<FGeometryIdentifier, TSharedPtr<FGeometrySelectionTarget>> TargetCache;
	UE_API void SleepOrShutdownTarget(TSharedPtr<FGeometrySelectionTarget> Target, bool bForceShutdown);
	UE_API TSharedPtr<FGeometrySelectionTarget> GetCachedTarget(FGeometryIdentifier Identifier, const IGeometrySelectorFactory* UseFactory);
	UE_API void ResetTargetCache();
	UE_API void SetTargetsOnUndoRedo(TArray<FGeometryIdentifier> NewTargets);
	UE_API TArray<FGeometryIdentifier> GetCurrentTargetIdentifiers() const;

	UE_API void OnTargetGeometryModified(IGeometrySelector* Selector);

	
	// todo [nickolas.drake]: cane we move CachedSelectionRenderElements, Cached[Un]selectedPreviewRenderElements, and bSelectionRenderCachesDirty to private?

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry = nullptr;

	/**
	 * Calls the CreateOrUpdate function for Triangle, Line, and Point sets to build sets for PreviewGeometry
	 * @param SelectionParams struct containing information (color, depth bias, line thickness, identifier) for the type of selection we are currently
	 * building a set for (as Selected, Selectable, HoverSelected, HoverUnselected all have different visual information needed to contruct their sets)
	 */
	UE_API void CreateOrUpdateAllSets(const FGeometrySelectionElements& Elements, const FMeshElementSelectionParams SelectionParams) const;
	
	TArray<FGeometrySelectionElements> CachedSelectionRenderElements;				// Cached 3D geometry for current selection
	UE_API void UpdateSelectionRenderCacheOnTargetChange();
	UE_API void RebuildSelectionRenderCaches();

    FGeometrySelection ActivePreviewSelection;
	FGeometrySelection SelectedActivePreviewSelection;
	FGeometrySelection UnselectedActivePreviewSelection;
	
	FGeometrySelectionElements CachedSelectedPreviewRenderElements;		// Cached 3D geometry for active preview elements that are in the current selection
	FGeometrySelectionElements CachedUnselectedPreviewRenderElements;  // Cached 3D geometry for active preview elements that are NOT in the current selection
    UE_API void ClearActivePreview();

	inline static TStaticArray<FString, 3> UnselectedSetIds = { "Unselected_Vertices", "Unselected_Lines", "Unselected_Triangles"};
	inline static TStaticArray<FString, 3> HoverOverSelectedSetIds = {  "HoverSelected_Vertices", "HoverSelected_Lines", "HoverSelected_Triangles"};
	inline static TStaticArray<FString, 3> HoverOverUnselectedSetIds = { "HoverUnselected_Vertices", "HoverUnselected_Lines", "HoverUnselected_Triangles" };
	inline static TStaticArray<FString, 3> SelectedSetIds = {"Selected_Vertices", "Selected_Lines", "Selected_Triangles"};
	inline static TStaticArray<FString, 3> SoftSelectedSetIds = {"SoftSelected_Vertices", "SoftSelected_Lines", "SoftSelected_Triangles"};

	UPROPERTY()
	FMeshElementSelectionParams UnselectedParams {UnselectedSetIds, 5.f, 2.f, 8.f};
	UPROPERTY()
	FMeshElementSelectionParams HoverOverSelectedParams {HoverOverSelectedSetIds, 10.f, 6.f, 10.f};
	UPROPERTY()
	FMeshElementSelectionParams HoverOverUnselectedParams {HoverOverUnselectedSetIds, 10.f, 6.f, 10.f};
	UPROPERTY()
	FMeshElementSelectionParams SelectedParams {SelectedSetIds, 6.f, 6.f, 10.f};
	UPROPERTY()
	FMeshElementSelectionParams SoftSelectedParams {SoftSelectedSetIds, 5.5f, 2.f, 10.f};

	
	// various change types need internal access

	friend class FGeometrySelectionManager_SelectionTypeChange;
	friend class FGeometrySelectionManager_ActiveTargetsChange;
	
	friend class FGeometrySelectionManager_TargetLockStateChange;
	UE_API void SetTargetLockStateOnUndoRedo(FGeometryIdentifier TargetIdentifier, bool bLocked);


	// support for complex selection changes that are driven externally
	bool bInTrackedSelectionChange = false;
	FGeometrySelectionUpdateConfig ActiveTrackedUpdateConfig;
	FGeometrySelection ActiveTrackedSelection;
	UE::Geometry::FGeometrySelectionDelta InitialTrackedDelta;
	UE::Geometry::FGeometrySelectionDelta ActiveTrackedDelta;
	bool bSelectionModifiedDuringTrackedChange = false;

private:

	//
	// 3D geometry for element selections of each ActiveTarget is cached
	// to improve rendering performance
	//

	UE_API void RebuildSelectionRenderCache();
	
	TArray<FGeometrySelectionElements> CachedSelectableRenderElements;				// Cached 3D geometry for all selectable elements
	UE_API void RebuildSelectableRenderCache();
	
	UE_API void RebuildPreviewRenderCache();

	TArray<FGeometrySelectionElements> CachedSoftSelectionRenderElements;
	UE_API void RebuildSoftSelectionRenderCache();

	EEnumerateRenderCachesDirtyFlags RenderCachesDirtyFlags = EEnumerateRenderCachesDirtyFlags::None;

	UE_API void RemoveAllSets() const;

	UE_API void RebuildSelectable() const;

	// Tracks saved selection state. Useful when the selection is temporarily cleared (e.g., for a tool)
	struct FSavedSelection
	{
		TArray<TWeakObjectPtr<UObject>> Targets;
		TArray<FGeometrySelection> Selections;
		void Empty()
		{
			Targets.Empty();
			Selections.Empty();
		}
		void Reset()
		{
			Targets.Reset();
			Selections.Reset();
		}
	};
	FSavedSelection SavedSelection;


	void UpdateSoftSelectionOnSelectionModified();
	void UpdateSoftSelectionOnConfigModified(bool bRecomputeDistances);

	// Soft Selection
	bool bEnableSoftSelection = false;

	double SoftSelectionRadius = 20.0;

	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	IGeometrySelector::ESoftSelectionDistanceMode SoftSelectionDistanceMode = IGeometrySelector::ESoftSelectionDistanceMode::Surface;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
};





#undef UE_API
