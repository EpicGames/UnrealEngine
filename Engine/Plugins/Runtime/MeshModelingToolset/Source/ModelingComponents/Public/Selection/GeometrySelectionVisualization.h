// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoxTypes.h"
#include "HAL/Platform.h" // Required by MODELINGCOMPONENTS_API

class UPreviewGeometry;
class UGeometrySelectionVisualizationProperties;

namespace UE::Geometry
{
	struct FGeometrySelection;
	class FDynamicMesh3;
	class FGroupTopology;

	/**
	 * Initialize the geometry selection preview geometry. Call this in your tool Setup function
	 */
	void MODELINGCOMPONENTS_API InitializeGeometrySelectionVisualization(
		UPreviewGeometry* PreviewGeom,
		UGeometrySelectionVisualizationProperties* Settings,
		const FDynamicMesh3& SourceMesh,
		const FGeometrySelection& Selection,
		const FGroupTopology* GroupTopology = nullptr,
		const FGeometrySelection* TriangleVertexSelection = nullptr,
		const TArray<int>* TriangleROI = nullptr,
		const FTransform* ApplyTransform = nullptr);

	/**
	 * Update the geometry selection preview geometry according to Settings. Call this in your tool OnTick function
	 */
	void MODELINGCOMPONENTS_API UpdateGeometrySelectionVisualization(
		UPreviewGeometry* PreviewGeom,
		UGeometrySelectionVisualizationProperties* Settings);

	/**
	 * Computes the axis aligned bounding box from a given triangle region of interest, useful to compute a DepthBias
	 */
	FAxisAlignedBox3d MODELINGCOMPONENTS_API ComputeBoundsFromTriangleROI(
		const FDynamicMesh3& Mesh,
		const TArray<int32>& TriangleROI,
		const FTransform3d* Transform = nullptr);

	/**
	 * Computes the axis aligned bounding box from a given vertex region of interest, useful to compute a DepthBias
	 */
	FAxisAlignedBox3d MODELINGCOMPONENTS_API ComputeBoundsFromVertexROI(
		const FDynamicMesh3& Mesh,
		const TArray<int32>& VertexROI,
		const FTransform3d* Transform = nullptr);

} // namespace UE::Geometry
