// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "ConvexVolume.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"
#include "FrameTypes.h"
#include "InputState.h"
#include "ToolContextInterfaces.h"

class IToolsContextRenderAPI;
class IToolsContextTransactionsAPI;



/**
 * FGeometryIdentifier is used to represent a specific Geometry-containing object.
 */
struct FGeometryIdentifier
{
	/**
	 * ETargetType specifies the category of target object
	 */
	enum class ETargetType
	{
		Unknown = 0,
		/** TargetObject is a UPrimitiveComponent subclass */
		PrimitiveComponent = 1,
		/** TargetObject is a UObject container for a Mesh, like UStaticMesh or UDynamicMesh */
		MeshContainer = 2,

		UserDefinedBase = 32
	};

	/**
	 * EObjectType is the specific UClass/UObject Type of TargetObject
	 */
	enum class EObjectType
	{
		Unknown = 0,
		DynamicMeshComponent = 1,
		DynamicMesh = 2,
		BrushComponent = 3,
		StaticMeshComponent = 4,

		UserDefinedBase = 64
	};

	/**
	 * Target Type specifies the category of TargetObject
	 */
	ETargetType TargetType = ETargetType::PrimitiveComponent;

	/**
	 * Object Type specifies the specific UClass/UObject Type of TargetObject
	 */
	EObjectType ObjectType = EObjectType::Unknown;	

	/**
	 * TargetObject is *optional*, depending on the identifier and situation it may or may not exist
	 */
	TWeakObjectPtr<UObject> TargetObject = nullptr;

	FGeometryIdentifier() = default;

	/**
	 * @return true if TargetObject is defined and still exists
	 */
	bool HasObject() const { return TargetObject.IsValid(); }


	bool operator==(const FGeometryIdentifier& Other) const
	{
		return Equals(Other);
	}
	bool Equals(const FGeometryIdentifier& Other) const 
	{
		return TargetType == Other.TargetType && ObjectType == Other.ObjectType && TargetObject == Other.TargetObject;
	}

	friend uint32 GetTypeHash(const FGeometryIdentifier& Object)
	{
		return HashCombine(
			HashCombine(GetTypeHash(Object.TargetType), GetTypeHash(Object.ObjectType)), 
			GetTypeHash(Object.TargetObject));
	}

	/**
	 * @return TargetObject cast to template ObjectTypeT, or nullptr if it is not defined or not this type
	 */
	template<typename ObjectTypeT>
	ObjectTypeT* GetAsObjectType() const
	{
		return Cast<ObjectTypeT>(TargetObject.Get());
	}

	/**
	 * @return TargetObject cast to template PrimitiveComponent subclass ComponentTypeT, or nullptr if it is not defined or not this type
	 */
	template<typename ComponentTypeT>
	ComponentTypeT* GetAsComponentType() const
	{
		if (ensure(TargetType == ETargetType::PrimitiveComponent) == false)
		{
			return nullptr;
		}
		return Cast<ComponentTypeT>(TargetObject.Get());
	}


	/**
	 * @return a FGeometryIdentifier for a given UPrimitiveComponent with the specified ObjectType (may be EObjectType::Unknown)
	 */
	static FGeometryIdentifier PrimitiveComponent(UPrimitiveComponent* Component, EObjectType ObjectType)
	{
		FGeometryIdentifier Identifier;
		Identifier.TargetType = ETargetType::PrimitiveComponent;
		Identifier.ObjectType = ObjectType;
		Identifier.TargetObject = Component;
		return Identifier;
	}

};



class IGeometrySelector;


/**
 * FGeometrySelectionHandle stores a Selection and an Identifier for
 * the geometry object that the Selection is defined relative to
 */
struct FGeometrySelectionHandle
{
	FGeometryIdentifier Identifier;
	const UE::Geometry::FGeometrySelection* Selection;

	/** optional Selector for SelectionHandle */
 	IGeometrySelector* Selector = nullptr;
};



/**
 * IGeometrySelectionTransformer is a transient object that is created by an IGeometrySelector
 * to provide external code with a way to manipulate/transform an active Selection.
 * This allows (eg) a Gizmo to be hooked up to an active Selection without either
 * explicitly knowing about each other. 
 * 
 * You should not construct an IGeometrySelectionTransformer implementation directly.
 * IGeometrySelector::InitializeTransformation() will create a suitable IGeometrySelectionTransformer
 * implementation if transformation is supported, and IGeometrySelector::ShutdownTransformation() is used
 * to return/destroy it. Only one active Transformer is supported on a given Selector.
 * 
 * The API functions BeginTransform/UpdateTransform/EndTransform are called by
 * external code with the relevant information, see details below.
 * 
 * The current API assumes the transformation will be applied "per vertex", ie that
 * the concept of "vertices" exists and that the transform is done pointwise.
 * This may be expanded/revisited in future.
 */
class IGeometrySelectionTransformer
{
public:
	virtual ~IGeometrySelectionTransformer() = default;

	/** @return the IGeometrySelector that owns this Transformer */
	virtual IGeometrySelector* GetSelector() const = 0;

	/**
	 * Start a transform (eg called at beginning of a user interaction with a 3D gizmo).
	 * The caller provides the active Selection that is intended to be transformed.
	 */
	virtual void BeginTransform(const UE::Geometry::FGeometrySelection& Selection) = 0;

	/**
	 * Update the active transform (eg called on each mouse update during 3D gizmo usage).
	 * The caller provides a VertexTransformFunc callback, which will only be called during
	 * the call to UpdateTransform to compute new 3D positions for 3D points. 
	 */
	UE_DEPRECATED(5.8, "Use the UpdateTransformation function that takes a lambda that includes per-vertex weight instead")
	virtual void UpdateTransform( TFunctionRef<FVector3d(int VertexID, const FVector3d& InitialPosition, const FTransform& WorldTransform)> VertexTransformFunc )
	{
		auto VertexTransformFuncWithUnusedWeight = [&](int32 VertexID, double Weight, const FVector3d& Position, const FTransform& Transform)
			{
				return VertexTransformFunc(VertexID, Position, Transform);
			};
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		UpdateTransform(VertexTransformFuncWithUnusedWeight);
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	};

	/**
	 * Update the active transform (eg called on each mouse update during 3D gizmo usage).
	 * The caller provides a VertexTransformFunc callback, which will only be called during
	 * the call to UpdateTransform to compute new 3D positions for 3D points. 
	 */
	UE_EXPERIMENTAL(5.8, "Per-vertex weighted transformation is experimental while the API is under active development.")
	virtual void UpdateTransform( TFunctionRef<FVector3d(int VertexID, double Weight, const FVector3d& InitialPosition, const FTransform& WorldTransform)> VertexTransformFunc ) = 0;
	
	/**
	 * Optional function called per-frame to visualize the active Transformation
	 */
	virtual void PreviewRender(IToolsContextRenderAPI* RenderAPI) { }

	/**
	 * Finish a transform (eg called on mouse-up during 3D gizmo interaction). 
	 * The caller provides a TransactionsAPI implementation which the Transformer/Selector can use to emit change events
	 */
	virtual void EndTransform(IToolsContextTransactionsAPI* TransactionsAPI) = 0;
};




/**
 * IGeometrySelector is a base API definition for, roughly, "an object that knows how to
 * select elements and/or work with element selections on some other object", where "element"
 * is some "part" of the object, like a mesh edge/vertex/triangle, or intermediate structure (eg a polygroup).
 * 
 * Essentially the purpose of IGeometrySelector is to provide the bridge between "object that can have a selection"
 * and "the active selection on that object". The Selection is stored as a FGeometrySelection which has 
 * no reference to a type or target object, and standard target objects (eg a StaticMesh, Volume, etc)
 * have no inherent knowledge of, or support for, low-level geometry selections. 
 * So the Selector provides these capabilities, as well as related functionality like 
 * being able to transform an active selection and emit necessary transactions/etc. 
 * 
 * Generally clients will not create a IGeometrySelector instance directly. For a given
 * implmentation, a IGeometrySelectorFactory is registered with the UGeometrySelectionManager,
 * and it creates instances based on the active selection.
 * 
 * Where possible, the UGeometrySelectionManager will also "sleep" a IGeometrySelector for a given
 * target object instead of destroying it when the object is deselected, and then "restore"
 * it if the object is re-selected. This avoids frequent churn in Selectors as the user
 * clicks between objects.
 * 
 * Selector implementations should ideally have a strategy to lazy-construct any large data
 * structures (eg mesh AABBTrees, etc). Again this is to avoid excessive overhead when the
 * user (for example) selects a large object but has no intention of doing element-level selection.
 * 
 */
class IGeometrySelector
{
public:
	using FGeometrySelection = UE::Geometry::FGeometrySelection;
	using FGeometrySelectionEditor = UE::Geometry::FGeometrySelectionEditor;
	using FGeometrySelectionBounds = UE::Geometry::FGeometrySelectionBounds;
	using FGeometrySelectionElements = UE::Geometry::FGeometrySelectionElements;
	using FGeometrySelectionUpdateConfig = UE::Geometry::FGeometrySelectionUpdateConfig;
	using FGeometrySelectionUpdateResult = UE::Geometry::FGeometrySelectionUpdateResult;

	virtual ~IGeometrySelector() = default;

	/**
	 * Disconnect the Selector from its target object and prepare for deletion.
	 */
	virtual void Shutdown() = 0;


	/**
	 * The SelectionManager will call SupportsSleep() to determine if an IGeometrySelector
	 * needs to be destroyed when the target object is deselected, or if it can be
	 * re-used (generally more efficient)
	 */
	virtual bool SupportsSleep() const { return false; }

	/**
	 * Temporarily disable the Selector, with the intention of re-enabling it (via Restore()) in
	 * the future. Sleep should (currently) free any large memory data structures.
	 * @return true if Sleep was successful, if not the Selector will likely be Shutdown
	 */
	virtual bool Sleep() { return false; }

	/**
	 * Restore a Selector that has had Sleep() called on it. 
	 * @return true on success
	 */
	virtual bool Restore() { return false; }

	/**
	 * @return a FGeometryIdentifier for this Selector. 
	 */
	virtual FGeometryIdentifier GetIdentifier() const = 0;


	//
	// Locking support. When a Selector is locked, it will not allow selection changes.
	// The primary use of this is to support configurable enable/disable of selection on 
	// certain types of objects (eg on static meshes). Although it could potentially be
	// supported at a higher level, this adds a lot of complications to state management
	// that are avoided if the Selector is always created.
	//

	virtual bool IsLockable() const { return false; }
	virtual bool IsLocked() const { return false; }
	virtual void SetLockedState(bool bLocked) { }



	enum class EInitializeSelectionMode
	{
		// create a selection of all geometry elements
		All,
		// create a selection of all geometry elements connected to a ReferenceSelection
		Connected,
		// create a selection of all geometry elements directly connected to a ReferenceSelection (ie adjacent/one-ring/etc)
		AdjacentToBorder
	};

	/**
	 * Populate a Selection using the provided ReferenceMode, with a Predicate and optional ReferenceSelection.
	 * Note that this method does not modify/access any selection that might be "owned" by the Selector.
	 * It is part of the IGeometrySelector API because often a Selector has access to internal data structures
	 * (like a Mesh, GroupTopology, etc) that are required to initialize a Selection.
	 * @param SelectionInOut the Selection to initialize, the input Geometry/Topology types define what type of element will be selected
	 * @param SelectionIDPredicate only Elements that pass this predicate will be included in SelectionInOut
	 * @param InitializeMode method to use to initialize SelectionInOut
	 * @param ReferenceSelection optional Selection to select "relative to", based on the InitializeMode
	 */
	virtual void InitializeSelectionFromPredicate(	
		FGeometrySelection& SelectionInOut,
		TFunctionRef<bool(UE::Geometry::FGeoSelectionID)> SelectionIDPredicate,
		EInitializeSelectionMode InitializeMode = EInitializeSelectionMode::All,
		const FGeometrySelection* ReferenceSelection = nullptr 
	) = 0;


	/**
	 * Combine FromSelection with the current Selection owned/referenced by the Selector, 
	 * using the provided SelectionEditor and UpdateConfig. This can be used to explicitly
	 * initialize the Selection, restore it to a previous state for undo/redo, etc.
	 * @param bAllowConversion if true, FromSelection's Geometry/Topology type will be converted to the internal types of the Selector/SelectionEditor
	 * @param SelectionDelta if passed as non-null, this Delta will be initialized based on the change in the internal Selection
	 */
	virtual void UpdateSelectionFromSelection(	
		const FGeometrySelection& FromSelection,
		bool bAllowConversion,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		UE::Geometry::FGeometrySelectionDelta* SelectionDelta = nullptr
	) = 0;


	struct FWorldRayQueryInfo
	{
		FRay3d WorldRay;
		FViewCameraState CameraState;
	};

	/**
	 * Check for intersection between a world-space Ray with the Selector's target object and return a FInputRayHit result.
	 * RayHitTest() must return true for other raycast-based functions like UpdateSelectionViaRaycast() to be called.
	 * @return true on hit, false on miss
	 */
	virtual bool RayHitTest(
		const FWorldRayQueryInfo& RayInfo,
		UE::Geometry::FGeometrySelectionHitQueryConfig QueryConfig,
		FInputRayHit& HitResultOut
	) = 0;

	/**
	 * Update the active selection based on a world-space Ray. Should only be called if RayHitTest() returns true.
	 * Uses the provided SelectionEditor and UpdateConfig to do the update
	 * Information about any changes actually made to the selection via the SelectionEditor are returned in ResultOut
	 */
	virtual void UpdateSelectionViaRaycast(	
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut 
	) = 0;

	/**
	 * Query the Selector for a potential selection based on world-space Ray. The intention of this query
	 * is that it be used for "selection preview", eg like hover-highlighting. The PreviewEditor will
	 * always be updated in 'Add' mode, and currently no result information will be returned. 
	 */
	virtual void GetSelectionPreviewForRaycast(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& PreviewEditor
	) = 0;
	
	/**
	 * Query the Selector for a potential selection based on world-space Ray. The intention of this query
	 * is that it be used for "selection preview", eg like hover-highlighting. The PreviewEditor will
	 * always be updated in 'Add' mode, and currently no result information will be returned.
	 *
	 * CurrentSelection is provided for selectors which have conditionally selectable geometry.
	 * While not enforced, the expectation is that we add elements to the PreviewEditor only if
	 * those elements are considered selectable with respect to CurrentSelection.
	 */
	UE_EXPERIMENTAL(5.7, "GetConditionalSelectionPreviewForRaycast is experimental while the API is under active development.")
	virtual void GetConditionalSelectionPreviewForRaycast(
		const FWorldRayQueryInfo& RayInfo,
		FGeometrySelectionEditor& PreviewEditor,
		const FGeometrySelection* CurrentSelection 
	) { GetSelectionPreviewForRaycast(RayInfo, PreviewEditor); }



	struct FWorldShapeQueryInfo
	{
		FConvexVolume Convex;
		FViewCameraState CameraState;
	};

	/**
	 * Update the active selection based on a world-space Shape.
	 * Uses the provided SelectionEditor and UpdateConfig to do the update
	 * Information about any changes actually made to the selection via the SelectionEditor are returned in ResultOut
	 */
	virtual void UpdateSelectionViaShape(	
		const FWorldShapeQueryInfo& ShapeInfo,
		FGeometrySelectionEditor& SelectionEditor,
		const FGeometrySelectionUpdateConfig& UpdateConfig,
		FGeometrySelectionUpdateResult& ResultOut 
	) = 0;


	enum class UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.") ESoftSelectionDistanceMode : uint8
	{
		Surface = 0,
		Volume = 1,
	};

	struct UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.") FUpdateSoftSelectionConfig
	{
		// When true, the selector should force a recompute of any cached per-vertex distance data.
		bool bRecomputeDistances = true;

		// When false, the update call just clears soft selection.
		bool bEnabled = false;

		// Distance to expand. Interpreted as either geodesic (Surface) or Euclidean (Volume) depending on DistanceMode.
		double Radius = 20.0;

		// Distance metric used to expand the soft selection.
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		ESoftSelectionDistanceMode DistanceMode = ESoftSelectionDistanceMode::Surface;
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS	
	};
	
	/**
	 * Generate the soft selection based on the input selection.
	 */
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	UE_EXPERIMENTAL(5.8, "Soft Selection is experimental while the API is under active development.")
	virtual void UpdateSoftSelection(FGeometrySelection& InSelection, const FUpdateSoftSelectionConfig& InSoftSelectionConfig) = 0;
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/**
	 * Converts selection from one type to another type
	 */
	UE_EXPERIMENTAL(5.8, "This function is experimental while the API is under active development.")
	virtual bool ConvertSelection(const FGeometrySelection& FromSelectionIn, FGeometrySelection& ToSelectionOut, const UE::Geometry::EEnumerateSelectionConversionParams ConversionParams) = 0;

	/**
	 * @return the World transform for the Selector's target object
	 */
	virtual FTransform GetLocalToWorldTransform() const = 0;

	/**
	 * Compute a 3D frame for the provided Selection. For example this could simply be a frame aligned to the XYZ axes at the center
	 * of the selection bounding-box, or something more sophisticated, like aligned to the tangent-space of the selected element(s), etc.
	 * @param bTransformToWorld if true the resulting SelectionFrameOut will be in world-space, based on GetLocalToWorldTransform()
	 */
	virtual void GetSelectionFrame(const FGeometrySelection& Selection, UE::Geometry::FFrame3d& SelectionFrameOut, bool bTransformToWorld) = 0;

	/**
	 * Compute a 3D frame for the Selector's target object
	 */
	virtual void GetTargetFrame(const FGeometrySelection& Selection, UE::Geometry::FFrame3d& SelectionFrameOut) = 0;

	/**
	 * Accumulate the bounds of the provided Selection in the provided BoundsInOut. BoundsInOut is not cleared.
	 * @param bTransformToWorld if true each bounded point/element will be transformed to World space before being "contained" in BoundsInOut
	 */
	virtual void AccumulateSelectionBounds(const FGeometrySelection& Selection, FGeometrySelectionBounds& BoundsInOut, bool bTransformToWorld) = 0;

	/** Prefer AccumulateSelectionElements with Flags parameter. */
	UE_DEPRECATED(5.5, "AccumulateSelectionElements which takes a bIsForPreview boolean is deprecated."
				"Please use the function of the same name which takes EEnumerateSelectionMapping flags instead")
	virtual void AccumulateSelectionElements(const FGeometrySelection& Selection, FGeometrySelectionElements& ElementsInOut, bool bTransformToWorld, bool bIsForPreview) = 0;
	
	/**
	 * Accumulate geometric elements (currently 3D triangles, line segments, and points) for the provided Selection in the provided ElementsInOut. 
	 * ElementsInOut is not cleared.
	 * @param bTransformToWorld if true each geometric element will be transformed to World space, based on GetLocalToWorldTransform()
	 * @param Flags determines which pieces of the geometry should be rendered
	 */
	virtual void AccumulateSelectionElements(const FGeometrySelection& Selection, FGeometrySelectionElements& ElementsInOut, bool bTransformToWorld, UE::Geometry::EEnumerateSelectionMapping Flags = UE::Geometry::EEnumerateSelectionMapping::Default) = 0;

	/**
	 * Accumulate all geometric elements (currently 3D triangles, line segments, and points) in the provided ElementsInOut
	 * by initializing vertex, edge, and face selections using the provided predicate and then accumulating elements for those selections.
	 * ElementsInOut is not cleared.
	 * @param bTransformToWorld if true each geometric element will be transformed to World space, based on GetLocalToWorldTransform()
	 * @param bIsForPreview if true, geometry is being collected for a preview of selection. Selector may return simplified geometry in this case
	 * @param bUseGroupTopology if true, attempt to use group topology when populating ElementsInOut
	 * @param Predicate if true for a given element type and element ID, this element will be collected, otherwise this element will be skipped.
	 */
	virtual void AccumulateElementsFromPredicate(
		FGeometrySelectionElements& ElementsInOut,
		bool bTransformToWorld,
		bool bIsForPreview,
		bool bUseGroupTopology,
		TFunctionRef<bool(UE::Geometry::EGeometryElementType, UE::Geometry::FGeoSelectionID)> Predicate
	) = 0;
	
	/**
	 * Create and initialize a IGeometrySelectionTransformer for the provided Selection.
	 * IGeometrySelector retains ownership of the active Transformer, it should not be
	 * deleted by the caller, but instead ShutdownTransformation() should be called.
	 */
	virtual IGeometrySelectionTransformer* InitializeTransformation(const FGeometrySelection& Selection) = 0;

	/**
	 * Shutdown an active IGeometrySelectionTransformer. Generally only one Transformer
	 * may be active at any time, and it is an error to shut it down multiple times.
	 */
	virtual void ShutdownTransformation(IGeometrySelectionTransformer* Transformer) = 0;


public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGeometryModified, IGeometrySelector* Selector);
	/**
	 * The OnGeometryModified delegate will fire if the geometry of the Selector's Target Object
	 * is modified. For example if a UDynamicMesh is edited, OnGeometryModified will be broadcast.
	 */
	FOnGeometryModified& GetOnGeometryModifed() { return OnGeometryModifed; }

	/**
	 * The OnSelectableElementsChanged delegate will fire if the set of selectable elements have changed.
	 * The set of selectable elements is a function of the state of the selection target and the set of
	 * currently selected elements (the selection 'owned' by this selector).
	 */
	UE_EXPERIMENTAL(5.7, "Selectable element invalidation notification is experimental while the API is under active development.")
	FSimpleMulticastDelegate& GetOnSelectableElementsChanged() { return OnSelectableElementsChanged; }

protected:
	FOnGeometryModified OnGeometryModifed;
	void NotifyGeometryModified() { OnGeometryModifed.Broadcast(this); }

	UE_EXPERIMENTAL(5.7, "NotifySelectableElementsChanged is experimental while the API is under active development.")
	void NotifySelectableElementsChanged() const { OnSelectableElementsChanged.Broadcast(); }
	
private:
	FSimpleMulticastDelegate OnSelectableElementsChanged;
};


/**
 * Factory for a specific type of IGeometrySelector
 */
class IGeometrySelectorFactory
{
public:
	virtual ~IGeometrySelectorFactory() = default;

	/**
	 * @return true if this Factory can build an IGeometrySelector for the provided TargetIdentifier
	 */
	virtual bool CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const = 0;

	/**
	 * @return a new IGeometrySelector for the provided TargetIdentifier
	 */
	virtual TUniquePtr<IGeometrySelector> BuildForTarget(FGeometryIdentifier TargetIdentifier) const = 0;
};




