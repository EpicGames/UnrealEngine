// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshTopologySelector.h"
#include "GroupTopology.h"
#include "UObject/ObjectPtr.h"

#define UE_API MODELINGCOMPONENTS_API

using UE::Geometry::FGroupTopology;

/**
 * TopologyProvider using FGroupTopology 
 */

class FGroupTopologyProvider : public FTopologyProvider
{
public:

	FGroupTopologyProvider(const FGroupTopology* GroupTopology) :
		GroupTopology(GroupTopology)
	{}
	
	virtual ~FGroupTopologyProvider() = default;

	virtual int GetNumCorners() const override
	{
		return GroupTopology->Corners.Num();
	}

	virtual int GetCornerVertexID(int CornerID) const override
	{
		return GroupTopology->Corners[CornerID].VertexID;
	}

	virtual int GetNumEdges() const override
	{
		return GroupTopology->Edges.Num();
	}

	virtual void GetEdgePolyline(int EdgeID, FPolyline3d& OutPolyline) const override
	{
		GroupTopology->Edges[EdgeID].Span.GetPolyline(OutPolyline);
	}

	virtual int FindGroupEdgeID(int MeshEdgeID) const override
	{
		return GroupTopology->FindGroupEdgeID(MeshEdgeID);
	}

	const TArray<int>& GetGroupEdgeEdges(int GroupEdgeID) const override
	{
		return GroupTopology->GetGroupEdgeEdges(GroupEdgeID);
	}

	const TArray<int>& GetGroupEdgeVertices(int GroupEdgeID) const override
	{
		return GroupTopology->GetGroupEdgeVertices(GroupEdgeID);
	}

	virtual int GetGroupIDForTriangle(int TriangleID) const override
	{
		return GroupTopology->GetGroupID(TriangleID);
	}

	virtual void ForGroupSetEdges(const TSet<int32>& GroupIDs, const TFunction<void (int EdgeID)>& EdgeFunc) const
	{
		GroupTopology->ForGroupSetEdges(GroupIDs, [EdgeFunc](FGroupTopology::FGroupEdge Edge, int EdgeID)
		{
			EdgeFunc(EdgeID);
		});
	}

	virtual int GetNumGroups() const override
	{
		return GroupTopology->Groups.Num();
	}

	virtual int GetGroupIDAt(int GroupIndex) const override
	{
		return GroupTopology->Groups[GroupIndex].GroupID;
	}

	virtual FFrame3d GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame = nullptr) const override
	{
		return GroupTopology->GetSelectionFrame(Selection, InitialLocalFrame);
	}

	virtual FAxisAlignedBox3d GetSelectionBounds(const FGroupTopologySelection& Selection, TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const override
	{
		return GroupTopology->GetSelectionBounds(Selection, TransformFunc);
	}

private:

	const FGroupTopology* GroupTopology;

};

/**
 * Additional functionality for operating on FGroupTopology
 */
struct FGroupTopologyUtils
{
	bool GetNextEdgeLoopEdge(int32 IncomingEdgeID, int32 CornerID, int32& NextEdgeIDOut) const;
	void AddNewEdgeLoopEdgesFromCorner(int32 EdgeID, int32 CornerID, TSet<int32>& EdgeSet) const;
	void AddNewBoundaryLoopEdges(int32 StartEdgeID, int32 ForwardCornerID, TSet<int32>& EdgeSet) const;
	bool GetNextBoundaryLoopEdge(int32 IncomingEdgeID, int32 CornerID, int32& NextEdgeIDOut) const;
	void AddNewEdgeRingEdges(int32 StartEdgeID, int32 ForwardGroupID, TSet<int32>& EdgeSet) const;
	bool GetQuadOppositeEdge(int32 EdgeIDIn, int32 GroupID, int32& OppositeEdgeIDOut) const;
	FIndex2i GetEdgeEndpointCorners(int EdgeID) const;
	FIndex2i GetEdgeGroups(int EdgeID) const;

	const FGroupTopology* GroupTopology;
};


/**
* FGroupTopologySelector is a MeshTopologySelector subclass using an FGroupTopology to define groups, edges, and vertices
*/

class FGroupTopologySelector : public FMeshTopologySelector
{

public:

	FGroupTopologySelector() = default;

	/**
	 * Create the selector with the given Mesh and Topology.
	 * This does not create the internal data structures, this happens lazily on GetGeometrySet()
	 */
	UE_API FGroupTopologySelector(const FDynamicMesh3* Mesh, const FGroupTopology* Topology);

	/**
	 * Initialize the selector with the given Mesh and Topology.
	 * This does not create the internal data structures, this happens lazily on GetGeometrySet()
	 */
	UE_API void Initialize(const FDynamicMesh3* Mesh, const FGroupTopology* Topology);

	UE_API virtual void DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle = ECornerDrawStyle::Point) override;

	/**
	 * Using the edges in the given selection as starting points, add any "edge loops" containing the edges. An
	 * edge loop is a sequence of edges that passes through valence-4 corners through the opposite edge, and may
	 * not actually form a complete loop if they hit a non-valence-4 corner.
	 *
	 * @param Selection Selection to expand.
	 * @returns true if selection was modified (i.e., were the already selected edges part of any edge loops whose
	 *  member edges were not yet all selected).
	 */
	UE_API bool ExpandSelectionByEdgeLoops(FGroupTopologySelection& Selection);

	/**
	 * Using the edges in the given selection as starting points, add any "boundary loops" containing the edges. A
	 * boundary loop is just the sequence of edges that surrounds a hole in the mesh, or the outside of an open mesh.
	 * The loop may not complete if it hits a bowtie vertex (a vertex with more than two incident boundary edges).
	 *
	 * @param Selection Selection to expand.
	 * @returns true if selection was modified (i.e., were the already selected edges part of any boundary loops whose
	 *  member edges were not yet all selected).
	 */
	UE_API bool ExpandSelectionByBoundaryLoops(FGroupTopologySelection& Selection);

	/**
	 * Using the edges in the given selection as starting points, add any "edge rings" containing the edges. An
	 * edge ring is a sequence of edges that lie on opposite sides of quad-like faces, meaning faces that have four
	 * corners.
	 *
	 * @param Selection Selection to expand.
	 * @returns true if selection was modified (i.e., were the already selected edges part of any edge rings whose
	 *  member edges were not yet all selected).
	 */
	UE_API bool ExpandSelectionByEdgeRings(FGroupTopologySelection& Selection);

private:

	FGroupTopologyUtils GroupTopologyUtils;

};

#undef UE_API
