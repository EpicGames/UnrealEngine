// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/TriangleSetComponent.h"
#include "InputBehavior.h"
#include "InteractiveTool.h"
#include "Mechanics/RectangleMarqueeMechanic.h"
#include "Components/DynamicMeshComponent.h"
#include "Selection/GroupTopologySelector.h"
#include "TransformTypes.h"
#include "ToolDataVisualizer.h"
#include "InteractionMechanic.h"
#include "MeshTopologySelectionMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

class FMeshTopologySelectionMechanicSelectionChange;
class UPersistentMeshSelection;
class UMouseHoverBehavior;
class UMeshTopologySelectionMechanic;
class URectangleMarqueeMechanic;
class UPolyLassoMarqueeMechanic;
class USingleClickOrDragInputBehavior;

using UE::Geometry::FDynamicMeshAABBTree3;
PREDECLARE_USE_GEOMETRY_CLASS(FCompactMaps);

/**
* define the tool to use when dragging the mouse when selecting 
*/
UENUM()
enum class EMeshTopologySelectionMechanicDragTool : uint8
{
	None,
	Marquee,
	Lasso
};

UCLASS(MinimalAPI)
class UMeshTopologySelectionMechanicProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = SelectionFilter)
	bool bSelectVertices = true;

	UPROPERTY(EditAnywhere, Category = SelectionFilter)
	bool bSelectEdges = true;

	UPROPERTY(EditAnywhere, Category = SelectionFilter)
	bool bSelectFaces = true;

	/** When true, will select edge loops. Edge loops are either paths through vertices with 4 edges, or boundaries of holes. */
	UPROPERTY(EditAnywhere, Category = SelectionFilter)
	bool bSelectEdgeLoops = false;

	/** When set, will select rings of edges that are opposite each other across a quad face. */
	UPROPERTY(EditAnywhere, Category = SelectionFilter)
	bool bSelectEdgeRings = false;

	/** When set, faces that face away from the camera are ignored in selection and occlusion. Useful for working with inside-out meshes. */
	UPROPERTY(EditAnywhere, Category = AdditionalSelectionOptions, AdvancedDisplay)
	bool bHitBackFaces = true;

	UPROPERTY(EditAnywhere, Category = AdditionalSelectionOptions, AdvancedDisplay)
	EMeshTopologySelectionMechanicDragTool SelectionDragTool = EMeshTopologySelectionMechanicDragTool::Marquee;

	UPROPERTY(EditAnywhere, Category = AdditionalSelectionOptions, meta = (EditCondition = "SelectionDragTool != EMeshTopologySelectionMechanicDragTool::None", EditConditionHides))
	bool bIgnoreOcclusionForDragTools = true;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "bEnableMarquee has been deprecated, use SelectionDragTool instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bEnableMarquee has been deprecated, use SelectionDragTool instead"))
	bool bEnableMarquee = true;

	/** Determines whether vertices should be checked for occlusion in marquee select (Note: marquee select currently only works with edges and vertices) */
	UE_DEPRECATED(5.8, "bMarqueeIgnoreOcclusion has been deprecated, use bIgnoreOcclusionForDragTools instead")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bMarqueeIgnoreOcclusion has been deprecated, use bIgnoreOcclusionForDragTools instead"))
	bool bMarqueeIgnoreOcclusion = true;
#endif

	// The following were originally in their own category, all marked as AdvancedDisplay. However, since there wasn't a non-AdvancedDisplay
	// property in the category, they started out as expanded and could not be collapsed.
	// The alternative approach, used below, is to have them in a nested category, which starts out as collapsed. This works nicely.

	/** Prefer to select an edge projected to a point rather than the point, or a face projected to an edge rather than the edge. */
	UPROPERTY(EditAnywhere, Category = "AdditionalSelectionOptions|Ortho Viewport Behavior")
	bool bPreferProjectedElement = true;

	/** If the closest element is valid, select other elements behind it that are aligned with it. */
	UPROPERTY(EditAnywhere, Category = "AdditionalSelectionOptions|Ortho Viewport Behavior")
	bool bSelectDownRay = true;

	/** Do not check whether the closest element is occluded from the current view. */
	UPROPERTY(EditAnywhere, Category = "AdditionalSelectionOptions|Ortho Viewport Behavior")
	bool bIgnoreOcclusion = false;

	// When true display toggles buttons to allow switching between various selection drag tools (in the detail customization)
	bool bDisplaySelectionDragToolsControls = false;

	// Used to avoid showing some of the selection filter buttons in triedit (in the detail customization)
	bool bDisplayPolygroupReliantControls = true;

	// Whether to enable the different selection modes (in the detail customization)
	bool bCanSelectVertices = true;
	bool bCanSelectEdges = true;
	bool bCanSelectFaces = true;

	/** Invert current selection. If selection is empty, has same effect as Select All, and is similarly dependent on selection filter. */
	UFUNCTION(CallInEditor, Category = SelectionActions)
	UE_API void InvertSelection();

	/** Select all elements. Depends on selection filter, where vertices are preferred to edges to faces. */
	UFUNCTION(CallInEditor, Category = SelectionActions)
	UE_API void SelectAll();

	void Initialize(UMeshTopologySelectionMechanic* MechanicIn)
	{
		Mechanic = MechanicIn;
	}

protected:

	TWeakObjectPtr<UMeshTopologySelectionMechanic> Mechanic;
};

/*
 * Selection update type when the marquee rectangle has changed.
 */

UENUM()
enum class UE_DEPRECATED(5.0, "EMarqueeSelectionUpdateType is deprecated, use ESelectionDragToolUpdateType instead") EMarqueeSelectionUpdateType
{
	OnDrag,
	OnTickAndRelease,
	OnRelease
};

/*
 * Selection update type that defines when to update the selection when using a drag selection tool
 */
UENUM()
enum class ESelectionDragToolUpdateType: uint8
{
	OnDrag,				// will update the selection for every mouse input events ( can be multiple time per tick )
	OnTickAndRelease,	// will update the selection only once per tick and on the tool release
	OnRelease			// will only update the selection on ethe tool release
};

/**
 * Base class mechanic for selecting a subset of mesh elements (edge loops, groups, corners, etc.)
 * Internally it relies on an FMeshTopologySelector to define which type of mesh topology is selectable.
 * 
 * NOTE: Users should not use this class directly, but rather subclass it and specify a particular FMeshTopologySelector to use.
 */

UCLASS(MinimalAPI, Abstract)
class UMeshTopologySelectionMechanic : public UInteractionMechanic, public IClickBehaviorTarget, public IHoverBehaviorTarget
{
	GENERATED_BODY()
	using FFrame3d = UE::Geometry::FFrame3d;
	using FAxisAlignedBox3d = UE::Geometry::FAxisAlignedBox3d;
public:

	UE_API virtual ~UMeshTopologySelectionMechanic();

	// configuration variables that must be set before bSetup is called
	bool bAddSelectionFilterPropertiesToParentTool = true;

	UE_API void Initialize(const FDynamicMesh3* MeshIn,
		FTransform3d TargetTransformIn,
		UWorld* WorldIn,
		TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFuncIn);

	UE_API virtual UMeshTopologySelectionMechanicProperties* CreateProperties();

	UE_API virtual void Setup(UInteractiveTool* ParentTool) override;
	UE_API virtual void Shutdown() override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void DrawHUD(FCanvas* Canvas, IToolsContextRenderAPI* RenderAPI);

	UE_API virtual void Tick(float DeltaTime) override;

	/**
	 * Removes the mechanic's own click/hover handlers, which means that the parent tool
	 * will need to call UpdateSelection(), UpdateHighlight(), ClearHighlight(), and 
	 * ClearSelection() from its own hover/click handlers.
	 *
	 * Must be called during tool Setup() after calling Setup() on the mechanic to have an effect.
	 *
	 * @param ParentToolIn The parent tool, needed to be able to remove the behaviors.
	 */
	UE_API void DisableBehaviors(UInteractiveTool* ParentToolIn);

	/**
	 * Enable/disable the selection behaviors in the mechanic without permanently removing them or
	 *  shutting the mechanic down.
	 * @param bBehaviorsEnabled Determines whether the mechanic selection behaviors are active
	 * @param bRenderTopology Determines whether the topology is rendered, which is independent
	 *   of the behaviors being active.
	 */
	UE_API void SetIsEnabled(bool bBehaviorsEnabled, bool bRenderTopology = true);

	/**
	 * Update the location of the mesh after Initialize()
	 */
	UE_API void SetTransform(const FTransform3d& InTargetTransform);

	/**
	 * Sets how/when the selection updates are handled.
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "deprecated, use SetSelectionDragToolUpdateType instead")
	UE_API void SetMarqueeSelectionUpdateType(EMarqueeSelectionUpdateType InType);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Sets how/when the selection updates are handled.
	 */
	UE_API void SetSelectionDragToolUpdateType(ESelectionDragToolUpdateType InType);

	/**
	 * Sets the base priority so that tools can make sure that their own behaviors are higher
	 * priority. The mechanic will not use any priority value higher than this, but it may use
	 * lower if it needs to stagger the priorities of behaviors it uses.
	 * Can be called before or after Setup().
	 */
	UE_API void SetBasePriority(const FInputCapturePriority& Priority);

	/**
	 * Gets the current priority range used by behaviors in the mechanic. The returned pair will
	 * have the base (highest) priority as the key, and the lowest priority as the value.
	 */
	UE_API TPair<FInputCapturePriority, FInputCapturePriority> GetPriorityRange() const;


	void SetShouldSelectEdgeLoopsFunc(TFunction<bool(void)> Func)
	{
		ShouldSelectEdgeLoopsFunc = Func;
	}

	void SetShouldSelectEdgeRingsFunc(TFunction<bool(void)> Func)
	{
		ShouldSelectEdgeRingsFunc = Func;
	}

	/**
	 * By default, the shift key will cause new clicks to add to the selection. However, this
	 * can be changed by supplying a different function to check here.
	 */
	void SetShouldAddToSelectionFunc(TFunction<bool(void)> Func)
	{
		ShouldAddToSelectionFunc = Func;
	}

	/**
	 * By default, the Ctrl key will cause new clicks to remove from the existing selection.
	 * However, this can be changed by supplying a different function to check here.
	 */
	void SetShouldRemoveFromSelectionFunc(TFunction<bool(void)> Func)
	{
		ShouldRemoveFromSelectionFunc = Func;
	}

	/**
	 * Notify internal data structures that the associated MeshComponent has been modified.
	 * @param bTopologyModified if true, the underlying mesh topology has been changed. This clears the current selection.
	 */
	UE_API void NotifyMeshChanged(bool bTopologyModified);

	/**
	 * Perform a hit test on the topology using the current selection settings. Note that to work properly,
	 *  the mechanic requires that Render() be called on it, as it needs to cache camera state.
	 * In cases of hitting edges and corners, OutHit contains the following:
	 *   OutHit.FaceIndex: edge or corner id in the topology
	 *   OutHit.ImpactPoint: closest point on the ray to the hit element (Note: not a point on the element!)
	 *   OutHit.Distance: distance along the ray to ImpactPoint
	 *   OutHit.Item: if hit item was an edge, index of the segment within the edge polyline. Otherwise undefined.
	 *
	 * @param bUseOrthoSettings If true, the ortho-relevant settings for selection are used (selecting down the view ray, etc)
	 */
	UE_API bool TopologyHitTest(const FRay& WorldRay, FHitResult& OutHit, FGroupTopologySelection& OutSelection, bool bUseOrthoSettings = false);
	UE_API bool TopologyHitTest(const FRay& WorldRay, FHitResult& OutHit, bool bUseOrthoSettings = false);

	TSharedPtr<FMeshTopologySelector, ESPMode::ThreadSafe> GetTopologySelector() { return TopoSelector; }

	//
	// Hover API
	//

	/**
	 * Update the hover highlight based on the hit elements at the given World Ray
	 * @return true if something was hit and is now being hovered
	 */
	virtual bool UpdateHighlight(const FRay& WorldRay) PURE_VIRTUAL(UMeshTopologySelectionMechanic::UpdateHighlight, return false; );

	/**
	 * Clear current hover-highlight
	 */
	UE_API void ClearHighlight();


	//
	// Selection API
	//

	/** 
	 * Intersect the ray with the mesh and update the selection based on the hit element, modifier states, etc
	 * @return true if selection was modified
	 */
	virtual bool UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut) PURE_VIRTUAL(UMeshTopologySelectionMechanic::UpdateSelection, return false; );

	/**
	 * Replace the current selection with an external selection. 
	 * @warning does not check that the selection is valid!
	 *
	 * @param bBroadcast If true, issues an OnSelectionChanged delegate broadcast.
	 */
	UE_API void SetSelection(const FGroupTopologySelection& Selection, bool bBroadcast = true);

	/**
	 * Clear the current selection.
	 */
	UE_API void ClearSelection();

	UE_API void InvertSelection();
	UE_API void SelectAll();

	UE_DEPRECATED(5.5, "Calling this function on the base class is not supported. It can be called on UPolygonSelectionMechanic.")
	void GrowSelection() {};
	UE_DEPRECATED(5.5, "Calling this function on the base class is not supported. It can be called on UPolygonSelectionMechanic.")
	void ShrinkSelection() {};
	UE_DEPRECATED(5.5, "Calling this function on the base class is not supported. It can be called on UPolygonSelectionMechanic.")
	void FloodSelection() {};
	UE_DEPRECATED(5.5, "Calling this function on the base class is not supported. It can be called on UPolygonSelectionMechanic.")
	void ConvertSelectionToBorder() {};

	/** 
	 * @return true if the current selection is non-empty 
	 */
	UE_API bool HasSelection() const;

	/**
	 * @return the current selection
	 */
	const FGroupTopologySelection& GetActiveSelection() const { return PersistentSelection; }

	/**
	 * Can be used by in an OnSelectionChanged event to inspect the clicked location (i.e., the
	 * values returned by the UpdateSelection() function when the click happened).
	 */
	UE_API void GetClickedHitPosition(FVector3d& PositionOut, FVector3d& NormalOut) const;

	/**
	 * @return The best-guess 3D frame for the current select
	 * @param bWorld if true, local-to-world transform of the target MeshComponent is applied to the frame
	 */
	UE_API FFrame3d GetSelectionFrame(bool bWorld, FFrame3d* InitialLocalFrame = nullptr) const;

	/**
	 * @return Bounding box for the current selection
	 * @param bWorld if true, the box is in world space, otherwise it is in local space of the target MeshComponent
	 */
	UE_API FAxisAlignedBox3d GetSelectionBounds(bool bWorld) const;

	UE_API void SetShowSelectableCorners(bool bShowCorners);

	//
	// Change Tracking
	//

	/**
	 * Begin a change record. Internally creates a FCommandChange and initializes it with current state
	 */
	UE_API void BeginChange();

	/**
	 * End the active change and return it. Returns empty change if the selection was not modified!
	 */
	UE_API TUniquePtr<FToolCommandChange> EndChange();

	/**
	 * Ends the active change and emits it via the parent tool, if the selection has been modified.
	 */
	UE_API bool EndChangeAndEmitIfModified();

	// IClickBehaviorTarget implementation
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget implementation
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;
	UE_API virtual void OnUpdateModifierState(int ModifierID, bool bIsOn) override;

protected:
	// common handling of the selection tools (marquee/lasso) 
	void OnSelectionToolDragStarted();
	void OnSelectionToolDragFinished();

	// These get bound to marquee mechanic delegates.
	UE_API virtual void OnDragRectangleStarted();
	UE_API virtual void OnDragRectangleChanged(const FCameraRectangle& CurrentRectangle);
	UE_API virtual void OnDragRectangleFinished(const FCameraRectangle& Rectangle, bool bCancelled);

	// These get bound to lasso mechanic delegates.
	void OnPolyLassoStarted();
	void OnPolyLassoChanged(const FCameraPolyLasso& Lasso);
	void OnPolyLassoFinished(const FCameraPolyLasso& Lasso, bool bCanceled);
	
	void UpdateSelectorEnabled();

	UE_API virtual void UpdateMarqueeEnabled();
	UE_API virtual void UpdateLassoEnabled();

public:
	/** 
	 * OnSelectionChanged is broadcast whenever the selection is modified (including by FChanges, which
	 * means that called functions should not issue undo transactions. 
	 */
	FSimpleMulticastDelegate OnSelectionChanged;

	/**
	 * This is broadcast during marquee rectangle dragging if selected faces change, to allow user to
	 * dynamically update face highlighting if desired (needed because OnSelectionChanged is only 
	 * fired once the rectangle is completed, not while it is being updated).
	 */
	FSimpleMulticastDelegate OnFaceSelectionPreviewChanged;

	/**
	 * When true, the selection mechanic is currently tracking a marquee rectangle drag, and acting
	 * on the selection may be unwise until it is over (and an OnSelectionChanged event is fired).
	 */
	bool IsCurrentlyMarqueeDragging() { return bCurrentlyMarqueeDragging; }

	/**
	* Render only the MarqueeMechanic, without rendering the current selection
	*/
	UE_API void RenderMarquee(IToolsContextRenderAPI* RenderAPI);

	/**
	* Toggle rendering of edges
	*/
	void SetShowEdges(const bool bRenderEdges) { bShowEdges = bRenderEdges; };
	

	// TODO: Is it worth issuing separate callbacks in normal selection changes and in FChange ones, to
	// allow the user to bundle in some FChanges into the normal callback?

	UPROPERTY()
	TObjectPtr<UMeshTopologySelectionMechanicProperties> Properties;

protected:

	//
	// Subclass should initialize this with a concrete subclass of FMeshTopologySelector
	//
	TSharedPtr<FMeshTopologySelector, ESPMode::ThreadSafe> TopoSelector;


	bool bIsEnabled = true;

	const FDynamicMesh3* Mesh;
	TFunction<FDynamicMeshAABBTree3*()> GetSpatialFunc;

	UPROPERTY()
	TObjectPtr<UMouseHoverBehavior> HoverBehavior;

	UPROPERTY()
	TObjectPtr<USingleClickOrDragInputBehavior> ClickOrDragBehavior;

	UPROPERTY()
	TObjectPtr<URectangleMarqueeMechanic> MarqueeMechanic;

	UPROPERTY()
	TObjectPtr<UPolyLassoMarqueeMechanic> PolyLassoMechanic;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "MarqueeSelectionUpdateType is deprecated, use SelectionDragToolUpdateType (see also SetSelectionDragToolUpdateType)")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "MarqueeSelectionUpdateType is deprecated, use SelectionDragToolUpdateType (see also SetSelectionDragToolUpdateType)"))
	EMarqueeSelectionUpdateType MarqueeSelectionUpdateType = EMarqueeSelectionUpdateType::OnDrag;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	/**
	 * Selection update type (default is OnDrag) as it may not need to be triggered for every rectangle change
	 * This can drastically improve the responsiveness of the UI for meshes high density meshes.
	 * - OnDrag: calls HandleRectangleChanged when dragging
	 * - OnTick: stores a PendingSelection function when dragging and calls it when ticking and on release (if any)
	 * - OnRelease: stores a PendingSelection function when dragging and calls it on release (if any)
	*/
	UPROPERTY()
	ESelectionDragToolUpdateType SelectionDragToolUpdateType = ESelectionDragToolUpdateType::OnDrag;
	
	FInputCapturePriority BasePriority = FInputCapturePriority(FInputCapturePriority::DEFAULT_TOOL_PRIORITY);

	// When bSelectEdgeLoops is true, this function is tested to see if we should select edge loops,
	// to allow edge loop selection to be toggled with some key (setting bSelectEdgeLoops to
	// false overrides this function).
	TFunction<bool(void)> ShouldSelectEdgeLoopsFunc = []() {return true; };

	// When bSelectEdgeRings is true, this function is tested to see if we should select edge rings,
	// to allow edge ring selection to be toggled with some key (setting bSelectEdgeRings to
	// false overrides this function).
	TFunction<bool(void)> ShouldSelectEdgeRingsFunc = []() {return true; };

	TFunction<bool(void)> ShouldAddToSelectionFunc = [this]() {return bShiftToggle; };
	TFunction<bool(void)> ShouldRemoveFromSelectionFunc = [this]() {return bCtrlToggle; };;

	FTransform3d TargetTransform;

	/** Pending selection function to be called if the selection is deferred to tick/release */
	TFunction<void()> PendingSelectionFunction;

	/** Calls actual selection using the input marquee rectangle. **/
	UE_API void HandleRectangleChanged(const FCameraRectangle& InRectangle);

	void HandleLassoChanged(const FCameraPolyLasso& InLasso);

	/**
	 * Get the topology selector settings to use given the current selection settings.
	 * 
	 * @param bUseOrthoSettings If true, the topology selector will be configured to use ortho settings,
	 *  which are generally different to allow for selection of projected elements, etc.
	 */
	UE_API FGroupTopologySelector::FSelectionSettings GetTopoSelectorSettings(bool bUseOrthoSettings = false);

	FGroupTopologySelection HilightSelection;
	FGroupTopologySelection PersistentSelection;
	int32 SelectionTimestamp = 0;
	TUniquePtr<FMeshTopologySelectionMechanicSelectionChange> ActiveChange;

	// Used for box selection
	FGroupTopologySelection PreDragPersistentSelection;
	FGroupTopologySelection LastUpdateRectangleSelection;
	FGroupTopologySelector::FSelectionSettings PreDragTopoSelectorSettings;
	TMap<int32, bool> TriIsOccludedCache;
	bool bCurrentlyMarqueeDragging = false;

	FVector3d LastClickedHitPosition;
	FVector3d LastClickedHitNormal;

	/** The actor we create internally to own the DrawnTriangleSetComponent */
	UPROPERTY()
	TObjectPtr<APreviewGeometryActor> PreviewGeometryActor;

	UPROPERTY()
	TObjectPtr<UTriangleSetComponent> DrawnTriangleSetComponent;

	TSet<int> CurrentlyHighlightedGroups;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> HighlightedFaceMaterial;

	FViewCameraState CameraState;

	bool bShiftToggle = false;
	bool bCtrlToggle = false;
	static const int32 ShiftModifierID = 1;
	static const int32 CtrlModifierID = 2;

	bool bShowSelectableCorners = true;
	bool bShowEdges = true;
public:
	FToolDataVisualizer PolyEdgesRenderer;
	FToolDataVisualizer HilightRenderer;
	FToolDataVisualizer SelectionRenderer;

	friend class FMeshTopologySelectionMechanicSelectionChange;

private:
	bool bRenderTopology = true;
};


class FMeshTopologySelectionMechanicSelectionChange : public FToolCommandChange
{
public:
	FGroupTopologySelection Before;
	FGroupTopologySelection After;
	int32 Timestamp = 0;

	UE_API virtual void Apply(UObject* Object) override;
	UE_API virtual void Revert(UObject* Object) override;
	UE_API virtual FString ToString() const override;
};

#undef UE_API
