// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Spatial/GeometrySet3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Polyline3.h"
#include "GroupTopology.h"		// FGroupTopologySelection

#define UE_API MODELINGCOMPONENTS_API

struct FCameraRectangle;
struct FCameraPolyLasso;
class FToolDataVisualizer;
struct FViewCameraState;

namespace UE::Geometry
{
	class FDynamicMesh3;
}

using UE::Geometry::FGroupTopologySelection;
using UE::Geometry::FDynamicMeshAABBTree3;
using UE::Geometry::FDynamicMesh3;
using UE::Geometry::FGeometrySet3;
using UE::Geometry::FPolyline3d;
using UE::Geometry::FIndex2i;
using UE::Geometry::FFrame3d;
using UE::Geometry::FAxisAlignedBox3d;

/**
* FTopologyProvider provides selectable edge/group/corner information for FMeshTopologySelector to use. Users should implement a concrete subclass
* based on the type of mesh topology they wish to make selectable (e.g. group topology, boundary curves, etc.). Then creating a new TopologySelector
* subclass should be fairly trivial.
*/
class FTopologyProvider
{
public:
	
	virtual ~FTopologyProvider() = default;

	virtual int GetNumCorners() const = 0;
	virtual int GetCornerVertexID(int CornerID) const = 0;
	
	virtual int GetNumEdges() const = 0;
	virtual void GetEdgePolyline(int EdgeID, FPolyline3d& OutPolyline) const = 0;
	virtual int FindGroupEdgeID(int MeshEdgeID) const = 0;
	
	virtual const TArray<int>& GetGroupEdgeEdges(int GroupEdgeID) const = 0;
	virtual const TArray<int>& GetGroupEdgeVertices(int GroupEdgeID) const = 0;

	virtual int GetNumGroups() const = 0;
	virtual int GetGroupIDAt(int GroupIndex) const = 0;
	virtual int GetGroupIDForTriangle(int TriangleID) const = 0;

	virtual void ForGroupSetEdges(const TSet<int32>& GroupIDs, const TFunction<void(int EdgeID)>& EdgeFunc) const = 0;

	virtual FFrame3d GetSelectionFrame(const FGroupTopologySelection& Selection, FFrame3d* InitialLocalFrame = nullptr) const = 0;
	virtual FAxisAlignedBox3d GetSelectionBounds(const FGroupTopologySelection& Selection, TFunctionRef<FVector3d(const FVector3d&)> TransformFunc) const = 0;
};

/**
 * FMeshTopologySelector implements selection behavior for a subset of mesh elements.
 * Topological Groups, Edges, and Corners can be selected depending on the subclass of FTopologyProvider used.
 *
 * Internally a FGeometrySet3 is constructed to support ray-hit testing against the edges and corners.
 *
 * Note that to hit-test against the mesh you have to provide a your own FDynamicMeshAABBTree3.
 * You do this by providing a callback via SetSpatialSource(). The reason for this is that
 * (1) owners of an FMeshTopologySelector likely already have a BVTree and (2) if the use case
 * is deformation, we need to make sure the owner has recomputed the BVTree before we call functions
 * on it. The callback you provide should do that!
 *
 * DrawSelection() can be used to visualize a selection via line/circle drawing.
 *
 * @todo optionally have an internal mesh AABBTree that can be used when owner does not provide one?
 */

class FMeshTopologySelector
{
public:

	virtual ~FMeshTopologySelector() = default;

	//
	// Configuration variables
	// 

	/**
	 * Determines the behavior of a FindSelectedElement() call.
	 */
	struct FSelectionSettings
	{
		bool bEnableFaceHits = true;
		bool bEnableEdgeHits = true;
		bool bEnableCornerHits = true;
		
		// When false, hits of a triangles that are facing away from the camera are not considered. This can
		// be useful when working with inside-out meshes, where we usually want to select the farther, visible
		// wall.
		bool bHitBackFaces = true;

		// The following are mainly useful for ortho viewport selection:

		// Prefer an edge projected to a point rather than the point, and a face projected to an edge
		// rather than the edge.
		bool bPreferProjectedElement = false;

		// If the first element is valid, select all elements behind it that are aligned with it.
		bool bSelectDownRay = false;

		//Do not check whether the closest element is occluded.
		bool bIgnoreOcclusion = false;
	};

	/** 
	 * This is the function we use to determine if a point on a corner/edge is close enough
	 * to the hit-test ray to treat as a "hit". By default this is Euclidean distance with
	 * a tolerance of 1.0. You probably need to replace this with your own function.
	 */
	TFunction<bool(const FVector3d&, const FVector3d&, double ToleranceScale)> PointsWithinToleranceTest;

public:

	UE_API FMeshTopologySelector();


	/**
	 * Provide a function that will return an AABBTree for the Mesh.
	 * See class comment for why this is necessary.
	 */
	void SetSpatialSource(TFunction<FDynamicMeshAABBTree3*(void)> GetSpatialFunc)
	{
		GetSpatial = GetSpatialFunc;
	}

	/**
	 * Notify the Selector that the mesh has changed. 
	 * @param bTopologyDeformed if this is true, the mesh vertices have been moved so we need to update bounding boxes/etc
	 * @param bTopologyModified if this is true, topology has changed and we need to rebuild spatial data structures from scratch
	 */
	UE_API void Invalidate(bool bTopologyDeformed, bool bTopologyModified);

	/**
	 * @return the internal GeometrySet. This does lazy updating of the GeometrySet, so this function may take some time.
	 */
	UE_API const FGeometrySet3& GetGeometrySet();

	/**
	 * Find which element was selected for a given ray.
	 *
	 * @param Settings settings that determine what can be selected.
	 * @param Ray hit-test ray
	 * @param ResultOut resulting selection. Will not be cleared before use. At most one of the Groups/Corners/Edges
	 *  members will be added to.
	 * @param SelectedPositionOut The point on the ray nearest to the selected element
	 * @param SelectedNormalOut the normal at that point if ResultOut contains a selected face, otherwise uninitialized
	 * @param EdgeSegmentIdOut When not null, if the selected element is a group edge, the segment id of the segment
	 *   that was actually clicked within the edge polyline will be stored here.
	 *
	 * @return true if something was selected
	 */
	UE_API bool FindSelectedElement(const FSelectionSettings& Settings, const FRay3d& Ray, FGroupTopologySelection& ResultOut,
		FVector3d& SelectedPositionOut, FVector3d& SelectedNormalOut, int32* EdgeSegmentIdOut = nullptr);

	/**
	 * Given a camera rectangle (from a marquee selection) gives the corners/edges/groups that are contained in
	 * the rectangle. Only selects one of the types, and prefers corners to edges to groups.
	 * 
	 * For group edges, occlusion is checked at endpoints, and if either is occluded, the edge is considered occluded.
	 * For faces, occlusion is checked at triangle centroids.
	 *
	 * @param Settings Settings that determine what is selected and whether occlusion is tested.
	 * @param CameraRectangle The camera rectangle.
	 * @param TargetTransform Transform from points to world space, since the rectangle is in world space.
	 * @param ResultOut Output selection (cleared before use).
	 * @param TriIsOccludedCache Cached values of whether a given Tid has an occluded centroid, since occlusion 
	 *  values will stay the same if the rectangle is resized as long as the camera is not moved. This should
	 *  be used when doing occlusion-based marquee face selection on meshes with many triangles, because the 
	 *  occlusion checks are expensive.
	 *
	 * @return true if something was selected.
	 */
	UE_API bool FindSelectedElement(const FSelectionSettings& Settings, const FCameraRectangle& CameraRectangle, 
		FTransform3d TargetTransform, FGroupTopologySelection& ResultOut, 
		TMap<int32, bool>* TriIsOccludedCache = nullptr);

	/**
	 * Given a camera lasso (from a lasso selection) gives the corners/edges/groups that are contained in
	 * the closed lasso curve. Only selects one of the types, and prefers corners to edges to groups.
	 *
	 * For group edges, occlusion is checked at endpoints, and if either is occluded, the edge is considered occluded.
	 * For faces, occlusion is checked at triangle centroids.
	 *
	 * @param Settings Settings that determine what is selected and whether occlusion is tested.
	 * @param CameraLasso The camera lasso .
	 * @param TargetTransform Transform from points to world space, since the rectangle is in world space.
	 * @param ResultOut Output selection (cleared before use).
	 * @param TriIsOccludedCache Cached values of whether a given Tid has an occluded centroid, since occlusion
	 *  values will stay the same if the rectangle is resized as long as the camera is not moved. This should
	 *  be used when doing occlusion-based marquee face selection on meshes with many triangles, because the
	 *  occlusion checks are expensive.
	 *
	 * @return true if something was selected.
	 */
	UE_API bool FindSelectedElement(const FSelectionSettings& Settings, const FCameraPolyLasso& CameraLasso, 
		FTransform3d TargetTransform, FGroupTopologySelection& ResultOut,
		TMap<int32, bool>* TriIsOccludedCache = nullptr);
	
	enum ECornerDrawStyle
	{
		Circle,
		Point
	};

	/**
	 * Render the given selection with the default settings of the FToolDataVisualizer.
	 * Selected edges are drawn as lines, and selected corners are drawn as points or small, view-facing circles, based on the CornerDrawStyle parameter.
	 * (Currently selected faces are not drawn)
	 */
	virtual void DrawSelection(const FGroupTopologySelection& Selection, FToolDataVisualizer* Renderer, const FViewCameraState* CameraState, ECornerDrawStyle CornerDrawStyle = ECornerDrawStyle::Point) = 0;

	const FTopologyProvider* GetTopologyProvider()
	{
		return TopologyProvider.Get();
	}

public:

	// internal rendering parameters
	float VisualAngleSnapThreshold = 0.5;


protected:

	const FDynamicMesh3* Mesh = nullptr;
	TUniquePtr<FTopologyProvider> TopologyProvider = nullptr;
	TFunction<FDynamicMeshAABBTree3*(void)> GetSpatial = nullptr;

	bool bGeometryInitialized = false;
	bool bGeometryUpToDate = false;
	FGeometrySet3 GeometrySet;

	UE_API bool DoCornerBasedSelection(const FSelectionSettings& Settings, const FRay3d& Ray,
		FDynamicMeshAABBTree3* Spatial, const FGeometrySet3& TopoSpatial, 
		FGroupTopologySelection& ResultOut, FVector3d& SelectedPositionOut, int32 *EdgeSegmentIdOut) const;
	UE_API bool DoEdgeBasedSelection(const FSelectionSettings& Settings, const FRay3d& Ray,
		FDynamicMeshAABBTree3* Spatial, const FGeometrySet3& TopoSpatial,
		FGroupTopologySelection& ResultOut, FVector3d& SelectedPositionOut, int32 *EdgeSegmentIdOut) const;
};

#undef UE_API
