// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IndexTypes.h"
#include "SegmentTypes.h"


class UPrimitiveComponent;
class UInteractiveTool;
PREDECLARE_USE_GEOMETRY_STRUCT(FGroupTopologySelection);
PREDECLARE_USE_GEOMETRY_CLASS(FGroupTopology);
PREDECLARE_USE_GEOMETRY_CLASS(FCompactMaps);




/**
 * FGenericMeshSelection represents various types of selection on a Mesh.
 * This includes various types of indices that could be interpreted in different ways.
 * 
 * In addition, "Render" geometry is stored, which can be used by higher-level
 * code to draw the selection in some way (eg a selection highlight)
 * 
 * @warning this class will be removed in the future
 */
struct UE_DEPRECATED(5.4, "Use FGeometrySelection instead") FGenericMeshSelection
{
	// selection type
	enum class ETopologyType
	{
		FGroupTopology,
		FTriangleGroupTopology,
		FUVGroupTopology,
	};



	//
	// Selection representation (may be interpreted differently depending on TopologyType)
	//

	// selection type
	ETopologyType TopologyType = ETopologyType::FGroupTopology;

	// selected vertices or "corners" (eg of polygroup topology)
	TArray<int32> VertexIDs;
	// selected edges, represented as index pairs because for many selections, 
	// using a pair of vertices defining/on the edge is more reliable (due to unstable edge IDs)
	TArray<UE::Geometry::FIndex2i> EdgeIDs;
	// selected triangles/faces/regions
	TArray<int32> FaceIDs;


	//
	// Selection Target Information
	//

	// Component this selection applies to (eg that owns mesh, etc)
	UPrimitiveComponent* SourceComponent = nullptr;

	//
	// Renderable Selection Representation
	//

	// set of 3D points representing selection (in world space)
	TArray<FVector3d> RenderVertices;
	// set of 3D lines representing selection (in world space)
	TArray<UE::Geometry::FSegment3d> RenderEdges;


	/** @return selected GroupIDs */
	const TArray<int32>& GetGroupIDs() const { return FaceIDs; }

	/** @return true if selection is empty */
	bool IsEmpty() const
	{
		return VertexIDs.IsEmpty() && EdgeIDs.IsEmpty() && FaceIDs.IsEmpty();
	}

	/** @return true if selection has 3D lines that can be rendered */
	bool HasRenderableLines() const { return RenderEdges.Num() > 0; }


	bool operator==(const FGenericMeshSelection& Other) const
	{
		return SourceComponent == Other.SourceComponent
			&& TopologyType == Other.TopologyType
			&& VertexIDs == Other.VertexIDs
			&& EdgeIDs == Other.EdgeIDs
			&& FaceIDs == Other.FaceIDs;
	}

};

