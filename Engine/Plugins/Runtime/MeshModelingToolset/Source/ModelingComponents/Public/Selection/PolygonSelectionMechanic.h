// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshTopologySelectionMechanic.h"
#include "PolygonSelectionMechanic.generated.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE::Geometry { struct FGeometrySelection; }

UCLASS(MinimalAPI)
class UPolygonSelectionMechanicProperties : public UMeshTopologySelectionMechanicProperties
{
	GENERATED_BODY()

public:

	/** Grow current selection. If selection is empty, nothing happens */
	UFUNCTION(CallInEditor, Category = SelectionActions)
	UE_API void GrowSelection();

	/** Shrink current selection. If selection is empty, nothing happens */
	UFUNCTION(CallInEditor, Category = SelectionActions)
	UE_API void ShrinkSelection();

	/** Expands selection to encompass connected components. */
	UFUNCTION(CallInEditor, Category = SelectionActions)
	UE_API void FloodSelection();

	/** Converts selection to a vertex/corner selection of just the boundary vertices/corners. */
	UFUNCTION(CallInEditor, Category = SelectionActions)
	UE_API void ConvertToBorder();

	void Initialize(UPolygonSelectionMechanic* MechanicIn)
	{
		Mechanic = MechanicIn;
	}
};


/**
 * UPolygonSelectionMechanic implements the interaction for selecting a set of faces/vertices/edges
 * from a FGroupTopology.
 */
UCLASS(MinimalAPI)
class UPolygonSelectionMechanic : public UMeshTopologySelectionMechanic
{
	GENERATED_BODY()

public:

	UE_API void Initialize(const FDynamicMesh3* Mesh,
		FTransform3d TargetTransform,
		UWorld* World,
		const FGroupTopology* Topology,
		TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFunc
	);

	UE_API void Initialize(UDynamicMeshComponent* MeshComponent, const FGroupTopology* Topology,
		TFunction<FDynamicMeshAABBTree3* ()> GetSpatialSourceFunc
	);

	UE_API virtual UMeshTopologySelectionMechanicProperties* CreateProperties() override;

	/*
	 * Expands selection at the borders.
	 * 
	 * @param bAsTriangleTopology Can be set true if the topology type is FTriangleGroupTopology, to
	 *  perform the operation a bit more efficiently by using the mesh topology directly.
	 */
	UE_API void GrowSelection(bool bAsTriangleTopology);
	/*
	 * Shrinks selection at the borders.
	 * 
	 * @param bAsTriangleTopology Can be set true if the topology type is FTriangleGroupTopology, to
	 *  perform the operation a bit more efficiently by using the mesh topology directly.
	 */
	UE_API void ShrinkSelection(bool bAsTriangleTopology);

	/*
	* Invert selection
	*
	* @param bAsTriangleTopology Can be set true if the topology type is FTriangleGroupTopology, to
	*  perform the operation a bit more efficiently by using the mesh topology directly.
	*/
	UE_API void InvertSelection(bool bAsTriangleTopology);

	/**
	 * Converts selection to a vertex/corner selection of just the boundary vertices/corners
	 * 
	 * @param bAsTriangleTopology Can be set true if the topology type is FTriangleGroupTopology, to
	 *  perform the operation a bit more efficiently by using the mesh topology directly.
	 */
	UE_API void ConvertSelectionToBorderVertices(bool bAsTriangleTopology);
	
	/**
	 * Expands selection to encompass connected components.
	 */
	UE_API void FloodSelection();

	// UMeshTopologySelectionMechanic
	UE_API virtual bool UpdateHighlight(const FRay& WorldRay) override;
	UE_API virtual bool UpdateSelection(const FRay& WorldRay, FVector3d& LocalHitPositionOut, FVector3d& LocalHitNormalOut) override;

	/**
	 * Convert the Active Selection to a PolyGroup-topology FGeometrySelection, with optional CompactMaps
	 */
	UE_API void GetSelection_AsGroupTopology(UE::Geometry::FGeometrySelection& SelectionOut, const FCompactMaps* CompactMapsToApply = nullptr) const;

	/**
	 * Convert the Active Selection to a Triangle-topology FGeometrySelection, with optional CompactMaps
	 */
	UE_API void GetSelection_AsTriangleTopology(UE::Geometry::FGeometrySelection& SelectionOut, const FCompactMaps* CompactMapsToApply = nullptr) const;


	/**
	 * Initialize the Active Selection based on the provided PolyGroup-topology FGeometrySelection
	 */
	UE_API void SetSelection_AsGroupTopology(const UE::Geometry::FGeometrySelection& Selection, const bool bBroadcastSelectionChanged = false);

	/**
	 * Initialize the Active Selection based on the provided Triangle-topology FGeometrySelection
	 */
	UE_API void SetSelection_AsTriangleTopology(const UE::Geometry::FGeometrySelection& Selection, const bool bBroadcastSelectionChanged = false);


private:

	// TODO: Would be nice to get rid of this and write everything in terms of TopologySelector and TopologyProvider
	const FGroupTopology* Topology;

	// Helper to execute selection actions through existing geometry selection code.
	bool ExecuteActionThroughGeometrySelection(bool bAsTriangleTopology, const FText& TransactionName,
		TFunctionRef<bool(UE::Geometry::FGeometrySelection& SelectionToModifyInPlace)> SelectionProcessor);
};

#undef UE_API
