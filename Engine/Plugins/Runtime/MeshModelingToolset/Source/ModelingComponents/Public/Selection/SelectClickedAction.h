// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseBehaviors/BehaviorTargetInterfaces.h"
#include "ToolSceneQueriesUtil.h"

struct FHitResult;

/**
 * BehaviorTarget to do world raycast selection from a click
 * Currently used to click-select reference planes in the world
 */
class FSelectClickedAction : public IClickBehaviorTarget
{
	MODELINGCOMPONENTS_API FInputRayHit DoRayCast(const FInputDeviceRay& ClickPos, bool callbackOnHit);

public:
	USceneSnappingManager* SnapManager = nullptr;
	UWorld* World = nullptr;
	TFunction<void(const FHitResult&)> OnClickedPositionFunc = nullptr;
	TUniqueFunction<bool()> ExternalCanClickPredicate = nullptr;

	// These lists can be used to modify which components are hit tested when doing a ray cast.
	// By default, all visible components are hit tested.
	TArray<const UPrimitiveComponent*> VisibleComponentsToIgnore;
	TArray<const UPrimitiveComponent*> InvisibleComponentsToHitTest;

	// can alternately track shift modifier, however client must register this modifier w/ behavior
	static const int ShiftModifier = 1;
	bool bShiftModifierToggle = false;

	virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override
	{
		if (ExternalCanClickPredicate && ExternalCanClickPredicate() == false)
		{
			return FInputRayHit();
		}
		return DoRayCast(ClickPos, false);
	}

	virtual void OnClicked(const FInputDeviceRay& ClickPos) override
	{
		DoRayCast(ClickPos, true);
	}


	virtual void OnUpdateModifierState(int ModifierID, bool bIsOn)
	{
		if (ModifierID == ShiftModifier)
		{
			bShiftModifierToggle = bIsOn;
		}
	}

};
