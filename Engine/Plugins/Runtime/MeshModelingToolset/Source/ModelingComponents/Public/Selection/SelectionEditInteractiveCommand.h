// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveCommand.h"
#include "Selection/GeometrySelector.h"    // for FGeometrySelectionHandle
#include "SelectionEditInteractiveCommand.generated.h"

/**
 * Arguments for a UGeometrySelectionEditCommand
 */
UCLASS(MinimalAPI)
class UGeometrySelectionEditCommandArguments : public UInteractiveCommandArguments
{
	GENERATED_BODY()
public:
	FGeometrySelectionHandle SelectionHandle;
	UE::Geometry::EGeometryElementType ElementType;
	UE::Geometry::EGeometryTopologyType TopologyMode;

	bool IsSelectionEmpty() const
	{
		return SelectionHandle.Selection == nullptr || SelectionHandle.Selection->IsEmpty();
	}

	bool IsMatchingType(FGeometryIdentifier::ETargetType TargetType, FGeometryIdentifier::EObjectType EngineType) const
	{
		return SelectionHandle.Identifier.TargetType == TargetType
			|| SelectionHandle.Identifier.ObjectType == EngineType;
	}
};


UCLASS(MinimalAPI)
class UGeometrySelectionEditCommandResult : public UInteractiveCommandResult
{
	GENERATED_BODY()
public:
	FGeometrySelectionHandle SourceHandle;

	UE::Geometry::FGeometrySelection OutputSelection;
};


/**
 * UGeometrySelectionEditCommand is a command that edits geometry based on a selection.
 * Requires a UGeometrySelectionEditCommandArguments
 */
UCLASS(MinimalAPI, Abstract)
class UGeometrySelectionEditCommand : public UInteractiveCommand
{
	GENERATED_BODY()
public:
	virtual bool AllowEmptySelection() const { return false; }
	virtual bool IsModifySelectionCommand() const { return false; }

	virtual bool CanExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* SelectionArgs)
	{
		return false;
	}
	
	virtual void ExecuteCommandForSelection(UGeometrySelectionEditCommandArguments* Arguments, UInteractiveCommandResult** Result = nullptr)
	{
	}


public:

	// UInteractiveCommand API

	virtual bool CanExecuteCommand(UInteractiveCommandArguments* Arguments) override
	{
		UGeometrySelectionEditCommandArguments* SelectionArgs = Cast<UGeometrySelectionEditCommandArguments>(Arguments);
		if  (SelectionArgs && (SelectionArgs->IsSelectionEmpty() == false || AllowEmptySelection()) )
		{
			return CanExecuteCommandForSelection(SelectionArgs);
		}
		return false;
	}

	virtual void ExecuteCommand(UInteractiveCommandArguments* Arguments, UInteractiveCommandResult** Result = nullptr) override
	{
		UGeometrySelectionEditCommandArguments* SelectionArgs = Cast<UGeometrySelectionEditCommandArguments>(Arguments);
		if ( SelectionArgs && (SelectionArgs->IsSelectionEmpty() == false || AllowEmptySelection()) )
		{
			ExecuteCommandForSelection(SelectionArgs, Result);
		}
	}


};
