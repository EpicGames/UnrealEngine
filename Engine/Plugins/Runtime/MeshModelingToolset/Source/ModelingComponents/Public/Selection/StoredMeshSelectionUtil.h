// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Selections/GeometrySelection.h"

struct FToolBuilderState;
class UToolTarget;
class UInteractiveTool;
class UInteractiveToolManager;
PREDECLARE_USE_GEOMETRY_CLASS(FGroupTopology);
PREDECLARE_USE_GEOMETRY_CLASS(FCompactMaps);

//
// Utility functions for Tool implementations to use to work with Stored Selections
//
namespace UE
{
namespace Geometry
{
	/**
	 * @return true if there is currently any geometry selection available in the SceneState
	 */
	MODELINGCOMPONENTS_API bool HaveAvailableGeometrySelection(const FToolBuilderState& SceneState);

	/**
	 * Given a FToolBuilderState, find an available FGeometrySelection for the given ToolTarget (via the UGeometrySelectionManager in the ContextObjectStore)
	 * @param SelectionOut non-empty selection will be returned here
	 * @return true if a non-empty selection was found
	 */
	MODELINGCOMPONENTS_API bool GetCurrentGeometrySelectionForTarget(const FToolBuilderState& SceneState, UToolTarget* Target,
		FGeometrySelection& SelectionOut);

	/**
	 * Find an available FGeometrySelection for the given ToolTarget (via the UGeometrySelectionManager in the ContextObjectStore)
	 * @param SelectionOut non-empty selection will be returned here
	 * @return true if a non-empty selection was found
	 */
	MODELINGCOMPONENTS_API bool GetCurrentGeometrySelectionForTarget(UInteractiveToolManager* ToolManager, UToolTarget* Target,
		FGeometrySelection& SelectionOut);

	/**
	 * Find the frame and bounds representative of the current selection, if there is one.
	 * @param SceneState				Scene state info, including current selection
	 * @param OutWorldFrame				Frame centered on the current selection and (when sensible) aligned to the selected elements
	 * @param OutWorldBounds			Bounds of the current selection
	 * @param bOutIsElementSelection	Whether the selection is a mesh element selection, or a regular editor/actor selection
	 * @return	true if a selection was found, false otherwise.
	 */
	MODELINGCOMPONENTS_API bool GetCurrentSelectionWorldFrameBounds(const FToolBuilderState& SceneState, FFrame3d& OutWorldFrame, FAxisAlignedBox3d& OutWorldBounds, bool& bOutIsElementSelection);

	/**
	 * Allow a Tool to return an "output" FGeometrySelection for the given ToolTarget (presumably on Tool Shutdown)
	 * Target must be an "active" selection target in the current UGeometrySelectionManager.
	 * This function will emit a selection-change transaction and should in most cases be nested inside a tool-shutdown transaction.
	 * @return true if the selection could be set, ie Target was valid
	 */
	MODELINGCOMPONENTS_API bool SetToolOutputGeometrySelectionForTarget(UInteractiveTool* Tool, UToolTarget* Target, const FGeometrySelection& OutputSelection);


	/*
	 * Helper class that can store a list of edges as pairs of triangle IDs and {0,1,2} indices into 
	 * the triangle edge triplet, because regular FDynamicMesh edge IDs may not stay the same across 
	 * some mesh operations that preserve the vertex/triangle ID topology. For instance, deleting and 
	 * reinserting triangles during an undo/redo transaction may preserve all the relevant vids and tids,
	 * but change the eids of edges even though these edges still exist in the mesh topology.
	 *
	 * Note this way of identifying edges will still not be stable in the case of triangles being
	 * reinserted with rotated indices (e.g. a triangle "a, b, c" reinserted as "b, c, d"). Storing
	 * Vid pairs would be necessary to be robust to that case, but finding an edge from vertices is
	 * a slower operation compared to the constant-time tri/index lookup used here.
	 */
	class FMeshEdgesFromTriangleSubIndices
	{
	public:
		template <typename EidContainerType>
		void InitializeFromEdgeIDs(const FDynamicMesh3& Mesh, const  EidContainerType& Eids)
		{
			EdgeTriIndexPairs.Reset(Eids.Num());
			for (int32 Eid : Eids)
			{
				if (ensure(Mesh.IsEdge(Eid)))
				{
					int32 Tid = Mesh.GetEdgeT(Eid).A;
					FIndex3i TriEids = Mesh.GetTriEdges(Tid);
					EdgeTriIndexPairs.Add(TPair<int32, int8>(Tid, IndexUtil::FindTriIndex(Eid, TriEids)));
				}
			}
		}

		template <typename EidContainerType>
		void GetEdgeIDs(const FDynamicMesh3& Mesh, EidContainerType& EidsOut)
		{
			// Can't pass in a desired capacity to Reset() because TSet does not support it
			EidsOut.Reset(); 
			for (const TPair<int32, int8>& VidPair : EdgeTriIndexPairs)
			{
				if (ensure(Mesh.IsTriangle(VidPair.Key)))
				{
					int32 Eid = Mesh.GetTriEdges(VidPair.Key)[VidPair.Value];
					checkSlow(Eid != IndexConstants::InvalidID);
					EidsOut.Add(Eid);
				}
			}
		}

		bool IsEmpty() const { return EdgeTriIndexPairs.IsEmpty(); }
		void Reset() { EdgeTriIndexPairs.Reset(); }
		void Empty() { EdgeTriIndexPairs.Empty(); }

	private:
		// The pair is (tid, {0,1,2}), where the second element is an index into the edge triplet
		// for that triangle.
		TArray<TPair<int32, int8>> EdgeTriIndexPairs;
	};

	//
	// Utility functions for group topology manipulation.
	// These should probably move to another location
	//


	/**
	 * Returns a pair of vertex ID's that are representative of a group edge, to be able to identify
	 * a selected group edge independently of a group topology object. 
	 *
	 * For non-loop group edges, this will be the vids of the lower-vid endpoint and its neighbor 
	 * in the group edge, arranged in increasing vid order. For loop group edges, this will be the 
	 * lowest vid in the group edge and its lower-vid neighbor in the group edge.
	 *
	 * This is basically just identifying the group edge with a specific component edge, but using
	 * vids instead of an edge id makes it a bit easier to apply a compact mapping if the mesh was
	 * compacted, and helps the representative survive translation to/from a mesh description in
	 * cases of compaction.
	 *
	 * @param CompactMaps This gets used to remap the vids given in the topology first (i.e., it assumes 
	 *  that the CompactMaps have not yet been applied to the contents of the topology object).
	 */
	MODELINGCOMPONENTS_API FIndex2i GetGroupEdgeRepresentativeVerts(const FGroupTopology& TopologyIn, int GroupEdgeID, const FCompactMaps& CompactMaps);

	/**
	 * Returns a pair of vertex ID's that are representative of a group edge, to be able to identify
	 * a selected group edge independently of a group topology object. 
	 *
	 * See other overload for more details.
	 */
	MODELINGCOMPONENTS_API FIndex2i GetGroupEdgeRepresentativeVerts(const FGroupTopology& TopologyIn, int GroupEdgeID);


}
}
