// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"
#include "CoreMinimal.h"

#define UE_API MODELINGCOMPONENTS_API

class UInteractiveToolManager;
class IToolsContextRenderAPI;
class AActor;

namespace UE::Geometry
{

class FDynamicMesh3;
class FGroupTopology;
struct FGeometrySelection;

}

/**
 * Utility functions for Tool implementations to use when doing selection
 */
namespace ToolSelectionUtil
{

	/**
	 * Change the active selection to the given Actor, via given ToolManager. Replaces existing selection.
	 */
	MODELINGCOMPONENTS_API void SetNewActorSelection(UInteractiveToolManager* ToolManager, AActor* Actor);

	/**
	 * Change the active selection to the given Actors, via given ToolManager. Replaces existing selection.
	 */
	MODELINGCOMPONENTS_API void SetNewActorSelection(UInteractiveToolManager* ToolManager, const TArray<AActor*>& Actors);

	/** Prefer AccumulateSelectionElements with Flags parameter. */
	UE_DEPRECATED(5.5, "AccumulateSelectionElements which takes a bMapFacesToEdges boolean is deprecated."
				"Please use the function of the same name which takes EEnumerateSelectionMapping flags instead")
	MODELINGCOMPONENTS_API bool AccumulateSelectionElements(
		UE::Geometry::FGeometrySelectionElements& Elements,
		const UE::Geometry::FGeometrySelection& Selection,
		const UE::Geometry::FDynamicMesh3& SourceMesh,
		const UE::Geometry::FGroupTopology* Topology = nullptr,
		const FTransform* ApplyTransform = nullptr,
		bool bMapFacesToEdges = false);

	/**
	 * Add the geometry selection elements corresponding to the given Selection to Elements. This function does not
	 * reset Elements before adding elements. If the Selection has Polygroup topology then use the given Topology to
	 * accumulate elements if it isn't null, otherwise compute a FGroupTopology from SourceMesh.TriangleGroups and use
	 * that to accumulate elements. If the Selection has Triangle topology then the Topology argument is ignored.
	 * Return false if there was an error and true otherwise
	 */
	MODELINGCOMPONENTS_API bool AccumulateSelectionElements(
		UE::Geometry::FGeometrySelectionElements& Elements,
		const UE::Geometry::FGeometrySelection& Selection,
		const UE::Geometry::FDynamicMesh3& SourceMesh,
		const UE::Geometry::FGroupTopology* Topology = nullptr,
		const FTransform* ApplyTransform = nullptr,
		const UE::Geometry::EEnumerateSelectionMapping Flags = UE::Geometry::EEnumerateSelectionMapping::Default);
	
	/**
	 * Render the given Elements using FPrimitiveDrawInterface
	 */
	MODELINGCOMPONENTS_API void DebugRenderGeometrySelectionElements(
		IToolsContextRenderAPI* RenderAPI,
		const UE::Geometry::FGeometrySelectionElements& Elements,
		bool bIsPreview = false);

	// todo [nickolas.drake]: remove this function when no longer used by GeometrySelectionManager
	void DebugRender(
		IToolsContextRenderAPI* RenderAPI,
		const UE::Geometry::FGeometrySelectionElements& Elements,
		float LineThickness,
		FLinearColor LineColor,
		float PointSize,
		FLinearColor PointColor,
		float DepthBias = 0.f,
		FLinearColor FillColor = FLinearColor(1.0f, 0.0f, 0.0f));
}

namespace UE::Geometry
{

struct FSelectionRenderHelper
{
	UE_API void Initialize(const FGeometrySelection& Selection,
		const FDynamicMesh3& SourceMesh,
		const FGroupTopology* Topology = nullptr,
		const FTransform* ApplyTransform = nullptr);

	UE_API void Render(IToolsContextRenderAPI* RenderAPI) const;

protected:

	FGeometrySelectionElements Elements;
};

}

#undef UE_API
