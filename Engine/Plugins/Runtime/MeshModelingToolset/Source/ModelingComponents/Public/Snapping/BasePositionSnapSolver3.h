// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VectorTypes.h"
#include "LineTypes.h"
#include "CircleTypes.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE
{
namespace Geometry
{

/**
 * FBasePositionSnapSolver3 is a base class for 3D position snapping implementations.
 * It is not usable on its own and the split between this class and the 
 * implementations is not incredibly clean. However there is lots of
 * shared functionality that is placed here.
 * 
 * 3D Point and Line targets are supported. Targets can also be "ignored".
 * 
 * The actual snap "solve" must be implemented by subclasses, depending on
 * their input data (ray, point, line, etc)
 * 
 * 
 */
class FBasePositionSnapSolver3
{
public:
	//
	// parameters
	// 

	/** This is the function we use to measure distances between snap points. Defaults to Euclidean distance. */
	TFunction<double(const FVector3d&, const FVector3d&)> SnapMetricFunc;
	/** Tolerance for snapping, in unit relative to SnapMetricFunc */
	double SnapMetricTolerance;

	/** If true, then we will prefer to keep current snap point over a new one */
	bool bEnableStableSnap = true;
	/** How much we have to improve the snap metric to discard the current stable snap */
	double StableSnapImproveThresh = 0.5;

	/** subclasses may have internal TargetID values, so external points should have IDs larger than this */
	static constexpr int BaseExternalPointID = 1000;
	/** subclasses may have internal TargetID values, so external lines should have IDs larger than this */
	static constexpr int BaseExternalLineID = 10000;


public:
	UE_API FBasePositionSnapSolver3();
	virtual ~FBasePositionSnapSolver3() = default;

	//
	// setup
	// 

	/** ECustomMetricType defines how a FCustomMetric should be interpreted  */
	enum class ECustomMetricType
	{
		ReplaceValue = 0,		/** Use given parameter instead of default metric value */
		Multiplier = 1,			/** Multiply default metric value by given parameter */
		MinValue = 2			/** Use minimum of default metric value and given parameter */
	};
	/** FCustomMetric overrides/modifies the default should-snap-happen metric */
	struct FCustomMetric
	{
		/** What kind of custom metric is this */
		ECustomMetricType Type;
		/** parameter for custom metric */
		double Value;
		static FCustomMetric Minimum(double Val) { return FCustomMetric{ ECustomMetricType::MinValue, Val }; }
		static FCustomMetric Replace(double Val) { return FCustomMetric{ ECustomMetricType::ReplaceValue, Val }; }
		static FCustomMetric Multiplier(double Val) { return FCustomMetric{ ECustomMetricType::Multiplier, Val }; }
	};

	/**
	 * Add a snap target point at the given Position
	 * @param Position snap target point location
	 * @param TargetID identifier for this target. Can be shared with other targets
	 * @param Priority importance of this snap point. Lower priority is more important.
	 */
	UE_API virtual void AddPointTarget(const FVector3d& Position, int TargetID, int Priority = 100);

	/**
	 * Add a snap target point at the given Position
	 * @param Position snap target point location
	 * @param TargetID identifier for this target. Can be shared with other targets
	 * @param CustomMetric Use the given custom metric instead of the default metric
	 * @param Priority importance of this snap point. Lower priority is more important.
	 */
	UE_API virtual void AddPointTarget(const FVector3d& Position, int TargetID, const FCustomMetric& CustomMetric, int Priority = 100);

	/** Remove any point targets with this TargetID */
	UE_API virtual bool RemovePointTargetsByID(int TargetID);

	/**
	 * Add a snap target line
	 * @param TargetID identifier for this target. Can be shared with other targets
	 * @param Priority importance of this snap line. Lower priority is more important.
	 */
	UE_API virtual void AddLineTarget(const FLine3d& Line, int TargetID, int Priority = 100);

	/** Remove any line targets with this TargetID */
	UE_API virtual bool RemoveLineTargetsByID(int TargetID);



	/**
	 * Add a snap target Circle
	 * @param TargetID identifier for this target. Can be shared with other targets
	 * @param Priority importance of this snap target. Lower priority is more important.
	 */
	UE_API virtual void AddCircleTarget(const FCircle3d& Circle, int TargetID, int Priority = 100);

	/** Remove any line targets with this TargetID */
	UE_API virtual bool RemoveCircleTargetsByID(int TargetID);


	/** Add given TargetID to the ignore list, so any points/lines with that ID will not be snapped-to */
	UE_API virtual void AddIgnoreTarget(int TargetID);
	/** Add given TargetID from the ignore list */
	UE_API virtual void RemoveIgnoreTarget(int TargetID);
	/** @return true if the given TargetID is in the ignore list */
	UE_API virtual bool IsIgnored(int TargetID) const;

	/** Discard the set of snap points and lines and clear the active snap */
	UE_API virtual void Reset();
	/** Clear the active snap */
	UE_API virtual void ResetActiveSnap();

	//
	// solving
	//

	//void UpdateSnappedPoint(const FRay3d& Ray);

	//
	// output
	//

	/** @return true if after the last snap solve we have an active snap */
	bool HaveActiveSnap() const { return bHaveActiveSnap; }
	/** @return the snapped-to point */
	FVector3d GetActiveSnapToPoint() const { return ActiveSnapToPoint; }
	/** @return the snapped-from point. This may be the original target, a point on a line, after projections, etc. Defined by subclasses. */
	FVector3d GetActiveSnapFromPoint() const { return ActiveSnapFromPoint; }
	/** @return targetID of original point or line that resulted in the snap */
	int GetActiveSnapTargetID() const { return (bHaveActiveSnap) ? ActiveSnapTarget.TargetID : -1; }

	/** @return true if the active snap target is a line */
	bool HaveActiveSnapLine() const { return HaveActiveSnap() && ActiveSnapTarget.bIsSnapLine; }
	/** @return 3D line for active snap target */
	const FLine3d& GetActiveSnapLine() const { return ActiveSnapTarget.SnapLine; }

	/** @return true if the active snap target is based on a distance along a line */
	bool HaveActiveSnapDistance() const { return HaveActiveSnap() && ActiveSnapTarget.bIsSnapDistance; }
	/** @return internal snap distance ID (interpretation defined by subclasses) */
	int GetActiveSnapDistanceID() const { return ActiveSnapTarget.SnapDistanceID; }

protected:
	/**
	 * Target point that might be snapped to
	 */
	struct FSnapTargetPoint
	{
		FVector3d Position;

		int TargetID;
		int Priority;

		FCustomMetric CustomMetric;
		bool bHaveCustomMetric = false;

		bool bIsSnapLine = false;
		FLine3d SnapLine;
		bool bIsSnapDistance = false;
		int SnapDistanceID = -1;

		// if true, then Position is the snap target point but ConstrainedPosition is the one we should return
		// Sounds weird but allows for things like axis-constrained grid snapping to work. We have to snap to
		// the line first, based on user interaction, and then to the grid once the line is selected
		bool bHaveConstrainedPosition = false;
		FVector3d ConstrainedPosition;
	};
	TArray<FSnapTargetPoint> TargetPoints;

	/**
	 * Target line that might be snapped to
	 */
	struct FSnapTargetLine
	{
		FLine3d Line;
		int TargetID;
		int Priority;
	};
	TArray<FSnapTargetLine> TargetLines;


	struct FSnapTargetCircle
	{
		FCircle3d Circle;
		int TargetID;
		int Priority;
	};
	TArray<FSnapTargetCircle> TargetCircles;


	/** list of TargetID values to ignore in snap queries */
	TSet<int> IgnoreTargets;

	//
	// information about active snap
	//
	bool bHaveActiveSnap;
	FSnapTargetPoint ActiveSnapTarget;
	FVector3d ActiveSnapFromPoint;
	FVector3d ActiveSnapToPoint;
	double SnappedPointMetric;

	UE_API virtual void SetActiveSnapData(const FSnapTargetPoint& TargetPoint, const FVector3d& FromPoint, const FVector3d& ToPoint, double Metric);
	UE_API virtual void ClearActiveSnapData();


	//
	// snap measurement functions
	// 

	UE_API int32 FindIndexOfBestSnapInSet(const TArray<FSnapTargetPoint>& TestTargets, double& MinMetric, int& MinPriority,
		const TFunction<FVector3d(const FVector3d&)>& GetSnapPointFromFunc);
	UE_API const FSnapTargetPoint* FindBestSnapInSet(const TArray<FSnapTargetPoint>& TestTargets, double& MinMetric, int& MinPriority,
		const TFunction<FVector3d(const FVector3d&)>& GetSnapPointFromFunc);

	UE_API bool TestSnapTarget(const FSnapTargetPoint& Target, double MinMetric, int MinPriority,
		const TFunction<FVector3d(const FVector3d&)>& GetSnapPointFromFunc);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
