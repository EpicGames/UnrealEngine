// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryBase.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "ModelingSceneSnappingManager.generated.h"

#define UE_API MODELINGCOMPONENTS_API


class IToolsContextQueriesAPI;
class UInteractiveToolsContext;
class UInteractiveToolManager;
class UDynamicMeshComponent;
PREDECLARE_GEOMETRY(class FSceneGeometrySpatialCache);

/**
 * UModelingSceneSnappingManager is an implementation of snapping suitable for use in
 * Modeling Tools/Gizmos (and potentially other places). 
 * 
 * Currently Supports:
 *    - snap to position/rotation grid
 *    - snap to mesh vertex position
 *    - snap to mesh edge position
 * 
 * Snapping to mesh vertex/edge positions currently works for Volume (BrushComponent), StaticMeshComponent, 
 * and DynamicMeshComponent. 
 * 
 * Currently the StaticMesh vertex/edge snapping is dependent on the Physics
 * system, and may fail or return nonsense results in some cases, due to the physics
 * complex-collision mesh deviating from the source-model mesh.
 */
UCLASS(MinimalAPI)
class UModelingSceneSnappingManager : public USceneSnappingManager
{
	GENERATED_BODY()

public:

	UE_API virtual void Initialize(TObjectPtr<UInteractiveToolsContext> ToolsContext);
	UE_API virtual void Shutdown();

	//
	// USceneSnappingManager API
	//


	/**
	* Try to find a Hit Point in the scene that satisfies the HitQuery Request.
	* @param Request hit query configuration
	* @param ResultOut hit query result, if return is true
	* @return true if a valid hit was found
	* @warning implementations are not required (and may not be able) to support hit-testing
	*/
	UE_API virtual bool ExecuteSceneHitQuery(const FSceneHitQueryRequest& Request, FSceneHitQueryResult& ResultOut) const override;

	/**
	* Try to find Snap Targets in the scene that satisfy the Snap Query.
	* @param Request snap query configuration
	* @param Results list of potential snap results
	* @return true if any valid snap target was found
	* @warning implementations are not required (and may not be able) to support snapping
	*/
	UE_API virtual bool ExecuteSceneSnapQuery(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& Results) const override;


	//
	// API for managing the set of Actors/Components that the snapping system knows about.
	// Currently this is for Volumes and DynamicMeshActors/Components, the SnappingManager builds
	// it's own spatial data structure cache for these types of meshes. StaticMeshComponents are
	// automatically included and handled via world-traces and the physics system.
	//
public:

	/** @return true if this Component type is supported by the spatial-cache tracking for hit and snap testing */
	UE_API virtual bool IsComponentTypeSupported(const UPrimitiveComponent* Component) const;

	/** Enable spatial-cache tracking for the Components of the Actor that pass the PrimitiveFilter */
	UE_API virtual void OnActorAdded(AActor* Actor, TFunctionRef<bool(UPrimitiveComponent*)> PrimitiveFilter);
	/** Disable spatial-cache tracking for any Components of the Actor */
	UE_API virtual void OnActorRemoved(AActor* Actor);

	/** Enable spatial-cache tracking for the Component */
	UE_API virtual void OnComponentAdded(UPrimitiveComponent* Component);
	/** Disable spatial-cache tracking for the Component */
	UE_API virtual void OnComponentRemoved(UPrimitiveComponent* Component);
	/** Notify the internal spatial-cache tracking system that the Component has been modified (ie cache needs to be rebiult) */
	UE_API virtual void OnComponentModified(UActorComponent* Component);

	/** Explicitly populate the spatial-cache tracking set with specific Actors/Components (alternative to OnActorAdded route)  */
	UE_API virtual void BuildSpatialCacheForWorld(
		UWorld* World,
		TFunctionRef<bool(AActor*)> ActorFilter,
		TFunctionRef<bool(UPrimitiveComponent*)> PrimitiveFilter );


	//
	// APIs for configuring the SceneSnappingManager behavior
	//
public:
	/***
	 * Temporarily disable live updates of any modified Actors/Components. Any changes that are
	 * detected while updates are paused will be executed on Unpause.
	 * This is mainly intended for use in interactive operations, eg while live-editing a mesh,
	 * changes may be posted every frame but the cache BVHs/etc only need to be updated on mouse-up, etc
	 */
	UE_API virtual void PauseSceneGeometryUpdates();
	/**
	 * Re-enable live updates that were prevented by PauseSceneGeometryUpdates(), and execute
	 * any pending updates that were requested while in the pause state.
	 * @param bImmediateRebuilds If true (default), things like BVH updates will happen immediately, instead of simply being marked as pending
	 */
	UE_API virtual void UnPauseSceneGeometryUpdates(bool bImmediateRebuilds = true);


protected:

	UPROPERTY()
	TObjectPtr<UInteractiveToolsContext> ParentContext;

	const IToolsContextQueriesAPI* QueriesAPI = nullptr;

	UE_API virtual bool ExecuteSceneSnapQueryRotation(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& Results) const;
	UE_API virtual bool ExecuteSceneSnapQueryPosition(const FSceneSnapQueryRequest& Request, TArray<FSceneSnapQueryResult>& Results) const;

	// This map allows us to identify the Components belonging to Actors. Need to store this
	// because if the Actor is deleted we will not be able to identify it's (previous) Components
	TMap<UPrimitiveComponent*, AActor*> ComponentToActorMap;

	// cache for objects in the level
	TSharedPtr<UE::Geometry::FSceneGeometrySpatialCache> SpatialCache;

	FDelegateHandle OnObjectModifiedHandler;
	UE_API void HandleGlobalObjectModifiedDelegate(UObject* Object);

	FDelegateHandle OnComponentTransformChangedHandle;
	UE_API void HandleGlobalComponentTransformChangedDelegate(USceneComponent* Component);

	TMap<UPrimitiveComponent*, TWeakObjectPtr<UDynamicMeshComponent>> DynamicMeshComponents;
	UE_API void HandleDynamicMeshModifiedDelegate(UDynamicMeshComponent* Component);

	// PauseSceneGeometryUpdates / UnPauseSceneGeometryUpdates support
	bool bQueueModifiedDynamicMeshUpdates = false;
	TSet<UDynamicMeshComponent*> PendingModifiedDynamicMeshes;
};





namespace UE
{
namespace Geometry
{
	//
	// The functions below are helper functions that simplify usage of a UModelingSceneSnappingManager 
	// that is registered as a ContextStoreObject in an InteractiveToolsContext
	//

	/**
	 * If one does not already exist, create a new instance of UModelingSceneSnappingManager and add it to the
	 * ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore now has a UModelingSceneSnappingManager (whether it already existed, or was created)
	 */
	MODELINGCOMPONENTS_API bool RegisterSceneSnappingManager(UInteractiveToolsContext* ToolsContext);

	/**
	 * Remove any existing UModelingSceneSnappingManager from the ToolsContext's ContextObjectStore
	 * @return true if the ContextObjectStore no longer has a UModelingSceneSnappingManager (whether it was removed, or did not exist)
	 */
	MODELINGCOMPONENTS_API bool DeregisterSceneSnappingManager(UInteractiveToolsContext* ToolsContext);


	/**
	 * Find an existing UModelingSceneSnappingManager in the ToolsContext's ContextObjectStore
	 * @return SelectionManager pointer or nullptr if not found
	 */
	MODELINGCOMPONENTS_API UModelingSceneSnappingManager* FindModelingSceneSnappingManager(UInteractiveToolManager* ToolManager);


}
}

#undef UE_API
