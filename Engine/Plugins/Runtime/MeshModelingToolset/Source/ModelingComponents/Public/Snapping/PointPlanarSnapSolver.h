// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Snapping/BasePositionSnapSolver3.h"

#define UE_API MODELINGCOMPONENTS_API

class IToolsContextRenderAPI;

namespace UE
{
namespace Geometry
{

/**
 * FPointPlanarSnapSolver solves for a Point snap location on a plane, based
 * on an input Point and a set of target points and lines in the plane.
 *
 * This implementation has the notion of a "history" of previous points,
 * from which line and distance constraints can be inferred.
 * This is useful for snapping in 2D polygon drawing.
 *
 * See FBasePositionSnapSolver3 for details on how to set up the snap problem
 * and get results.
 */
class FPointPlanarSnapSolver : public FBasePositionSnapSolver3
{
public:
	// configuration variables
	FFrame3d Plane;

	bool bEnableSnapToKnownLengths = true;

	static constexpr int CardinalAxisTargetID = 10;
	static constexpr int LastSegmentTargetID = 11; 
	static constexpr int IntersectionTargetID = 12;

	int CardinalAxisPriority = 150;
	int LastSegmentPriority = 140;

	/** How much more important a known length is than its line's priority. */
	int KnownLengthPriorityDelta = 10;

	/** How much more important an intersection is than the more important of the intersecting lines. */
	int32 IntersectionPriorityDelta = 11;

	int MinInternalPriority() const { return FMath::Min(CardinalAxisPriority, LastSegmentPriority) - FMath::Max(IntersectionPriorityDelta, KnownLengthPriorityDelta); }

public:
	UE_API FPointPlanarSnapSolver();

	/**
	 * Creates snap lines based on the last point in the point history. Calling the function 
	 * with both parameters set to false clears the generated snap lines.
	 *
	 * @param bCardinalAxes If true, the generated lines are parallel to the axes of the plane
	 *  and pass through the last point.
	 * @param bLastHistorySegment If true, adds a snap line through the last point that is 90
	 *  degrees to the last segment.
	 */
	UE_API void RegenerateTargetLines(bool bCardinalAxes, bool bLastHistorySegment);

	/**
	 * Sets the snapping lines to be based on the history points *adjacent* to the point with a 
	 * given history index. The given index can be one beyond the ends of the current history
	 * (i.e., -1 or PointHistoryLength) to base the lines on the first/last points.
	 * The snap lines will be parallel to the plane axes.
	 * Useful for moving a point to be aligned with one of its neighbors.
	 *
	 * @param HistoryIndex Index in the range [-1, PointHistoryLength()]. The points adjacent
	 *  to that index will be used for line generation.
	 * @param bWrapAround If true, the first point will be considered adjacent to the last
	 *  when HistoryIndex is 0 or PointHistoryLength()-1.
	 * @param bGenerateIntersections If true, intersections will be generated as higher-priority
	 *  points. Intersections are performed with generated and user-specified lines that
	 *  lie in the plane.
	 */
	UE_API void RegenerateTargetLinesAround(int32 HistoryIndex, bool bWrapAround = false, bool bGenerateIntersections = true);

	UE_API virtual void Reset() override;

	/** Point history manipulation functions. All of them remove the currently generated snap*/
	UE_API void UpdatePointHistory(const TArray<FVector3d>& Points);
	UE_API void UpdatePointHistory(const TArray<FVector3f>& Points);
	UE_API void AppendHistoryPoint(const FVector3d& Point);
	UE_API void InsertHistoryPoint(const FVector3d& Point, int32 Index);
	UE_API void RemoveHistoryPoint(int32 Index);
	int32 PointHistoryLength()
	{
		return PointHistory.Num();
	}

	UE_API void UpdateSnappedPoint(const FVector3d& PointIn);

	/** 
	 * Returns true when the active snap represents an intersection of multiple target lines in the plane. In such a case, one line
	 * can be obtained by the usual GetActiveSnapLine(), and the second can be obtained from GetIntersectionSecondLine()
	 */
	bool HaveActiveSnapIntersection() { return HaveActiveSnap() && bActiveSnapIsIntersection; }

	/**
	 * When the active snap is an intersection, holds the second intersecting line (the first can be obtained with GetActiveSnapLine())
	 */
	const FLine3d& GetIntersectionSecondLine() { return IntersectionSecondLine; }

	/** Draws the current snap targets (for debugging) */
	UE_API void DebugRender(IToolsContextRenderAPI* RenderAPI);

protected:

	TArray<FSnapTargetLine> GeneratedLines;

	TArray<FSnapTargetPoint> IntersectionPoints;
	TArray<const FSnapTargetLine*> IntersectionSecondLinePointers; // [1:1] with IntersectionPoints
	bool bActiveSnapIsIntersection = false;
	FLine3d IntersectionSecondLine;
	
	TSet<int> IgnoreTargets;

	TArray<FVector3d> PointHistory;

	TArray<FSnapTargetPoint> GeneratedTargets;

	UE_API void ResetGenerated();
	UE_API void GenerateLineIntersectionTargets();
	UE_API void GenerateTargets(const FVector3d& PointIn);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
