// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Snapping/BasePositionSnapSolver3.h"

#define UE_API MODELINGCOMPONENTS_API

class FToolDataVisualizer;

namespace UE
{
namespace Geometry
{


/**
 * FRaySpatialSnapSolver solves for a Point snap location based on an input Ray
 * and a set of snap targets (3D points and 3D lines). 
 * 
 * See FBasePositionSnapSolver3 for details on how to set up the snap problem
 * and get results.
 */
class FRaySpatialSnapSolver : public FBasePositionSnapSolver3
{
public:
	UE_API FRaySpatialSnapSolver();

	/**
	 * Optional function that will be used to project potential snap points onto constraints.
	 * Note that Line/Curve constraints are still respected, so eg if this projects to a 3D grid,
	 * then when calculating possible line-snap positions, the 3D grid point will be projected back 
	 * onto the line targets.
	 */
	TFunction<FVector3d(const FVector3d&)> PointConstraintFunc = nullptr;

	//
	// solving
	//

	/** Solve the snapping problem */
	UE_API void UpdateSnappedPoint(const FRay3d& Ray);

	//
	// Utility rendering
	//

	/** Visualization of snap targets and result (if available) */
	UE_API void Draw(FToolDataVisualizer* Renderer, float LineLength, TMap<int,FLinearColor>* ColorMap = nullptr);

protected:

	TArray<FSnapTargetPoint> GeneratedTargetPoints;
	UE_API void GenerateTargetPoints(const FRay3d& Ray);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
