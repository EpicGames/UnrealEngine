// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API MODELINGCOMPONENTS_API

class USplineComponent;
class IToolsContextRenderAPI;

namespace UE {
namespace Geometry {

class FDynamicMesh3;
template<typename T> class TMeshAABBTree3;
typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;


namespace SplineUtil {

	// Used for DrawSpline()
	struct FDrawSplineSettings
	{
		UE_API FDrawSplineSettings();

		// Defaults to FStyleColors::White
		FColor RegularColor;
		// Defaults to FStyleColors::AccentOrange
		FColor SelectedColor;
		// If non-positive, the spline is drawn just as points and curves. If positive, the
		// orientation and scale are visualized with the given base width.
		double ScaleVisualizationWidth = 0;

		// Keys to use SelectedColor for
		TSet<int32>* SelectedKeys = nullptr;
	};

	MODELINGCOMPONENTS_API void DrawSpline(const USplineComponent& SplineComp, IToolsContextRenderAPI& RenderAPI, const FDrawSplineSettings& Settings);

	/**
	* Iteratively subdivide and project a spline so that it lies on the given surface.
	* NOTE: Point tangents for newly added points are computed using the spline derivative and local surface tangent plane. Rotations and Scales are not handled by this function.
	*/
	MODELINGCOMPONENTS_API void ProjectSplineToSurface(FInterpCurveVector& OutputSpline,
		const FInterpCurveVector& InputSpline,
		const UE::Geometry::FDynamicMeshAABBTree3& SurfaceAABBTree, 
		const FTransform& SplineTransform, 
		const FTransform& MeshTransform, 
		double DistanceThreshold = 10.0,
		int32 MaxNewPoints = 100);

}
}
}

#undef UE_API
