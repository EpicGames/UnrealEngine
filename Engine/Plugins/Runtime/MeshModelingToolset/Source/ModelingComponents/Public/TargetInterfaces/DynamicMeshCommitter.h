// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "GeometryBase.h"

#include "DynamicMeshCommitter.generated.h"

PREDECLARE_GEOMETRY(class FDynamicMesh3);


UINTERFACE(MinimalAPI)
class UDynamicMeshCommitter : public UInterface
{
	GENERATED_BODY()
};

class IDynamicMeshCommitter
{
	GENERATED_BODY()

public:

	/**
	 * Extra information that can be passed to a CommitMesh call to potentially make
	 * the commit faster. Note that setting any of these to false doesn't mean that
	 * the corresponding data won't be updated, because a target may choose to always
	 * update everything. But it may help some targets do faster updates by not
	 * updating things that stayed the same.
	 */
	struct FDynamicMeshCommitInfo
	{
		/** Initializes each of the b*Changed members to bInitValue */
		FDynamicMeshCommitInfo(bool bInitValue)
		{
			bPositionsChanged =
			bTopologyChanged =
			bPolygroupsChanged =
			bNormalsChanged =
			bTangentsChanged =
			bUVsChanged =
			bVertexColorsChanged = bInitValue;
		}

		/** Leaves everything initialized to default (true) */
		FDynamicMeshCommitInfo() {}

		bool bPositionsChanged = true;
		bool bTopologyChanged = true;
		bool bPolygroupsChanged = true;
		bool bNormalsChanged = true;
		bool bTangentsChanged = true;
		bool bUVsChanged = true;
		bool bVertexColorsChanged = true;

		/**
		 * Intentionally left out of the constructor. This is a different
		 * parameter than the b*Changed members that augments how vertex
		 * colors are transformed during the commit.
		 */
		bool bTransformVertexColorsSRGBToLinear = false;
	};

	virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh)
	{
		FDynamicMeshCommitInfo CommitInfo;
		CommitDynamicMesh(Mesh, CommitInfo);
	};
	virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo) = 0;
};
