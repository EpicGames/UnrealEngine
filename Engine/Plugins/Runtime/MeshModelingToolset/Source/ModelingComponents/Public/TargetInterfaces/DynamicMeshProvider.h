// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "GeometryBase.h"

#include "DynamicMeshProvider.generated.h"

#define UE_API MODELINGCOMPONENTS_API

struct FGetMeshParameters;

PREDECLARE_GEOMETRY(class FDynamicMesh3);

UINTERFACE(MinimalAPI)
class UDynamicMeshProvider : public UInterface
{
	GENERATED_BODY()
};

class IDynamicMeshProvider
{
	GENERATED_BODY()

public:
	/**
	 * Gives a copy of a dynamic mesh for tools to operate on.
	 */
	virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() = 0;

	/**
	 * Gives a copy of a dynamic mesh for tools to operate on.
	 * 
	 * @param bRequestTangents Request tangents on the returned mesh. Not required if tangents are not on the source data and the provider does not have a standard way to generate them.
	 *
	 * Note: Default implementation simply returns GetDynamicMesh(). Overloaded implementations for e.g., Static and Skeletal Mesh sources will enable (and compute if needed) additional tangent data.
	 */
	UE_DEPRECATED(5.5, "Use GetDynamicMesh which takes a FGetMeshParameters instead.")
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh(bool bRequestTangents);
	
	/**
	 * Gives a copy of a dynamic mesh for tools to operate on.
	 * 
	 * @param InGetMeshParams Request specific LOD and/or tangents on the returned mesh.
	 * bWantMeshTangents not required if tangents are not on the source data and the provider does not have a standard way to generate them.
	 *
	 * Note: Default implementation simply returns GetDynamicMesh(). Overloaded implementations for e.g., Static and Skeletal Mesh sources will enable (and compute if needed) additional tangent data.
	 */
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh(const FGetMeshParameters& InGetMeshParams);
};

#undef UE_API
