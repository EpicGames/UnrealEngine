// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "InteractiveToolChange.h"

#include "DynamicMeshSource.generated.h"

class UDynamicMesh;
class UDynamicMeshComponent;

UINTERFACE(MinimalAPI)
class UPersistentDynamicMeshSource : public UInterface
{
	GENERATED_BODY()
};

class IPersistentDynamicMeshSource
{
	GENERATED_BODY()

public:

	/**
	 * @return a UDynamicMesh for this source. 
	 */
	virtual UDynamicMesh* GetDynamicMeshContainer() = 0;

	/**
	 * Commit a change to the UDynamicMesh. The assumption here is that the Change has already been applied to
	 * the UDynamicMesh returned by GetDynamicMeshContainer(). This function is used by Tools to provide that
	 * change to higher levels for storage in undo systems/etc (eg the Editor Transaction system)
	 */
	virtual void CommitDynamicMeshChange(TUniquePtr<FToolCommandChange> Change, const FText& ChangeMessage) = 0;

	//
	// optional
	//

	/**
	 * @return true if the UDynamicMesh comes from a UDynamicMeshComponent (or subclass)
	 */
	virtual bool HasDynamicMeshComponent() const { return false; }

	/**
	 * @return the UDynamicMeshComponent that owns the source UDynamicMesh, if available
	 */
	virtual UDynamicMeshComponent* GetDynamicMeshComponent() { return nullptr; }


};
