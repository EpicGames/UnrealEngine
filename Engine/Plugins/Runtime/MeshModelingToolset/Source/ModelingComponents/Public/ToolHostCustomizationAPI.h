// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/Function.h"
#include "UObject/Interface.h"
#include "UObject/ScriptInterface.h"

#include "ToolHostCustomizationAPI.generated.h"

class UInteractiveToolManager;
struct FSlateBrush;

UINTERFACE(MinimalAPI)
class UToolHostCustomizationAPI : public UInterface
{
	GENERATED_BODY()
};

/**
 * An API (to be stored as a context object) that would allow tools to customize aspects of their
 * hosts' appearance. For instance, it can allow a tool to customize tool shutdown accept/cancel
 * buttons to handle some tool sub-action, if the host allows this.
 */
class IToolHostCustomizationAPI
{
	GENERATED_BODY()
public:

	/** If true, the tool host is able to override its tool shutdown buttons to custom accept/cancel buttons. */
	virtual bool SupportsAcceptCancelButtonOverride() { return false; }

	struct FAcceptCancelButtonOverrideParams
	{
		/** Function to be called when the accept/cancel buttons are clicked. Passed in arg is true if it was the accept button. */
		TFunction<void(bool /*bAccept*/)> OnAcceptCancelTriggered;
		
		/** If this returns false, the accept button is not clickable. */
		TFunction<bool()> CanAccept = []() { return true; };
		
		/** Label to use to show that the buttons now control a sub action. */
		FText Label;

		// Optional parameters:

		/** The host will try to look up an icon with this name in its style sheet. Icon will not be used if host does not find a match. */
		TOptional<FName> IconName;

		TOptional<FText> OverrideAcceptButtonText;
		TOptional<FText> OverrideAcceptButtonTooltip;
		TOptional<FText> OverrideCancelButtonText;
		TOptional<FText> OverrideCancelButtonTooltip;
	};

	/**
	 * Requests that a tool host's tool shutdown buttons be temporarily overriden to an accept/cancel pair
	 * that accepts/cancels some sub action.
	 */
	virtual bool RequestAcceptCancelButtonOverride(FAcceptCancelButtonOverrideParams& Params) { return false; }

	virtual bool SupportsCompleteButtonOverride() { return false; }

	struct FCompleteButtonOverrideParams
	{
		/** Function to be called when complete button is clicked. */
		TFunction<void()> OnCompleteTriggered;

		/** Label to use to show that the buttons now control a sub action. */
		FText Label;
		
		// Optional parameters:

		/** The host will try to look up an icon with this name in its style sheet. Icon will not be used if host does not find a match. */
		TOptional<FName> IconName;

		TOptional<FText> OverrideCompleteButtonText;
		TOptional<FText> OverrideCompleteButtonTooltip;
	};

	/**
	 * Requests that a tool host's shutdown buttons be temporarily overriden to a complete button that
	 * completes some sub action.
	 */
	virtual bool RequestCompleteButtonOverride(FCompleteButtonOverrideParams& Params) { return false; }

	/**
	 * Remove any requested button customizations. 
	 * 
	 * This can be imporant to call on tool shutdown if the tool registered button overrides
	 * whose handlers hold a reference to the tool, or else, for example, a CanAccept override
	 * could end up trying to query the tool after shutdown.
	 */
	virtual void ClearButtonOverrides() = 0;

public:
	/**
	 * @return existing IToolHostCustomizationAPI registered in context store via the ToolManager, or nullptr if not found
	 */
	static MODELINGCOMPONENTS_API TScriptInterface<IToolHostCustomizationAPI> Find(UInteractiveToolManager* ToolManager);
};