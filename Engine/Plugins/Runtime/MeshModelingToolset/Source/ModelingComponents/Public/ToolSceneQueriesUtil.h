// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "InteractiveTool.h"
#include "InteractiveToolManager.h"
#include "VectorTypes.h"

class USceneSnappingManager;

/**
 * Utility functions for Tool implementations to use to do scene queries, generally via IToolsContextQueriesAPI
 */
namespace ToolSceneQueriesUtil
{
	using namespace UE::Geometry;

	/**
	 * @return global visual angle snap threshold (default is 1 degree)
	 */
	MODELINGCOMPONENTS_API double GetDefaultVisualAngleSnapThreshD();

	/**
	 * Test if two points are close enough to snap together.
	 * This is done by computing visual angle between points for current camera position.
	 * @param VisualAngleThreshold visual angle threshold to use. If 0, GetDefaultVisualAngleSnapThresh() is used
	 * @return true if visual angle is < threshold
	 */
	MODELINGCOMPONENTS_API bool PointSnapQuery(const UInteractiveTool* Tool, const FVector3d& Point1, const FVector3d& Point2, double VisualAngleThreshold = 0);

	/**
	 * Test if two points are close enough to snap together.
	 * This is done by computing visual angle between points for current camera position.
	 * @param VisualAngleThreshold visual angle threshold to use. If 0, GetDefaultVisualAngleSnapThresh() is used
	 */
	MODELINGCOMPONENTS_API bool PointSnapQuery(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2, double VisualAngleThreshold = 0);

	/**
	 * Get a measurement for testing whether two points can snap together, useful for choosing the best snap point among multiple
	 * options. For perspective mode, the returned metric is the "visual angle" between the points, which is the angle between the points
	 * to the camera scaled such that the visual angle between the horizontal bounds of the view is 90. For orthographic mode, it is a 
	 * projected distance onto the view plane scaled such that the distance between the horizontal bounds of the view is 90.
	 * Thus, the metric is suitable for comparing against a visual angle snap threshold to determine if snapping should happen.
	 */
	MODELINGCOMPONENTS_API double PointSnapMetric(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2);

	/**
	 * @return visual angle between two 3D points, relative to the current camera position
	 */
	MODELINGCOMPONENTS_API double CalculateViewVisualAngleD(const UInteractiveTool* Tool, const FVector3d& Point1, const FVector3d& Point2);


	/**
	 * @return visual angle between two 3D points, relative to the current camera position
	 */
	MODELINGCOMPONENTS_API double CalculateViewVisualAngleD(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2);

	/**
	 * @return visual angle between two 3D points, relative to the current camera position, normalized relative to 90-degree FOV
	 */
	MODELINGCOMPONENTS_API double CalculateNormalizedViewVisualAngleD(const FViewCameraState& CameraState, const FVector3d& Point1, const FVector3d& Point2);



	/**
	 * @return (approximate) 3D dimension that corresponds to a radius of target visual angle around Point, for current camera position
	 */
	MODELINGCOMPONENTS_API double CalculateDimensionFromVisualAngleD(const UInteractiveTool* Tool, const FVector3d& Point, double TargetVisualAngleDeg);

	/**
	 * @return (approximate) 3D dimension that corresponds to a radius of target visual angle around Point
	 */
	MODELINGCOMPONENTS_API double CalculateDimensionFromVisualAngleD(const FViewCameraState& CameraState, const FVector3d& Point, double TargetVisualAngleDeg);



	/**
	 * @return false if point is not currently visible (approximately)
	 */
	MODELINGCOMPONENTS_API bool IsPointVisible(const FViewCameraState& CameraState, const FVector3d& Point);



	/**
	 * FSnapGeometry stores information about geometry data of a snap, which we might use for highlights/etc
	 */
	struct FSnapGeometry
	{
		/** Geometry that was snapped to. only PointCount elements are initialized */
		FVector3d Points[3];
		/** Number of initialized elements in Points */
		int PointCount = 0;
	};

	struct FFindSceneSnapPointParams
	{
		// Required inputs/outputs
		const USceneSnappingManager* SnappingManager = nullptr;
		FViewCameraState CameraState;
		const FVector3d* Point = nullptr;
		FVector3d* SnapPointOut = nullptr;

		bool bVertices = true; // If true, try to snap to vertices
		bool bEdges = false; // If true, try to snap to edges
		double VisualAngleThreshold = 0; // Visual angle threshold to use. If 0, GetDefaultVisualAngleSnapThresh() is used
		const TArray<const UPrimitiveComponent*>* ComponentsToIgnore = nullptr;
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr;
		FSnapGeometry* SnapGeometryOut = nullptr; // world-space position of the snap geometry(point / line / polygon)
		FVector* DebugTriangleOut = nullptr; // if non - null, triangle containing snap is returned if a snap is found
	};

	/**
	 * Run a query against the scene to find the best SnapPointOut for the given Point
	 */
	MODELINGCOMPONENTS_API bool FindSceneSnapPoint(FFindSceneSnapPointParams& Params);

	/**
	 * Run a query against the scene to find the best SnapPointOut for the given Point
	 * @param bVertices if true, try snapping to mesh vertices in the scene
	 * @param bEdges if true, try snapping to mesh triangle edges in the scene
	 * @param VisualAngleThreshold visual angle threshold to use. If 0, GetDefaultVisualAngleSnapThresh() is used
	 * @param SnapGeometry world-space position of the snap geometry (point/line/polygon)
	 * @param DebugTriangleOut if non-null, triangle containing snap is returned if a snap is found
	 * @return true if a valid snap point was found
	 */
	MODELINGCOMPONENTS_API bool FindSceneSnapPoint(const UInteractiveTool* Tool, const FVector3d& Point, FVector3d& SnapPointOut,
		bool bVertices = true, bool bEdges = false, double VisualAngleThreshold = 0, 
		FSnapGeometry* SnapGeometry = nullptr, FVector* DebugTriangleOut = nullptr);

	/**
	 * Run a query against the scene to find the nearest WorldGrid snap point
	 */
	MODELINGCOMPONENTS_API bool FindWorldGridSnapPoint(const UInteractiveTool* Tool, const FVector3d& QueryPoint, FVector3d& GridSnapPointOut);

	/**
	 * Round the given distance to the nearest multiple of the world grid cell size
	 */
	MODELINGCOMPONENTS_API double SnapDistanceToWorldGridSize(const UInteractiveTool* Tool, const double Distance);

	/**
	 * @return true if HitResult is a hit on a visible Component of a visible Actor (provides correct result in Editor)
	 */
	MODELINGCOMPONENTS_API bool IsVisibleObjectHit(const FHitResult& HitResult);

	/**
	 * Find the nearest object hit by the LineTrace from Start to End that is currently visible (provides correct result in Editor)
	 * @param World the world to trace into
	 * @param HitResultOut the resulting hit, if true is returned
	 * @param Start start point of line
	 * @param End end point of line
	 * @param IgnoreComponents optional list of Components to ignore
	 * @param InvisibleComponentsToInclude optional list of Components to explicitly include, even if they are not visible
	 * @return true if a visible hit was found
	 */
	MODELINGCOMPONENTS_API bool FindNearestVisibleObjectHit(UWorld* World, FHitResult& HitResultOut, const FVector& Start, const FVector& End,
		const TArray<const UPrimitiveComponent*>* IgnoreComponents = nullptr, 
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);


	/**
	 * Find the nearest object hit by the LineTrace from Start to End that is currently visible (provides correct result in Editor)
	 * @param World the world to trace into
	 * @param HitResultOut the resulting hit, if true is returned
	 * @param Ray hit ray
	 * @param IgnoreComponents optional list of Components to ignore
	 * @param InvisibleComponentsToInclude optional list of Components to explicitly include, even if they are not visible
	 * @return true if a visible hit was found
	 */
	MODELINGCOMPONENTS_API bool FindNearestVisibleObjectHit(UWorld* World, FHitResult& HitResultOut, const FRay& Ray,
		const TArray<const UPrimitiveComponent*>* IgnoreComponents = nullptr, 
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);



	/**
	* Find the nearest object hit by the LineTrace from Start to End that is currently visible (provides correct result in Editor)
	* @param HitResultOut the resulting hit, if true is returned
	* @param Start start point of line
	* @param End end point of line
	* @param IgnoreComponents optional list of Components to ignore
	* @param InvisibleComponentsToInclude optional list of Components to explicitly include, even if they are not visible
	* @return true if a visible hit was found
	*/
	MODELINGCOMPONENTS_API bool FindNearestVisibleObjectHit(USceneSnappingManager* SnappingManager, 
		FHitResult& HitResultOut, const FRay& Ray,
		const TArray<const UPrimitiveComponent*>* IgnoreComponents = nullptr, 
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);



	/**
	* Find the nearest object hit by the LineTrace from Start to End that is currently visible (provides correct result in Editor)
	* @param HitResultOut the resulting hit, if true is returned
	* @param Start start point of line
	* @param End end point of line
	* @param IgnoreComponents optional list of Components to ignore
	* @param InvisibleComponentsToInclude optional list of Components to explicitly include, even if they are not visible
	* @return true if a visible hit was found
	*/
	MODELINGCOMPONENTS_API bool FindNearestVisibleObjectHit(const UInteractiveTool* Tool, FHitResult& HitResultOut, const FVector& Start, const FVector& End,
		const TArray<const UPrimitiveComponent*>* IgnoreComponents = nullptr, 
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);


	/**
	* Find the nearest object hit by the LineTrace from Start to End that is currently visible (provides correct result in Editor)
	* @param HitResultOut the resulting hit, if true is returned
	* @param Ray hit ray
	* @param IgnoreComponents optional list of Components to ignore
	* @param InvisibleComponentsToInclude optional list of Components to explicitly include, even if they are not visible
	* @return true if a visible hit was found
	*/
	MODELINGCOMPONENTS_API bool FindNearestVisibleObjectHit(const UInteractiveTool* Tool, FHitResult& HitResultOut, const FRay& Ray,
		const TArray<const UPrimitiveComponent*>* IgnoreComponents = nullptr, 
		const TArray<const UPrimitiveComponent*>* InvisibleComponentsToInclude = nullptr);

}