// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "FrameTypes.h"

class UCurveFloat;
class UTexture;
class UInteractiveToolManager;

class UToolTarget;
class UBaseDynamicMeshComponent;
class UPreviewMesh;

/**
 * Utility functions for Tool implementations to use when doing configuration/setup
 */
namespace ToolSetupUtil
{
	/**
	 * Get the default material for surfaces
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultMaterial();


	/**
	 * Get the default material to use for objects in an InteractiveTool. Optionally use SourceMaterial if it is valid.
	 * @param SourceMaterial optional material to use if available
	 * @return default material to use for objects in a tool.
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultMaterial(UInteractiveToolManager* ToolManager, UMaterialInterface* SourceMaterial = nullptr);

	/**
	 * @return configurable vertex color material
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetVertexColorMaterial(UInteractiveToolManager* ToolManager, bool bTwoSided = false);

	/**
	 * @return a vertex color material with sculpt-material shading
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetVertexColorSculptMaterial(UInteractiveToolManager* ToolManager);


	/**
	 * @return default material to use for "Working"/In-Progress animations
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultWorkingMaterial(UInteractiveToolManager* ToolManager);

	/**
	 * @return default material instance to use for "Working"/In-Progress animations
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetDefaultWorkingMaterialInstance(UInteractiveToolManager* ToolManager);

	/**
	 * @return default material instance to use for "Error" animations
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetDefaultErrorMaterial(UInteractiveToolManager* ToolManager);


	/**
	 * Get a black-and-white NxN checkerboard material
	 * @param CheckerDensity Number of checks along row/column
	 * @return default material to use for uv checkerboard visualizations
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetUVCheckerboardMaterial(double CheckerDensity = 20.0);


	/**
	 * @return default material to use for brush volume indicators (for instance, a spherical sculpt brush).
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetDefaultBrushVolumeMaterial(UInteractiveToolManager* ToolManager);

	/**
	 * Get a Material to use with a square brush plane/ROI indicator that has an analytic falloff function combined w/ an alpha mask texture.
	 * Falloff value will be used for both color and opacity, rendered as Additive with depth-testing disabled
	 * Material Parameters:
	 *  "BrushAlpha" - Texture2D Parameter for brush alpha texture map
	 *  "AlphaPower" - configure whether BrushAlpha texture is multiplied w/ analytic falloff (0 -> disabled, 1 -> enabled)
	 *  "FalloffRatio" - float in range [0,1] that corresponds to width of brush falloff region
	 *  "FalloffMode" - float in range [0,1] used to interpolate between Smooth (0) -> Linear (0.3333) -> Inverse (0.6666) -> Round (1.0)
	 *  "FalloffShape" - float in range [0,1] used to interpolate between Circular (0.0) and Square (1.0) 
	 *  "IntensityScale" - float in range [0,1], multiplier on brightness of falloff highlight (default 0.2)
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetDefaultBrushAlphaMaterial(UInteractiveToolManager* ToolManager);



	/**
	 * @return Sculpt Material 1
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultSculptMaterial(UInteractiveToolManager* ToolManager);

	/**
	 * @param bTwoSided A two sided material has some rendering artifacts in a transparent material because of indeterminate
	 *  ordering of triangles within the mesh. Still, it is sometimes useful despite these flaws.
	 * @return Transparent two-sided material suitable for sculpting (has some shine and a Fresnel effect)
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetTransparentSculptMaterial(UInteractiveToolManager* ToolManager, 
		const FLinearColor& Color, double Opacity, bool bTwoSided);

	/** Types of image-based material that we can create */
	enum class ImageMaterialType
	{
		DefaultBasic,
		DefaultSoft,
		TangentNormalFromView
	};

	/**
	 * @return Image-based sculpt material instance, based ImageMaterialType
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetImageBasedSculptMaterial(UInteractiveToolManager* ToolManager, ImageMaterialType Type);

	/**
	 * @return Image-based sculpt material that supports changing the image
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetCustomImageBasedSculptMaterial(UInteractiveToolManager* ToolManager, UTexture* SetImage);


	/**
	 * @return Selection Material 1
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetSelectionMaterial(UInteractiveToolManager* ToolManager);

	/**
	 * @return Selection Material 1 with custom color and optional depth bias (depth bias moves vertices towards the camera)
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetSelectionMaterial(const FLinearColor& UseColor, UInteractiveToolManager* ToolManager, float DepthBias = 0.0f);

	/**
	 * @return Simple material with configurable color and opacity. Note that the material 
	 *  will have translucent blend mode, which can interact poorly with overlapping translucent
	 *  objects, so use the other overload if you do not need opacity control.
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetSimpleCustomMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float Opacity);

	/**
	 * @return Simple material with configurable color. The material will have opaque blend mode.
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetSimpleCustomMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color);

	 /**
	  * @return Simple material with configurable depth bias, color, and opacity. Note that the material
	  *  will have translucent blend mode, which can interact poorly with overlapping translucent
	  *  objects, so use the other overload if you do not need opacity control.
	  */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetCustomDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias, float Opacity);

	/**
	 * @return Simple material with configurable depth bias and color. The material will have opaque blend mode.
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetCustomDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias);
	
	/**
	 * @return Simple two-sided material with configurable depth bias, color, and opacity. Note that 
	  *  the material will have translucent blend mode, which can interact poorly with overlapping translucent
	  *  objects, so use the other overload if you do not need opacity control.
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetCustomTwoSidedDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias, float Opacity);

	/**
	 * @return Simple two-sided material with configurable depth bias and color. The material will have opaque blend mode.
	 */
	MODELINGCOMPONENTS_API UMaterialInstanceDynamic* GetCustomTwoSidedDepthOffsetMaterial(UInteractiveToolManager* ToolManager, const FLinearColor& Color, float DepthBias);

	/**
	 * @return Material used when editing AVolume objects using our tools.
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultEditVolumeMaterial();

	/**
	 * Gets a custom material suitable for use with UPointSetComponent for square points.
	 * 
	 * @param bDepthTested If true, the material will be depth tested as normal. If false, occluded points will still be
	 *  displayed but dimmed.
	 *  Note that the current implementations of the depth-tested and non-depth-tested modes use opaque and translucent 
	 *  blend modes, respectively, and so inherit their limitations. Specifically, opaque does not support opacity, 
	 *  and translucent does not always follow correct draw order relative to other translucent objects, which means
	 *  that depth bias cannot reliably order lines within a non-depth-tested line set component.
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultPointComponentMaterial(UInteractiveToolManager* ToolManager, bool bDepthTested = true);

	/**
	 * Gets a custom material suitable for use with UPointSetComponent for round points.
	 * Note that this material uses translucent blend mode, and therefore can't always follow the correct draw order
	 * relative to other translucent objects (and within the point set). If this is not acceptible, you will need to use
	 * the depth tested square point material (GetDefaultPointComponentMaterial with bDepthTested = true).
	 * 
	 * @param bDepthTested If true, the material will be depth tested as normal. If false, occluded portions of lines
	 *  will still be displayed, but dashed and dimmed.
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetRoundPointComponentMaterial(UInteractiveToolManager* ToolManager, bool bDepthTested = true);

	/**
	 * Gets a custom material suitable for use with ULineSetComponent.
	 * 
	 * @param bDepthTested If true, the material will be depth tested as normal. If false, occluded portions of lines
	 *  will still be displayed, but dashed and dimmed. 
	 *  Note that the current implementations of the depth-tested and non-depth-tested modes use opaque and translucent 
	 *  blend modes, respectively, and so inherit their limitations. Specifically, opaque does not support opacity, 
	 *  and translucent does not always follow correct draw order relative to other translucent objects, which means
	 *  that depth bias cannot reliably order lines within a non-depth-tested line set component.
	 */
	MODELINGCOMPONENTS_API UMaterialInterface* GetDefaultLineComponentMaterial(UInteractiveToolManager* ToolManager, bool bDepthTested = true);

	/**
	 * @return a curve asset used for contrast adjustments when using a texture map for displacements.
	 */
	MODELINGCOMPONENTS_API UCurveFloat* GetContrastAdjustmentCurve(UInteractiveToolManager* ToolManager);


	/**
	 * Use the current selection, if any, or otherwise the camera position to find a relevant reference frame for creating new objects.
	 * Note: Should be called in tool setup to access the active geometry element selection (if any) before it is cleared by the tool.
	 * @param DefaultOrientation			Orientation to use if there is no geometry element selection providing a natural orientation
	 * @param DefaultPlacementDistance		Distance from camera to place the frame if there is no selection and a raycast doesn't hit anything
	 */
	MODELINGCOMPONENTS_API UE::Geometry::FFrame3d GetDefaultWorldReferenceFrame(UInteractiveToolManager* ToolManager, UE::Geometry::FQuaterniond DefaultOrientation = UE::Geometry::FQuaterniond(), double NoSelectionPlacementDistance = 500);


	//
	// Rendering Configuration/Setup Functions
	// These utility functions are used to configure rendering settings on Preview Meshes created internally by Modeling Tools.
	// 
	//

	MODELINGCOMPONENTS_API void ApplyRenderingConfigurationToPreview(UBaseDynamicMeshComponent* Component, UToolTarget* SourceTarget = nullptr);
	MODELINGCOMPONENTS_API void ApplyRenderingConfigurationToPreview(UPreviewMesh* PreviewMesh, UToolTarget* SourceTarget = nullptr);

}