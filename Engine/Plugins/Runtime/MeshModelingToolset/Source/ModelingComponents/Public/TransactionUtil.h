// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

class UInteractiveToolManager;

// Utilities for managing logic around undo/redo transactions
namespace UE::TransactionUtil
{
	// A simple manager class to track transactions that will be open for a duration of time, e.g. during a mouse drag, within a single tool or mechanic.
	// Provides some error checking to help catch cases where we open without closing or close without opening, or if we nest opening more transactions than expected.
	// One way these can be used is to keep a transaction open during a mouse drag interaction as a way to prevent the user from undoing during the mouse drag.
	// Otherwise, undoing during a mouse drag can cause bugs in many tools if they are not careful to undo any pending work in OnTerminateDragSequence.
	class FLongTransactionTracker
	{
	public:
		FLongTransactionTracker(int32 MaxOpen = 1) : OpenCount(0), MaxOpen(MaxOpen) {}
		~FLongTransactionTracker()
		{
			ensureMsgf(OpenCount == 0, TEXT("Expected all transactions to be closed, but found %d open."), OpenCount);
		}

		// Open a single long transaction.
		MODELINGCOMPONENTS_API void Open(FText TransactionName, UInteractiveToolManager* ToolManager);

		// Close a single long transaction. Expect that at least one is open.
		MODELINGCOMPONENTS_API void Close(UInteractiveToolManager* ToolManager);

		// Close any/all open transactions as part of shutdown of the owning tool/mechanic. Always safe to call on shutdown; OK if none are open.
		MODELINGCOMPONENTS_API void CloseAll(UInteractiveToolManager* ToolManager);

	private:
		// Current number of open long transactions we are tracking.  (Note only the first will actually call BeginUndoTransaction)
		int32 OpenCount;
		// Maximum number of long transactions we allow to be open at once before triggering an ensure()
		int32 MaxOpen;
	};
}
