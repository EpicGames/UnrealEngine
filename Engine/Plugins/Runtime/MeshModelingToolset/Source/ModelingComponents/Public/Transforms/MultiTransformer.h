// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolContextInterfaces.h"
#include "FrameTypes.h"
#include "InteractiveGizmo.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "InteractiveGizmoManager.h"

#include "MultiTransformer.generated.h"

#define UE_API MODELINGCOMPONENTS_API


class UCombinedTransformGizmo;
class UTransformProxy;

UENUM()
enum class EMultiTransformerMode
{
	DefaultGizmo = 1,
	QuickAxisTranslation = 2
};


/**
 * UMultiTransformer abstracts both a default TRS Gizmo, and the "Quick" translate/rotate Gizmos.
 * The "Quick" part is not yet implemented, and we might end up phasing out this class.
 */
UCLASS(MinimalAPI)
class UMultiTransformer : public UObject
{
	GENERATED_BODY()
	using FFrame3d = UE::Geometry::FFrame3d;
public:
	
	UE_API virtual void Setup(UInteractiveGizmoManager* GizmoManager, IToolContextTransactionProvider* TransactionProviderIn);
	UE_API virtual void Shutdown();

	UE_API virtual void Tick(float DeltaTime);

	UE_API virtual void InitializeGizmoPositionFromWorldFrame(const FFrame3d& Frame, bool bResetScale = true);
	UE_API virtual void UpdateGizmoPositionFromWorldFrame(const FFrame3d& Frame, bool bResetScale = true);

	UE_API virtual void ResetScale();

	virtual const FFrame3d& GetCurrentGizmoFrame() const { return ActiveGizmoFrame; }
	virtual const FVector3d& GetCurrentGizmoScale() const { return ActiveGizmoScale; }
	virtual bool InGizmoEdit() const { return bInGizmoEdit;	}

	virtual EMultiTransformerMode GetMode() const { return ActiveMode; }
	UE_API virtual void SetMode(EMultiTransformerMode NewMode);

	UE_API virtual void SetGizmoVisibility(bool bVisible);

	UE_API virtual void SetOverrideGizmoCoordinateSystem(EToolContextCoordinateSystem CoordSystem);

	UE_API virtual void SetEnabledGizmoSubElements(ETransformGizmoSubElements EnabledSubElements);

	UE_API virtual void SetGizmoRepositionable(bool bOn);

	UE_API virtual EToolContextCoordinateSystem GetGizmoCoordinateSystem();

	UE_API void SetSnapToWorldGridSourceFunc(TUniqueFunction<bool()> EnableSnapFunc);
	UE_API void SetIsNonUniformScaleAllowedFunction(TFunction<bool()> IsNonUniformScaleAllowedIn);

	UE_API void SetDisallowNegativeScaling(bool bDisallow);

	UE_API void AddAlignmentMechanic(UDragAlignmentMechanic* AlignmentMechanic);

	DECLARE_MULTICAST_DELEGATE(FMultiTransformerEvent);

	// Note that the following delegates don't fire on pivot repositioning drags.

	/** This delegate is fired when a drag is started */
	FMultiTransformerEvent OnTransformStarted;

	/** This delegate is fired when a drag is updated */
	FMultiTransformerEvent OnTransformUpdated;

	/** This delegate is fired when the drag is completed */
	FMultiTransformerEvent OnTransformCompleted;

	/** This delegate is fired when a repositioning drag is completed */
	FMultiTransformerEvent OnEndPivotEdit;

	// Note: we could have more pivot change delegates, but we don't yet need them,
	// and we might phase out the UMultiTransformer.

protected:
	UPROPERTY()
	TObjectPtr<UInteractiveGizmoManager> GizmoManager;

	IToolContextTransactionProvider* TransactionProvider;

	EMultiTransformerMode ActiveMode;

	ETransformGizmoSubElements ActiveGizmoSubElements = ETransformGizmoSubElements::FullTranslateRotateScale;

	EToolContextCoordinateSystem GizmoCoordSystem = EToolContextCoordinateSystem::World;
	bool bForceGizmoCoordSystem = false;

	bool bShouldBeVisible = true;
	FFrame3d ActiveGizmoFrame;
	FVector3d ActiveGizmoScale;

	bool bRepositionableGizmo = false;

	bool bDisallowNegativeScaling = false;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> TransformGizmo;

	UPROPERTY()
	TObjectPtr<UTransformProxy> TransformProxy;

	// We have to hold on to the mechanic only because the MultiTransformer has the capacity to delete and
	// recreate its gizmo, in which case we'll need to attach the alignment mechanic again.
	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	TUniqueFunction<bool()> EnableSnapToWorldGridFunc;
	TFunction<bool()> IsNonUniformScaleAllowed;

	// called on PlaneTransformProxy.OnTransformChanged
	UE_API void OnProxyTransformChanged(UTransformProxy* Proxy, FTransform Transform);
	UE_API void OnBeginProxyTransformEdit(UTransformProxy* Proxy);
	UE_API void OnEndProxyTransformEdit(UTransformProxy* Proxy);
	bool bInGizmoEdit = false;

	UE_API void UpdateShowGizmoState(bool bNewVisibility);
};

#undef UE_API
