// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "QuickTransformer.h"
#include "ToolDataVisualizer.h"
#include "Snapping/RaySpatialSnapSolver.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE
{
namespace Geometry
{

/**

 */
class FQuickAxisRotator : public FQuickTransformer
{
public:
	virtual ~FQuickAxisRotator() override = default;

	/**
	 * Set up internal data structures
	 */
	UE_API virtual void Initialize() override;

	/**
	 * Set current transform frame to the unit axes at the given Origin
	 */
	UE_API virtual void SetActiveFrameFromWorldAxes(const FVector3d& Origin) override;

	/**
	 * Set current transform frame to the given frame
	 */
	UE_API virtual void SetActiveWorldFrame(const FFrame3d& Frame) override;

	/**
	 * @return the current transform frame
	 */
	const FFrame3d& GetActiveWorldFrame() const 
	{
		return AxisFrameWorld; 
	}

	/**
	 * Set current snap-axis frame to a frame at the given Origin with Z aligned to the given Normal.
	 * @param bAlignToWorldAxes if true, the frame is rotated around the Normal so that the X and Y axes are best-aligned to the World axes
	 */
	UE_API virtual void SetActiveFrameFromWorldNormal(const FVector3d& Origin, const FVector3d& Normal, bool bAlignToWorldAxes);

	/**
	 * Update the current snap-axis frame with a new origin
	 */
	UE_API virtual void UpdateActiveFrameOrigin(const FVector3d& NewOrigin) override;


	/**
	 * Try to find the best snap point for the given Ray, and store in SnapPointOut if found
	 * @return true if found
	 */
	UE_API virtual bool UpdateSnap(const FRay3d& Ray, FVector3d& SnapPointOut);

	/** @return true if there is an active snap */
	UE_API virtual bool HaveActiveSnap() const;


	UE_API virtual bool HaveActiveSnapRotation() const;

	UE_API virtual FFrame3d GetActiveRotationFrame() const;

	/** Reset transformer state */
	UE_API virtual void Reset() override;


	UE_API void ClearAxisLock();
	UE_API void SetAxisLock();
	bool GetHaveLockedToAxis() const { return bHaveLockedToAxis; }


	/**
	 * Update internal copy of camera state. You must call this for snapping to work!
	 */
	UE_API virtual void UpdateCameraState(const FViewCameraState& CameraState) override;

	/**
	 * Draw a visualization of the current snap axes and active snap point
	 * @param RenderAPI provide access to context rendering info
	 */
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * Draw a visualization of the current snap axes and active snap point
	 * @param RenderAPI provide access to context rendering info
	 */
	UE_API virtual void PreviewRender(IToolsContextRenderAPI* RenderAPI) override;


protected:
	// camera state saved at last Render()
	FViewCameraState CameraState;

	bool bHaveLockedToAxis = false;
	FIndex3i IgnoredAxes;

	FFrame3d AxisFrameWorld;

	FRaySpatialSnapSolver MoveAxisSolver;
	UE_API void UpdateSnapAxes();

	FToolDataVisualizer QuickAxisRenderer;
	TMap<int, FLinearColor> AxisColorMap;

	FToolDataVisualizer QuickAxisPreviewRenderer;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
