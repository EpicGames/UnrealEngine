// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "QuickTransformer.h"
#include "ToolDataVisualizer.h"
#include "Snapping/RaySpatialSnapSolver.h"

#define UE_API MODELINGCOMPONENTS_API

namespace UE
{
namespace Geometry
{

/**
 * FQuickAxisTranslater implements the underpinnings for "quick" axis transformations, ie axis-gizmo-like
 * behavior without having to explicitly click on an axis.
 * 
 * To use this class, you first configure the internal world-axis-frame using the SetActiveX() functions.
 * Then as you collect input updates, you call UpdateSnap() with the input ray, and this will output
 * a snapped 3D world-space point. The delta (Frame.Origin - SnapPoint) is the move axis.
 * 
 * You must also call UpdateCameraState() each time the camera changes (typically each frame in a tool Render())
 * 
 * A default visualization is provided via the Render() function
 * 
 * A small snap-ball around the frame origin prevents small movements, which are unstable with this approach.
 * @todo add ability to do small movements once an axis is chosen?
 */
class FQuickAxisTranslater : public FQuickTransformer
{
public:
	virtual ~FQuickAxisTranslater() override = default;

	/**
	 * Set up internal data structures
	 */
	UE_API virtual void Initialize() override;

	/**
	 * Set current snap-axis frame to the unit axes at the given Origin
	 */
	UE_API virtual void SetActiveFrameFromWorldAxes(const FVector3d& Origin) override;

	/**
	 * Set current snap-axis frame to the given frame
	 */
	UE_API virtual void SetActiveWorldFrame(const FFrame3d& Frame) override;

	/**
	 * Set current snap-axis frame to a frame at the given Origin with Z aligned to the given Normal.
	 * @param bAlignToWorldAxes if true, the frame is rotated around the Normal so that the X and Y axes are best-aligned to the World axes
	 */
	UE_API virtual void SetActiveFrameFromWorldNormal(const FVector3d& Origin, const FVector3d& Normal, bool bAlignToWorldAxes);

	/**
	 * Update the current snap-axis frame with a new origin
	 */
	UE_API virtual void UpdateActiveFrameOrigin(const FVector3d& NewOrigin) override;


	/**
	 * Try to find the best snap point for the given Ray, and store in SnapPointOut if found
	 * @param Ray 3D ray in space of snap points
	 * @param SnapPointOut found snap point
	 * @param PositionConstraintFunc Function that projects potential snap points onto constraint surfaces (eg grid points)
	 * @return true if found
	 */
	UE_API virtual bool UpdateSnap(const FRay3d& Ray, FVector3d& SnapPointOut,
		TFunction<FVector3d(const FVector3d&)> PositionConstraintFunc = nullptr);

	/** @return true if there is an active snap */
	UE_API virtual bool HaveActiveSnap() const;


	/**
	 * Update internal copy of camera state. You must call this for snapping to work!
	 */
	UE_API virtual void UpdateCameraState(const FViewCameraState& CameraState) override;

	/**
	 * Draw a visualization of the current snap axes and active snap point
	 * @param RenderAPI provide access to context rendering info
	 */
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	/**
	 * Draw a visualization of the current snap axes and active snap point
	 * @param RenderAPI provide access to context rendering info
	 */
	UE_API virtual void PreviewRender(IToolsContextRenderAPI* RenderAPI) override;

	/** Reset transformer state */
	UE_API virtual void Reset() override;


protected:
	// camera state saved at last Render()
	FViewCameraState CameraState;

	FFrame3d AxisFrameWorld;

	FRaySpatialSnapSolver MoveAxisSolver;
	UE_API void UpdateSnapAxes();

	FToolDataVisualizer QuickAxisRenderer;
	TMap<int, FLinearColor> AxisColorMap;

	FToolDataVisualizer QuickAxisPreviewRenderer;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
