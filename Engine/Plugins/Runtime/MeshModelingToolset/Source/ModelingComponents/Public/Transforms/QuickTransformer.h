// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolContextInterfaces.h"
#include "FrameTypes.h"

namespace UE
{
namespace Geometry
{

/**

 */
class FQuickTransformer
{
public:
	virtual ~FQuickTransformer() = default;

	/**
	 * Set up internal data structures
	 */
	virtual void Initialize() = 0;


	/**
	 * Set current transform frame to the unit axes at the given Origin
	 */
	virtual void SetActiveFrameFromWorldAxes(const FVector3d& Origin) = 0;

	/**
	 * Set current transform frame  to the given frame
	 */
	virtual void SetActiveWorldFrame(const FFrame3d& Frame) = 0;

	/**
	 * Update the current snap-axis frame with a new origin
	 */
	virtual void UpdateActiveFrameOrigin(const FVector3d& NewOrigin) = 0;

	/**
	 * Reset transformer state
	 */
	virtual void Reset() = 0;


	/**
	 * Update internal copy of camera state. You must call this for snapping to work!
	 */
	virtual void UpdateCameraState(const FViewCameraState& CameraState) = 0;

	/**
	 * Draw a visualization of the current snap axes and active snap point
	 * @param RenderAPI provide access to context rendering info
	 */
	virtual void Render(IToolsContextRenderAPI* RenderAPI) = 0;

	/**
	 * Draw a visualization of the current snap axes and active snap point
	 * @param RenderAPI provide access to context rendering info
	 */
	virtual void PreviewRender(IToolsContextRenderAPI* RenderAPI) = 0;

};

} // end namespace UE::Geometry
} // end namespace UE
