// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "BaseGizmos/TransformGizmoUtil.h" // UCombinedTransformGizmoContextObject::IGizmoBinder

#define UE_API MODELINGCOMPONENTS_API

class UCombinedTransformGizmo;
class UTransformProxy;

/**
 * Helper class for binding a UCombinedTransformGizmo to several FVector3d's such that they
 * are updated when the gizmo is changed, and the gizmo is updated from the vectors if the
 * approriate function is called. The vectors can serve as a display of the gizmo's values,
 * and if they are made modifiable, they can serve as a numerical control of the gizmo.
 * 
 * To use, just call InitializeBoundVectors with 3 vectors (it can be fewer, but the binder
 * currently does the same amount of work regardless of whether 3 or fewer are bound), and
 * then either call BindToGizmoContextObject with the context object that will be used to
 * create the gizmos (in which case the vectors will bind to all newly created gizmos, and 
 * will switch bindings as different gizmos are dragged), or just bind to a particular gizmo
 * using BindToInitializedGizmo after its SetActiveTarget call has been made..
 */
class FTransformGizmoDataBinder : public TSharedFromThis<FTransformGizmoDataBinder, ESPMode::ThreadSafe>
{
public:

	UE_API virtual ~FTransformGizmoDataBinder();

	/**
	 * Set the vectors that should reflect the gizmo transform. Any of these pointers can be null,
	 * but currently the same amount of work is done by the binder (it will simply bind to some
	 * internal vectors).
	 */
	UE_API virtual void InitializeBoundVectors(FVector3d* Translation, FVector3d* RotaionEulerAngles, FVector3d* Scale);
	
	/**
	 * Makes it so that the gizmo binder attaches to any gizmos created by the context object
	 * in the future. The binding is automatically removed if FTransformGizmoDataBinder is
	 * destroyed.
	 */
	UE_API virtual void BindToGizmoContextObject(UCombinedTransformGizmoContextObject* ContextObject);

	/**
	 * Binds to a specific gizmo for tracking. Requires ActiveTarget to be set, and is
	 * done automatically for gizmos when binding via the gizmo context object.
	 */
	UE_API virtual void BindToInitializedGizmo(UCombinedTransformGizmo* Gizmo, UTransformProxy*);
	/**
	 * Unbinds from a given gizmo. Done automatically for gizmos when their ActiveTarget
	 * is cleared when binding via the gizmo context object.
	 */
	UE_API virtual void UnbindFromGizmo(UCombinedTransformGizmo* Gizmo, UTransformProxy*);

	// These are used to push updates from the current value of vectors to the gizmo:

	/**
	 * Should be called at the beginning of a sequence of UpdateAfterDataEdit calls, for instance
	 * if the bound values are being scrubbed in a display.
	 */
	UE_API void BeginDataEditSequence();
	/**
	 * Called to update the gizmo after updating the bound data. This acts slightly differently
	 * depending on whether we're between begin/end sequence calls (for instance it will trigger
	 * individual undo transactions if not between Begin/End calls).
	 */
	UE_API void UpdateAfterDataEdit();
	/**
	 * Should be called at the end of a sequence of UpdateAfterDataEdit calls, for instance
	 * if the bound values are being scrubbed in a display.
	 */
	UE_API void EndDataEditSequence();

	/**
	 * Can either be called on tick or in response to some broadcast to allow the binder
	 * to respond to coordinate system changes. If not called, the coordinate system will
	 * still be updated the next time that the gizmo is modified.
	 */
	UE_API virtual void UpdateCoordinateSystem();

	/**
	 * Determines whether there is a visible currently tracked gizmo.
	 */
	UE_API virtual bool HasVisibleGizmo();

	/**
	 * In delta mode, the bound values hold the difference of the current gizmo transform relative
	 * to some previous start transform. When false, the binder is in "destination mode", where the
	 * bound values represent the final gizmo transform.
	 */
	virtual bool IsInDeltaMode() { return bUsingDeltaMode; }

	/**
	 * Sets to delta mode with given start transform. This call will update the bound values
	 * appropriately from the current gizmo transform.
	 */
	UE_API virtual void SetToDeltaMode(const FTransform& StartTransform);

	/**
	 * Much like SetToDeltaMode, but uses the current gizmo transform as the start transform, meaning
	 * that the bound data values will be reset to zero.
	 */
	UE_API virtual void ResetToDeltaMode();

	/**
	 * Sets to destination mode (as opposed to delta mode). Calling this will update the bound values
	 * to reflect the current tracked gizmo transform. If the gizmo is in local coordinate mode and it
	 * has a custom reference transform, that will be used as the reference.
	 */
	UE_API virtual void SetToDestinationMode();

	/**
	 * If the gizmo doesn't have all three axes, then it likely needs to stay constrained in some
	 * plane/line. In this situation, allowing destination mode is only safe if the gizmo has a 
	 * DisplaySpaceTransform that (presumably) orients the measurement axes correctly, or if the
	 * coordinate mode is global, in which case the world axes are known to be fine.
	 * 
	 * Currently, the binder will automatically switch to delta mode when tracking a new gizmo
	 * that doesn't satisfy the above definition of safety, though it won't prevent the mode from
	 * being switched from there...
	 */
	UE_API virtual bool ShouldDestinationModeBeAllowed();

	/**
	 * Changes which gizmo is being tracked, out of those to which the data binder has been
	 * bound to via BindToInitializedGizmo().
	 *
	 * Note that if the same data binder is bound to multiple gizmos, then it will switch tracked gizmos
	 * automatically whenever a gizmo is dragged, or whenever a gizmo becomes visible without the previously
	 * tracked gizmo being visible.
	 */
	UE_API virtual void SetTrackedGizmo(UCombinedTransformGizmo* Gizmo);

	/**
	 * Allows the data bound to the translation component to be stored in different units than
	 * used by the gizmo. For instance, in a UV editor where the world coordinates are not actually 0 to 1.
	 */
	virtual void SetTranslationConversionFunctions(
		TFunction<FVector3d(const FVector3d& InternalValue)> ActualToBoundConversionIn,
		TFunction<FVector3d(const FVector3d& DisplayValue)> BoundToActualConversionIn)
	{
		ActualToBoundConversion = ActualToBoundConversionIn;
		BoundToActualConversion = BoundToActualConversionIn;
	}

	/**
	 * In local coordinate destination mode, when a DisplaySpaceTransform is not available on
	 * the gizmo, this default transform can be used instead. Currently, this is used to allow
	 * destination mode for the 2D gizmos in UV editor, but that's a fairly specific use case.
	 */
	void SetDefaultLocalReferenceTransform(TOptional<FTransform> DefaultTransform)
	{
		DefaultCustomLocalReferenceTransform = DefaultTransform;
	}

	/**
	 * Unbinds from everything
	 */
	UE_API virtual void Reset();

	/**
	 * Broadcast whenever the tracked gizmo is changed (because one of multiple gizmos is
	 * dragged, or a new gizmo appears after no others are present).
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTrackedGizmoChanged, UCombinedTransformGizmo*);
	FOnTrackedGizmoChanged OnTrackedGizmoChanged;


protected:
	// Gizmo to which the values are currently bound.
	TWeakObjectPtr<UCombinedTransformGizmo> CurrentlyTrackedGizmo;

	// All the gizmos the numerical UI can correspond to. When one of them is dragged,
	// CurrentlyTrackedGizmo is updated.
	TSet<TWeakObjectPtr<UCombinedTransformGizmo>> BoundGizmos;

	FVector3d VectorsToUseIfUnbound[3]{};

	FVector3d* BoundTranslation = VectorsToUseIfUnbound;
	FVector3d* BoundEulerAngles = VectorsToUseIfUnbound + 1;
	FVector3d* BoundScale = VectorsToUseIfUnbound + 2;

	// Used to detect whether anything actually changed.
	FVector3d LastTranslation;
	FVector3d LastEulerAngles;
	FVector3d LastScale;

	UE_API FTransform GetGizmoTransformFromDataValues(bool bLocalCoordinates);
	UE_API void UpdateDataValuesFromGizmo();

	// Controls delta mode, where the gizmo specifies a difference from some start transform.
	bool bUsingDeltaMode = false;
	FTransform DeltaStartTransform;

	// Used for debouncing when editing the gizmo via the UI
	bool bIgnoreCallbackForDebouncing = false;

	// Additional state that lets us respond appropriately to various notifications
	bool bInDataEditSequence = false;
	bool bGizmoIsBeingDragged = false;

	// If a gizmo doesn't have all three axes and is not aligned to world (i.e. is in local mode) then 
	// we currently require a reference transform to allow the user to be in destination mode.
	bool bCurrentGizmoLacksDegreeOfFreedom = false;

	bool bCurrentGizmoOnlyHasUniformScale = false;

	// Reference transform to use when in local gizmo mode. If it is not specified, the
	// global reference transform is used.
	TOptional<FTransform> CurrentCustomLocalReferenceTransform;

	// Reference transform to use in local gizmo mode by default.
	TOptional<FTransform> DefaultCustomLocalReferenceTransform;

	// See comment for SetTranslationConversionFunctions
	TFunction<FVector3d(const FVector3d& UIValues)> ActualToBoundConversion = nullptr;
	TFunction<FVector3d(const FVector3d& UIValues)> BoundToActualConversion = nullptr;

	// Used when we only allow uniform scaling, to scale everything proportionally
	FVector3d ProportionalDragInitialVector;

	// Used to determine whether the current gizmo's coordinate system has changed,
	// when we receive an UpdateCoordinateSystem() call.
	EToolContextCoordinateSystem LastCoordinateSystem;

	// Used in binding to gizmos. Currently these need to be member variables
	// to allow the use of AddSP, as there is not an AddLambda overload that allows
	// for subsequent RemoveAll() calls (because AddWeakLambda only works with UObjects).
	UE_API void OnProxyBeginTransformEdit(UTransformProxy* TransformProxy, UCombinedTransformGizmo* Gizmo);
	UE_API void OnProxyTransformChanged(UTransformProxy* TransformProxy, FTransform Transform, UCombinedTransformGizmo* Gizmo);
	UE_API void OnProxyEndTransformEdit(UTransformProxy* TransformProxy, UCombinedTransformGizmo* Gizmo);
	UE_API void OnDisplaySpaceTransformChanged(UCombinedTransformGizmo* Gizmo, TOptional<FTransform> Transform);
	UE_API void OnGizmoVisibilityChanged(UCombinedTransformGizmo* Gizmo, bool bVisible);

	// Used for binding to context object, called from the OnGizmoCreated delegate.
	UE_API void BindToUninitializedGizmo(UCombinedTransformGizmo* Gizmo);

	// Used to clean up our registration with context object when being destroyed
	TArray<TWeakObjectPtr<UCombinedTransformGizmoContextObject>> ContextObjectsToUnregisterWith;

private:

	// Currently we don't expose any of these because it is not clear whether we want them to be
	// customizable. However, if we decide that we want some of our behavior choices to be configurable,
	// we may expose them.
	bool bTriggerSequenceBookendsForNonSequenceUpdates = true;
	bool bEnforceUniformScaleConstraintsIfPresent = true;
	bool bChangeDisplayedGizmoOnDrag = true;
	bool bAvoidDestinationModeWhenUnsafe = true;
};

#undef UE_API
