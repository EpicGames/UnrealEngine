// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BoxTypes.h"
#include "WeightMapTypes.h"

struct FMeshDescription;


namespace UE
{
	namespace WeightMaps
	{
		using namespace UE::Geometry;

		/**
		 * Find the set of per-vertex weight map attributes on a MeshDescription
		 */
		MODELINGCOMPONENTS_API void FindVertexWeightMaps(const FMeshDescription* Mesh, TArray<FName>& PropertyNamesOut);

		/**
		 * Extract a per-vertex weight map from a MeshDescription
		 * If the attribute with the given name is not found, a WeightMap initialized with the default value is returned
		 * @return false if weight map was not found
		 */
		MODELINGCOMPONENTS_API bool GetVertexWeightMap(const FMeshDescription* Mesh, FName AttributeName, FIndexedWeightMap1f& WeightMap, float DefaultValue = 1.0f);

	}
}