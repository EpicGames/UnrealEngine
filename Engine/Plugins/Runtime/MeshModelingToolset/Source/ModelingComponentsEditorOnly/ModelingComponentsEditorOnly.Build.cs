// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelingComponentsEditorOnly : ModuleRules
{
	public ModelingComponentsEditorOnly(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "InteractiveToolsFramework",
                "MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"DynamicMesh",
				"MeshConversion",
				"ModelingOperators",
				"ModelingComponents"
				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BSPUtils",
				"CoreUObject",
				"Engine",
				"InputCore",
                "RenderCore",
				"PhysicsCore",
				"Slate",
                "RHI",
				"AssetTools",
				"UnrealEd", 
				"SkeletalMeshUtilitiesCommon", // required for asset factories,
				"SkeletalMeshModifiers"
			}
			);

		// OpenSubdiv has Windows, Mac and Unix support
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.Platform == UnrealTargetPlatform.Mac || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"OpenSubdiv",
				}
				);
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
