// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetUtils/CreateMaterialUtil.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Factories/MaterialInstanceConstantFactoryNew.h"

#include "AssetToolsModule.h"
#include "IAssetTools.h"


UE::AssetUtils::ECreateMaterialResult UE::AssetUtils::CreateDuplicateMaterial(
	UMaterialInterface* BaseMaterial,
	FMaterialAssetOptions& Options,
	FMaterialAssetResults& ResultsOut)
{
	// Duplicate operation will create a new Package, so calling code should not!
	if (ensure(Options.UsePackage == nullptr) == false)
	{
		return ECreateMaterialResult::InvalidPackage;
	}

	if (ensure(BaseMaterial) == false)
	{
		return ECreateMaterialResult::InvalidBaseMaterial;
	}
	UMaterial* SourceMaterial = BaseMaterial->GetBaseMaterial();
	if (ensure(SourceMaterial) == false)
	{
		return ECreateMaterialResult::InvalidBaseMaterial;
	}

	// cannot create new package if path already exists
	if (FPackageName::DoesPackageExist(Options.NewAssetPath))
	{
		ensure(false);
		return ECreateMaterialResult::NameError;
	}

	FString DestinationLongPackagePath = FPackageName::GetLongPackagePath(Options.NewAssetPath);
	FString DestinationObjectName = FPackageName::GetLongPackageAssetName(Options.NewAssetPath);

	UObject* NewAsset = IAssetTools::Get().DuplicateAsset(DestinationObjectName, DestinationLongPackagePath, SourceMaterial);
	UMaterial* NewMaterial = Cast<UMaterial>(NewAsset);

	if ( ensure(NewMaterial) == false )
	{
		return ECreateMaterialResult::DuplicateFailed;
	}

	if (Options.bDeferPostEditChange == false)
	{
		NewMaterial->PostEditChange();
	}

	ResultsOut.NewMaterial = NewMaterial;
	return ECreateMaterialResult::Ok;
}

UE::AssetUtils::ECreateMaterialResult UE::AssetUtils::CreateDerivedMaterialInstance(
	UMaterialInterface* BaseMaterial,
	FMaterialAssetOptions& Options,
	FMaterialAssetResults& ResultsOut)
{
	// Duplicate operation will create a new Package, so calling code should not!
	if (ensure(Options.UsePackage == nullptr) == false)
	{
		return ECreateMaterialResult::InvalidPackage;
	}

	if (ensure(BaseMaterial) == false)
	{
		return ECreateMaterialResult::InvalidBaseMaterial;
	}

	// cannot create new package if path already exists
	if (FPackageName::DoesPackageExist(Options.NewAssetPath))
	{
		ensure(false);
		return ECreateMaterialResult::NameError;
	}

	FString DestinationLongPackagePath = FPackageName::GetLongPackagePath(Options.NewAssetPath);
	FString DestinationObjectName = FPackageName::GetLongPackageAssetName(Options.NewAssetPath);

	UMaterialInstanceConstantFactoryNew* Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
	Factory->InitialParent = BaseMaterial;

	UObject* NewAsset = IAssetTools::Get().CreateAsset(DestinationObjectName, DestinationLongPackagePath, UMaterialInstanceConstant::StaticClass(), Factory);

	UMaterialInstanceConstant* NewMaterialInstance = Cast<UMaterialInstanceConstant>(NewAsset);
	if (ensure(NewMaterialInstance) == false)
	{
		return ECreateMaterialResult::DuplicateFailed;
	}

	if (Options.bDeferPostEditChange == false)
	{
		NewMaterialInstance->PostEditChange();
	}

	ResultsOut.NewMaterialInstance = NewMaterialInstance;
	return ECreateMaterialResult::Ok;
}
