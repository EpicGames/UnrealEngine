// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetUtils/CreateSkeletalMeshUtil.h"

#include "Animation/Skeleton.h"
#include "DynamicMeshToMeshDescription.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Misc/PackageName.h"
#include "StaticToSkeletalMeshConverter.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Components/SkeletalMeshComponent.h"

UE::AssetUtils::ECreateSkeletalMeshResult UE::AssetUtils::CreateSkeletalMeshAsset(
	const FSkeletalMeshAssetOptions& Options,
	FSkeletalMeshResults& ResultsOut
	)
{
	const FString NewObjectName = FPackageName::GetLongPackageAssetName(Options.NewAssetPath);

	if (!ensure(Options.Skeleton))
	{
		return ECreateSkeletalMeshResult::InvalidSkeleton;
	}

	UPackage* UsePackage;
	if (Options.UsePackage != nullptr)
	{
		UsePackage = Options.UsePackage;
	}
	else
	{
		UsePackage = CreatePackage(*Options.NewAssetPath);
	}
	if (ensure(UsePackage != nullptr) == false)
	{
		return ECreateSkeletalMeshResult::InvalidPackage;
	}

	UsePackage->FullyLoad();

	USkeletalMesh* NewSkeletalMesh = nullptr;

	// Avoid creating new skeletal mesh every time as this can crash. Instead, first
	// try to find an existing skeletal mesh for the given package and object name.
	USkeletalMesh* ExistingSkelMesh = LoadObject<USkeletalMesh>(UsePackage, *NewObjectName, nullptr, LOAD_NoWarn | LOAD_Quiet);
	bool bSkelMeshUpdate = false;
	constexpr EObjectFlags UseFlags = RF_Public | RF_Standalone;

	if (ExistingSkelMesh)
	{
		bSkelMeshUpdate = true;
		NewSkeletalMesh = ExistingSkelMesh;

		NewSkeletalMesh->PreEditChange(nullptr);

		if (!NewSkeletalMesh->GetMorphTargets().IsEmpty())
		{
			NewSkeletalMesh->UnregisterAllMorphTarget();
		}

		FSkeletalMeshModel* ImportedResource = NewSkeletalMesh->GetImportedModel();
		ImportedResource->EmptyOriginalReductionSourceMeshData();
		ImportedResource->LODModels.Empty();
		ImportedResource->InlineReductionCacheDatas.Empty();

		NewSkeletalMesh->SetNumSourceModels(0);
		NewSkeletalMesh->GetMaterials().Empty();
		NewSkeletalMesh->GetRefSkeleton().Empty();
		NewSkeletalMesh->SetSkeleton(nullptr);
		NewSkeletalMesh->SetPhysicsAsset(nullptr);

		NewSkeletalMesh->ReleaseResources();
		NewSkeletalMesh->InvalidateDeriveDataCacheGUID();

		// Make sure that we create non-transient, standalone assets
		NewSkeletalMesh->ClearFlags(RF_Transient);
		NewSkeletalMesh->SetFlags(UseFlags);
	}
	else
	{
		NewSkeletalMesh = NewObject<USkeletalMesh>(UsePackage, FName(*NewObjectName), UseFlags);
		if (ensure(NewSkeletalMesh != nullptr) == false)
		{
			return ECreateSkeletalMeshResult::UnknownError;
		}
	}

	const int32 UseNumSourceModels = FMath::Max(1, Options.NumSourceModels);
	
	TArray<const FMeshDescription*> MeshDescriptions;
	TArray<FMeshDescription> ConstructedMeshDescriptions;
	if (!Options.SourceMeshes.MoveMeshDescriptions.IsEmpty())
	{
		if (ensure(Options.SourceMeshes.MoveMeshDescriptions.Num() == UseNumSourceModels))
		{
			MeshDescriptions.Append(Options.SourceMeshes.MoveMeshDescriptions);
		}
	}
	else if (!Options.SourceMeshes.MeshDescriptions.IsEmpty())
	{
		if (ensure(Options.SourceMeshes.MeshDescriptions.Num() == UseNumSourceModels))
		{
			MeshDescriptions.Append(Options.SourceMeshes.MeshDescriptions);
		}
	}
	else if (!Options.SourceMeshes.DynamicMeshes.IsEmpty())
	{
		if (ensure(Options.SourceMeshes.DynamicMeshes.Num() == UseNumSourceModels))
		{
			for (const FDynamicMesh3* DynamicMesh : Options.SourceMeshes.DynamicMeshes)
			{
				ConstructedMeshDescriptions.AddDefaulted();
				FConversionToMeshDescriptionOptions ConverterOptions;
				ConverterOptions.bConvertBackToNonManifold = Options.bConvertBackToNonManifold;
				FDynamicMeshToMeshDescription Converter(ConverterOptions);
				FSkeletalMeshAttributes Attributes(ConstructedMeshDescriptions.Last());
				Attributes.Register();
				Converter.Convert(DynamicMesh, ConstructedMeshDescriptions.Last(), !Options.bEnableRecomputeTangents);
				MeshDescriptions.Add(&ConstructedMeshDescriptions.Last());
			}
		}
	}

	TArray<FSkeletalMaterial> Materials;
	TConstArrayView<FSkeletalMaterial> MaterialView;
	if (!Options.SkeletalMaterials.IsEmpty())
	{
		MaterialView = Options.SkeletalMaterials;
	}
	else if (!Options.AssetMaterials.IsEmpty())
	{
		for (UMaterialInterface* MaterialInterface : Options.AssetMaterials)
		{
			Materials.Add(FSkeletalMaterial(MaterialInterface));
		}
		MaterialView = Materials;
	}

	// ensure there is at least one material
	if (MaterialView.IsEmpty())
	{
		Materials.Add(FSkeletalMaterial());
		MaterialView = Materials;
	}

	if (Options.bApplyNaniteSettings)
	{
		NewSkeletalMesh->NaniteSettings = Options.NaniteSettings;
	}
	
	FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
	Parameters.Materials = MaterialView;
	Parameters.bRecomputeNormals = Options.bEnableRecomputeNormals;
	Parameters.bRecomputeTangents = Options.bEnableRecomputeTangents;
	
	if (!FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
		NewSkeletalMesh,
		MeshDescriptions, 
		Options.RefSkeleton ? *Options.RefSkeleton : Options.Skeleton->GetReferenceSkeleton(),
		Parameters))
	{
		return ECreateSkeletalMeshResult::UnknownError;
	}

	// Update the skeletal mesh and the skeleton so that their ref skeletons are in sync and the skeleton's preview mesh
	// is the one we just created.
	NewSkeletalMesh->SetSkeleton(Options.Skeleton);
	Options.Skeleton->MergeAllBonesToBoneTree(NewSkeletalMesh);
	if (!Options.Skeleton->GetPreviewMesh())
	{
		Options.Skeleton->SetPreviewMesh(NewSkeletalMesh);
	}

	if (bSkelMeshUpdate)
	{
		// Force materials to update in the asset editor, if opened
		for (TObjectIterator<USkeletalMeshComponent> It; It; ++It)
		{
			USkeletalMeshComponent* Component = *It;
			if (Component && Component->GetSkeletalMeshAsset() == NewSkeletalMesh)
			{
				Component->MarkRenderStateDirty();
			}
		}
	}

	ResultsOut.SkeletalMesh = NewSkeletalMesh;
	return ECreateSkeletalMeshResult::Ok;
}
