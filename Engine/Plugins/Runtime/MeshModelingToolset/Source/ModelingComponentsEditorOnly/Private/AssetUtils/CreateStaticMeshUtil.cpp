// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetUtils/CreateStaticMeshUtil.h"

#include "CoreGlobals.h" // GUndo
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"

#include "Misc/PackageName.h"
#include "PhysicsEngine/BodySetup.h"

#include "MeshDescription.h"
#include "DynamicMeshToMeshDescription.h"
#include "UObject/Package.h"

#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "AssetUtils/StaticMeshMaterialUtil.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"

using namespace UE::AssetUtils;

using namespace UE::Geometry;


UE::AssetUtils::ECreateStaticMeshResult UE::AssetUtils::CreateStaticMeshAsset(
	FStaticMeshAssetOptions& Options,
	FStaticMeshResults& ResultsOut)
{
	// MINOR HACK: undoing static mesh asset creation causes a bunch of problems because the asset is not deleted,
	// and is instead left in an invalid state that can cause crashes later. Since we can't undo full asset creation
	// anyway, we are going to make sure that we don't transact the static mesh asset creation at all. We still want
	// to allow the calling code to enclose this in another transaction because upstream code may not want to care
	// about what type of object we are creating, so it shouldn't have to be careful with its transaction placement.
	// So we temporarily disconnect the current transaction object, if any.
	ITransaction* UndoState = GUndo;
	GUndo = nullptr; // Pretend we're not in a transaction
	ON_SCOPE_EXIT{ GUndo = UndoState; }; // Revert

	FString NewObjectName = FPackageName::GetLongPackageAssetName(Options.NewAssetPath);

	UPackage* UsePackage;
	if (Options.UsePackage != nullptr)
	{
		UsePackage = Options.UsePackage;
	}
	else
	{
		UsePackage = CreatePackage(*Options.NewAssetPath);
	}
	if (ensure(UsePackage != nullptr) == false)
	{
		return ECreateStaticMeshResult::InvalidPackage;
	}

	//Make sure the package is fully load.
	UsePackage->FullyLoad();

	// create new UStaticMesh object
	EObjectFlags UseFlags = EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;
	UStaticMesh* NewStaticMesh = NewObject<UStaticMesh>(UsePackage, FName(*NewObjectName), UseFlags);
	if (ensure(NewStaticMesh != nullptr) == false)
	{
		return ECreateStaticMeshResult::UnknownError;
	}

	// initialize the MeshDescription SourceModel LODs
	int32 UseNumSourceModels = FMath::Clamp(Options.NumSourceModels, 1, MAX_STATIC_MESH_LODS);
	NewStaticMesh->SetNumSourceModels(UseNumSourceModels);
	for (int32 k = 0; k < UseNumSourceModels; ++k)
	{
		FMeshBuildSettings& BuildSettings = NewStaticMesh->GetSourceModel(k).BuildSettings;

		BuildSettings.bRecomputeNormals = Options.bEnableRecomputeNormals;
		BuildSettings.bRecomputeTangents = Options.bEnableRecomputeTangents;
		BuildSettings.bGenerateLightmapUVs = Options.bGenerateLightmapUVs;
		BuildSettings.bUseFullPrecisionUVs = Options.bUseFullPrecisionUVs;
		BuildSettings.bBuildReversedIndexBuffer = Options.bBuildReversedIndexBuffer;
		
		if (!Options.bAllowDistanceField)
		{
			BuildSettings.DistanceFieldResolutionScale = 0.0f;
		}

		NewStaticMesh->CreateMeshDescription(k);
	}

	// create physics body and configure appropriately
	if (Options.bCreatePhysicsBody)
	{
		NewStaticMesh->CreateBodySetup();
		NewStaticMesh->GetBodySetup()->CollisionTraceFlag = Options.CollisionType;
	}

	// Must always have one material slot.
	int32 UseNumMaterialSlots = FMath::Max(1, Options.NumMaterialSlots);
	// initialize list of Static Materials. Set Default Surface Material in each slot if no
	// other Material was provided. Derive slot names from material names 
	TArray<FStaticMaterial> StaticMaterials;
	for (int32 MatIdx = 0; MatIdx < UseNumMaterialSlots; ++MatIdx)
	{
		FStaticMaterial NewMaterial;
		// fallback to default material if no material is found
		if (MatIdx < Options.AssetMaterials.Num() && Options.AssetMaterials[MatIdx] != nullptr)
		{
			NewMaterial.MaterialInterface = Options.AssetMaterials[MatIdx];
			NewMaterial.MaterialSlotName = UE::AssetUtils::GenerateNewMaterialSlotName(StaticMaterials, NewMaterial.MaterialInterface, MatIdx);
		}
		else
		{
			NewMaterial.MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
			NewMaterial.MaterialSlotName = UE::AssetUtils::GenerateNewMaterialSlotName(StaticMaterials, nullptr, MatIdx);
		}
		NewMaterial.ImportedMaterialSlotName = NewMaterial.MaterialSlotName;
		NewMaterial.UVChannelData = FMeshUVChannelInfo(1.f);		// this avoids an ensure in  UStaticMesh::GetUVChannelData
		StaticMaterials.Add(NewMaterial);
	}
	NewStaticMesh->SetStaticMaterials(StaticMaterials);


	// determine maximum number of sections across all mesh LODs
	int32 MaxNumSections = 0;

	// if options included SourceModel meshes, copy them over
	if (Options.SourceMeshes.MoveMeshDescriptions.Num() > 0)
	{
		if (ensure(Options.SourceMeshes.MoveMeshDescriptions.Num() == UseNumSourceModels))
		{
			for (int32 k = 0; k < UseNumSourceModels; ++k)
			{
				FMeshDescription* Mesh = NewStaticMesh->GetMeshDescription(k);
				*Mesh = MoveTemp(*Options.SourceMeshes.MoveMeshDescriptions[k]);
				MaxNumSections = FMath::Max(MaxNumSections, Mesh->PolygonGroups().Num());
				NewStaticMesh->CommitMeshDescription(k);
			}
		}
	}
	else if (Options.SourceMeshes.MeshDescriptions.Num() > 0)
	{
		if (ensure(Options.SourceMeshes.MeshDescriptions.Num() == UseNumSourceModels))
		{
			for (int32 k = 0; k < UseNumSourceModels; ++k)
			{
				FMeshDescription* Mesh = NewStaticMesh->GetMeshDescription(k);
				*Mesh = *Options.SourceMeshes.MeshDescriptions[k];
				MaxNumSections = FMath::Max(MaxNumSections, Mesh->PolygonGroups().Num());
				NewStaticMesh->CommitMeshDescription(k);
			}
		}
	}
	else if (Options.SourceMeshes.DynamicMeshes.Num() > 0)
	{
		if (ensure(Options.SourceMeshes.DynamicMeshes.Num() == UseNumSourceModels))
		{
			for (int32 k = 0; k < UseNumSourceModels; ++k)
			{
				FMeshDescription* Mesh = NewStaticMesh->GetMeshDescription(k);
				FConversionToMeshDescriptionOptions ConverterOptions;
				ConverterOptions.bConvertBackToNonManifold = Options.bConvertBackToNonManifold;
				FDynamicMeshToMeshDescription Converter(ConverterOptions);
				Converter.Convert(Options.SourceMeshes.DynamicMeshes[k], *Mesh, !Options.bEnableRecomputeTangents);
				MaxNumSections = FMath::Max(MaxNumSections, Mesh->PolygonGroups().Num());
				NewStaticMesh->CommitMeshDescription(k);
			}
		}
	}

	// Ensure that we have a Material Slot for each Section that exists on the mesh.
	// This is not technically correct as the Sections on the MeshDescription (ie PolygonGroups)
	// may reference any MaterialSlot (via the PolygonGroup::ImportedMaterialSlotName attribute),
	// so it is valid for there to be fewer Materials than Sections. *However* if a Section does
	// not have an ImportedMaterialSlotName, then the Section Index is used as Material Slot Index,
	// and if that Material Slot Index does not exist, we are in a bit of an undefined state, for
	// example the Section will not appear in the Static Mesh Editor, and various other code that
	// does not properly handle Sections/Slots will be broken. 
	// So, some of these extra Slots may be unused, but this is not a catastrophe. Note that the
	// extra Slots will end up with no Material assigned.
	int32 CurNumValidSections = UseNumMaterialSlots;
	while (CurNumValidSections < MaxNumSections)
	{
		NewStaticMesh->GetStaticMaterials().Add(FStaticMaterial());
		CurNumValidSections++;
	}

	// populate section info map
	FMeshSectionInfoMap SectionInfoMap;
	for (int32 LODIndex = 0; LODIndex < UseNumSourceModels; ++LODIndex)
	{
		if (FMeshDescription* Mesh = NewStaticMesh->GetMeshDescription(LODIndex))
		{
			FStaticMeshConstAttributes MeshDescriptionAttributes(*Mesh);
			TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshDescriptionAttributes.GetPolygonGroupMaterialSlotNames();
			int32 SectionIndex = 0;
			for (FPolygonGroupID PolygonGroupID : Mesh->PolygonGroups().GetElementIDs())
			{
				// Material index is either from the matching material slot name or the section index if that name is not found
				int32 MaterialIndex = INDEX_NONE;
				if (MaterialSlotNames[PolygonGroupID] != FName())
				{
					MaterialIndex = NewStaticMesh->GetStaticMaterials().IndexOfByPredicate(
						[&MaterialSlotName = MaterialSlotNames[PolygonGroupID]](const FStaticMaterial& StaticMaterial) { return StaticMaterial.MaterialSlotName == MaterialSlotName; }
					);
				}
				if (MaterialIndex == INDEX_NONE)
				{
					MaterialIndex = SectionIndex;
				}
				SectionInfoMap.Set(LODIndex, SectionIndex, FMeshSectionInfo(MaterialIndex));
				SectionIndex++;
			}
		}
	}
	NewStaticMesh->GetSectionInfoMap().CopyFrom(SectionInfoMap);
	NewStaticMesh->GetOriginalSectionInfoMap().CopyFrom(SectionInfoMap);

	// Nanite options
	NewStaticMesh->GetNaniteSettings().bEnabled = false;
	if (Options.bGenerateNaniteEnabledMesh)
	{
		NewStaticMesh->SetNaniteSettings(Options.NaniteSettings);
	}

	// Ray tracing
	NewStaticMesh->bSupportRayTracing = Options.bSupportRayTracing;

	// Distance field
	NewStaticMesh->bGenerateMeshDistanceField = Options.bAllowDistanceField;

	// Set the initial overall static mesh lightmap settings to match the first lightmap build settings
	if (Options.bGenerateLightmapUVs)
	{
		FMeshBuildSettings& BuildSettings = NewStaticMesh->GetSourceModel(0).BuildSettings;
		NewStaticMesh->SetLightMapCoordinateIndex(BuildSettings.DstLightmapIndex);
		NewStaticMesh->SetLightMapResolution(BuildSettings.MinLightmapResolution);
	}

	[[maybe_unused]] bool MarkedDirty = NewStaticMesh->MarkPackageDirty();
	if (Options.bDeferPostEditChange == false)
	{
		NewStaticMesh->PostEditChange();
	}

	ResultsOut.StaticMesh = NewStaticMesh;
	return ECreateStaticMeshResult::Ok;
}



