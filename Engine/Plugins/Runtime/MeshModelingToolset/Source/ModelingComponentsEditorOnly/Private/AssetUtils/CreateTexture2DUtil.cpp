// Copyright Epic Games, Inc. All Rights Reserved. 

#include "AssetUtils/CreateTexture2DUtil.h"
#include "AssetUtils/Texture2DBuilder.h"
#include "AssetUtils/Texture2DUtil.h"
#include "Misc/PackageName.h"
#include "UObject/Package.h"

UE::AssetUtils::ECreateTexture2DResult UE::AssetUtils::SaveGeneratedTexture2DAsset(
	UTexture2D* GeneratedTexture,
	FTexture2DAssetOptions& Options,
	FTexture2DAssetResults& ResultsOut)
{
	// validate input texture
	if (ensure(GeneratedTexture) == false)
	{
		return ECreateTexture2DResult::InvalidInputTexture;
	}
	if (ensure(GeneratedTexture->GetOuter() == GetTransientPackage()) == false)
	{
		return ECreateTexture2DResult::InvalidInputTexture;
	}
	if (ensure(GeneratedTexture->Source.IsValid()) == false)
	{
		return ECreateTexture2DResult::InvalidInputTexture;
	}

	FString NewObjectName = FPackageName::GetLongPackageAssetName(Options.NewAssetPath);

	UPackage* UsePackage;
	if (Options.UsePackage != nullptr)
	{
		UsePackage = Options.UsePackage;
	}
	else
	{
		UsePackage = CreatePackage(*Options.NewAssetPath);
	}
	if (ensure(UsePackage != nullptr) == false)
	{
		return ECreateTexture2DResult::InvalidPackage;
	}

	// check for existing asset of the same type.
	if (UObject* ExistingAsset = StaticFindObject(nullptr, UsePackage, *NewObjectName, EFindObjectFlags::None))
	{
		UTexture2D* ExistingTexture2D = Cast<UTexture2D>(ExistingAsset);
		if (ExistingTexture2D && Options.bOverwriteIfExists)
		{
			// TODO: Direct copy between UTexture2D
			UE::Geometry::TImageBuilder<FVector4f> TexImage; // This will contain FLinearColor data
			if (!UE::AssetUtils::ReadTexture(GeneratedTexture, TexImage))
			{
				return ECreateTexture2DResult::InvalidInputTexture;
			}
			
			UE::Geometry::FTexture2DBuilder Builder;
			Builder.InitializeAndReplaceExistingTexture(ExistingTexture2D, GeneratedTexture);
			Builder.Copy(TexImage, GeneratedTexture->SRGB);
			Builder.Commit(true);
			GeneratedTexture = Builder.GetTexture2D();
		}
		else if (!ExistingTexture2D && Options.bOverwriteIfExists)
		{
			return ECreateTexture2DResult::OverwriteTypeError;
		}
		else
		{
			return ECreateTexture2DResult::NameError;
		}
	}
	else
	{
		// move texture from Transient package to real package
		const bool bRenameOK = GeneratedTexture->Rename(*NewObjectName, UsePackage, REN_None);
		if (ensure(bRenameOK) == false)
		{
			return ECreateTexture2DResult::NameError;
		}

		// remove transient flag, add public/standalone/transactional
		GeneratedTexture->ClearFlags(RF_Transient);
		GeneratedTexture->SetFlags(RF_Public | RF_Standalone | RF_Transactional);
	}

	[[maybe_unused]] bool MarkedDirty = GeneratedTexture->MarkPackageDirty();

	// do we need to Modify() it? we are not doing any undo/redo
	GeneratedTexture->Modify();

	// force texture to rebuild (can we defer this?)
	GeneratedTexture->UpdateResource();

	if (Options.bDeferPostEditChange == false)
	{
		GeneratedTexture->PostEditChange();		// this may be necessary if any Materials are using this texture
	}

	ResultsOut.Texture = GeneratedTexture;
	return ECreateTexture2DResult::Ok;
}