// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConversionUtils/DynamicMeshToVolume.h"

#include "BSPOps.h"
#include "CompGeom/PolygonTriangulation.h"
#include "ConstrainedDelaunay2.h"
#include "Curve/PlanarComplex.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshEditor.h"
#include "Engine/Polys.h"
#include "GameFramework/Volume.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshConstraintsUtil.h"
#include "MeshRegionBoundaryLoops.h"
#include "Model.h"
#include "Operations/PlanarHoleFiller.h"
#include "Selections/MeshConnectedComponents.h"
#include "Util/ProgressCancel.h"

#include "MeshSimplification.h"

using namespace UE::Geometry;

namespace DynamicMeshToVolumeLocals
{
	// This is meant as a replacement for the FPoly Triangulate method, which occasionally
	// removes colinear verts yet still sometimes creates degenerate triangles that trigger an
	// ensure in that function.
	bool TriangulatePolygon(FPoly& PolygonIn, TArray<FPoly>& PolygonsOut)
	{
		PolygonsOut.Empty();
		if (PolygonIn.Vertices.Num() < 3)
		{
			return false;
		}

		if (PolygonIn.Vertices.Num() == 3)
		{
			if (!PolygonIn.IsConvex())
			{
				// This means that the triangle was given to us with wrong orientation (according to legacy code). This can
				// actually happen as a result of removing verts in certain near-degenerate cases, so no ensure here.
				Swap(PolygonIn.Vertices[1], PolygonIn.Vertices[2]);
			}
			PolygonsOut.Add(PolygonIn);
			return true;
		}

		FFrame3d PolygonFrame((FVector3d)PolygonIn.Base, (FVector3d)PolygonIn.Normal);
		TArray<FVector2D> Polygon2DCoordinates;
		for (const FVector3f& Vert : PolygonIn.Vertices)
		{
			Polygon2DCoordinates.Add(PolygonFrame.ToPlaneUV((FVector3d)Vert));
		}
		FPolygon2d Polygon(Polygon2DCoordinates);

		FConstrainedDelaunay2d Delaunay;
		Delaunay.Add(Polygon);
		// Output needs to be wound the same way as input
		Delaunay.bOutputCCW = true;

		TArray<FIndex3i>* TrianglesToUse = &Delaunay.Triangles;
		TArray<FIndex3i> PolyTriangles;
		if (!Delaunay.Triangulate() || Delaunay.Triangles.Num() == 0)
		{
			PolygonTriangulation::TriangulateSimplePolygon(Polygon.GetVertices(), PolyTriangles, false);
			TrianglesToUse = &PolyTriangles;
		}

		for (const FIndex3i& Triangle : *TrianglesToUse)
		{
			FPoly PolyToEmit;
			PolyToEmit.Init();

			for (int i = 0; i < 3; ++i)
			{
				// Note: the ordering between PolygonIn.Vertices and Polygon2DCoordinates should be identical so that
				// we can index back to get the original location easily.
				PolyToEmit.Vertices.Add(PolygonIn.Vertices[Triangle[i]]);
			}
			PolyToEmit.Base = PolyToEmit.Vertices[0];

			// At this point we could call PolyToEmit.Finalize() but it has some questionable
			// error handling, in particular where some degenerate triangles get kept despite
			// triggering an ensure and having their normal set to zero. We'll just do it ourselves.

			if (FVector3f::PointsAreSame(PolyToEmit.Vertices[0], PolyToEmit.Vertices[1])
				|| FVector3f::PointsAreSame(PolyToEmit.Vertices[0], PolyToEmit.Vertices[2])
				|| FVector3f::PointsAreSame(PolyToEmit.Vertices[1], PolyToEmit.Vertices[2]))
			{
				// Don't emit the degenerate poly
				continue;
			}
			
			// Note that we use the vertices in backwards order because FPoly stores vertices clockwise if the normal is towards you.
			PolyToEmit.Normal = VectorUtil::Normal(PolyToEmit.Vertices[0], PolyToEmit.Vertices[2], PolyToEmit.Vertices[1]);

			if (!PolyToEmit.Normal.Equals(PolygonIn.Normal))
			{
				// We shouldn't really have non-coplanar triangles, but the original handled non-coplanarity, so we do too.
				FQuaternionf Rotation(PolygonIn.Normal, PolyToEmit.Normal);
				PolyToEmit.TextureU = Rotation * PolygonIn.TextureU;
				PolyToEmit.TextureV = Rotation * PolygonIn.TextureV;
			}
			else
			{
				PolyToEmit.TextureU = PolygonIn.TextureU;
				PolyToEmit.TextureV = PolygonIn.TextureV;
			}

			if (!ensure(PolyToEmit.IsConvex()))
			{
				// This means that the polygon we got had vertices in the wrong order
				Swap(PolyToEmit.Vertices[0], PolyToEmit.Vertices[1]);

				if (!ensure(PolyToEmit.IsConvex()))
				{
					// Not sure whether this could happen, but don't emit in this case
					continue;
				}
			}

			PolygonsOut.Add(PolyToEmit);
		}

		return PolygonsOut.Num() > 0;
	}

	/**
	 * This is a way to triangulate polygons that have multiple boundary loops (i.e. polygons with
	 * holes in them) into nicer triangles/convexes than just outputting all triangles separately.
	 * (We could actually use this for any polygon, but for normal polygons we just emit the whole
	 * boundary and let the legacy code retriangulate/merge if they are not convex).
	 */
	bool TriangulateLoopsIntoFaces(
		const FDynamicMesh3& InputMesh, const TArray<int32>& Tids,
		FVector3d PlaneNormal, const TArray<TArray<int32>>& VidLoopsIn,
		TArray<UE::Conversion::FDynamicMeshFace>& FacesToAppendTo)
	{
		// The mesh we have access to is const, and while we could do the retriangulation separately
		// without a mesh, it helps to have neighbor relationships between the created triangles for
		// merging them later. So instead of doing a plain retriangulation, we'll create a little
		// submesh and do a hole fill.
		FDynamicMesh3 Submesh;
		TMap<int32, int32> InputVidToSubmeshVid;

		TArray<TArray<int32>> HoleFillVidLoops;
		for (const TArray<int32>& BaseLoop : VidLoopsIn)
		{
			TArray<int32>& HoleFillLoop = HoleFillVidLoops.Emplace_GetRef();
			// Prep the loops backwards because we found them as boundaries of a component rather than
			// a hole, and now they are boundaries of a hole.
			for (int32 i = BaseLoop.Num() - 1; i >= 0; --i)
			{
				int32 InputVid = BaseLoop[i];
				int32* ExistingSubmeshVid = InputVidToSubmeshVid.Find(InputVid);
				if (!ExistingSubmeshVid)
				{
					int32 SubmeshVid = Submesh.AppendVertex(InputMesh.GetVertex(InputVid));
					ExistingSubmeshVid = &InputVidToSubmeshVid.Add(InputVid, SubmeshVid);
				}
				
				HoleFillLoop.Add(*ExistingSubmeshVid);
			}
		}

		// Retriangulation
		// We use a slightly modified version of UE::Geometry::ConstrainedDelaunayTriangulate<double> for our retriangulation
		// function in order to know when a retriangulation fails.
		// TODO: our constrained delaunay triangulation fails in some pathological self intersecting cases that can come
		// about from degenerate triangles, and it would be nice to be able to fall back to something, but we don't have
		// an alternative for triangulating with holes.
		bool bRetriangulationSucceeded = true;
		auto RetriangulationFunc = [&bRetriangulationSucceeded](const TGeneralPolygon2<double>& GeneralPolygon) -> TArray<FIndex3i>
		{
			TConstrainedDelaunay2<double> Triangulation;
			Triangulation.FillRule = TConstrainedDelaunay2<double>::EFillRule::Positive;
			Triangulation.Add(GeneralPolygon);
			if (!Triangulation.Triangulate() || !ensure(Triangulation.Triangles.Num() > 0))
			{
				bRetriangulationSucceeded = false;
			}
			return Triangulation.Triangles;
		};

		FPlanarHoleFiller HoleFiller(&Submesh, &HoleFillVidLoops, RetriangulationFunc,
			Submesh.GetVertex(HoleFillVidLoops[0][0]), PlaneNormal);

		if (!HoleFiller.Fill() || !bRetriangulationSucceeded)
		{
			return false;
		}

		// Now merge the triangles into convexes. 
		// We could use OptimizeIntoConvexPolys, but we have a bunch of knowledge that lets us do it more
		// efficiently: we have adjacency information, know that everything is a triangle, and most importantly,
		// we know that there aren't any interior verts. The lack of interior vertices means that we don't need 
		// multiple passes (once a triangle is determined to break convexity, another triangle can't fix it because 
		// that would require a surrounded/interior vertex).

		TSet<int32> ProcessedTids;
		int32 TidCount = 0; // For sanity check
		for (int32 Tid : Submesh.TriangleIndicesItr())
		{
			if (ProcessedTids.Contains(Tid))
			{
				continue;
			}
			ProcessedTids.Add(Tid);

			FIndex3i TriVids = Submesh.GetTriangle(Tid);

			// We'll walk around the boundary seeing if we can add neighboring triangles. Top of VidStack is the next
			// vid to walk to, and we add verts onto our path in front of ourselves as we add triangles.
			TArray<int32> PolygonVids;
			TArray<int32> VidStack{ TriVids[2], TriVids[1], TriVids[0] };
			int32 FromVid = VidStack[0]; // Vert we're walking from (wrap around to bottom of stack)
			int32 PreviousVid = VidStack[1]; // Vert before FromVid, for convexity check.

			while (!VidStack.IsEmpty())
			{
				int32 ToVid = VidStack.Pop();
				int32 CurrentEid = Submesh.FindEdge(FromVid, ToVid); // edge we're walking along

				// If we determine that the triangle on the other side cannot be added to the polygon, 
				// then we'll add ToVid to our polygon and move on to the next vid in our stack.
				auto AdvanceToNext = [&PolygonVids, ToVid, &PreviousVid, &FromVid]()
				{
					PolygonVids.Add(ToVid);
					PreviousVid = FromVid;
					FromVid = ToVid;
				};

				// Find the triangle on the other side of the current edge
				FIndex2i EdgeTids = Submesh.GetEdgeT(CurrentEid);
				if (EdgeTids.B == IndexConstants::InvalidID) // was a boundary edge
				{
					AdvanceToNext();
					continue;
				}
				int32 OtherTid = ProcessedTids.Contains(EdgeTids.A) ? EdgeTids.B : EdgeTids.A;
				if (ProcessedTids.Contains(OtherTid))
				{
					// Both triangles being processed means that the other triangle was already placed into another convex
					AdvanceToNext();
					continue;
				}

				// Look to see whether the two edges of the triangle stay on the correct side of the preceding and following
				// polygon edges. This test is done similar to the way that it is done in FPoly::IsConvex().
				FVector3d PreviousVector = (Submesh.GetVertex(FromVid) - Submesh.GetVertex(PreviousVid)).GetSafeNormal();
				FVector3d PreviousOutVector = PlaneNormal.Cross(PreviousVector); // points out into the disallowed half space

				int32 OtherVid = IndexUtil::FindTriOtherVtxUnsafe(FromVid, ToVid, Submesh.GetTriangle(OtherTid));
				FVector3d VectorToOther = (Submesh.GetVertex(OtherVid) - Submesh.GetVertex(FromVid)).GetSafeNormal();
				if (PreviousOutVector.Dot(VectorToOther) > UE_KINDA_SMALL_NUMBER)
				{
					AdvanceToNext();
					continue;
				}

				int32 VidAfterToVid = VidStack.IsEmpty() ? PolygonVids[0] : VidStack.Top();
				FVector3d NextVector = (Submesh.GetVertex(VidAfterToVid) - Submesh.GetVertex(ToVid)).GetSafeNormal();
				FVector3d NextOutVector = PlaneNormal.Cross(NextVector); // points out into the disallowed half space
				VectorToOther = (Submesh.GetVertex(OtherVid) - Submesh.GetVertex(ToVid)).GetSafeNormal();
				if (NextOutVector.Dot(VectorToOther) > UE_KINDA_SMALL_NUMBER)
				{
					AdvanceToNext();
					continue;
				}

				// If we got to here, then we can add in this triangle to the polygon.
				ProcessedTids.Add(OtherTid);

				// This means that we're not ready to emit ToVid yet, so put it back on the stack, with the new ToVid on top of it.
				VidStack.Push(ToVid);
				VidStack.Push(OtherVid);

				// Sanity check
				if (!ensure(++TidCount <= Submesh.TriangleCount()))
				{
					return false;
				}
			}

			// Once we've got here, we have a polygon that we can't add more triangles to
			UE::Conversion::FDynamicMeshFace Face;

			Face.BoundaryLoop.SetNum(PolygonVids.Num());
			FVector3d PlaneOrigin = FVector3d::Zero();
			for (int i = 0; i < PolygonVids.Num(); ++i)
			{
				Face.BoundaryLoop[i] = Submesh.GetVertex(PolygonVids[i]);
				PlaneOrigin += Face.BoundaryLoop[i];
			}
			PlaneOrigin /= PolygonVids.Num();
			Face.Plane = FFrame3d(PlaneOrigin, PlaneNormal);

			// Poly vertices are stored backwards
			Algo::Reverse(Face.BoundaryLoop);

			FacesToAppendTo.Add(Face);
		}//end emitting faces

		return true;
	}

	/**
	 * This filters the passed in FMeshRegionBoundaryLoops to create vid loops that do not include
	 * unnecessary colinear verts, where "unnecessary" means that the colinear vert is not part of
	 * more than just the two adjacent faces (i.e., it is not actually a T-junction).
	 * 
	 * @param VertCanBeSkippedMap A map used to cache knowledge of whether a vert is skippable, if
	 *  it has already been examined as part of another loop.
	 * @param TrianglesShouldBeInSameComponent Function that returns true when the two triangles are
	 *  in the same face, i.e. the function used in the connected component search.
	 */
	void GetVidLoopsWithoutUnneededColinearVerts(const FDynamicMesh3& InputMesh, const FMeshRegionBoundaryLoops& Loops,
		TMap<int32, bool>& VertCanBeSkippedMap, TFunctionRef<bool(int32, int32)> TrianglesShouldBeInSameComponent,
		TArray<TArray<int32>>& VidLoopsOut)
	{
		auto IsVertMeetingPointOfMultipleBoundaries = [&InputMesh, TrianglesShouldBeInSameComponent](int32 Vid)
		{
			int NumAttachedBoundaries = 0;
			for (int32 Eid : InputMesh.VtxEdgesItr(Vid))
			{
				FIndex2i EdgeTids = InputMesh.GetEdgeT(Eid);
				if (EdgeTids.B == IndexConstants::InvalidID || !TrianglesShouldBeInSameComponent(EdgeTids.A, EdgeTids.B))
				{
					++NumAttachedBoundaries;
				}

				if (NumAttachedBoundaries > 2)
				{
					return true;
				}
			}
			return false;
		};

		auto CanVertBeSkipped = [&InputMesh, &VertCanBeSkippedMap, &IsVertMeetingPointOfMultipleBoundaries]
			(int32 Vid, int32 NextVid, const FVector3d& LastAddedVertPosition)
		{
			// See if we've already dealt with this vert
			bool* CachedCanSkip = VertCanBeSkippedMap.Find(Vid);
			if (CachedCanSkip)
			{
				return *CachedCanSkip;
			}

			FVector3d PreviousVector = InputMesh.GetVertex(Vid) - LastAddedVertPosition;
			FVector3d NextVector = InputMesh.GetVertex(NextVid) - InputMesh.GetVertex(Vid);

			// Skippability depends in part on colinearity
			bool bCanSkip = !PreviousVector.Normalize() || !NextVector.Normalize()
				// We use abs here because we don't want to keep backwards folding degenerate "fangs" either. This helps
				// clean up some pathological degenerate soup cases.
				|| FMath::Abs(NextVector.Dot(PreviousVector)) >= 1 - KINDA_SMALL_NUMBER;

			// However, even if we can skip based on colinearity, we may need to keep the vert if it's something like a T junction
			if (bCanSkip)
			{
				bCanSkip = !IsVertMeetingPointOfMultipleBoundaries(Vid);
			}
			
			VertCanBeSkippedMap.Add(Vid, bCanSkip);
			return bCanSkip;
		};

		VidLoopsOut.Reset();
		for (const FEdgeLoop& Loop : Loops.Loops)
		{
			int32 VidLoopIndex = VidLoopsOut.Emplace();
			TArray<int32>& VidLoop = VidLoopsOut[VidLoopIndex];

			int32 NumVerts = Loop.Vertices.Num();
			if (!ensure(NumVerts >= 3))
			{
				continue;
			}

			// The initialization of LastAddedVertPosition might not be accurate if we don't later end up adding the 
			// last vid. If the edge between last and first vid is not degenerate, the mistake won't matter because
			// the vector to the true previous vid would be colinear with our initialized one. If that edge is degenerate,
			// then we can account for the situation when processing the last vid, to make sure we add the final vert if needed.
			FVector3d LastAddedVertPosition = InputMesh.GetVertex(Loop.Vertices[NumVerts - 1]);

			for (int32 i = 0; i < NumVerts - 1; ++i) // Stop right before the last one
			{
				int32 Vid = Loop.Vertices[i];
				int32 NextVid = Loop.Vertices[(i + 1) % NumVerts];

				if (!CanVertBeSkipped(Vid, NextVid, LastAddedVertPosition))
				{
					VidLoop.Add(Vid);
					LastAddedVertPosition = InputMesh.GetVertex(Vid);
				}
			}

			// Process the last vid using knowledge of the next unskipped vert.
			if (VidLoop.Num() > 0 && !CanVertBeSkipped(Loop.Vertices.Last(), VidLoop[0], LastAddedVertPosition))
			{
				VidLoop.Add(Loop.Vertices.Last());
			}

			if (VidLoop.Num() < 3)
			{
				// It's not obvious what we should do with an entirely degenerate loop... We choose to throw it away.
				// The up side of this choice is that this helps a lot in some cases we run into with high-triangle-count
				// meshes, where preceding simplification steps occasionally create degenerate tris that still end
				// up getting a nonzero normal that differs from their surroundings, and which create problematic
				// cracks and flaps in the volume. Removing degenerate loops here ends up removing these cracks and
				// often improves the result quite a bit.
				// 
				// The down side is that in the extreme case, we could end up throwing away a whole mesh if it is
				// nothing but degenerates, but it seems that having a volume without faces does not crash anything
				// (it's possible to delete all faces in brush editing mode, for instance). Another drawback/limitation
				// of the approach here is that it doesn't deal with all degenerate cases, only ones that can be reduced
				// to a line or point (so bent ribbon of degenerates would end up staying).

				VidLoopsOut.Pop();
			}
		}//end for each loop
	}//end GetVidLoopsWithoutColinearVerts()

}//end DynamicMeshToVolumeLocals

namespace UE {
namespace Conversion {

/**
 * A slightly modified version of FPoly::OptimizeIntoConvexPolys. Unlike the original, it won't 
 * arbitrarily delete colinear verts or snap things to grid. Like the original, it assumes that 
 * the passed in polys are convex and coplanar (usually just a triangulation), and it will not 
 * merge polygons sharing a multi-edge (colinear) side.
 *
 * @param	InOwnerBrush	The brush that owns the polygons.
 * @param	InPolygons		An array of FPolys that will be replaced with a new set of polygons that are merged together as much as possible.
 */
static void OptimizeIntoConvexPolys(ABrush* InOwnerBrush, TArray<FPoly>& InPolygons)
{
	bool bDidMergePolygons = true;

	while (bDidMergePolygons)
	{
		bDidMergePolygons = false;

		for (int32 PolyIndex = 0; PolyIndex < InPolygons.Num() && !bDidMergePolygons; ++PolyIndex)
		{
			const FPoly* PolyMain = &InPolygons[PolyIndex];

			// Find all the polygons that neighbor this one (aka share an edge)

			for (int32 p2 = PolyIndex + 1; p2 < InPolygons.Num() && !bDidMergePolygons; ++p2)
			{
				const FPoly* PolyNeighbor = &InPolygons[p2];

				if (PolyMain->Normal.Equals(PolyNeighbor->Normal))
				{
					// See if PolyNeighbor is sharing an edge with Poly

					int32 PolyMainIndex1 = INDEX_NONE, PolyMainIndex2 = INDEX_NONE;
					int32 PolyNeighborIndex1 = INDEX_NONE, PolyNeighborIndex2 = INDEX_NONE;

					int32 PolyMainNumVerts = PolyMain->Vertices.Num();
					int32 PolyNeighborNumVerts = PolyNeighbor->Vertices.Num();

					for (int32 v = 0; v < PolyMainNumVerts; ++v)
					{
						FVector3f vtx = PolyMain->Vertices[v];

						int32 idx = INDEX_NONE;

						for (int32 v2 = 0; v2 < PolyNeighborNumVerts; ++v2)
						{
							const FVector3f* vtx2 = &PolyNeighbor->Vertices[v2];

							if (vtx.Equals(*vtx2))
							{
								idx = v2;
								break;
							}
						}

						// TODO: Could clean this up. We only need to find the first shared vert
						// and then check the next neighbor (and maybe previous, if at start)
						if (idx != INDEX_NONE)
						{
							if (PolyMainIndex1 == INDEX_NONE)
							{
								PolyMainIndex1 = v;
								PolyNeighborIndex1 = idx;
							}
							else if (PolyMainIndex2 == INDEX_NONE)
							{
								PolyMainIndex2 = v;
								PolyNeighborIndex2 = idx;
								break;
							}
						}
					}

					// Check that we found two shared verts and that their indices are adjacent in PolyMain.
					if (!(PolyMainIndex1 != INDEX_NONE && PolyMainIndex2 != INDEX_NONE
						&& (PolyMainIndex1 + 1 == PolyMainIndex2 || (PolyMainIndex1 == 0 && PolyMainIndex2 == PolyMainNumVerts - 1))))
					{
						continue; // skip neighbor
					}

					// Swap indices around so that they correspond to the start and end of the edge counterclockwise
					// in PolyMain (this will only not be the case if the edge spans end and start of vert array).
					if (PolyMainIndex1 + 1 != PolyMainIndex2)
					{
						Swap(PolyMainIndex1, PolyMainIndex2);
						Swap(PolyNeighborIndex1, PolyNeighborIndex2);
					}

					// Check that the edge goes the opposite direction in PolyNeighbor
					if ((PolyNeighborIndex2 + 1) % PolyNeighborNumVerts != PolyNeighborIndex1)
					{
						continue; // skip neighbor
					}

					// Found a shared edge.  Let's see if we can merge these polygons together.
					// The most robust thing to do is to just go ahead and try the merge and see if FPoly::IsConvex() 
					// returns true. 

					FPoly PolyMerged = *PolyMain;
					int32 NumTotalVerts = PolyMainNumVerts + PolyNeighborNumVerts - 2;
					PolyMerged.Vertices.SetNumUninitialized(NumTotalVerts);

					// The new vertices get added after PolyMainIndex1. First shift over everything that
					// is to the right of PolyMainIndex1
					int32 NumVertsToShift = PolyMainNumVerts - 1 - PolyMainIndex1;
					for (int32 i = 0; i < NumVertsToShift; ++i)
					{
						PolyMerged.Vertices[NumTotalVerts - 1 - i] = PolyMerged.Vertices[PolyMainNumVerts - 1 - i];
					}
					// Now splice in PolyNeighbor vertices.
					for (int32 i = 1; i < PolyNeighborNumVerts - 1; ++i) // Skipping the shared verts
					{
						int32 NeighborIndex = (PolyNeighborIndex1 + i) % PolyNeighborNumVerts;
						PolyMerged.Vertices[PolyMainIndex1 + i] = PolyNeighbor->Vertices[NeighborIndex];
					}

					// Check if the result is both coplanar and convex, and then finalize it.
					if (PolyMerged.IsCoplanar() && PolyMerged.IsConvex() && PolyMerged.Finalize(InOwnerBrush, 1) == 0)
					{
						// Remove the original polygons from the list. Uses knowledge that p2 > PolyIndex
						InPolygons.RemoveAtSwap(p2);
						InPolygons.RemoveAtSwap(PolyIndex);

						// Add the newly merged polygon into the list
						InPolygons.Add(PolyMerged);

						// Tell the outside loop that we merged polygons and need to run through it all again
						bDidMergePolygons = true;
					}
				}
			}
		}
	}
}

/**
 * Largely copied from FGeomObject::FinalizeSourceData(). Does various cleanup
 * operations, particularly to make the volume polygons all planar and convex.
 */
static void ApplyLegacyVolumeCleanup(ABrush* brush)
{
	using namespace DynamicMeshToVolumeLocals;

	// Remove invalid polygons from the brush
	for (int32 x = 0; x < brush->Brush->Polys->Element.Num(); ++x)
	{
		FPoly* Poly = &brush->Brush->Polys->Element[x];
		if (Poly->Vertices.Num() < 3)
		{
			brush->Brush->Polys->Element.RemoveAt(x);
			x = -1;
		}
	}
	for (int32 p = 0; p < brush->Brush->Polys->Element.Num(); ++p)
	{
		FPoly* Poly = &(brush->Brush->Polys->Element[p]);
		Poly->iLink = p;
		int32 SaveNumVertices = Poly->Vertices.Num();
		if (!Poly->IsCoplanar() || !Poly->IsConvex())
		{
			// If the polygon is no longer coplanar and/or convex, break it up into separate triangles.
			FPoly WkPoly = *Poly;
			brush->Brush->Polys->Element.RemoveAt(p);
			TArray<FPoly> Polygons;

			// We use our own triangulation function because the legacy one (WkPoly.Triangulate(brush, Polygons)) is not 
			// very reliable. It deletes colinear verts (which we don't always want) yet still sometimes creates degenerate
			// triangles that trigger an ensure inside it.
			if (TriangulatePolygon(WkPoly, Polygons))
			{
				OptimizeIntoConvexPolys(brush, Polygons);
				for (int32 t = 0; t < Polygons.Num(); ++t)
				{
					brush->Brush->Polys->Element.Add(Polygons[t]);
				}
			}
			p = -1;
		}
		else
		{
			int32 FixResult = Poly->Fix();
			if (FixResult != SaveNumVertices)
			{
				// If the polygon collapses after running "Fix" against it, it needs to be
				// removed from the brushes polygon list.
				if (FixResult == 0)
				{
					brush->Brush->Polys->Element.RemoveAt(p);
				}
				p = -1;
			}
			else
			{
				// If we get here, the polygon is valid and needs to be kept.  Finalize its internals.
				Poly->Finalize(brush, 1);
			}
		}
	}
}

void GetPolygonFaces(const FDynamicMesh3& InputMesh, const FMeshToVolumeOptions& Options, 
	TArray<FDynamicMeshFace>& FacesOut, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	FDynamicMesh3 LocalMesh;
	const FDynamicMesh3* UseMesh = &InputMesh;
	if (Options.bAutoSimplify && InputMesh.TriangleCount() > Options.MaxTriangles)
	{
		LocalMesh = InputMesh;
		LocalMesh.DiscardAttributes();

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		// collapse to minimal planar first
		FQEMSimplification PlanarSimplifier(&LocalMesh);

		if (Options.bRespectGroupBoundaries)
		{
			FMeshConstraints Constraints;
			FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(
				Constraints,
				FMeshConstraintsUtil::EBoundaryType::Group,
				LocalMesh,
				/*BoundaryCornerAngleThreshold = */ 30);
			PlanarSimplifier.SetExternalConstraints(Constraints);
		}

		PlanarSimplifier.SimplifyToMinimalPlanar(0.1);

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		// If we still have too many triangles, don't bother trying to stay planar or preserve groups.
		if (LocalMesh.TriangleCount() > Options.MaxTriangles)
		{
			FVolPresMeshSimplification Simplifier(&LocalMesh);
			Simplifier.SimplifyToTriangleCount(Options.MaxTriangles);
		}

		UseMesh = &LocalMesh;

		if (Progress && Progress->Cancelled())
		{
			return;
		}
	}

	if (Options.bCleanDegenerate)
	{
		// If the above simplification hasn't already switched us to local mesh, initialize local mesh
		if (UseMesh != &LocalMesh)
		{
			LocalMesh = InputMesh;
			LocalMesh.DiscardAttributes();
		}
		
		// Delete remaining triangles with small area or small edges
		TArray<int32> ToDelete;
		for (int32 tid : LocalMesh.TriangleIndicesItr())
		{
			if (LocalMesh.GetTriArea(tid) < Options.MinTriangleArea)
			{
				ToDelete.Add(tid);
			}
		}
		FDynamicMeshEditor Editor(&LocalMesh);
		Editor.RemoveTriangles(ToDelete, true);
		ToDelete.Reset();
		const double MinEdgeSq = Options.MinEdgeLength * Options.MinEdgeLength;
		for (int32 eid : LocalMesh.EdgeIndicesItr())
		{
			FDynamicMesh3::FEdge Edge = LocalMesh.GetEdge(eid);
			if ( FVector::DistSquared(LocalMesh.GetVertex(Edge.Vert.A), LocalMesh.GetVertex(Edge.Vert.B)) < MinEdgeSq )
			{
				// Note: It's ok that triangles may be added multiple times; RemoveTriangles will just skip triangles were already deleted
				ToDelete.Add(Edge.Tri.A);
				if (Edge.Tri.B != IndexConstants::InvalidID)
				{
					ToDelete.Add(Edge.Tri.B);
				}
			}
		}
		Editor.RemoveTriangles(ToDelete, true);

		UseMesh = &LocalMesh;
	}

	GetPolygonFaces(*UseMesh, FacesOut, Options.bRespectGroupBoundaries);
}


void DynamicMeshToVolume(const FDynamicMesh3& InputMesh, AVolume* TargetVolume, const FMeshToVolumeOptions& Options)
{
	TArray<FDynamicMeshFace> Faces;
	GetPolygonFaces(InputMesh, Options, Faces);
	DynamicMeshToVolume(InputMesh, Faces, TargetVolume);
}

void DynamicMeshToVolume(const FDynamicMesh3&, TArray<FDynamicMeshFace>& Faces, AVolume* TargetVolume)
{
	check(TargetVolume->Brush);

	UModel* Model = TargetVolume->Brush;

	Model->Modify(false);

	Model->Initialize(TargetVolume);
	UPolys* Polys = Model->Polys;

	for (FDynamicMeshFace& Face : Faces)
	{
		int32 NumVertices = Face.BoundaryLoop.Num();
		FVector Normal = (FVector)Face.Plane.Z();
		FVector U = (FVector)Face.Plane.X();
		FVector V = (FVector)Face.Plane.Y();

		// create FPoly. This is Editor-only and I'm not entirely sure we need it?
		// int32 PolyIndex = Polys->Element.Num();
		FPoly NewPoly;
		NewPoly.Base = (FVector3f)Face.BoundaryLoop[0];
		NewPoly.Normal = (FVector3f)Normal;
		NewPoly.TextureU = (FVector3f)U;
		NewPoly.TextureV = (FVector3f)V;
		NewPoly.Vertices.SetNum(NumVertices);
		for (int32 k = 0; k < NumVertices; ++k)
		{
			NewPoly.Vertices[k] = (FVector3f)Face.BoundaryLoop[k];
		}
		NewPoly.PolyFlags = 0;
		NewPoly.iLink = NewPoly.iLinkSurf = NewPoly.iBrushPoly = -1;
		NewPoly.SmoothingMask = 0;
		Polys->Element.Add(NewPoly);

		/*  
		 *  // This is an alternate implementation that does not depend on Polys,
		 *  // and creates the FBspNode directly. However it's not clear that it
		 *  // works properly on non-convex shapes (or at all). Leaving this code
		 *  // here as we would like to have a path that is independent of Polys
		 *  // in the future (to potentially work at Runtime)

			// create points for this face in UModel::Points
			// TODO: can we share points between faces?
			TArray<int32> PointIndices;
			PointIndices.SetNum(NumVertices);
			for (int32 k = 0; k < NumVertices; ++k)
			{
				int32 NewIdx = Model->Points.Num();
				Model->Points.Add((FVector)Face.BoundaryLoop[k]);
				PointIndices[k] = NewIdx;
			}
			int32 BasePointIndex = PointIndices[0];

			// create normal for this face in UModel::Vectors along with U and V direction vectors
			int32 NormalIdx = Model->Vectors.Num();
			Model->Vectors.Add(Normal);
			int32 TextureUIdx = Model->Vectors.Num();
			Model->Vectors.Add(U);
			int32 TextureVIdx = Model->Vectors.Num();
			Model->Vectors.Add(V);

			// create FVerts for this face in UModel::Verts
			int32 iVertPoolStart = Model->Verts.Num();
			for (int32 k = 0; k < NumVertices; ++k)
			{
				FVert NewVert;
				NewVert.pVertex = PointIndices[k];		// Index of vertex point.
				NewVert.iSide = INDEX_NONE;				// If shared, index of unique side. Otherwise INDEX_NONE.
				NewVert.ShadowTexCoord = FVector2D::ZeroVector;			// The vertex's shadow map coordinate.
				NewVert.BackfaceShadowTexCoord = FVector2D::ZeroVector;	// The vertex's shadow map coordinate for the backface of the node.
				Model->Verts.Add(NewVert);
			}

			// create Surf

			int32 SurfIndex = Model->Surfs.Num();
			FBspSurf NewSurf;
			NewSurf.Material = nullptr;			// 4 Material.
			NewSurf.PolyFlags = 0;				// 4 Polygon flags.
			NewSurf.pBase = BasePointIndex;		// 4 Polygon & texture base point index (where U,V==0,0).
			NewSurf.vNormal = NormalIdx;		// 4 Index to polygon normal.
			NewSurf.vTextureU = TextureUIdx;	// 4 Texture U-vector index.
			NewSurf.vTextureV = TextureVIdx;	// 4 Texture V-vector index.
			//NewSurf.iBrushPoly = PolyIndex;		// 4 Editor brush polygon index.
			NewSurf.iBrushPoly = -1;
			//NewSurf.Actor = NewVolume;			// 4 Brush actor owning this Bsp surface.
			NewSurf.Actor = nullptr;
			NewSurf.Plane = FacePlane;			// 16 The plane this surface lies on.
			Model->Surfs.Add(NewSurf);


			// create nodes for this face in UModel::Nodes

			FBspNode NewNode;
			NewNode.Plane = FacePlane;					// 16 Plane the node falls into (X, Y, Z, W).
			NewNode.iVertPool = iVertPoolStart;			// 4  Index of first vertex in vertex pool, =iTerrain if NumVertices==0 and NF_TerrainFront.
			NewNode.iSurf = SurfIndex;					// 4  Index to surface information.
			NewNode.iVertexIndex = INDEX_NONE;			// The index of the node's first vertex in the UModel's vertex buffer.
														// This is initialized by UModel::UpdateVertices()
			NewNode.NumVertices = NumVertices;			// 1  Number of vertices in node.
			NewNode.NodeFlags = 0;						// 1  Node flags.
			Model->Nodes.Add(NewNode);
		*/
	}

	ApplyLegacyVolumeCleanup(TargetVolume);
	FBSPOps::bspUnlinkPolys(Model);

	// requires editor
	FBSPOps::csgPrepMovingBrush(TargetVolume);

	// do we need to do this?
	[[maybe_unused]] bool MarkedDirty = TargetVolume->MarkPackageDirty();
}


void GetPolygonFaces(const FDynamicMesh3& InputMesh, TArray<FDynamicMeshFace>& Faces, bool bRespectGroupBoundaries)
{
	using namespace DynamicMeshToVolumeLocals;

	Faces.SetNum(0);

	// We'll find faces using a connected component search based on normals. Note that we give
	// degenerate tris the normal of their neighbor so that they don't connect non-planar
	// components.

	FMeshNormals Normals(&InputMesh);
	Normals.ComputeTriangleNormals();
	Normals.SetDegenerateTriangleNormalsToNeighborNormal(); // See comment above

	double NormalTolerance = FMathf::ZeroTolerance;

	auto TrianglesShouldBeInSameComponent = [&InputMesh, &Normals, NormalTolerance, bRespectGroupBoundaries](int32 Triangle0, int32 Triangle1)
	{
		return (!bRespectGroupBoundaries || InputMesh.GetTriangleGroup(Triangle0) == InputMesh.GetTriangleGroup(Triangle1))

			// This test is only performed if triangles share an edge, so checking the normal is 
			// sufficient for coplanarity.
			&& Normals[Triangle0].Dot(Normals[Triangle1]) >= 1 - NormalTolerance;
	};

	FMeshConnectedComponents Components(&InputMesh);
	Components.FindConnectedTriangles(TrianglesShouldBeInSameComponent);

	// Used for removing colinear verts in the loop ahead
	TMap<int32, bool> VertCanBeSkippedMap;

	for (const FMeshConnectedComponents::FComponent& Component : Components)
	{
		FVector3d FaceNormal = Normals[Component.Indices[0]];
		FMeshRegionBoundaryLoops Loops(&InputMesh, Component.Indices);

		// Remove colinear verts from the loop where it is safe to do so
		TArray<TArray<int32>> VidLoops;
		GetVidLoopsWithoutUnneededColinearVerts(InputMesh, Loops, VertCanBeSkippedMap,
			TrianglesShouldBeInSameComponent, VidLoops);

		if (VidLoops.Num() > 1)
		{
			int32 PreviousNumFaces = Faces.Num();
			if (!TriangulateLoopsIntoFaces(InputMesh, Component.Indices, FaceNormal, VidLoops, Faces))
			{
				// Sanity check to make sure we didn't add new faces despite a failure
				if (!ensure(Faces.Num() == PreviousNumFaces))
				{
					Faces.SetNum(PreviousNumFaces);
				}

				// Our triangulation function for multiple loops can fail in some self-intersecting cases possible with degenerate
				// triangles. Revert to outputting all the triangles individually.
				// TODO: It would be nice to try to lump the triangles into convexes, but it's hard in this general case,
				// and possibly not worth it.
				for (int32 tid : Component.Indices)
				{
					FDynamicMeshFace Face;
					Face.BoundaryLoop.SetNum(3);
					InputMesh.GetTriVertices(tid, Face.BoundaryLoop[0], Face.BoundaryLoop[1], Face.BoundaryLoop[2]);
					Algo::Reverse(Face.BoundaryLoop);
					Face.Plane = FFrame3d(InputMesh.GetTriCentroid(tid), FaceNormal);
					Faces.Add(Face);
				}
			}
		}
		else if (VidLoops.Num() == 1) // if polygon with no holes
		{
			const TArray<int32>& LoopVids = VidLoops[0];
			FDynamicMeshFace Face;

			FVector3d AvgPos(0, 0, 0);
			for (int32 vid : LoopVids)
			{
				FVector3d Position = InputMesh.GetVertex(vid);
				Face.BoundaryLoop.Add(Position);
				AvgPos += Position;
			}
			AvgPos /= (double)LoopVids.Num();
			Algo::Reverse(Face.BoundaryLoop);

			Face.Plane = FFrame3d(AvgPos, FaceNormal);

			Faces.Add(Face);
		}
	}
}

void GetTriangleFaces(const FDynamicMesh3& InputMesh, TArray<FDynamicMeshFace>& Faces)
{
	Faces.SetNum(0);

	for (int32 tid : InputMesh.TriangleIndicesItr())
	{
		FVector3d A, B, C;
		InputMesh.GetTriVertices(tid, A, B, C);
		FVector3d Centroid, Normal; double Area;
		InputMesh.GetTriInfo(tid, Normal, Area, Centroid);

		FDynamicMeshFace Face;
		Face.Plane = FFrame3d(Centroid, Normal);
		Face.BoundaryLoop.Add(A);
		Face.BoundaryLoop.Add(C);
		Face.BoundaryLoop.Add(B);

		Faces.Add(Face);
	}
}

}//end namespace UE::Conversion
}//end namespace UE