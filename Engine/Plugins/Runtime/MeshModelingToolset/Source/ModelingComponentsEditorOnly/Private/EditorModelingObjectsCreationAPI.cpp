// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorModelingObjectsCreationAPI.h"
#include "InteractiveToolsContext.h"
#include "InteractiveToolManager.h"
#include "ContextObjectStore.h"

#include "AssetUtils/CreateStaticMeshUtil.h"
#include "AssetUtils/CreateTexture2DUtil.h"
#include "AssetUtils/CreateMaterialUtil.h"
#include "Physics/ComponentCollisionUtil.h"

#include "ConversionUtils/DynamicMeshToVolume.h"
#include "MeshDescriptionToDynamicMesh.h"

#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"

#include "ToolTargets/VolumeComponentToolTarget.h"  // for CVarModelingMaxVolumeTriangleCount
#include "Engine/BlockingVolume.h"
#include "Components/BrushComponent.h"
#include "Engine/Polys.h"
#include "Model.h"
#include "BSPOps.h"		// in UnrealEd
#include "Editor/EditorEngine.h"		// for FActorLabelUtilities

#include "DynamicMeshActor.h"
#include "Components/DynamicMeshComponent.h"

#include "ActorFactories/ActorFactory.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "AssetSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorModelingObjectsCreationAPI)

using namespace UE::Geometry;

extern UNREALED_API UEditorEngine* GEditor;

UEditorModelingObjectsCreationAPI* UEditorModelingObjectsCreationAPI::Register(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UEditorModelingObjectsCreationAPI* CreationAPI = ToolsContext->ContextObjectStore->FindContext<UEditorModelingObjectsCreationAPI>();
		if (CreationAPI)
		{
			return CreationAPI;
		}
		CreationAPI = NewObject<UEditorModelingObjectsCreationAPI>(ToolsContext);
		ToolsContext->ContextObjectStore->AddContextObject(CreationAPI);
		if (ensure(CreationAPI))
		{
			return CreationAPI;
		}
	}
	return nullptr;
}

UEditorModelingObjectsCreationAPI* UEditorModelingObjectsCreationAPI::Find(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UEditorModelingObjectsCreationAPI* CreationAPI = ToolsContext->ContextObjectStore->FindContext<UEditorModelingObjectsCreationAPI>();
		if (CreationAPI != nullptr)
		{
			return CreationAPI;
		}
	}
	return nullptr;
}

bool UEditorModelingObjectsCreationAPI::Deregister(UInteractiveToolsContext* ToolsContext)
{
	if (ensure(ToolsContext))
	{
		UEditorModelingObjectsCreationAPI* CreationAPI = ToolsContext->ContextObjectStore->FindContext<UEditorModelingObjectsCreationAPI>();
		if (CreationAPI != nullptr)
		{
			ToolsContext->ContextObjectStore->RemoveContextObject(CreationAPI);
		}
		return true;
	}
	return false;
}


FCreateMeshObjectResult UEditorModelingObjectsCreationAPI::CreateMeshObject(const FCreateMeshObjectParams& CreateMeshParams)
{
	// TODO: implement this path
	check(false);
	return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_InvalidMesh };
}


FCreateTextureObjectResult UEditorModelingObjectsCreationAPI::CreateTextureObject(const FCreateTextureObjectParams& CreateTexParams)
{
	FCreateTextureObjectParams LocalParams = CreateTexParams;
	return CreateTextureObject(MoveTemp(LocalParams));
}

FCreateMaterialObjectResult UEditorModelingObjectsCreationAPI::CreateMaterialObject(const FCreateMaterialObjectParams& CreateMaterialParams)
{
	FCreateMaterialObjectParams LocalParams = CreateMaterialParams;
	return CreateMaterialObject(MoveTemp(LocalParams));
}

FCreateActorResult UEditorModelingObjectsCreationAPI::CreateNewActor(const FCreateActorParams& CreateActorParams)
{
	FCreateActorParams LocalParams = CreateActorParams;
	return CreateNewActor(MoveTemp(LocalParams));
}

FCreateComponentResult UEditorModelingObjectsCreationAPI::CreateNewComponentOnActor(const FCreateComponentParams& CreateComponentParams)
{
	FCreateComponentParams LocalParams = CreateComponentParams;
	return CreateNewComponentOnActor(MoveTemp(LocalParams));		
}


FCreateMeshObjectResult UEditorModelingObjectsCreationAPI::CreateMeshObject(FCreateMeshObjectParams&& CreateMeshParams)
{
	FCreateMeshObjectResult ResultOut;
	if (CreateMeshParams.TypeHint == ECreateObjectTypeHint::Volume)
	{
		ResultOut = CreateVolume(MoveTemp(CreateMeshParams));
	}
	else if (CreateMeshParams.TypeHint == ECreateObjectTypeHint::DynamicMeshActor)
	{
		ResultOut = CreateDynamicMeshActor(MoveTemp(CreateMeshParams));
	}
	else
	{
		ResultOut = CreateStaticMeshAsset(MoveTemp(CreateMeshParams));
	}

	if (ResultOut.IsOK())
	{
		OnModelingMeshCreated.Broadcast(ResultOut);
	}

	return ResultOut;
}


TArray<UMaterialInterface*> UEditorModelingObjectsCreationAPI::FilterMaterials(const TArray<UMaterialInterface*>& MaterialsIn)
{
	TArray<UMaterialInterface*> OutputMaterials = MaterialsIn;
	for (int32 k = 0; k < OutputMaterials.Num(); ++k)
	{
		FString AssetPath = OutputMaterials[k]->GetPathName();
		if (AssetPath.StartsWith(TEXT("/MeshModelingToolsetExp/")))
		{
			OutputMaterials[k] = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}
	return OutputMaterials;
}


FCreateMeshObjectResult UEditorModelingObjectsCreationAPI::CreateVolume(FCreateMeshObjectParams&& CreateMeshParams)
{
	// determine volume actor type
	UClass* VolumeClass = ABlockingVolume::StaticClass();
	if (CreateMeshParams.TypeHintClass
		&& Cast<AVolume>(CreateMeshParams.TypeHintClass.Get()->GetDefaultObject(false)) != nullptr)
	{
		VolumeClass = CreateMeshParams.TypeHintClass;
	}

	// create new volume actor using factory
	AVolume* const NewVolumeActor = [VolumeClass, &CreateMeshParams]() -> AVolume*
	{
		if (UActorFactory* const VolumeFactory = FActorFactoryAssetProxy::GetFactoryForAssetObject(VolumeClass))
		{
			AActor* const Actor = VolumeFactory->CreateActor(VolumeClass, CreateMeshParams.TargetWorld->GetCurrentLevel(), FTransform::Identity);
			FActorLabelUtilities::SetActorLabelUnique(Actor, CreateMeshParams.BaseName);
			return Cast<AVolume>(Actor);
		}
		return nullptr;
	}();
	if (!NewVolumeActor)
	{
		return FCreateMeshObjectResult{ECreateModelingObjectResult::Failed_ActorCreationFailed};
	}

	NewVolumeActor->BrushType = EBrushType::Brush_Add;
	UModel* Model = NewObject<UModel>(NewVolumeActor);
	NewVolumeActor->Brush = Model;
	NewVolumeActor->GetBrushComponent()->Brush = NewVolumeActor->Brush;

	UE::Conversion::FMeshToVolumeOptions Options;
	Options.bAutoSimplify = true;
	Options.MaxTriangles = FMath::Max(1, CVarModelingMaxVolumeTriangleCount.GetValueOnGameThread());
	if (CreateMeshParams.MeshType == ECreateMeshObjectSourceMeshType::DynamicMesh)
	{
		UE::Conversion::DynamicMeshToVolume(CreateMeshParams.DynamicMesh.GetValue(), NewVolumeActor, Options);
	}
	else if (CreateMeshParams.MeshType == ECreateMeshObjectSourceMeshType::MeshDescription)
	{
		FMeshDescriptionToDynamicMesh Converter;
		FMeshDescription* MeshDescription = &CreateMeshParams.MeshDescription.GetValue();
		FDynamicMesh3 ConvertedMesh;
		Converter.Convert(MeshDescription, ConvertedMesh);
		UE::Conversion::DynamicMeshToVolume(ConvertedMesh, NewVolumeActor, Options);
	}
	else
	{
		return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_InvalidMesh };
	}

	NewVolumeActor->SetActorTransform(CreateMeshParams.Transform);
	FActorLabelUtilities::SetActorLabelUnique(NewVolumeActor, CreateMeshParams.BaseName);
	NewVolumeActor->PostEditChange();
	
	// emit result
	FCreateMeshObjectResult ResultOut;
	ResultOut.ResultCode = ECreateModelingObjectResult::Ok;
	ResultOut.NewActor = NewVolumeActor;
	ResultOut.NewComponent = NewVolumeActor->GetBrushComponent();
	ResultOut.NewAsset = nullptr;
	return ResultOut;
}



FCreateMeshObjectResult UEditorModelingObjectsCreationAPI::CreateDynamicMeshActor(FCreateMeshObjectParams&& CreateMeshParams)
{
	// spawn new actor
	FActorSpawnParameters SpawnInfo;
	ADynamicMeshActor* NewActor = CreateMeshParams.TargetWorld->SpawnActor<ADynamicMeshActor>(SpawnInfo);

	UDynamicMeshComponent* NewComponent = NewActor->GetDynamicMeshComponent();

	// assume that DynamicMeshComponent always has tangents on it's internal UDynamicMesh
	NewComponent->SetTangentsType(EDynamicMeshComponentTangentsMode::ExternallyProvided);

	if (CreateMeshParams.MeshType == ECreateMeshObjectSourceMeshType::DynamicMesh)
	{
		FDynamicMesh3 SetMesh = MoveTemp(CreateMeshParams.DynamicMesh.GetValue());
		if (SetMesh.IsCompact() == false)
		{
			SetMesh.CompactInPlace();
		}
		NewComponent->SetMesh(MoveTemp(SetMesh));
		NewComponent->NotifyMeshUpdated();
	}
	else if (CreateMeshParams.MeshType == ECreateMeshObjectSourceMeshType::MeshDescription)
	{
		const FMeshDescription* MeshDescription = &CreateMeshParams.MeshDescription.GetValue();
		FDynamicMesh3 Mesh(EMeshComponents::FaceGroups);
		Mesh.EnableAttributes();
		FMeshDescriptionToDynamicMesh Converter;
		Converter.Convert(MeshDescription, Mesh, true);
		NewComponent->SetMesh(MoveTemp(Mesh));
	}
	else
	{
		return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_InvalidMesh };
	}

	NewActor->SetActorTransform(CreateMeshParams.Transform);
	FActorLabelUtilities::SetActorLabelUnique(NewActor, CreateMeshParams.BaseName);

	// set materials
	TArray<UMaterialInterface*> ComponentMaterials = FilterMaterials(CreateMeshParams.Materials);
	for (int32 k = 0; k < ComponentMaterials.Num(); ++k)
	{
		NewComponent->SetMaterial(k, ComponentMaterials[k]);
	}

	// configure collision
	if (CreateMeshParams.bEnableCollision)
	{
		if (CreateMeshParams.CollisionShapeSet.IsSet())
		{
			UE::Geometry::SetSimpleCollision(NewComponent, CreateMeshParams.CollisionShapeSet.GetPtrOrNull());
		}

		NewComponent->CollisionType = CreateMeshParams.CollisionMode;
		// enable complex collision so that raycasts can hit this object
		NewComponent->bEnableComplexCollision = true;

		// force collision update
		NewComponent->UpdateCollision(false);
	}

	// configure raytracing
	NewComponent->SetEnableRaytracing(CreateMeshParams.bEnableRaytracingSupport);

	NewActor->PostEditChange();

	// emit result
	FCreateMeshObjectResult ResultOut;
	ResultOut.ResultCode = ECreateModelingObjectResult::Ok;
	ResultOut.NewActor = NewActor;
	ResultOut.NewComponent = NewComponent;
	ResultOut.NewAsset = nullptr;
	return ResultOut;
}




FCreateMeshObjectResult UEditorModelingObjectsCreationAPI::CreateStaticMeshAsset(FCreateMeshObjectParams&& CreateMeshParams)
{
	UE::AssetUtils::FStaticMeshAssetOptions AssetOptions;
	
	ECreateModelingObjectResult AssetPathResult = GetNewAssetPath(
		AssetOptions.NewAssetPath,
		CreateMeshParams.BaseName,
		nullptr,
		CreateMeshParams.TargetWorld);
		
	if (AssetPathResult != ECreateModelingObjectResult::Ok)
	{
		return FCreateMeshObjectResult{ AssetPathResult };
	}

	AssetOptions.NumSourceModels = 1;
	AssetOptions.NumMaterialSlots = CreateMeshParams.Materials.Num();
	AssetOptions.AssetMaterials = (CreateMeshParams.AssetMaterials.Num() == AssetOptions.NumMaterialSlots) ?
		FilterMaterials(CreateMeshParams.AssetMaterials) : FilterMaterials(CreateMeshParams.Materials);

	AssetOptions.bEnableRecomputeNormals = CreateMeshParams.bEnableRecomputeNormals;
	AssetOptions.bEnableRecomputeTangents = CreateMeshParams.bEnableRecomputeTangents;
	AssetOptions.bGenerateNaniteEnabledMesh = CreateMeshParams.bEnableNanite;
	AssetOptions.NaniteSettings = CreateMeshParams.NaniteSettings;
	AssetOptions.bGenerateLightmapUVs = CreateMeshParams.bGenerateLightmapUVs;

	AssetOptions.bCreatePhysicsBody = CreateMeshParams.bEnableCollision;
	AssetOptions.CollisionType = CreateMeshParams.CollisionMode;

	if (CreateMeshParams.MeshType == ECreateMeshObjectSourceMeshType::DynamicMesh)
	{
		FDynamicMesh3* DynamicMesh = &CreateMeshParams.DynamicMesh.GetValue();
		AssetOptions.SourceMeshes.DynamicMeshes.Add(DynamicMesh);
	}
	else if (CreateMeshParams.MeshType == ECreateMeshObjectSourceMeshType::MeshDescription)
	{
		FMeshDescription* MeshDescription = &CreateMeshParams.MeshDescription.GetValue();
		AssetOptions.SourceMeshes.MoveMeshDescriptions.Add(MeshDescription);
	}
	else
	{
		return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_InvalidMesh };
	}

	UE::AssetUtils::FStaticMeshResults ResultData;
	UE::AssetUtils::ECreateStaticMeshResult AssetResult = UE::AssetUtils::CreateStaticMeshAsset(AssetOptions, ResultData);

	if (AssetResult != UE::AssetUtils::ECreateStaticMeshResult::Ok)
	{
		return FCreateMeshObjectResult{ ECreateModelingObjectResult::Failed_AssetCreationFailed };
	}

	UStaticMesh* NewStaticMesh = ResultData.StaticMesh;

	// create new StaticMeshActor
	AStaticMeshActor* const StaticMeshActor = [NewStaticMesh, &CreateMeshParams]() -> AStaticMeshActor*
	{
		if (UActorFactory* const StaticMeshFactory = FActorFactoryAssetProxy::GetFactoryForAssetObject(NewStaticMesh))
		{
			AActor* const Actor = StaticMeshFactory->CreateActor(NewStaticMesh, CreateMeshParams.TargetWorld->GetCurrentLevel(), FTransform::Identity);
			FActorLabelUtilities::SetActorLabelUnique(Actor, CreateMeshParams.BaseName);
			return Cast<AStaticMeshActor>(Actor);
		}
		return nullptr;
	}();
	if (!StaticMeshActor)
	{
		return FCreateMeshObjectResult{ECreateModelingObjectResult::Failed_ActorCreationFailed};
	}

	// set the mesh
	UStaticMeshComponent* const StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();

	// this disconnects the component from various events
	StaticMeshComponent->UnregisterComponent();
	// replace the UStaticMesh in the component
	StaticMeshComponent->SetStaticMesh(NewStaticMesh);

	// set materials
	TArray<UMaterialInterface*> ComponentMaterials = FilterMaterials(CreateMeshParams.Materials);
	for (int32 k = 0; k < ComponentMaterials.Num(); ++k)
	{
		StaticMeshComponent->SetMaterial(k, ComponentMaterials[k]);
	}

	// set simple collision geometry
	if (CreateMeshParams.CollisionShapeSet.IsSet())
	{
		UE::Geometry::SetSimpleCollision(StaticMeshComponent, CreateMeshParams.CollisionShapeSet.GetPtrOrNull(),
			UE::Geometry::GetCollisionSettings(StaticMeshComponent));
	}

	// re-connect the component (?)
	StaticMeshComponent->RegisterComponent();

	NewStaticMesh->PostEditChange();

	StaticMeshComponent->RecreatePhysicsState();

	// update transform
	StaticMeshActor->SetActorTransform(CreateMeshParams.Transform);

	// emit result
	FCreateMeshObjectResult ResultOut;
	ResultOut.ResultCode = ECreateModelingObjectResult::Ok;
	ResultOut.NewActor = StaticMeshActor;
	ResultOut.NewComponent = StaticMeshComponent;
	ResultOut.NewAsset = NewStaticMesh;
	return ResultOut;
}




FCreateTextureObjectResult UEditorModelingObjectsCreationAPI::CreateTextureObject(FCreateTextureObjectParams&& CreateTexParams)
{
	// currently we cannot create a new texture without an existing generated texture to store
	if (!ensure(CreateTexParams.GeneratedTransientTexture))
	{
		return FCreateTextureObjectResult{ ECreateModelingObjectResult::Failed_InvalidTexture };
	}
	UE::AssetUtils::FTexture2DAssetOptions AssetOptions;
	UE::AssetUtils::FTexture2DAssetResults ResultData;
	
	if (CreateTexParams.FullAssetPath != "")
	{
		AssetOptions.NewAssetPath = CreateTexParams.FullAssetPath;
	}
	else
	{
		const ECreateModelingObjectResult AssetPathResult = GetNewAssetPath(
			AssetOptions.NewAssetPath,
			CreateTexParams.BaseName,
			CreateTexParams.StoreRelativeToObject,
			CreateTexParams.TargetWorld);

		if (AssetPathResult != ECreateModelingObjectResult::Ok)
		{
			return FCreateTextureObjectResult{ AssetPathResult };
		}

	}
	const UE::AssetUtils::ECreateTexture2DResult AssetResult = UE::AssetUtils::SaveGeneratedTexture2DAsset(
			CreateTexParams.GeneratedTransientTexture, AssetOptions, ResultData);
	

	if (AssetResult != UE::AssetUtils::ECreateTexture2DResult::Ok)
	{
		return FCreateTextureObjectResult{ ECreateModelingObjectResult::Failed_AssetCreationFailed };
	}

	// emit result
	FCreateTextureObjectResult ResultOut;
	ResultOut.ResultCode = ECreateModelingObjectResult::Ok;
	ResultOut.NewAsset = ResultData.Texture;

	OnModelingTextureCreated.Broadcast(ResultOut);

	return ResultOut;
}




FCreateMaterialObjectResult UEditorModelingObjectsCreationAPI::CreateMaterialObject(FCreateMaterialObjectParams&& CreateMaterialParams)
{
	UE::AssetUtils::FMaterialAssetOptions AssetOptions;

	ECreateModelingObjectResult AssetPathResult = GetNewAssetPath(
		AssetOptions.NewAssetPath,
		CreateMaterialParams.BaseName,
		CreateMaterialParams.StoreRelativeToObject,
		CreateMaterialParams.TargetWorld);

	if (AssetPathResult != ECreateModelingObjectResult::Ok)
	{
		return FCreateMaterialObjectResult{ AssetPathResult };
	}

	// Currently we cannot create a new material without duplicating an existing material
	if (!ensure(CreateMaterialParams.MaterialToDuplicate))
	{
		return FCreateMaterialObjectResult{ ECreateModelingObjectResult::Failed_InvalidMaterial };
	}

	UE::AssetUtils::FMaterialAssetResults ResultData;
	UE::AssetUtils::ECreateMaterialResult AssetResult = UE::AssetUtils::CreateDuplicateMaterial(
		CreateMaterialParams.MaterialToDuplicate,
		AssetOptions,
		ResultData);

	if (AssetResult != UE::AssetUtils::ECreateMaterialResult::Ok)
	{
		return FCreateMaterialObjectResult{ ECreateModelingObjectResult::Failed_AssetCreationFailed };
	}

	// emit result
	FCreateMaterialObjectResult ResultOut;
	ResultOut.ResultCode = ECreateModelingObjectResult::Ok;
	ResultOut.NewAsset = ResultData.NewMaterial;

	OnModelingMaterialCreated.Broadcast(ResultOut);

	return ResultOut;
}




FCreateActorResult UEditorModelingObjectsCreationAPI::CreateNewActor(FCreateActorParams&& CreateActorParams)
{
	// create new Actor
	AActor* NewActor = nullptr;
	
	if (GEditor && CreateActorParams.TemplateAsset)
	{
		if (UActorFactory* const ActorFactory =  FActorFactoryAssetProxy::GetFactoryForAssetObject(CreateActorParams.TemplateAsset))
		{
			NewActor = ActorFactory->CreateActor(CreateActorParams.TemplateAsset, CreateActorParams.TargetWorld->GetCurrentLevel(), CreateActorParams.Transform);
			FActorLabelUtilities::SetActorLabelUnique(NewActor, CreateActorParams.BaseName);
		}
	}

	// If we don't have an asset template, we're going to just create a blank, empty actor instance.
	if (GEditor && !CreateActorParams.TemplateAsset)
	{
		UActorFactoryEmptyActor* EmptyActorFactory = NewObject<UActorFactoryEmptyActor>();
		FAssetData AssetData(EmptyActorFactory->GetDefaultActorClass(FAssetData()));
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = FName(CreateActorParams.BaseName);
		SpawnParams.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;
		NewActor = EmptyActorFactory->CreateActor(AssetData.GetAsset(),
			CreateActorParams.TargetWorld->GetCurrentLevel(),
			CreateActorParams.Transform,
			SpawnParams);

		// This is also editor-only: it's the label that shows up in the hierarchy
		FActorLabelUtilities::SetActorLabelUnique(NewActor, CreateActorParams.BaseName);
	}


	if (!NewActor)
	{
		return FCreateActorResult{ ECreateModelingObjectResult::Failed_ActorCreationFailed };
	}

	FCreateActorResult ResultOut;
	ResultOut.ResultCode = ECreateModelingObjectResult::Ok;
	ResultOut.NewActor = NewActor;

	OnModelingActorCreated.Broadcast(ResultOut);

	return ResultOut;
}

FCreateComponentResult UEditorModelingObjectsCreationAPI::CreateNewComponentOnActor(FCreateComponentParams&& CreateComponentParams)
{
	FCreateComponentResult Result;

	if (!ensure(CreateComponentParams.HostActor))
	{
		Result.ResultCode = ECreateModelingObjectResult::Failed_InvalidActor;
		return Result;
	}

	if (CreateComponentParams.bTransact)
	{
		CreateComponentParams.HostActor->Modify();
	}

	FName NewComponentName = *FComponentEditorUtils::GenerateValidVariableName(CreateComponentParams.ComponentClass, CreateComponentParams.HostActor);

	if (!CreateComponentParams.BaseName.IsEmpty())
	{
		NewComponentName = FName(CreateComponentParams.BaseName);
	}

	// Note that the RF_Transactional is important here for the spline to undo/redo properly in the future
	UActorComponent* Component = NewObject<UActorComponent>(CreateComponentParams.HostActor, CreateComponentParams.ComponentClass, NewComponentName,
		CreateComponentParams.bTransact ? RF_Transactional : RF_NoFlags);
	
	USceneComponent* SceneComponent = StaticCast<USceneComponent*>(Component);
	if (SceneComponent) {
		if (CreateComponentParams.bSetAsRoot)
		{
			CreateComponentParams.HostActor->SetRootComponent(SceneComponent);
		}
		else
		{
			SceneComponent->SetupAttachment(CreateComponentParams.HostActor->GetRootComponent());
		}
	}

	Component->OnComponentCreated();
	CreateComponentParams.HostActor->AddInstanceComponent(Component);
	Component->RegisterComponent();
	if (SceneComponent) {
		SceneComponent->ResetRelativeTransform();
	}
	CreateComponentParams.HostActor->PostEditChange();

	Result.ResultCode = ECreateModelingObjectResult::Ok;
	Result.NewComponent = Component;

	OnModelingComponentCreated.Broadcast(Result);

	return Result;
}

ECreateModelingObjectResult UEditorModelingObjectsCreationAPI::GetNewAssetPath(
	FString& OutNewAssetPath,
	const FString& BaseName,
	const UObject* StoreRelativeToObject,
	const UWorld* TargetWorld)
{
	FString RelativeToObjectFolder;
	if (StoreRelativeToObject != nullptr)
	{
		// find path to asset
		UPackage* AssetOuterPackage = CastChecked<UPackage>(StoreRelativeToObject->GetOuter());
		if (ensure(AssetOuterPackage))
		{
			FString AssetPackageName = AssetOuterPackage->GetName();
			RelativeToObjectFolder = FPackageName::GetLongPackagePath(AssetPackageName);
		}
	}
	else
	{
		if (!ensure(TargetWorld)) {
			return ECreateModelingObjectResult::Failed_InvalidWorld;
		}
	}

	if (GetNewAssetPathNameCallback.IsBound())
	{
		OutNewAssetPath = GetNewAssetPathNameCallback.Execute(BaseName, TargetWorld, RelativeToObjectFolder);
		if (OutNewAssetPath.Len() == 0)
		{
			return ECreateModelingObjectResult::Cancelled;
		}
	}
	else
	{
		FString UseBaseFolder = (RelativeToObjectFolder.Len() > 0) ? RelativeToObjectFolder : TEXT("/Game");
		OutNewAssetPath = FPaths::Combine(UseBaseFolder, BaseName);
	}

	return ECreateModelingObjectResult::Ok;
}
