// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModelingToolTargetEditorOnlyUtil.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "ModelingToolTargetEditorOnlyUtil"

TArray<EMeshLODIdentifier> UE::ToolTarget::GetAvailableLODs(const USkeletalMesh* SkeletalMesh, bool bSkipAutoGenerated)
{
	TArray<EMeshLODIdentifier> AvailableLODs;

	if (!SkeletalMesh)
	{
		return AvailableLODs;
	}

	const int32 MaxLODs = FMath::Min(8, SkeletalMesh->GetNumSourceModels());

	for (int32 LODIndex = 0; LODIndex < MaxLODs; LODIndex++)
	{
		if (bSkipAutoGenerated && !SkeletalMesh->HasMeshDescription(LODIndex))
		{
			continue;
		}

		AvailableLODs.Add(static_cast<EMeshLODIdentifier>(LODIndex));
	}

	return AvailableLODs;
}

#undef LOCTEXT_NAMESPACE