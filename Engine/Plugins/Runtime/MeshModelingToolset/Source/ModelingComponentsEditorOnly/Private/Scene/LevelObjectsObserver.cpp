// Copyright Epic Games, Inc. All Rights Reserved.

#include "Scene/LevelObjectsObserver.h"


#include "Engine/Engine.h"
#include "EngineUtils.h" // for TActorIterator<>

#include "Editor.h"
#include "EditorSupportDelegates.h"


FLevelObjectsObserver::~FLevelObjectsObserver()
{	// These ensures are meant to catch missing Shutdown() calls. However if the Editor is closed, Shutdown()
	// (which is generally called by UEdMode exit) will not get chance to happen before objects are GC'd.
	if (GEditor && GEngine)
	{
		ensure(!OnActorDeletedHandle.IsValid());
		ensure(!OnActorAddedHandle.IsValid());
		ensure(!OnActorListChangedHandle.IsValid());
	}
}


void FLevelObjectsObserver::Initialize(UWorld* WorldIn)
{
	this->World = WorldIn;

	GEditor->RegisterForUndo(this);

	OnActorDeletedHandle = GEngine->OnLevelActorDeleted().AddLambda([this](AActor* Actor) { HandleActorDeletedEvent(Actor); });
	OnActorAddedHandle = GEngine->OnLevelActorAdded().AddLambda([this](AActor* Actor) { HandleActorAddedEvent(Actor); });
	OnActorListChangedHandle = GEngine->OnLevelActorListChanged().AddLambda([this]() { OnUntrackedLevelChange(); });

	OnUntrackedLevelChange();
}


void FLevelObjectsObserver::Shutdown()
{
	for (AActor* Actor : Actors)
	{
		OnActorRemoved.Broadcast(Actor);
	}
	Actors.Reset();

	this->World = nullptr;

	OnActorAdded.Clear();
	OnActorRemoved.Clear();

	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}

	if (GEngine != nullptr)
	{
		GEngine->OnLevelActorDeleted().Remove(OnActorDeletedHandle);
		GEngine->OnLevelActorAdded().Remove(OnActorAddedHandle);
		GEngine->OnLevelActorListChanged().Remove(OnActorListChangedHandle);
	}

	OnActorDeletedHandle.Reset();
	OnActorAddedHandle.Reset();
	OnActorListChangedHandle.Reset();
}


void FLevelObjectsObserver::PostUndo(bool bSuccess)
{
	// on undo/redo anything might have happend so we have to check the entire actor set
	OnUntrackedLevelChange();
}
void FLevelObjectsObserver::PostRedo(bool bSuccess) 
{ 
	PostUndo(bSuccess);
}



void FLevelObjectsObserver::OnUntrackedLevelChange()
{
	// the world changed and we don't know how, so check all actors

	TSet<AActor*> RemainingActors;
	RemainingActors.Append(Actors);

	// find any new Actors that we don't know about
	for ( TActorIterator<AActor> ActorItr(World); ActorItr; ++ActorItr)
	{
		AActor* Actor = *ActorItr;
		if (Actors.Contains(Actor) == false)
		{
			Actors.Add(Actor);
			OnActorAdded.Broadcast(Actor);
		}
		RemainingActors.Remove(Actor);		// we saw this actor
	}

	// remove any Actors we were tracking that no longer exist
	for (AActor* Actor : RemainingActors)
	{
		Actors.Remove(Actor);
		OnActorRemoved.Broadcast(Actor);
	}
}


void FLevelObjectsObserver::HandleActorAddedEvent(AActor* Actor)
{
	if (Actors.Contains(Actor) == false)
	{
		Actors.Add(Actor);
		OnActorAdded.Broadcast(Actor);
	}
}


void FLevelObjectsObserver::HandleActorDeletedEvent(AActor* Actor)
{
	if (Actors.Contains(Actor))
	{
		Actors.Remove(Actor);
		OnActorRemoved.Broadcast(Actor);
	}
}
