// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/StaticMeshSelector.h"
#include "Selection/GeometrySelector.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"

#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Changes/MeshVertexChange.h"

#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "Selection/DynamicMeshPolygroupTransformer.h"

#include "StaticMeshOperations.h"

#include "RenderingThread.h"
#include "UObject/Package.h"

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FStaticMeshSelector"

static TAutoConsoleVariable<bool> CVarEnableModelingSelectionStaticMeshLocking(
	TEXT("modeling.Selection.EnableStaticMeshLocking"),
	true,
	TEXT("Control whether Selection Locking is enabled by default for Static Meshes"));

namespace UEGlobal
{
	/**
	 * Only Unlocked Mesh Assets can be edited. This is currently implemented as a global set based on UObject pointers.
	 * Obviously not an ideal solution, however this is really only used to implement a UI gate, if we get a stale pointer
	 * here that happens to still point to a valid UStaticMesh, it just means that asset will be unlocked for mesh selection
	 * even if the user did not explicitly unlock it. This is not a disaster.
	 */
	TSet<UStaticMesh*> UnlockedStaticMeshes;
}

bool FStaticMeshSelector::IsLockable() const 
{ 
	return CVarEnableModelingSelectionStaticMeshLocking.GetValueOnGameThread();
}


bool FStaticMeshSelector::IsLocked() const
{
	if (CVarEnableModelingSelectionStaticMeshLocking.GetValueOnGameThread())
	{
		return WeakStaticMesh != nullptr && (UEGlobal::UnlockedStaticMeshes.Contains(WeakStaticMesh.Get()) == false);
	}
	else
	{
		return (WeakStaticMesh == nullptr);
	}
}

void FStaticMeshSelector::SetLockedState(bool bLocked)
{
	if (UStaticMesh* StaticMesh = WeakStaticMesh.Get())
	{
		if (bLocked)
		{
			UEGlobal::UnlockedStaticMeshes.Remove(StaticMesh);
		}
		else
		{
			UEGlobal::UnlockedStaticMeshes.Add(StaticMesh);

			// if the mesh was locked, we were not updating it on changes, so we
			// need to do an update now
			if (IsLocked() == false)
			{
				CopyFromStaticMesh();
				InvalidateOnMeshChange(FDynamicMeshChangeInfo());
			}
		}
	}
}


void FStaticMeshSelector::SetAssetUnlockedOnCreation(UStaticMesh* StaticMesh)
{
	if (StaticMesh != nullptr)
	{
		UEGlobal::UnlockedStaticMeshes.Add(StaticMesh);
	}
}

void FStaticMeshSelector::ResetUnlockedStaticMeshAssets()
{
	UEGlobal::UnlockedStaticMeshes.Reset();
}



bool FStaticMeshSelector::Initialize(
	FGeometryIdentifier SourceGeometryIdentifierIn)
{
	if (SourceGeometryIdentifierIn.TargetType != FGeometryIdentifier::ETargetType::PrimitiveComponent ||
		SourceGeometryIdentifierIn.ObjectType != FGeometryIdentifier::EObjectType::StaticMeshComponent)
	{
		return false;
	}
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SourceGeometryIdentifierIn.TargetObject); 
	if (StaticMeshComponent == nullptr)
	{
		return false;
	}
	WeakStaticMeshComponent = StaticMeshComponent;
	
	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	if (StaticMesh == nullptr)
	{
		return false;
	}
	WeakStaticMesh = StaticMesh;

	// To know if the static mesh changed, we will listen to this OnMeshChanged event.
	// It is currently a very large hammer, though...
	StaticMesh_OnMeshChangedHandle = StaticMesh->OnMeshChanged.AddLambda([this]()
	{
		if (IsLocked() == false)
		{
			CopyFromStaticMesh();
			InvalidateOnMeshChange(FDynamicMeshChangeInfo());
		}
	});

	// create new transient UDyanmicMesh that will be owned by this Selector
	LocalTargetMesh.Reset( NewObject<UDynamicMesh>() );

	if (IsLocked() == false)
	{
		CopyFromStaticMesh();
	}

	FBaseDynamicMeshSelector::Initialize(SourceGeometryIdentifierIn, LocalTargetMesh.Get(), 
		[this]()
		{
			FTransformSRT3d Result = FTransformSRT3d::Identity();
			if (const UStaticMeshComponent* StaticMeshComponent = this->WeakStaticMeshComponent.Get())
			{
				Result = StaticMeshComponent->GetComponentTransform();
			}
			return Result;
		});

	return true;
}

void FStaticMeshSelector::Shutdown()
{
	if (UStaticMesh* StaticMesh = WeakStaticMesh.Get())
	{
		StaticMesh->OnMeshChanged.Remove(StaticMesh_OnMeshChangedHandle);
	}
	StaticMesh_OnMeshChangedHandle.Reset();

	LocalTargetMesh.Reset();
	FBaseDynamicMeshSelector::Shutdown();
}



IGeometrySelectionTransformer* FStaticMeshSelector::InitializeTransformation(const FGeometrySelection& Selection)
{
	check(!ActiveTransformer);

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		ActiveTransformer = MakeShared<FDynamicMeshPolygroupTransformer>();
	}
	else
	{
		ActiveTransformer = MakeShared<FBasicDynamicMeshSelectionTransformer>();
	}
	ActiveTransformer->bEnableSelectionTransformDrawing = true;

	ActiveTransformer->Initialize(this);
	ActiveTransformer->OnEndTransformFunc = [this](IToolsContextTransactionsAPI*) 
	{ 
		CommitMeshTransform();
	};
	return ActiveTransformer.Get();
}

void FStaticMeshSelector::ShutdownTransformation(IGeometrySelectionTransformer* Transformer)
{
	ActiveTransformer.Reset();
}


void FStaticMeshSelector::UpdateAfterGeometryEdit(
	IToolsContextTransactionsAPI* TransactionsAPI,
	bool bInTransaction,
	TUniquePtr<FDynamicMeshChange> DynamicMeshChange,
	FText GeometryEditTransactionString)
{
	if (!bInTransaction)
	{
		TransactionsAPI->BeginUndoTransaction(GeometryEditTransactionString);
	}

	CommitMeshTransform();

	if (!bInTransaction)
	{
		TransactionsAPI->EndUndoTransaction();
	}
}


void FStaticMeshSelector::CopyFromStaticMesh()
{
	int32 UseLODIndex = 0;
	const FMeshDescription* SourceMesh = nullptr;
	FVector BuildScale = FVector::OneVector;
	if (const UStaticMesh* StaticMesh = WeakStaticMesh.Get())
	{
		SourceMesh = StaticMesh->GetMeshDescription(UseLODIndex);
		if (StaticMesh->IsSourceModelValid(UseLODIndex))
		{
			BuildScale = StaticMesh->GetSourceModel(UseLODIndex).BuildSettings.BuildScale3D;
		}
	}
	
	if (SourceMesh != nullptr)
	{
		FDynamicMesh3 NewMesh;
		FMeshDescriptionToDynamicMesh Converter;
		Converter.Convert(SourceMesh, NewMesh, false /*AssetOptions.bRequestTangents*/ );
		if (!BuildScale.Equals(FVector::OneVector))
		{
			MeshTransforms::Scale(NewMesh, BuildScale, FVector::ZeroVector, true);
		}

		LocalTargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			EditMesh = MoveTemp(NewMesh);
		});
	}
	else
	{
		LocalTargetMesh->ResetToCube();
	}
}


void FStaticMeshSelector::CommitMeshTransform()
{
	int32 UseLODIndex = 0;
	FlushRenderingCommands();

	// emit transaction here??
	
	if (UStaticMesh* StaticMesh = WeakStaticMesh.Get())
	{
		// make sure transactional flag is on for this asset
		StaticMesh->SetFlags(RF_Transactional);
		// mark as modified
		StaticMesh->Modify();

		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(UseLODIndex);
		if (MeshDescription == nullptr)
		{
			MeshDescription = StaticMesh->CreateMeshDescription(UseLODIndex);
		}
		StaticMesh->ModifyMeshDescription(UseLODIndex);

		FConversionToMeshDescriptionOptions ConversionOptions;
		FDynamicMeshToMeshDescription Converter(ConversionOptions);
		TargetMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh)
		{
			Converter.Convert(&ReadMesh, *MeshDescription, false /*bCopyTangents*/ );
		});
		FVector BuildScale = StaticMesh->GetSourceModel(UseLODIndex).BuildSettings.BuildScale3D;
		if (!BuildScale.Equals(FVector::OneVector))
		{
			FVector InvBuildScale;
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				InvBuildScale[SubIdx] = FMath::IsNearlyZero(BuildScale[SubIdx], FMathd::Epsilon) ? 1.0 : 1.0 / BuildScale[SubIdx];
			}
			FTransform InvScaleTransform = FTransform::Identity;
			InvScaleTransform.SetScale3D(InvBuildScale);
			FStaticMeshOperations::ApplyTransform(*MeshDescription, InvScaleTransform, true);
		}

		StaticMesh->CommitMeshDescription(UseLODIndex);
		StaticMesh->PostEditChange();
	}
}




bool FStaticMeshComponentSelectorFactory::CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	if (TargetIdentifier.TargetType == FGeometryIdentifier::ETargetType::PrimitiveComponent)
	{
		UStaticMeshComponent* Component = TargetIdentifier.GetAsComponentType<UStaticMeshComponent>();
		if (Component != nullptr )
		{
			// Ensure that the Component is not an InstancedStaticMeshComponent.
			// The StaticMeshSelector does not properly support that component.
			if (Cast<UInstancedStaticMeshComponent>(Component))
			{
				return false;
			}

			// ensure that we have a static mesh and it's not a built-in Engine mesh
			UStaticMesh* StaticMesh = Component->GetStaticMesh();
			if (StaticMesh == nullptr ||
				StaticMesh->GetPathName().StartsWith(TEXT("/Engine/")))
			{
				return false;
			}

			// ensure that this is not a cooked static mesh
			if (StaticMesh->GetOutermost()->bIsCookedForEditor)
			{
				return false;
			}

			return true;
		}
	}
	return false;
}

TUniquePtr<IGeometrySelector> FStaticMeshComponentSelectorFactory::BuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	UStaticMeshComponent* Component = TargetIdentifier.GetAsComponentType<UStaticMeshComponent>();
	check(Component != nullptr);

	TUniquePtr<FStaticMeshSelector> Selector = MakeUnique<FStaticMeshSelector>();
	bool bInitialized = Selector->Initialize(TargetIdentifier);
	return (bInitialized) ? MoveTemp(Selector) : TUniquePtr<IGeometrySelector>();
}


#undef LOCTEXT_NAMESPACE 