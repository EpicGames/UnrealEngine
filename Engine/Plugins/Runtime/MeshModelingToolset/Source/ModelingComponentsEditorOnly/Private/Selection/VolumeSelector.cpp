// Copyright Epic Games, Inc. All Rights Reserved.


#include "Selection/VolumeSelector.h"
#include "Selection/GeometrySelector.h"
#include "Selections/GeometrySelection.h"
#include "Selections/GeometrySelectionUtil.h"

#include "DynamicMesh/MeshIndexUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Changes/MeshVertexChange.h"

#include "GameFramework/Volume.h"
#include "Components/BrushComponent.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "ConversionUtils/DynamicMeshToVolume.h"

#include "Selection/DynamicMeshPolygroupTransformer.h"

#include "Editor.h"		// for GEditor

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "FVolumeSelector"

static TAutoConsoleVariable<bool> CVarEnableModelingSelectionVolumeLocking(
	TEXT("modeling.Selection.EnableVolumeLocking"),
	true,
	TEXT("Control whether Selection Locking is enabled by default for Volumes"));


namespace UEGlobal
{
	/**
	 * Only Unlocked Brush Components can be edited. This is currently implemented as a global set based on UObject pointers.
	 * Obviously not an ideal solution, however this is really only used to implement a UI gate, if we get a stale pointer
	 * here that happens to still point to a valid UBrushComponent, it just means that Volume will be unlocked for mesh selection
	 * even if the user did not explicitly unlock it. This is not a disaster.
	 */
	TSet<UBrushComponent*> UnlockedBrushComponents;
}


bool FVolumeSelector::IsLockable() const
{
	return CVarEnableModelingSelectionVolumeLocking.GetValueOnGameThread();
}


bool FVolumeSelector::IsLocked() const
{
	if (CVarEnableModelingSelectionVolumeLocking.GetValueOnGameThread())
	{
		return BrushComponent != nullptr && (UEGlobal::UnlockedBrushComponents.Contains(BrushComponent) == false);
	}
	else
	{
		return (BrushComponent == nullptr);
	}
}


void FVolumeSelector::SetLockedState(bool bLocked)
{
	if (BrushComponent != nullptr)
	{
		if (bLocked)
		{
			UEGlobal::UnlockedBrushComponents.Remove(BrushComponent);
		}
		else
		{
			UEGlobal::UnlockedBrushComponents.Add(BrushComponent);
			if (IsLocked() == false)
			{
				UpdateDynamicMeshFromVolume();
			}
		}
	}
}

void FVolumeSelector::SetComponentUnlockedOnCreation(UBrushComponent* BrushComponent)
{
	if (BrushComponent != nullptr)
	{
		UEGlobal::UnlockedBrushComponents.Add(BrushComponent);
	}
}

void FVolumeSelector::ResetUnlockedBrushComponents()
{
	UEGlobal::UnlockedBrushComponents.Reset();
}


bool FVolumeSelector::Initialize(
	FGeometryIdentifier SourceGeometryIdentifierIn)
{
	if (SourceGeometryIdentifierIn.TargetType != FGeometryIdentifier::ETargetType::PrimitiveComponent ||
		SourceGeometryIdentifierIn.ObjectType != FGeometryIdentifier::EObjectType::BrushComponent)
	{
		return false;
	}
	BrushComponent = Cast<UBrushComponent>(SourceGeometryIdentifierIn.TargetObject);
	if (BrushComponent == nullptr)
	{
		return false;
	}
	ParentVolume = Cast<AVolume>(BrushComponent->GetOwner());
	if (ParentVolume == nullptr)
	{
		return false;
	}
	
	LocalTargetMesh.Reset( NewObject<UDynamicMesh>() );
	UpdateDynamicMeshFromVolume();

	FBaseDynamicMeshSelector::Initialize(SourceGeometryIdentifierIn, LocalTargetMesh.Get(), 
		[this]() { return IsValid(this->BrushComponent) ? (UE::Geometry::FTransformSRT3d)this->BrushComponent->GetComponentTransform() : FTransformSRT3d::Identity(); });

	// AVolume and BrushComponent seem to have no way of signalling when they are modified,
	// so on undo and redo, we have no idea that the mesh has changed. The brute-force way
	// to work around this is simply to rebuild the mesh on every undo/redo event. 
	// This is expensive and may have other negative reprecussions...
	if (GEditor != nullptr)
	{
		GEditor->RegisterForUndo(this);
	}

	return true;
}


void FVolumeSelector::Shutdown()
{
	LocalTargetMesh.Reset();
	FBaseDynamicMeshSelector::Shutdown();

	if (GEditor != nullptr)
	{
		GEditor->UnregisterForUndo(this);
	}
}


void FVolumeSelector::PostUndo(bool bSuccess)
{
	// see comment in Initialize() for details
	if (IsLocked() == false)
	{
		UpdateDynamicMeshFromVolume();
	}
}
void FVolumeSelector::PostRedo(bool bSuccess)
{
	// see comment in Initialize() for details
	if (IsLocked() == false)
	{
		UpdateDynamicMeshFromVolume();
	}
}


void FVolumeSelector::UpdateDynamicMeshFromVolume()
{
	LocalTargetMesh->EditMesh([&](FDynamicMesh3& EditMesh)
	{
		UE::Conversion::FVolumeToMeshOptions ConvertOptions;
		UE::Conversion::BrushComponentToDynamicMesh(BrushComponent, EditMesh, ConvertOptions);
	});
}


IGeometrySelectionTransformer* FVolumeSelector::InitializeTransformation(const FGeometrySelection& Selection)
{
	check(!ActiveTransformer);

	if (Selection.TopologyType == EGeometryTopologyType::Polygroup)
	{
		ActiveTransformer = MakeShared<FDynamicMeshPolygroupTransformer>();
	}
	else
	{
		ActiveTransformer = MakeShared<FBasicDynamicMeshSelectionTransformer>();
	}
	ActiveTransformer->bEnableSelectionTransformDrawing = true;

	ActiveTransformer->Initialize(this);
	ActiveTransformer->OnEndTransformFunc = [this](IToolsContextTransactionsAPI*) 
	{ 
		CommitMeshTransform();
	};
	return ActiveTransformer.Get();
}

void FVolumeSelector::ShutdownTransformation(IGeometrySelectionTransformer* Transformer)
{
	ActiveTransformer.Reset();
}



void FVolumeSelector::UpdateAfterGeometryEdit(
	IToolsContextTransactionsAPI* TransactionsAPI,
	bool bInTransaction,
	TUniquePtr<FDynamicMeshChange> DynamicMeshChange,
	FText GeometryEditTransactionString)
{
	if (!bInTransaction)
	{
		TransactionsAPI->BeginUndoTransaction(GeometryEditTransactionString);
	}

	CommitMeshTransform();

	if (!bInTransaction)
	{
		TransactionsAPI->EndUndoTransaction();
	}
}




void FVolumeSelector::CommitMeshTransform()
{
	FTransform SetTransform = ParentVolume->GetActorTransform();

	ParentVolume->Modify(true);
	BrushComponent->Modify();

	UE::Conversion::FMeshToVolumeOptions ConvertOptions;
	FDynamicMesh3 TmpMesh;
	TargetMesh->ProcessMesh([&](const UE::Geometry::FDynamicMesh3& SourceMesh)
	{
		TmpMesh = SourceMesh;
	});

	// This conversion will (somehow) mess w/ the transform on the Component/Volume? What seems like
	// is happening is that the mesh is interpreted as being in world-space, and the Actor transform recomputed,
	// so basically it moves to the Origin. So we need to re-set the previous Transform afterwards.
	UE::Conversion::DynamicMeshToVolume(TmpMesh, ParentVolume, ConvertOptions);
	ParentVolume->SetActorTransform(SetTransform);

	ParentVolume->PostEditChange();
}




bool FBrushComponentSelectorFactory::CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	if (TargetIdentifier.TargetType == FGeometryIdentifier::ETargetType::PrimitiveComponent)
	{
		UBrushComponent* Component = TargetIdentifier.GetAsComponentType<UBrushComponent>();
		if (Component && Cast<AVolume>(Component->GetOwner()) != nullptr)
		{
			return true;
		}
	}
	return false;
}

TUniquePtr<IGeometrySelector> FBrushComponentSelectorFactory::BuildForTarget(FGeometryIdentifier TargetIdentifier) const
{
	UBrushComponent* Component = TargetIdentifier.GetAsComponentType<UBrushComponent>();
	check(Component != nullptr);

	TUniquePtr<FVolumeSelector> Selector = MakeUnique<FVolumeSelector>();
	bool bInitialized = Selector->Initialize(TargetIdentifier);
	return (bInitialized) ? MoveTemp(Selector) : TUniquePtr<IGeometrySelector>();
}


#undef LOCTEXT_NAMESPACE 