// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/DynamicMeshComponentToolTarget.h"

#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "StaticMeshAttributes.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ModelingToolTargetUtil.h"

#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "SkeletalMeshModifiers/Public/SkeletonModifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshComponentToolTarget)


#define LOCTEXT_NAMESPACE "UDynamicMeshComponentToolTarget"

namespace DynamicMeshComponentToolTargetLocals
{
	const FText CommitMeshTransactionName = LOCTEXT("DynamicMeshComponentToolTargetCommit", "Update Mesh");
}

bool UDynamicMeshComponentToolTarget::IsValid() const
{
	if (!UPrimitiveComponentToolTarget::IsValid())
	{
		return false;
	}
	UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component);
	if (DynamicMeshComponent == nullptr)
	{
		return false;
	}
	UDynamicMesh* DynamicMesh = DynamicMeshComponent->GetDynamicMesh();
	if (DynamicMesh == nullptr)
	{
		return false;
	}

	return true;
}


int32 UDynamicMeshComponentToolTarget::GetNumMaterials() const
{
	return ensure(IsValid()) ? Component->GetNumMaterials() : 0;
}

UMaterialInterface* UDynamicMeshComponentToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return ensure(IsValid()) ? Component->GetMaterial(MaterialIndex) : nullptr;
}

void UDynamicMeshComponentToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;

	int32 NumMaterials = Component->GetNumMaterials();
	MaterialSetOut.Materials.SetNum(NumMaterials);
	for (int32 k = 0; k < NumMaterials; ++k)
	{
		MaterialSetOut.Materials[k] = Component->GetMaterial(k);
	}
}

bool UDynamicMeshComponentToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!ensure(IsValid())) return false;

	// filter out any Engine materials that we don't want to be permanently assigning
	TArray<UMaterialInterface*> FilteredMaterials = MaterialSet.Materials;
	for (int32 k = 0; k < FilteredMaterials.Num(); ++k)
	{
		FString AssetPath = FilteredMaterials[k]->GetPathName();
		if (AssetPath.StartsWith(TEXT("/MeshModelingToolsetExp/")))
		{
			FilteredMaterials[k] = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component);

	int32 NumMaterialsNeeded = Component->GetNumMaterials();
	int32 NumMaterialsGiven = FilteredMaterials.Num();

	DynamicMeshComponent->Modify();
	for (int32 k = 0; k < NumMaterialsGiven; ++k)
	{
		DynamicMeshComponent->SetMaterial(k, FilteredMaterials[k]);
	}

	return true;
}


const FMeshDescription* UDynamicMeshComponentToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()))
	{
		ensure(GetMeshParams.bHaveRequestLOD == false);	// not supported yet, just returning default LOD

		if (bHaveMeshDescription)
		{
			return ConvertedMeshDescription.Get();
		}

		UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component);

		ConvertedMeshDescription = MakeUnique<FMeshDescription>();
		FStaticMeshAttributes Attributes(*ConvertedMeshDescription);
		Attributes.Register();

		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(DynamicMeshComponent->GetMesh(), *ConvertedMeshDescription, true);

		bHaveMeshDescription = true;
		return ConvertedMeshDescription.Get();
	}
	return nullptr;
}

FMeshDescription UDynamicMeshComponentToolTarget::GetEmptyMeshDescription()
{
	// We use StaticMeshAttributes here because they are the standard used across the engine
	// with regard to setting up FMeshDescriptions in the majority of cases.

	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes Attributes(EmptyMeshDescription);
	Attributes.Register();
	return EmptyMeshDescription;
}

void UDynamicMeshComponentToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams)
{
	if (ensure(IsValid()) == false) return;

	ensure(CommitMeshParams.bHaveTargetLOD == false);		// not supporting this yet

	UDynamicMesh* DynamicMesh = GetDynamicMeshContainer();
	TUniquePtr<FDynamicMesh3> CurrentMesh = DynamicMesh->ExtractMesh();
	TSharedPtr<FDynamicMesh3> CurrentMeshShared(CurrentMesh.Release());

	FMeshDescription EditingMeshDescription(*GetMeshDescription());
	FCommitterParams CommitterParams;
	CommitterParams.MeshDescriptionOut = &EditingMeshDescription;
	Committer(CommitterParams);
	FMeshDescriptionToDynamicMesh Converter;
	TSharedPtr<FDynamicMesh3> NewMeshShared = MakeShared<FDynamicMesh3>();
	NewMeshShared->EnableAttributes();
	Converter.Convert(CommitterParams.MeshDescriptionOut, *NewMeshShared, true);

	DynamicMesh->EditMesh([&](FDynamicMesh3& EditMesh) { EditMesh = *NewMeshShared; });

	TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(CurrentMeshShared, NewMeshShared);

	CommitDynamicMeshChange(MoveTemp(ReplaceChange), DynamicMeshComponentToolTargetLocals::CommitMeshTransactionName);
}


UDynamicMesh* UDynamicMeshComponentToolTarget::GetDynamicMeshContainer()
{
	UDynamicMeshComponent* DynamicMeshComponent = Cast<UDynamicMeshComponent>(Component);
	return DynamicMeshComponent->GetDynamicMesh();
}

bool UDynamicMeshComponentToolTarget::HasDynamicMeshComponent() const
{
	return true;
}

UDynamicMeshComponent* UDynamicMeshComponentToolTarget::GetDynamicMeshComponent()
{
	return Cast<UDynamicMeshComponent>(Component);
}



void UDynamicMeshComponentToolTarget::CommitDynamicMeshChange(TUniquePtr<FToolCommandChange> Change, const FText& ChangeMessage)
{
	FScopedTransaction Transaction(ChangeMessage);
	check(GUndo != nullptr);
	UDynamicMesh* DynamicMesh = GetDynamicMeshContainer();
	DynamicMesh->Modify();
	GUndo->StoreUndo(DynamicMesh, MoveTemp(Change));

	// Invalidate any cached MeshDescription. 
	if (bHaveMeshDescription)
	{
		ConvertedMeshDescription = nullptr;
		bHaveMeshDescription = false;
	}
}

FDynamicMesh3 UDynamicMeshComponentToolTarget::GetDynamicMesh()
{
	UDynamicMesh* DynamicMesh = GetDynamicMeshContainer();
	FDynamicMesh3 Mesh;
	DynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { Mesh = ReadMesh; });
	return Mesh;
}

void UDynamicMeshComponentToolTarget::CommitDynamicMesh(const FDynamicMesh3& UpdatedMesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	UE::ToolTarget::Internal::CommitDynamicMeshViaIPersistentDynamicMeshSource(
		*this, UpdatedMesh, CommitInfo.bTopologyChanged);
}

UBodySetup* UDynamicMeshComponentToolTarget::GetBodySetup() const
{
	return IsValid() ? Cast<UDynamicMeshComponent>(Component)->GetBodySetup() : nullptr;
}


IInterface_CollisionDataProvider* UDynamicMeshComponentToolTarget::GetComplexCollisionProvider() const
{
	return IsValid() ? Cast<UDynamicMeshComponent>(Component) : nullptr;
}

FReferenceSkeleton UDynamicMeshComponentToolTarget::GetSkeleton() const
{
	if (!IsValid())
	{
		return FReferenceSkeleton();
	}
	
	const FDynamicMesh3& EditingMesh = *(Cast<UDynamicMeshComponent>(Component)->GetMesh());
	
	FReferenceSkeleton RefSkeleton;

	{
		FReferenceSkeletonModifier RefSkeletonModifier(RefSkeleton, nullptr);

		using namespace UE::Geometry;
		const FDynamicMeshBoneNameAttribute* BoneNameAttrib = EditingMesh.Attributes()->GetBoneNames();
		const FDynamicMeshBoneParentIndexAttribute* BoneParentIndexAttrib = EditingMesh.Attributes()->GetBoneParentIndices();
		const FDynamicMeshBonePoseAttribute* BonePoses = EditingMesh.Attributes()->GetBonePoses();
		const int32 NumBones = EditingMesh.Attributes()->GetNumBones();
		
		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const int32 ParentBoneIndex = BoneParentIndexAttrib->GetValue(BoneIndex);
			FName BoneName = BoneNameAttrib->GetValue(BoneIndex);
			FMeshBoneInfo BoneInfo(BoneName, BoneName.ToString(), ParentBoneIndex);

			FTransform Transform = BonePoses->GetValue(BoneIndex);
			RefSkeletonModifier.Add(BoneInfo, Transform);
		}
	}
	
	return RefSkeleton;
}

void UDynamicMeshComponentToolTarget::SetupSkeletonModifier(USkeletonModifier* InModifier)
{
	InModifier->SetDynamicMesh(GetDynamicMeshContainer());
}

void UDynamicMeshComponentToolTarget::CommitSkeletonModifier(USkeletonModifier* InModifier)
{
	InModifier->CommitSkeletonToDynamicMesh();
}


// Factory

bool UDynamicMeshComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	UDynamicMeshComponent* Component = Cast<UDynamicMeshComponent>(SourceObject);
	return Component 
		&& IsValidChecked(Component)
		&& !Component->IsUnreachable() 
		&& Component->IsValidLowLevel() 
		&& Component->GetDynamicMesh()
		&& Component->IsEditable()
		&& Requirements.AreSatisfiedBy(UDynamicMeshComponentToolTarget::StaticClass());
}

UToolTarget* UDynamicMeshComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UDynamicMeshComponentToolTarget* Target = NewObject<UDynamicMeshComponentToolTarget>();// TODO: Should we set an outer here?
	Target->InitializeComponent(Cast<UDynamicMeshComponent>(SourceObject));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}


#undef LOCTEXT_NAMESPACE
