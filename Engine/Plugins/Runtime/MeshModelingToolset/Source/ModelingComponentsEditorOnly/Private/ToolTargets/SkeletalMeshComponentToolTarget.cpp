// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/SkeletalMeshComponentToolTarget.h"

#include "Components/SkeletalMeshComponent.h"
#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ToolTargets/SkeletalMeshToolTarget.h"
#include "SkeletalMeshAttributes.h"
#include "Framework/Notifications/NotificationManager.h"
#include "UObject/Package.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SkeletalMeshModifiers/Public/SkeletonModifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshComponentToolTarget)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "SkeletalMeshComponentToolTarget"

namespace SkeletalMeshComponentToolTargetLocals
{
static void DisplayCriticalWarningMessage(const FText& InMessage)
{
	FNotificationInfo Info(InMessage);
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info);

	UE_LOGF(LogGeometry, Warning, "%ls", *InMessage.ToString());
}
}



//
// USkeletalMeshComponentReadOnlyToolTarget
//

bool USkeletalMeshComponentReadOnlyToolTarget::IsValid() const
{
	if (!UPrimitiveComponentToolTarget::IsValid())
	{
		return false;
	}
	USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Component);
	if (SkeletalMeshComponent == nullptr)
	{
		return false;
	}
	const USkeletalMesh* SkeletalMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();

	return SkeletalMesh != nullptr;
}

int32 USkeletalMeshComponentReadOnlyToolTarget::GetNumMaterials() const
{
	return ensure(IsValid()) ? Component->GetNumMaterials() : 0;
}

UMaterialInterface* USkeletalMeshComponentReadOnlyToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return ensure(IsValid()) ? Component->GetMaterial(MaterialIndex) : nullptr;
}

void USkeletalMeshComponentReadOnlyToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;

	if (bPreferAssetMaterials)
	{
		const USkeletalMesh* SkeletalMesh = Cast<USkeletalMeshComponent>(Component)->GetSkeletalMeshAsset();
		USkeletalMeshToolTarget::GetMaterialSet(SkeletalMesh, MaterialSetOut, bPreferAssetMaterials);
	}
	else
	{
		int32 NumMaterials = Component->GetNumMaterials();
		MaterialSetOut.Materials.SetNum(NumMaterials);
		for (int32 k = 0; k < NumMaterials; ++k)
		{
			MaterialSetOut.Materials[k] = Component->GetMaterial(k);
		}
	}
}

bool USkeletalMeshComponentReadOnlyToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!ensure(IsValid())) return false;

	if (bApplyToAsset)
	{
		USkeletalMesh* SkeletalMesh = Cast<USkeletalMeshComponent>(Component)->GetSkeletalMeshAsset();

		// unregister the component while we update it's static mesh
		TUniquePtr<FComponentReregisterContext> ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component.Get());

		return USkeletalMeshToolTarget::CommitMaterialSetUpdate(SkeletalMesh, MaterialSet, bApplyToAsset);
	}
	else
	{
		const int32 NumMaterialsNeeded = Component->GetNumMaterials();
		const int32 NumMaterialsGiven = MaterialSet.Materials.Num();

		// We wrote the below code to support a mismatch in the number of materials.
		// However, it is not yet clear whether this might be desirable, and we don't
		// want to inadvertently hide bugs in the meantime. So, we keep this ensure here
		// for now, and we can remove it if we decide that we want the ability.
		ensure(NumMaterialsNeeded == NumMaterialsGiven);

		check(NumMaterialsGiven > 0);

		for (int32 i = 0; i < NumMaterialsNeeded; ++i)
		{
			const int32 MaterialToUseIndex = FMath::Min(i, NumMaterialsGiven - 1);
			Component->SetMaterial(i, MaterialSet.Materials[MaterialToUseIndex]);
		}
	}

	return true;
}

const FMeshDescription* USkeletalMeshComponentReadOnlyToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()))
	{
		const USkeletalMesh* SkeletalMesh = GetSkeletalMesh();
		EMeshLODIdentifier UseLOD = GetMeshParams.bHaveRequestLOD ? GetMeshParams.RequestLOD : EditingLOD;
		const int32 LODIndex = USkeletalMeshReadOnlyToolTarget::GetValidEditingLOD(SkeletalMesh, UseLOD);
	
		return SkeletalMesh->GetMeshDescription(LODIndex);
	}
	
	return nullptr;
}

FMeshDescription USkeletalMeshComponentReadOnlyToolTarget::GetEmptyMeshDescription()
{
	FMeshDescription EmptyMeshDescription;
	FSkeletalMeshAttributes MeshAttributes(EmptyMeshDescription);
	MeshAttributes.Register();

	return EmptyMeshDescription;
}

TArray<EMeshLODIdentifier> USkeletalMeshComponentReadOnlyToolTarget::GetAvailableLODs(bool bSkipAutoGenerated) const
{
	return USkeletalMeshReadOnlyToolTarget::GetAvailableLODs(GetSkeletalMesh(), bSkipAutoGenerated);
}

FDynamicMesh3 USkeletalMeshComponentReadOnlyToolTarget::GetDynamicMesh()
{
	return GetDynamicMeshViaMeshDescription(*this);
}

FDynamicMesh3 USkeletalMeshComponentReadOnlyToolTarget::GetDynamicMesh(const FGetMeshParameters& InGetMeshParams)
{
	return GetDynamicMeshViaMeshDescription(*this, InGetMeshParams);
}


USkeletalMesh* USkeletalMeshComponentReadOnlyToolTarget::GetSkeletalMesh() const
{
	return IsValid() ? Cast<USkeletalMeshComponent>(Component)->GetSkeletalMeshAsset() : nullptr;
}

FReferenceSkeleton USkeletalMeshComponentReadOnlyToolTarget::GetSkeleton() const
{
	return IsValid() ? GetSkeletalMesh()->GetRefSkeleton() : FReferenceSkeleton();
}


//
// USkeletalMeshComponentToolTarget
//


void USkeletalMeshComponentToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams)
{
	if (ensure(IsValid()))
	{
		USkeletalMesh* SkeletalMesh = GetSkeletalMesh();
		EMeshLODIdentifier UseLOD = CommitMeshParams.bHaveTargetLOD ? CommitMeshParams.TargetLOD : EditingLOD;
		const int32 LODIndex = USkeletalMeshReadOnlyToolTarget::GetValidEditingLOD(SkeletalMesh, UseLOD);

		// Unregister the component while we update its skeletal mesh
		FComponentReregisterContext ComponentReregisterContext(Component.Get());

		USkeletalMeshToolTarget::CommitMeshDescription(SkeletalMesh, Committer, LODIndex);

		// this rebuilds physics, but it doesn't undo!
		Component->RecreatePhysicsState();
	}
}


void USkeletalMeshComponentToolTarget::CommitDynamicMesh(const FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	const FMeshDescription* CurrentMeshDescription = GetMeshDescription();
	if (ensureMsgf(CurrentMeshDescription, TEXT("Unable to commit mesh, perhaps the user deleted "
		"the asset while the tool was active?")))
	{
		CommitDynamicMeshViaMeshDescription(FMeshDescription(*CurrentMeshDescription), *this, Mesh, CommitInfo);
	}
}

void USkeletalMeshComponentToolTarget::SetupSkeletonModifier(USkeletonModifier* InModifier)
{
	InModifier->SetSkeletalMesh(GetSkeletalMesh());
}

void USkeletalMeshComponentToolTarget::CommitSkeletonModifier(USkeletonModifier* InModifier)
{
	InModifier->CommitSkeletonToSkeletalMesh();
}


// Factory

bool USkeletalMeshComponentReadOnlyToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of USkeletalMeshComponent,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)

	return Cast<USkeletalMeshComponent>(SourceObject) && Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset() &&
		ExactCast<USkeletalMesh>(Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset()) &&
		!ExactCast<USkeletalMesh>(Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset())->GetOutermost()->bIsCookedForEditor &&
		Requirements.AreSatisfiedBy(USkeletalMeshComponentReadOnlyToolTarget::StaticClass());
}

UToolTarget* USkeletalMeshComponentReadOnlyToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	USkeletalMeshComponentReadOnlyToolTarget* Target = NewObject<USkeletalMeshComponentReadOnlyToolTarget>();
	Target->InitializeComponent(Cast<USkeletalMeshComponent>(SourceObject));
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}

bool USkeletalMeshComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMeshComponent, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of USkeletalMeshComponent,
	// just add another factory that allows that class specifically(but make sure that
	// GetMeshDescription and such work properly)
	
	bool bValid = Cast<USkeletalMeshComponent>(SourceObject) && Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset() &&
		ExactCast<USkeletalMesh>(Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset()) &&
		!ExactCast<USkeletalMesh>(Cast<USkeletalMeshComponent>(SourceObject)->GetSkeletalMeshAsset())->GetOutermost()->bIsCookedForEditor;
	if (!bValid)
	{
		return false;
	}

	if (CanWriteToSource(SourceObject))
	{
		return Requirements.AreSatisfiedBy(USkeletalMeshComponentToolTarget::StaticClass());
	}
	else
	{
		return Requirements.AreSatisfiedBy(USkeletalMeshComponentReadOnlyToolTarget::StaticClass());
	}
}

UToolTarget* USkeletalMeshComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	USkeletalMeshComponentReadOnlyToolTarget* Target;
	if (CanWriteToSource(SourceObject))
	{
		Target = NewObject<USkeletalMeshComponentToolTarget>();
	}
	else
	{
		Target = NewObject<USkeletalMeshComponentReadOnlyToolTarget>();
	}	
	Target->InitializeComponent(Cast<USkeletalMeshComponent>(SourceObject));
	Target->EditingLOD = EditingLOD;
	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}

bool USkeletalMeshComponentToolTargetFactory::CanWriteToSource(const UObject* SourceObject)
{
	if (const USkeletalMeshComponent* Component = GetValid(Cast<USkeletalMeshComponent>(SourceObject)))
	{
		if (const USkeletalMesh* SkeletalMesh = GetValid(Cast<USkeletalMesh>(Component->GetSkeletalMeshAsset())))
		{
			return !SkeletalMesh->GetPathName().StartsWith(TEXT("/Engine/"));
		}
	}
	return false;
}

void USkeletalMeshComponentToolTargetFactory::SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD)
{
	EditingLOD = NewEditingLOD;
}

#undef LOCTEXT_NAMESPACE
