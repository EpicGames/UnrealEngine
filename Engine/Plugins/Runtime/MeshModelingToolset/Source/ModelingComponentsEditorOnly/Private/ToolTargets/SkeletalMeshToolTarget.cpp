// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/SkeletalMeshToolTarget.h"

#include "LODUtilities.h"
#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ModelingToolTargetEditorOnlyUtil.h"
#include "ModelingToolTargetUtil.h"
#include "SkeletalMeshAttributes.h"
#include "Misc/SlowTask.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshToolTarget)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "SkeletalMeshToolTarget"


namespace SkeletalMeshToolTargetLocals
{
static void DisplayCriticalWarningMessage(const FText& InMessage)
{
	FNotificationInfo Info(InMessage);
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info);

	UE_LOGF(LogGeometry, Warning, "%ls", *InMessage.ToString());
}
}


//
// USkeletalMeshReadOnlyToolTarget
//


bool USkeletalMeshReadOnlyToolTarget::IsValid() const
{
	return SkeletalMesh.IsValid();
}

bool USkeletalMeshReadOnlyToolTarget::IsValid(const USkeletalMesh* SkeletalMeshIn)
{
	if (!SkeletalMeshIn || !IsValidChecked(SkeletalMeshIn) || SkeletalMeshIn->IsUnreachable() || !SkeletalMeshIn->IsValidLowLevel())
	{
		return false;
	}

	return true;
}

TArray<EMeshLODIdentifier> USkeletalMeshReadOnlyToolTarget::GetAvailableLODs(const USkeletalMesh* SkeletalMesh, bool bSkipAutoGenerated)
{
	return UE::ToolTarget::GetAvailableLODs(SkeletalMesh, bSkipAutoGenerated);
}

int32 USkeletalMeshReadOnlyToolTarget::GetNumMaterials() const
{
	return ensure(IsValid()) ? SkeletalMesh->GetMaterials().Num() : 0;
}

UMaterialInterface* USkeletalMeshReadOnlyToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return ensure(IsValid() && MaterialIndex < SkeletalMesh->GetMaterials().Num()) ? 
		SkeletalMesh->GetMaterials()[MaterialIndex].MaterialInterface : nullptr;
}

void USkeletalMeshReadOnlyToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;
	GetMaterialSet(SkeletalMesh.Get(), MaterialSetOut, bPreferAssetMaterials);
}

void USkeletalMeshReadOnlyToolTarget::GetMaterialSet(const USkeletalMesh* SkeletalMeshIn, FComponentMaterialSet& MaterialSetOut,
	bool bPreferAssetMaterials)
{
	const TArray<FSkeletalMaterial>& Materials = SkeletalMeshIn->GetMaterials(); 
	MaterialSetOut.Materials.SetNum(Materials.Num());
	for (int32 k = 0; k < Materials.Num(); ++k)
	{
		MaterialSetOut.Materials[k] = Materials[k].MaterialInterface;
	}
}

bool USkeletalMeshReadOnlyToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!ensure(IsValid())) return false;
	return CommitMaterialSetUpdate(SkeletalMesh.Get(), MaterialSet, bApplyToAsset);
}

bool USkeletalMeshReadOnlyToolTarget::CommitMaterialSetUpdate(USkeletalMesh* SkeletalMeshIn, 
	const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!bApplyToAsset)
	{
		return false;
	}

	if (SkeletalMeshIn->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		UE_LOGF(LogTemp, Warning, "CANNOT MODIFY BUILT-IN ENGINE ASSET %ls", *SkeletalMeshIn->GetPathName());
		return false;
	}

	// flush any pending rendering commands, which might touch a component while we are rebuilding its mesh
	FlushRenderingCommands();

	// make sure transactional flag is on
	SkeletalMeshIn->SetFlags(RF_Transactional);

	SkeletalMeshIn->Modify();

	const int NewNumMaterials = MaterialSet.Materials.Num();
	TArray<FSkeletalMaterial> &SkeletalMaterials = SkeletalMeshIn->GetMaterials(); 
	if (NewNumMaterials != SkeletalMaterials.Num())
	{
		SkeletalMaterials.SetNum(NewNumMaterials);
	}
	for (int k = 0; k < NewNumMaterials; ++k)
	{
		if (SkeletalMaterials[k].MaterialInterface != MaterialSet.Materials[k])
		{
			SkeletalMaterials[k].MaterialInterface = MaterialSet.Materials[k];
			if (SkeletalMaterials[k].MaterialSlotName.IsNone())
			{
				SkeletalMaterials[k].MaterialSlotName = MaterialSet.Materials[k]->GetFName();
			}
		}
	}

	UE::ToolTarget::Internal::PostEditChangeWithConditionalUndo(SkeletalMeshIn);

	return true;
}

int32 USkeletalMeshReadOnlyToolTarget::GetValidEditingLOD(const USkeletalMesh* SkeletalMesh, EMeshLODIdentifier RequestedEditingLOD)
{
	using namespace SkeletalMeshToolTargetLocals;

	int32 ValidEditingLOD = 0;

	if (RequestedEditingLOD == EMeshLODIdentifier::MaxQuality)
	{
		ValidEditingLOD = 0;
	}
	else if (RequestedEditingLOD == EMeshLODIdentifier::HiResSource)
	{
		DisplayCriticalWarningMessage(LOCTEXT("SkelmeshHiresLODUnavailable", "HiRes LOD not available on skeletal meshes. Falling back to using LOD 0."));
	}
	else if (RequestedEditingLOD != EMeshLODIdentifier::Default)
	{
		ValidEditingLOD = static_cast<int32>(RequestedEditingLOD);
	}

	
	const int32 MaxLOD = SkeletalMesh->GetLODNum() - 1;
	if (ValidEditingLOD > MaxLOD)
	{
		DisplayCriticalWarningMessage(FText::Format(LOCTEXT("SkelmeshLODUnavailable", "LOD {0} requested but is not available. Falling back to LOD {1}."), FText::AsNumber(ValidEditingLOD), FText::AsNumber(MaxLOD)));
		ValidEditingLOD = MaxLOD;
	}

	return ValidEditingLOD;
}

const FMeshDescription* USkeletalMeshReadOnlyToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()))
	{
		EMeshLODIdentifier UseLOD = GetMeshParams.bHaveRequestLOD ? GetMeshParams.RequestLOD : EditingLOD;
		const int32 LODIndex = GetValidEditingLOD(SkeletalMesh.Get(),UseLOD);
	
		return SkeletalMesh->GetMeshDescription(LODIndex);
	}

	return nullptr;
}


FMeshDescription USkeletalMeshReadOnlyToolTarget::GetEmptyMeshDescription()
{
	FMeshDescription EmptyMeshDescription;
	FSkeletalMeshAttributes MeshAttributes(EmptyMeshDescription);
	MeshAttributes.Register();

	return EmptyMeshDescription;
}

TArray<EMeshLODIdentifier> USkeletalMeshReadOnlyToolTarget::GetAvailableLODs(bool bSkipAutoGenerated) const
{
	return GetAvailableLODs(SkeletalMesh.Get(), bSkipAutoGenerated);
}

FDynamicMesh3 USkeletalMeshReadOnlyToolTarget::GetDynamicMesh()
{
	return GetDynamicMeshViaMeshDescription(*this);
}

FDynamicMesh3 USkeletalMeshReadOnlyToolTarget::GetDynamicMesh(const FGetMeshParameters& InGetMeshParams)
{
	return GetDynamicMeshViaMeshDescription(*this, InGetMeshParams);
}

USkeletalMesh* USkeletalMeshReadOnlyToolTarget::GetSkeletalMesh() const
{
	return IsValid() ? SkeletalMesh.Get() : nullptr;
}

//
// USkeletalMeshToolTarget
//

void USkeletalMeshToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams)
{
	if (ensure(IsValid()))
	{
		USkeletalMesh* Mesh = GetSkeletalMesh();
		EMeshLODIdentifier UseLOD = CommitMeshParams.bHaveTargetLOD ? CommitMeshParams.TargetLOD : EditingLOD;
		const int32 LODIndex = GetValidEditingLOD(Mesh, UseLOD);
		
		CommitMeshDescription(Mesh, Committer, LODIndex);
	}
}

void USkeletalMeshToolTarget::CommitMeshDescription(USkeletalMesh* SkeletalMesh, const FCommitter& Committer, int32 LODIndex)
{
	using namespace SkeletalMeshToolTargetLocals;

	if (SkeletalMesh->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		DisplayCriticalWarningMessage(FText::Format(LOCTEXT("CannotModifyBuiltInAssetError", "Cannot modify built-in engine asset: {0}"), FText::FromString(*SkeletalMesh->GetPathName())));
		return;
	}

	// This scope will invoke the skeletal mesh render data rebuild machinery on scope exit
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
	// flush any pending rendering commands, which might touch a component while we are rebuilding it's mesh
	FlushRenderingCommands();

	// Ensure the asset is set to transactional, so that undo works proper.
	SkeletalMesh->SetFlags(RF_Transactional);
	SkeletalMesh->Modify();

	// Mark the mesh at this LOD as modified as well, so that we can undo any geometry changes.
	SkeletalMesh->ModifyMeshDescription(LODIndex);
	
	FCommitterParams CommitterParams;
	CommitterParams.MeshDescriptionOut = SkeletalMesh->GetMeshDescription(LODIndex);
	Committer(CommitterParams);

	// Commit the edited mesh description to bulk data.
	SkeletalMesh->CommitMeshDescription(LODIndex);

	// Update the vertex attribute infos now that we have committed the mesh description
	// the actual copying from SourceModel to LODModel happens during build time
	FLODUtilities::UpdateLODInfoVertexAttributes(SkeletalMesh, LODIndex, LODIndex, false);
	
}

void USkeletalMeshToolTarget::CommitDynamicMesh(const FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	FMeshDescription CurrentMeshDescription = *GetMeshDescription();
	CommitDynamicMeshViaMeshDescription(MoveTemp(CurrentMeshDescription), *this, Mesh, CommitInfo);
}


// Factory

bool USkeletalMeshReadOnlyToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMesh, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of USkeletalMesh,
	// just add another factory that allows that class specifically (but make sure that
	// GetMeshDescription and such work properly)

	const USkeletalMesh* SkeletalMesh =  ExactCast<USkeletalMesh>(SourceObject);

	return SkeletalMesh
		&& !SkeletalMesh->GetOutermost()->bIsCookedForEditor
		&& Requirements.AreSatisfiedBy(USkeletalMeshReadOnlyToolTarget::StaticClass());
}

UToolTarget* USkeletalMeshReadOnlyToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	USkeletalMeshReadOnlyToolTarget* Target = NewObject<USkeletalMeshReadOnlyToolTarget>();
	Target->SkeletalMesh = Cast<USkeletalMesh>(SourceObject);
	checkSlow(Target->SkeletalMesh.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}

bool USkeletalMeshToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	// We are using an exact cast here to prevent subclasses, which might not meet all
	// requirements for functionality such as the deprecated DestructibleMesh, from 
	// being caught up as valid targets.
	// If you want to make the tool target work with some subclass of USkeletalMesh,
	// just add another factory that allows that class specifically (but make sure that
	// GetMeshDescription and such work properly)
	
	const USkeletalMesh* SkeletalMesh =  ExactCast<USkeletalMesh>(SourceObject);
	
	bool bValid = SkeletalMesh
		&& !SkeletalMesh->GetOutermost()->bIsCookedForEditor;
	if (!bValid)
	{
		return false;
	}
	
	if (CanWriteToSource(SourceObject))
	{
		return Requirements.AreSatisfiedBy(USkeletalMeshToolTarget::StaticClass());
	}
	else
	{
		return Requirements.AreSatisfiedBy(USkeletalMeshReadOnlyToolTarget::StaticClass());
	}
}

UToolTarget* USkeletalMeshToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	USkeletalMeshReadOnlyToolTarget* Target;
	if (CanWriteToSource(SourceObject))
	{
		Target = NewObject<USkeletalMeshToolTarget>();
	}
	else
	{
		Target = NewObject<USkeletalMeshReadOnlyToolTarget>();
	}
	Target->SkeletalMesh = Cast<USkeletalMesh>(SourceObject);
	Target->EditingLOD = EditingLOD;
	checkSlow(Target->SkeletalMesh.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}

bool USkeletalMeshToolTargetFactory::CanWriteToSource(const UObject* SourceObject)
{
	if (const USkeletalMesh* SkeletalMesh = GetValid(ExactCast<USkeletalMesh>(SourceObject)))
	{
		return !SkeletalMesh->GetPathName().StartsWith(TEXT("/Engine/"));
	}
	return false;
}

void USkeletalMeshToolTargetFactory::SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD)
{
	EditingLOD = NewEditingLOD;
}

#undef LOCTEXT_NAMESPACE

