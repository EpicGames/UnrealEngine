// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/StaticMeshComponentToolTarget.h"

#include "ComponentReregisterContext.h"
#include "Components/StaticMeshComponent.h"
#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/StaticMesh.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ToolTargets/StaticMeshToolTarget.h"
#include "AssetUtils/MeshDescriptionUtil.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshComponentToolTarget)

using namespace UE::Geometry;

void UStaticMeshComponentReadOnlyToolTarget::SetEditingLOD(EMeshLODIdentifier RequestedEditingLOD)
{
	EMeshLODIdentifier ValidEditingLOD = EMeshLODIdentifier::LOD0;

	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (ensure(StaticMeshComponent != nullptr))
	{
		UStaticMesh* StaticMeshAsset = StaticMeshComponent->GetStaticMesh();
		ValidEditingLOD = UStaticMeshReadOnlyToolTarget::GetValidEditingLOD(StaticMeshAsset, RequestedEditingLOD);
	}

	EditingLOD = ValidEditingLOD;
}


bool UStaticMeshComponentReadOnlyToolTarget::IsValid() const
{
	if (!UPrimitiveComponentToolTarget::IsValid())
	{
		return false;
	}
	UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component);
	if (StaticMeshComponent == nullptr)
	{
		return false;
	}
	UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
	return ::IsValid(StaticMesh) && !StaticMesh->IsUnreachable() && StaticMesh->IsValidLowLevel();
}



int32 UStaticMeshComponentReadOnlyToolTarget::GetNumMaterials() const
{
	return ensure(IsValid()) ? Component->GetNumMaterials() : 0;
}

UMaterialInterface* UStaticMeshComponentReadOnlyToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return ensure(IsValid()) ? Component->GetMaterial(MaterialIndex) : nullptr;
}

void UStaticMeshComponentReadOnlyToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;

	if (bPreferAssetMaterials)
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
		UStaticMeshReadOnlyToolTarget::GetMaterialSet(StaticMesh, MaterialSetOut, bPreferAssetMaterials);
	}
	else
	{
		int32 NumMaterials = Component->GetNumMaterials();
		MaterialSetOut.Materials.SetNum(NumMaterials);
		for (int32 k = 0; k < NumMaterials; ++k)
		{
			MaterialSetOut.Materials[k] = Component->GetMaterial(k);
		}
	}
}

bool UStaticMeshComponentReadOnlyToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!ensure(IsValid())) return false;

	if (bApplyToAsset)
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();

		// unregister the component while we update it's static mesh
		TUniquePtr<FComponentReregisterContext> ComponentReregisterContext = MakeUnique<FComponentReregisterContext>(Component.Get());

		return UStaticMeshReadOnlyToolTarget::CommitMaterialSetUpdate(StaticMesh, MaterialSet, bApplyToAsset);
	}
	else
	{
		// filter out any Engine materials that we don't want to be permanently assigning
		TArray<UMaterialInterface*> FilteredMaterials = MaterialSet.Materials;
		for (int32 k = 0; k < FilteredMaterials.Num(); ++k)
		{
			FString AssetPath = FilteredMaterials[k]->GetPathName();
			if (AssetPath.StartsWith(TEXT("/MeshModelingToolsetExp/")))
			{
				FilteredMaterials[k] = UMaterial::GetDefaultMaterial(MD_Surface);
			}
		}

		int32 NumMaterialsNeeded = Component->GetNumMaterials();
		int32 NumMaterialsGiven = FilteredMaterials.Num();

		// We wrote the below code to support a mismatch in the number of materials.
		// However, it is not yet clear whether this might be desirable, and we don't
		// want to inadvertantly hide bugs in the meantime. So, we keep this ensure here
		// for now, and we can remove it if we decide that we want the ability.
		ensure(NumMaterialsNeeded == NumMaterialsGiven);

		check(NumMaterialsGiven > 0);

		for (int32 i = 0; i < NumMaterialsNeeded; ++i)
		{
			int32 MaterialToUseIndex = FMath::Min(i, NumMaterialsGiven - 1);
			Component->SetMaterial(i, FilteredMaterials[MaterialToUseIndex]);
		}
	}

	return true;
}


const FMeshDescription* UStaticMeshComponentReadOnlyToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()))
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
		EMeshLODIdentifier UseLOD = EditingLOD;
		if (StaticMesh && GetMeshParams.bHaveRequestLOD)
		{
			UseLOD = UStaticMeshReadOnlyToolTarget::GetValidEditingLOD(StaticMesh, GetMeshParams.RequestLOD);
			ensure(UseLOD == GetMeshParams.RequestLOD);		// probably a bug somewhere if this is not true
		}

		return UStaticMeshReadOnlyToolTarget::GetMeshDescriptionWithScaleApplied(StaticMesh, (int32)UseLOD, CachedMeshDescriptions);
	}
	return nullptr;
}

TArray<int32> UStaticMeshComponentReadOnlyToolTarget::GetPolygonGroupToMaterialIndexMap() const
{
	if (IsValid())
	{
		UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
		return UStaticMeshReadOnlyToolTarget::MapSectionToMaterialID(StaticMesh, EditingLOD);
	}
	return TArray<int32>();
}

FMeshDescription UStaticMeshComponentReadOnlyToolTarget::GetEmptyMeshDescription()
{
	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes Attributes(EmptyMeshDescription);
	Attributes.Register();
	return EmptyMeshDescription;
}

FMeshDescription UStaticMeshComponentReadOnlyToolTarget::GetMeshDescriptionCopy(const FGetMeshParameters& GetMeshParams)
{
	auto ApplyBuildScaleIfNeeded = [](FMeshDescription& MeshDescription, FVector BuildScale)
	{
		if (!BuildScale.Equals(FVector::OneVector))
		{
			FTransform ScaleTransform = FTransform::Identity;
			ScaleTransform.SetScale3D(BuildScale);
			FStaticMeshOperations::ApplyTransform(MeshDescription, ScaleTransform, true);
		}
	};
	if (ensure(IsValid()))
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh())
		{
			EMeshLODIdentifier UseLOD = EditingLOD;
			if (GetMeshParams.bHaveRequestLOD)
			{
				UseLOD = UStaticMeshReadOnlyToolTarget::GetValidEditingLOD(StaticMesh, GetMeshParams.RequestLOD);
				ensure(UseLOD == GetMeshParams.RequestLOD);		// probably a bug somewhere if this is not true
			}
			if (UseLOD == EMeshLODIdentifier::HiResSource )
			{
				if (StaticMesh->IsHiResMeshDescriptionValid())
				{
					FMeshDescription MeshDescriptionCopy = *StaticMesh->GetHiResMeshDescription();
					const FStaticMeshSourceModel& SourceModel = StaticMesh->GetHiResSourceModel();
					ApplyBuildScaleIfNeeded(MeshDescriptionCopy, SourceModel.BuildSettings.BuildScale3D);
					UE::MeshDescription::InitializeAutoGeneratedAttributes(MeshDescriptionCopy, &SourceModel.BuildSettings);
					return MeshDescriptionCopy;
				}
			}
			else
			{
				if (StaticMesh->IsMeshDescriptionValid(static_cast<int32>(UseLOD)))
				{
					if (FMeshDescription* SourceMesh = StaticMesh->GetMeshDescription(static_cast<int32>(UseLOD)))
					{
						FMeshDescription MeshDescriptionCopy = *SourceMesh;
						ApplyBuildScaleIfNeeded(MeshDescriptionCopy, StaticMesh->GetSourceModel((int32)UseLOD).BuildSettings.BuildScale3D);
						UE::MeshDescription::InitializeAutoGeneratedAttributes(MeshDescriptionCopy, StaticMesh, static_cast<int32>(UseLOD));
						return MeshDescriptionCopy;
					}
				}
			}
		}
	}
	

	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes Attributes(EmptyMeshDescription);
	Attributes.Register();
	return EmptyMeshDescription;
}


TArray<EMeshLODIdentifier> UStaticMeshComponentReadOnlyToolTarget::GetAvailableLODs(bool bSkipAutoGenerated) const
{
	UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();
	return UStaticMeshReadOnlyToolTarget::GetAvailableLODs(StaticMesh, bSkipAutoGenerated);
}


void UStaticMeshComponentToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams)
{
	if (ensure(IsValid()) == false) return;

	UStaticMesh* StaticMesh = Cast<UStaticMeshComponent>(Component)->GetStaticMesh();

	EMeshLODIdentifier WriteToLOD = (CommitParams.bHaveTargetLOD && CommitParams.TargetLOD != EMeshLODIdentifier::Default) ? CommitParams.TargetLOD : EditingLOD;

	// unregister the component while we update its static mesh
	FComponentReregisterContext ComponentReregisterContext(Component.Get());

	UStaticMeshToolTarget::CommitMeshDescription(StaticMesh, Committer, WriteToLOD);

	// this rebuilds physics, but it doesn't undo!
	Component->RecreatePhysicsState();
}

FDynamicMesh3 UStaticMeshComponentReadOnlyToolTarget::GetDynamicMesh()
{
	return GetDynamicMeshViaMeshDescription(*this);
}

FDynamicMesh3 UStaticMeshComponentReadOnlyToolTarget::GetDynamicMesh(const FGetMeshParameters& InGetMeshParams)
{
	return GetDynamicMeshViaMeshDescription(*this, InGetMeshParams);
}

void UStaticMeshComponentToolTarget::CommitDynamicMesh(const FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	const FMeshDescription* CurrentMeshDescription = GetMeshDescription();
	if (ensureMsgf(CurrentMeshDescription, TEXT("Unable to commit mesh, perhaps the user deleted "
		"the asset while the tool was active?")))
	{
		CommitDynamicMeshViaMeshDescription(FMeshDescription(*CurrentMeshDescription), *this, Mesh, CommitInfo);
	}
}

UStaticMesh* UStaticMeshComponentReadOnlyToolTarget::GetStaticMesh() const
{
	return IsValid() ? Cast<UStaticMeshComponent>(Component)->GetStaticMesh() : nullptr;
}


UBodySetup* UStaticMeshComponentReadOnlyToolTarget::GetBodySetup() const
{
	UStaticMesh* StaticMesh = GetStaticMesh();
	if (StaticMesh)
	{
		return StaticMesh->GetBodySetup();
	}
	return nullptr;
}


IInterface_CollisionDataProvider* UStaticMeshComponentReadOnlyToolTarget::GetComplexCollisionProvider() const
{
	UStaticMesh* StaticMesh = GetStaticMesh();
	if (StaticMesh)
	{
		return Cast<IInterface_CollisionDataProvider>(StaticMesh);
	}
	return nullptr;
}


// Factory

const UStaticMesh* UStaticMeshComponentToolTargetFactory::SourceToStaticMesh(const UObject* SourceObject)
{
	const UStaticMeshComponent* Component = GetValid(Cast<UStaticMeshComponent>(SourceObject));
	const UStaticMesh* StaticMesh = (Component && !Component->IsUnreachable() && Component->IsValidLowLevel()) ? Component->GetStaticMesh().Get() : nullptr;
	return StaticMesh;
}

bool UStaticMeshComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	const UStaticMesh* StaticMesh = SourceToStaticMesh(SourceObject);
	bool bValid = StaticMesh
		&& !StaticMesh->GetOutermost()->bIsCookedForEditor
		&& UStaticMeshToolTarget::HasNonGeneratedLOD(StaticMesh, EditingLOD);
	if (!bValid)
	{
		return false;
	}
	if (CanWriteToSource(SourceObject))
	{
		return Requirements.AreSatisfiedBy(UStaticMeshComponentToolTarget::StaticClass());
	}
	else
	{
		return Requirements.AreSatisfiedBy(UStaticMeshComponentReadOnlyToolTarget::StaticClass());
	}
}

UToolTarget* UStaticMeshComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UStaticMeshComponentReadOnlyToolTarget* Target;
	if (CanWriteToSource(SourceObject))
	{
		Target = NewObject<UStaticMeshComponentToolTarget>();
	}
	else
	{
		Target = NewObject<UStaticMeshComponentReadOnlyToolTarget>();
	}
	Target->InitializeComponent(Cast<UStaticMeshComponent>(SourceObject));
	Target->SetEditingLOD(EditingLOD);
	checkSlow(Target->Component.Get() && Requirements.AreSatisfiedBy(Target));

	return Target;
}


void UStaticMeshComponentToolTargetFactory::SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD)
{
	EditingLOD = NewEditingLOD;
}


bool UStaticMeshComponentToolTargetFactory::CanWriteToSource(const UObject* Source)
{
	const UStaticMesh* StaticMesh = SourceToStaticMesh(Source);
	if (StaticMesh)
	{
		return !StaticMesh->GetPathName().StartsWith(TEXT("/Engine/"));
	}
	return false;
}
