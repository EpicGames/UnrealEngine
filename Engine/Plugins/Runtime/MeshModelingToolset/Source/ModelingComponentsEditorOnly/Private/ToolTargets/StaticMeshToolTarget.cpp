// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/StaticMeshToolTarget.h"

#include "AssetUtils/MeshDescriptionUtil.h"
#include "ConversionUtils/DynamicMeshViaMeshDescriptionUtil.h"
#include "ConversionUtils/SceneComponentToDynamicMesh.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/StaticMesh.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "ModelingToolTargetUtil.h"
#include "RenderingThread.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshToolTarget)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "StaticMeshToolTarget"

namespace StaticMeshToolTargetLocals
{
	static void DisplayCriticalWarningMessage(const FString& Message)
	{
		FNotificationInfo Info(FText::FromString(Message));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		UE_LOGF(LogTemp, Warning, "%ls", *Message);
	}
}

void UStaticMeshReadOnlyToolTarget::SetEditingLOD(EMeshLODIdentifier RequestedEditingLOD)
{
	EditingLOD = GetValidEditingLOD(StaticMesh.Get(), RequestedEditingLOD);
}

EMeshLODIdentifier UStaticMeshReadOnlyToolTarget::GetValidEditingLOD(const UStaticMesh* StaticMeshIn, 
	EMeshLODIdentifier RequestedEditingLOD)
{
	using namespace StaticMeshToolTargetLocals;

	EMeshLODIdentifier ValidEditingLOD = EMeshLODIdentifier::LOD0;

	if (ensure(StaticMeshIn != nullptr))
	{
		if (RequestedEditingLOD == EMeshLODIdentifier::MaxQuality)
		{
			ValidEditingLOD = StaticMeshIn->IsHiResMeshDescriptionValid() ? EMeshLODIdentifier::HiResSource : EMeshLODIdentifier::LOD0;
		}
		else if (RequestedEditingLOD == EMeshLODIdentifier::HiResSource)
		{
			ValidEditingLOD = StaticMeshIn->IsHiResMeshDescriptionValid() ? EMeshLODIdentifier::HiResSource : EMeshLODIdentifier::LOD0;
			if (ValidEditingLOD != EMeshLODIdentifier::HiResSource)
			{
				DisplayCriticalWarningMessage(FString(TEXT("HiRes Source selected but not available - Falling Back to LOD0")));
			}
		}
		else
		{
			ValidEditingLOD = RequestedEditingLOD;
			int32 MaxExistingLOD = StaticMeshIn->GetNumSourceModels() - 1;
			if ((int32)ValidEditingLOD > MaxExistingLOD)
			{
				DisplayCriticalWarningMessage(FString::Printf(TEXT("LOD%d Requested but not available - Falling Back to LOD%d"), (int32)ValidEditingLOD, 0));
				ValidEditingLOD = EMeshLODIdentifier::LOD0;
			}
		}

		// Ensure the selected Editing LOD is a non-generated LOD
		if (ValidEditingLOD != EMeshLODIdentifier::HiResSource)
		{
			const int32 LODint = static_cast<int32>(ValidEditingLOD);
			if (!StaticMeshIn->GetSourceModel(LODint).IsSourceModelInitialized())
			{
				DisplayCriticalWarningMessage(FString::Printf(TEXT("LOD%d Requested but is a generated LOD - Falling Back to LOD%d"), LODint, 0));
				ValidEditingLOD = EMeshLODIdentifier::LOD0;
			}
		}
	}

	return ValidEditingLOD;
}

TArray<EMeshLODIdentifier> UStaticMeshReadOnlyToolTarget::GetAvailableLODs(bool bSkipAutoGenerated) const
{
	return GetAvailableLODs(StaticMesh.Get(), bSkipAutoGenerated);
}

TArray<EMeshLODIdentifier> UStaticMeshReadOnlyToolTarget::GetAvailableLODs(const UStaticMesh* InStaticMesh, bool bSkipAutoGenerated)
{
	TArray<EMeshLODIdentifier> AvailableLODs;
	
	if (!InStaticMesh)
	{
		return AvailableLODs;
	}

	if (InStaticMesh->IsHiResMeshDescriptionValid())
	{
		AvailableLODs.Add(EMeshLODIdentifier::HiResSource);
	}
	int32 NumExistingLOD = FMath::Min(8, InStaticMesh->GetNumSourceModels());
	for (int32 LOD = 0; LOD < NumExistingLOD; ++LOD)
	{
		if (bSkipAutoGenerated)
		{
			// Generated LODs have no source model data
			if (!InStaticMesh->GetSourceModel(LOD).IsSourceModelInitialized())
			{
				continue;
			}
		}
		AvailableLODs.Add((EMeshLODIdentifier)LOD);
	}

	return AvailableLODs;
}


bool UStaticMeshReadOnlyToolTarget::IsValid() const
{
	if (UStaticMesh* Mesh = StaticMesh.Get())
	{
		return ::IsValid(Mesh) && !Mesh->IsUnreachable() && Mesh->IsValidLowLevel();
	}
	return false;
}

bool UStaticMeshReadOnlyToolTarget::HasNonGeneratedLOD(const UStaticMesh* StaticMeshIn, EMeshLODIdentifier MeshLOD)
{
	if (!StaticMeshIn)
	{
		return false;
	}

	// Early out if the static mesh is still compiling. A significant number of functions checking the state of the static mesh are
	// blocking on static mesh builds such as `GetNumSourceModels` used below. It is best to avoid long blocking calls in CanBuildTarget to
	// ensure editor interactions are smooth. A tool target can't be properly initialized until the static mesh is finished building anyways.
	if (StaticMeshIn->IsCompiling())
	{
		return false;
	}
	
	if (MeshLOD == EMeshLODIdentifier::Default)
	{
		MeshLOD = EMeshLODIdentifier::LOD0;
	}
	if (MeshLOD == EMeshLODIdentifier::HiResSource)
	{
		return StaticMeshIn->IsHiResMeshDescriptionValid();
	}
	const int32 NumExistingLOD = FMath::Min(8, StaticMeshIn->GetNumSourceModels());
	if (MeshLOD == EMeshLODIdentifier::MaxQuality)
	{
		const bool bHasHiRes = StaticMeshIn->IsHiResMeshDescriptionValid();
		const bool bHasLOD0 = (NumExistingLOD > 0) ? StaticMeshIn->GetSourceModel(0).IsSourceModelInitialized() : false;
		return (bHasHiRes || bHasLOD0);
	}
	else
	{
		const int32 LODint = static_cast<int32>(MeshLOD);
		if (LODint >= NumExistingLOD)
		{ 
			return false; 
		}

		return StaticMeshIn->GetSourceModel(LODint).IsSourceModelInitialized();
	}
}

int32 UStaticMeshReadOnlyToolTarget::GetNumMaterials() const
{
	return ensure(IsValid()) ? StaticMesh->GetStaticMaterials().Num() : 0;
}

UMaterialInterface* UStaticMeshReadOnlyToolTarget::GetMaterial(int32 MaterialIndex) const
{
	return ensure(IsValid()) ? StaticMesh->GetMaterial(MaterialIndex) : nullptr;
}

void UStaticMeshReadOnlyToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	if (!ensure(IsValid())) return;

	GetMaterialSet(StaticMesh.Get(), MaterialSetOut, bPreferAssetMaterials);
}

void UStaticMeshReadOnlyToolTarget::GetMaterialSet(const UStaticMesh* StaticMeshIn, 
	FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials)
{
	int32 NumMaterials = StaticMeshIn->GetStaticMaterials().Num();
	MaterialSetOut.Materials.SetNum(NumMaterials);
	for (int32 k = 0; k < NumMaterials; ++k)
	{
		MaterialSetOut.Materials[k] = StaticMeshIn->GetMaterial(k);
	}
}

bool UStaticMeshReadOnlyToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!ensure(IsValid())) return false;

	return CommitMaterialSetUpdate(StaticMesh.Get(), MaterialSet, bApplyToAsset);
}

bool UStaticMeshReadOnlyToolTarget::CommitMaterialSetUpdate(UStaticMesh* StaticMeshIn,
	const FComponentMaterialSet& MaterialSet, bool bApplyToAsset)
{
	if (!bApplyToAsset)
	{
		return false;
	}

	if (StaticMeshIn->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		UE_LOGF(LogTemp, Warning, "CANNOT MODIFY BUILT-IN ENGINE ASSET %ls", *StaticMeshIn->GetPathName());
		return false;
	}

	// filter out any Engine materials that we don't want to be permanently assigning
	TArray<UMaterialInterface*> FilteredMaterials = MaterialSet.Materials;
	for (int32 k = 0; k < FilteredMaterials.Num(); ++k)
	{
		FString AssetPath = FilteredMaterials[k]->GetPathName();
		if (AssetPath.StartsWith(TEXT("/MeshModelingToolsetExp/")))
		{
			FilteredMaterials[k] = UMaterial::GetDefaultMaterial(MD_Surface);
		}
	}

	// flush any pending rendering commands, which might touch this component while we are rebuilding its mesh
	FlushRenderingCommands();

	// make sure transactional flag is on
	StaticMeshIn->SetFlags(RF_Transactional);

	StaticMeshIn->Modify();

	int NewNumMaterials = FilteredMaterials.Num();
	if (NewNumMaterials != StaticMeshIn->GetStaticMaterials().Num())
	{
		StaticMeshIn->GetStaticMaterials().SetNum(NewNumMaterials);
	}
	for (int k = 0; k < NewNumMaterials; ++k)
	{
		if (StaticMeshIn->GetMaterial(k) != FilteredMaterials[k])
		{
			StaticMeshIn->SetMaterial(k, FilteredMaterials[k]);
		}
	}

	UE::ToolTarget::Internal::PostEditChangeWithConditionalUndo(StaticMeshIn);
	return true;
}

const FMeshDescription* UStaticMeshReadOnlyToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (ensure(IsValid()))
	{
		EMeshLODIdentifier UseLOD = EditingLOD;
		if (GetMeshParams.bHaveRequestLOD)
		{
			UseLOD = GetValidEditingLOD(StaticMesh.Get(), GetMeshParams.RequestLOD);
			ensure(UseLOD == GetMeshParams.RequestLOD);		// probably a bug somewhere if this is not true
		}

		return GetMeshDescriptionWithScaleApplied(StaticMesh.Get(), (int32)UseLOD, CachedMeshDescriptions);
	}
	return nullptr;
}

TArray<int32> UStaticMeshReadOnlyToolTarget::GetPolygonGroupToMaterialIndexMap() const
{
	if (IsValid())
	{
		return UStaticMeshReadOnlyToolTarget::MapSectionToMaterialID(StaticMesh.Get(), EditingLOD);
	}
	return TArray<int32>();
}

TArray<int32> UStaticMeshReadOnlyToolTarget::MapSectionToMaterialID(const UStaticMesh* Mesh, EMeshLODIdentifier EditingLOD)
{
	UE::Conversion::EMeshLODType LODType = [EditingLOD]() {
		switch (EditingLOD)
		{
		case EMeshLODIdentifier::MaxQuality:
			return UE::Conversion::EMeshLODType::MaxAvailable;
		case EMeshLODIdentifier::HiResSource:
			return UE::Conversion::EMeshLODType::HiResSourceModel;
		default:
			return UE::Conversion::EMeshLODType::SourceModel;
		}
	}();
	int32 UseLOD = (int32)EditingLOD;
	if (UseLOD > 7) // map the special LODs (default, hi res, max) to index 0
	{
		UseLOD = 0;
	}
	return UE::Conversion::GetPolygonGroupToMaterialIndexMap(Mesh, LODType, UseLOD);
}

const FMeshDescription* UStaticMeshReadOnlyToolTarget::GetMeshDescriptionWithScaleApplied(const UStaticMesh* StaticMesh, int32 UseLOD, FMeshDescriptionCache& CachedMeshDescriptions, const FGetMeshDescriptionOptions& Options)
{
	// Note: Static init is thread-safe, as long as it's done entirely in this assignment; do not change the EmptyMeshDescription object after.
	static const FMeshDescription EmptyMeshDescription = []
	{
		FMeshDescription MD;
		FStaticMeshAttributes(MD).Register();
		return MD;
	}();

	if (!StaticMesh)
	{
		return &EmptyMeshDescription;
	}

	FMeshDescription* FoundMeshDescription = nullptr;
	TOptional<FMeshBuildSettings> FoundBuildSettings;
	if (UseLOD == (int32)EMeshLODIdentifier::HiResSource)
	{
		FoundMeshDescription = StaticMesh->GetHiResMeshDescription();
		FoundBuildSettings = StaticMesh->GetHiResSourceModel().BuildSettings;
	}
	else
	{
		FoundMeshDescription = StaticMesh->GetMeshDescription(UseLOD);
		if (FoundMeshDescription)
		{
			FoundBuildSettings = StaticMesh->GetSourceModel(UseLOD).BuildSettings;
		}
	}
	
	// Apply allow-recompute overrides, if present
	if (FoundBuildSettings)
	{
		FoundBuildSettings->bRecomputeNormals &= Options.bAllowRecomputeNormals;
		FoundBuildSettings->bRecomputeTangents &= Options.bAllowRecomputeTangents;
	}

	if (CachedMeshDescriptions.Contains(UseLOD))
	{
		if (CachedMeshDescriptions[UseLOD].Source == FoundMeshDescription)
		{
			return CachedMeshDescriptions[UseLOD].Copy.Get();
		}
		else // cache was stale; clear it
		{
			CachedMeshDescriptions.Remove(UseLOD);
		}
	}

	if (FoundMeshDescription)
	{
		bool bNeedsBuildScale = !FoundBuildSettings->BuildScale3D.Equals(FVector::OneVector);
		bool bNeedsNormalTangent = FoundBuildSettings->bRecomputeNormals || FoundBuildSettings->bRecomputeTangents;

		if (bNeedsBuildScale || bNeedsNormalTangent)
		{
			FCachedMeshDescription& Cache = CachedMeshDescriptions.Emplace(UseLOD);
			Cache.Source = FoundMeshDescription;
			Cache.Copy = MakeUnique<FMeshDescription>(*FoundMeshDescription);

			if (bNeedsBuildScale)
			{
				FTransform ScaleTransform = FTransform::Identity;
				ScaleTransform.SetScale3D(FoundBuildSettings->BuildScale3D);
				FStaticMeshOperations::ApplyTransform(*Cache.Copy, ScaleTransform, true);
			}
			if (bNeedsNormalTangent)
			{
				UE::MeshDescription::InitializeAutoGeneratedAttributes(*Cache.Copy, FoundBuildSettings.GetPtrOrNull());
			}

			FoundMeshDescription = Cache.Copy.Get();
		}	
	}

	return (FoundMeshDescription != nullptr) ? FoundMeshDescription : &EmptyMeshDescription;
}

FMeshDescription UStaticMeshReadOnlyToolTarget::GetEmptyMeshDescription()
{
	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes Attributes(EmptyMeshDescription);
	Attributes.Register();
	return EmptyMeshDescription;
}

void UStaticMeshToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams)
{
	if (ensure(IsValid()) == false) return;

	EMeshLODIdentifier WriteToLOD = (CommitParams.bHaveTargetLOD && CommitParams.TargetLOD != EMeshLODIdentifier::Default) ? CommitParams.TargetLOD : EditingLOD;

	CommitMeshDescription(StaticMesh.Get(), Committer, WriteToLOD);
}

void UStaticMeshToolTarget::CommitMeshDescription(UStaticMesh* StaticMeshIn, const FCommitter& Committer, EMeshLODIdentifier EditingLODIn)
{
	using namespace StaticMeshToolTargetLocals;

	if ( ! ensure(EditingLODIn != EMeshLODIdentifier::Default && EditingLODIn != EMeshLODIdentifier::MaxQuality) )
	{
		UE_LOGF(LogGeometry, Warning, "UStaticMeshToolTarget::CommitMeshDescription: invalid Target LOD, must specify explicit LOD");
		return;
	}
	if (StaticMeshIn->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		DisplayCriticalWarningMessage(FString::Printf(TEXT("CANNOT MODIFY BUILT-IN ENGINE ASSET %s"), *StaticMeshIn->GetPathName()));
		return;
	}

	// flush any pending rendering commands, which might touch this component while we are rebuilding it's mesh
	FlushRenderingCommands();

	// make sure transactional flag is on for this asset
	StaticMeshIn->SetFlags(RF_Transactional);
	// mark as modified
	StaticMeshIn->Modify();

	FMeshDescription* UpdateMeshDescription = nullptr;
	if (EditingLODIn == EMeshLODIdentifier::HiResSource)
	{
		UpdateMeshDescription = StaticMeshIn->GetHiResMeshDescription();
		if (UpdateMeshDescription == nullptr)
		{
			UpdateMeshDescription = StaticMeshIn->CreateHiResMeshDescription();
		}
	}
	else
	{
		int32 UseLODIndex = static_cast<int32>(EditingLODIn);
		if (StaticMeshIn->GetNumSourceModels() < UseLODIndex+1)
		{
			StaticMeshIn->SetNumSourceModels(UseLODIndex+1);
		}

		UpdateMeshDescription = StaticMeshIn->GetMeshDescription(UseLODIndex);
		if (UpdateMeshDescription == nullptr)
		{
			UpdateMeshDescription = StaticMeshIn->CreateMeshDescription(UseLODIndex);
		}
	}

	// disable auto-generated normals StaticMesh build setting
	UE::MeshDescription::FStaticMeshBuildSettingChange SettingsChange;
	SettingsChange.AutoGeneratedNormals = UE::MeshDescription::EBuildSettingBoolChange::Disable;
	if (static_cast<int32>(EditingLODIn) <= (int32)EMeshLODIdentifier::LOD7)
	{
		UE::MeshDescription::ConfigureBuildSettings(StaticMeshIn, static_cast<int32>(EditingLODIn), SettingsChange);
	}
	// do we need to configure build settings for highres LOD?

	FVector BuildScale = FVector::OneVector;
	if (EditingLODIn == EMeshLODIdentifier::HiResSource)
	{
		StaticMeshIn->ModifyHiResMeshDescription();
		BuildScale = StaticMeshIn->GetHiResSourceModel().BuildSettings.BuildScale3D;
	}
	else
	{
		StaticMeshIn->ModifyMeshDescription((int32)EditingLODIn);
		if (StaticMeshIn->IsSourceModelValid((int32)EditingLODIn))
		{
			BuildScale = StaticMeshIn->GetSourceModel((int32)EditingLODIn).BuildSettings.BuildScale3D;
		}
	}

	FCommitterParams CommitterParams;
	CommitterParams.MeshDescriptionOut = UpdateMeshDescription;

	Committer(CommitterParams);
	if (!BuildScale.Equals(FVector::OneVector))
	{
		FTransform InverseBuildScaleTransform = FTransform::Identity;
		FVector InverseBuildScale;
		// Safely invert BuildScale
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			InverseBuildScale[Idx] = FMath::IsNearlyZero(BuildScale[Idx], FMathd::Epsilon) ? 1.0 : 1.0 / BuildScale[Idx];
		}
		InverseBuildScaleTransform.SetScale3D(InverseBuildScale);
		FStaticMeshOperations::ApplyTransform(*UpdateMeshDescription, InverseBuildScaleTransform, true);
	}

	if (EditingLODIn == EMeshLODIdentifier::HiResSource)
	{
		StaticMeshIn->CommitHiResMeshDescription();
	}
	else
	{
		StaticMeshIn->CommitMeshDescription((int32)EditingLODIn);

		// configure build settings to prevent the standard static mesh reduction from running and replacing the render LOD.
		FStaticMeshSourceModel& ThisSourceModel = StaticMeshIn->GetSourceModel((int32)EditingLODIn);
		ThisSourceModel.ResetReductionSetting();
	}

	UE::ToolTarget::Internal::PostEditChangeWithConditionalUndo(StaticMeshIn);
}

FDynamicMesh3 UStaticMeshReadOnlyToolTarget::GetDynamicMesh()
{
	return GetDynamicMeshViaMeshDescription(*this);
}

FDynamicMesh3 UStaticMeshReadOnlyToolTarget::GetDynamicMesh(const FGetMeshParameters& InGetMeshParams)
{
	return GetDynamicMeshViaMeshDescription(*this, InGetMeshParams);
}

void UStaticMeshToolTarget::CommitDynamicMesh(const FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo)
{
	FMeshDescription CurrentMeshDescription = *GetMeshDescription();
	CommitDynamicMeshViaMeshDescription(MoveTemp(CurrentMeshDescription), *this, Mesh, CommitInfo);
}


UStaticMesh* UStaticMeshReadOnlyToolTarget::GetStaticMesh() const
{
	return IsValid() ? StaticMesh.Get() : nullptr;
}


// Factory

bool UStaticMeshToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	const UStaticMesh* StaticMesh = GetValid(Cast<UStaticMesh>(SourceObject));
	bool bValid = StaticMesh && !StaticMesh->IsUnreachable() && StaticMesh->IsValidLowLevel()
			&& !StaticMesh->GetOutermost()->bIsCookedForEditor
			&& UStaticMeshToolTarget::HasNonGeneratedLOD(StaticMesh, EditingLOD);
	if (!bValid)
	{
		return false;
	}
	if (CanWriteToSource(SourceObject))
	{
		return Requirements.AreSatisfiedBy(UStaticMeshToolTarget::StaticClass());
	}
	else
	{
		return Requirements.AreSatisfiedBy(UStaticMeshReadOnlyToolTarget::StaticClass());
	}
}

UToolTarget* UStaticMeshToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UStaticMeshReadOnlyToolTarget* Target;
	if (CanWriteToSource(SourceObject))
	{ 
		Target = NewObject<UStaticMeshToolTarget>();
	}
	else
	{
		Target = NewObject<UStaticMeshReadOnlyToolTarget>();
	}
	Target->StaticMesh = Cast<UStaticMesh>(SourceObject);
	Target->SetEditingLOD(EditingLOD);
	checkSlow(Target->StaticMesh.IsValid() && Requirements.AreSatisfiedBy(Target));

	return Target;
}


void UStaticMeshToolTargetFactory::SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD)
{
	EditingLOD = NewEditingLOD;
}


bool UStaticMeshToolTargetFactory::CanWriteToSource(const UObject* SourceObject)
{
	if (const UStaticMesh* StaticMesh = GetValid(Cast<UStaticMesh>(SourceObject)))
	{
		return !StaticMesh->GetPathName().StartsWith(TEXT("/Engine/"));
	}
	return false;
}

#undef LOCTEXT_NAMESPACE

