// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/VolumeComponentTarget.h"

#include "Components/BrushComponent.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "ConversionUtils/DynamicMeshToVolume.h"
#include "DynamicMeshToMeshDescription.h"
#include "GameFramework/Volume.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "StaticMeshAttributes.h"
#include "ToolSetupUtil.h"
#include "TargetInterfaces/MaterialProvider.h"

using namespace UE::Geometry;

FVolumeComponentTarget::FVolumeComponentTarget(UPrimitiveComponent* Component)
	: FPrimitiveComponentTarget(Cast<UBrushComponent>(Component)) 
{
	// TODO: These should be user-configurable somewhere
	VolumeToMeshOptions.bInWorldSpace = false;
	VolumeToMeshOptions.bSetGroups = true;
	VolumeToMeshOptions.bMergeVertices = true;
	VolumeToMeshOptions.bAutoRepairMesh = false;
	VolumeToMeshOptions.bOptimizeMesh = true;
}

FMeshDescription* FVolumeComponentTarget::GetMesh()
{
	if (!ConvertedMeshDescription.IsValid())
	{
		UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component);
		if (!BrushComponent)
		{
			return nullptr;
		}
		AVolume* Volume = Cast<AVolume>(BrushComponent->GetOwner());
		if (!Volume)
		{
			return nullptr;
		}

		// Note: We can go directly from a volume to a mesh description using GetBrushMesh() in
		// Editor.h. However, that path doesn't assign polygroups to the result, which we
		// typically want when using this target, hence the two-step path we use here.
		// All of this may also someday be fixed if we allow targets to be converted directly
		// to dynamic meshes rather than mesh descriptions, which is not currently possible
		// because FDynamicMesh3 is a class specific to the geometry plugin.
		FDynamicMesh3 DynamicMesh;
		UE::Conversion::VolumeToDynamicMesh(Volume, DynamicMesh, GetVolumeToMeshOptions());
		FMeshNormals::InitializeMeshToPerTriangleNormals(&DynamicMesh);

		ConvertedMeshDescription = MakeUnique<FMeshDescription>();
		FStaticMeshAttributes Attributes(*ConvertedMeshDescription);
		Attributes.Register();
		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(&DynamicMesh, *ConvertedMeshDescription);
	}
	return ConvertedMeshDescription.Get();
}

void FVolumeComponentTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bAssetMaterials) const
{
	MaterialSetOut.Materials.Reset();
	if (IsValid() == false)
	{
		return;
	}

	UMaterialInterface* Material = ToolSetupUtil::GetDefaultEditVolumeMaterial();
	if (Material)
	{
		MaterialSetOut.Materials.Add(Material);
	}
}

bool FVolumeComponentTarget::HasSameSourceData(const FPrimitiveComponentTarget& OtherTarget) const
{
	if (IsValid())
	{
		const UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component);
		const UBrushComponent* OtherBrushComponent = Cast<UBrushComponent>(OtherTarget.Component);
		return BrushComponent && BrushComponent == OtherBrushComponent;
	}
	return false;
}

void FVolumeComponentTarget::CommitMesh(const FCommitter& Committer)
{
	check(IsValid());

	UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component);
	check(BrushComponent);
	AVolume* Volume = Cast<AVolume>(BrushComponent->GetOwner());
	check(Volume);

	FCommitParams CommitParams;
	CommitParams.MeshDescription = ConvertedMeshDescription.Get();
	Committer(CommitParams);

	// TODO: We should probably have a path directly from a mesh description to a volume
	// rather than going through a dynamic mesh, though this may eventually not be necessary
	// if our targets are changed to hold dynamic meshes to begin with.
	FDynamicMesh3 DynamicMesh;
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(ConvertedMeshDescription.Get(), DynamicMesh);

	FTransform Transform = GetWorldTransform();
	UE::Conversion::DynamicMeshToVolume(DynamicMesh, Volume);

	Volume->SetActorTransform(Transform);
	Volume->PostEditChange();
}

bool FVolumeComponentTargetFactory::CanBuild(UActorComponent* Component)
{
	if (UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		return Cast<AVolume>(BrushComponent->GetOwner()) != nullptr;
	}
	return false;
}

TUniquePtr<FPrimitiveComponentTarget> FVolumeComponentTargetFactory::Build(UPrimitiveComponent* Component)
{
	UBrushComponent* VolumeBrushComponent = Cast<UBrushComponent>(Component);
	if (!VolumeBrushComponent)
	{
		return {};
	}
	AVolume* Volume = Cast<AVolume>(VolumeBrushComponent->GetOwner());
	if (!Volume)
	{
		return {};
	}

	return TUniquePtr<FPrimitiveComponentTarget> { new FVolumeComponentTarget(Component) };
}
