// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolTargets/VolumeComponentToolTarget.h"

#include "Components/BrushComponent.h"
#include "ConversionUtils/DynamicMeshToVolume.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "GameFramework/Volume.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ModelingToolTargetUtil.h"
#include "DynamicMesh/MeshNormals.h"
#include "StaticMeshAttributes.h"
#include "ToolSetupUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VolumeComponentToolTarget)

using namespace UE::Geometry;

TAutoConsoleVariable<int32> CVarModelingMaxVolumeTriangleCount(
	TEXT("modeling.VolumeMaxTriCount"),
	500,
	TEXT("Limit on triangle count for Volumes that will be emitted by modeling tools. Meshes above this limit will be auto-simplified."));

UVolumeComponentToolTarget::UVolumeComponentToolTarget()
{
	// TODO: These should be user-configurable somewhere
	VolumeToMeshOptions.bInWorldSpace = false;
	VolumeToMeshOptions.bSetGroups = true;
	VolumeToMeshOptions.bMergeVertices = true;

	// When a volume has cracks, this option seems to make the geometry
	// worse rather than better, since the filled in triangles are sometimes
	// degenerate, folded in on themselves, etc.
	VolumeToMeshOptions.bAutoRepairMesh = false;

	VolumeToMeshOptions.bOptimizeMesh = true;
}

int32 UVolumeComponentToolTarget::GetNumMaterials() const
{
	return IsValid() ? 1 : 0;
}

UMaterialInterface* UVolumeComponentToolTarget::GetMaterial(int32 MaterialIndex) const
{
	if (!IsValid() || MaterialIndex > 0)
	{
		return nullptr;
	}

	return ToolSetupUtil::GetDefaultEditVolumeMaterial();
}

void UVolumeComponentToolTarget::GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const
{
	MaterialSetOut.Materials.Reset();
	if (IsValid() == false)
	{
		return;
	}

	UMaterialInterface* Material = ToolSetupUtil::GetDefaultEditVolumeMaterial();
	if (Material)
	{
		MaterialSetOut.Materials.Add(Material);
	}
}

FDynamicMesh3 UVolumeComponentToolTarget::GetDynamicMesh()
{
	UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component);
	if (!BrushComponent)
	{
		return nullptr;
	}
	AVolume* Volume = Cast<AVolume>(BrushComponent->GetOwner());
	if (!Volume)
	{
		return nullptr;
	}

	FDynamicMesh3 DynamicMesh;
	UE::Conversion::VolumeToDynamicMesh(Volume, DynamicMesh, GetVolumeToMeshOptions());

	return DynamicMesh;
}

void UVolumeComponentToolTarget::CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo&)
{
	check(IsValid());

	UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component);
	check(BrushComponent);
	AVolume* Volume = Cast<AVolume>(BrushComponent->GetOwner());
	check(Volume);

	FTransform Transform = GetWorldTransform();

	UE::Conversion::FMeshToVolumeOptions ConversionOptions;
	ConversionOptions.bAutoSimplify = true;
	ConversionOptions.MaxTriangles = FMath::Max(1, CVarModelingMaxVolumeTriangleCount.GetValueOnGameThread());
	UE::Conversion::DynamicMeshToVolume(Mesh, Volume, ConversionOptions);

	Volume->SetActorTransform(Transform);
	
	UE::ToolTarget::Internal::PostEditChangeWithConditionalUndo(Volume);
}

const FMeshDescription* UVolumeComponentToolTarget::GetMeshDescription(const FGetMeshParameters& GetMeshParams)
{
	if (!ConvertedMeshDescription.IsValid())
	{
		// Note: We can go directly from a volume to a mesh description using GetBrushMesh() in
		// Editor.h. However, that path doesn't assign polygroups to the result, which we
		// typically want when using this target, hence the path through a dynamic mesh.

		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> DynamicMesh = 
			MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(GetDynamicMesh());

		ConvertedMeshDescription = MakeShared<FMeshDescription, ESPMode::ThreadSafe>();
		FStaticMeshAttributes Attributes(*ConvertedMeshDescription);
		Attributes.Register();
		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(DynamicMesh.Get(), *ConvertedMeshDescription);
	}
	return ConvertedMeshDescription.Get();
}

FMeshDescription UVolumeComponentToolTarget::GetEmptyMeshDescription()
{
	// We use StaticMeshAttributes here because they are the standard used across the engine
	// with regard to setting up FMeshDescriptions in the majority of cases.

	FMeshDescription EmptyMeshDescription;
	FStaticMeshAttributes Attributes(EmptyMeshDescription);
	Attributes.Register();
	return EmptyMeshDescription;
}

void UVolumeComponentToolTarget::CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams)
{
	// no LODs so LODIdentifier is ignored
	check(IsValid());

	// Let the user fill our mesh description with the Committer
	if (!ConvertedMeshDescription.IsValid())
	{
		ConvertedMeshDescription = MakeShared<FMeshDescription, ESPMode::ThreadSafe>();
		FStaticMeshAttributes Attributes(*ConvertedMeshDescription);
		Attributes.Register();
	}
	FCommitterParams CommitParams;
	CommitParams.MeshDescriptionOut = ConvertedMeshDescription.Get();
	Committer(CommitParams);

	// The conversion we have right now is from dynamic mesh to volume, so we convert
	// to dynamic mesh first.
	FDynamicMesh3 DynamicMesh;
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(ConvertedMeshDescription.Get(), DynamicMesh);

	CommitDynamicMesh(DynamicMesh);
}

UBodySetup* UVolumeComponentToolTarget::GetBodySetup() const
{
	if (UBrushComponent* BrushComponent = Cast<UBrushComponent>(Component))
	{
		return BrushComponent->GetBodySetup();
	}
	return nullptr;
}


// Factory

bool UVolumeComponentToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements) const
{
	UBrushComponent* BrushComponent = Cast<UBrushComponent>(SourceObject);
	if (!BrushComponent)
	{
		return false;
	}

	return Cast<AVolume>(BrushComponent->GetOwner()) && Requirements.AreSatisfiedBy(UVolumeComponentToolTarget::StaticClass());
}

UToolTarget* UVolumeComponentToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& Requirements)
{
	UVolumeComponentToolTarget* Target = NewObject<UVolumeComponentToolTarget>();
	Target->InitializeComponent(Cast<UBrushComponent>(SourceObject));

	checkSlow(Target->Component.IsValid() && Requirements.AreSatisfiedBy(Target)
		&& Cast<AVolume>(Cast<UBrushComponent>(SourceObject)->GetOwner()));

	return Target;
}
