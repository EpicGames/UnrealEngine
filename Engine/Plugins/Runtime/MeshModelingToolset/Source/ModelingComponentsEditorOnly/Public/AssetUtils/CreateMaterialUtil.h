// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <UObject/Package.h>

class UMaterial;
class UMaterialInstanceConstant;
class UMaterialInterface;

namespace UE
{
namespace AssetUtils
{
	/**
	 * Result enum returned by Create() functions below to indicate succes/error conditions
	 */
	enum class ECreateMaterialResult
	{
		Ok = 0,
		InvalidPackage = 1,
		InvalidBaseMaterial = 2,
		NameError = 3,
		DuplicateFailed = 4,

		UnknownError = 100
	};

	/**
	 * Options for new UMaterial asset created by Create() functions below.
	 */
	struct FMaterialAssetOptions
	{
		// this package will be used if it is not null, otherwise a new package will be created at NewAssetPath
		UPackage* UsePackage = nullptr;
		FString NewAssetPath;

		bool bDeferPostEditChange = false;
	};

	/**
	 * Output information about a newly-created UMaterial, returned by Create functions below.
	 * Some fields may be null, if the relevant function did not create that type of object
	 */
	struct FMaterialAssetResults
	{
		UMaterial* NewMaterial = nullptr;
		UMaterialInstanceConstant* NewMaterialInstance = nullptr;
	};

	/**
	 * Create a new UMaterial Asset by duplicating the parent UMaterial of BaseMaterial
	 * @param Options defines configuration for the new UMaterial Asset
	 * @param ResultsOut new UMaterial is returned here
	 * @return Ok, or error flag if some part of the creation process failed. On failure, no Asset is created.
	 */
	MODELINGCOMPONENTSEDITORONLY_API ECreateMaterialResult CreateDuplicateMaterial(
		UMaterialInterface* BaseMaterial,
		FMaterialAssetOptions& Options,
		FMaterialAssetResults& ResultsOut);


	/**
	 * Create a new UMaterialInstanceConstant Asset derived from the parent UMaterial of BaseMaterial.
	 * If the input is a UMaterialInstanceConstant, duplicate it to preserve existing parameters/etc
	 * @param Options defines configuration for the new UMaterialInstanceConstant Asset
	 * @param ResultsOut new UMaterialInstanceConstant is returned here
	 * @return Ok, or error flag if some part of the creation process failed. On failure, no Asset is created.
	 */
	MODELINGCOMPONENTSEDITORONLY_API ECreateMaterialResult CreateDerivedMaterialInstance(
		UMaterialInterface* BaseMaterial,
		FMaterialAssetOptions& Options,
		FMaterialAssetResults& ResultsOut);


}  // end namespace UE
}  // end namespace AssetUtils

