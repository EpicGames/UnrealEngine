// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"

#include "Engine/EngineTypes.h" // for NaniteSettings

class UMaterialInterface;
class USkeletalMesh;
class USkeleton;
struct FMeshDescription;
struct FSkeletalMaterial;
struct FReferenceSkeleton;

namespace UE::AssetUtils
{
	using namespace Geometry;

	/**
	 * Result enum returned by Create functions below to indicate success/error conditions
	 */
	enum class ECreateSkeletalMeshResult
	{
		Ok = 0,
		InvalidPackage = 1,
		InvalidSkeleton = 2,

		UnknownError = 100
	};

	/**
	 * Set of input meshes for StaticMesh Create() functions below.
	 * Only one of the arrays should be initialized, and it's size should be equal to 
	 * the number of LODs specified for the Asset.
	 */
	struct FSkeletalMeshAssetMeshes
	{
		TArray<const FDynamicMesh3*> DynamicMeshes;
		TArray<const FMeshDescription*> MeshDescriptions;
		TArray<FMeshDescription*> MoveMeshDescriptions;		// these will be MoveTemp()'d into the SkeletalMesh, so if initialized, this array will be invalid after Create call()
	};

	/**
	 * Options for new USkeletalMesh asset created by Create() functions below.
	 */
	struct FSkeletalMeshAssetOptions
	{
		// This package will be used if it is not nullptr, otherwise a new package will be created at NewAssetPath
		UPackage* UsePackage = nullptr;
		FString NewAssetPath;
		
		// The associated USkeleton object
		USkeleton* Skeleton = nullptr;

		// The associated FReferenceSkeleton object. If null, the reference skeleton of the Skeleton object is used instead.
		FReferenceSkeleton* RefSkeleton = nullptr;

		// Number of SourceModels (ie LODs)
		int32 NumSourceModels = 1;
		// Number of Material Slots desired on the Asset. At least one will always be created.
		int32 NumMaterialSlots = 1;

		// Controls whether or not the RecomputeNormals option will be enabled on the Asset
		bool bEnableRecomputeNormals = false;
		// Controls whether or not the RecomputeTangents option will be enabled on the Asset
		bool bEnableRecomputeTangents = true;

		// Whether to use the specified NaniteSettings when generating the Asset. If false, the default settings (nanite disabled) will be used.
		bool bApplyNaniteSettings = false;
		// Nanite settings to set on skeletal mesh asset, if bApplyNaniteSettings is true
		FMeshNaniteSettings NaniteSettings = FMeshNaniteSettings();

		// Whether to attempt to re-create the original non-manifold mesh from the (optional) non-manifold index map on the dynamic mesh.
		bool bConvertBackToNonManifold = false;

		// List of skeletal materials. These will be used over the AssetMaterials, if provided.
		TArray<FSkeletalMaterial> SkeletalMaterials;
		
		// Optional list of materials to initialize the Asset with. If defined, size must be the same as NumMaterialSlots
		TArray<UMaterialInterface*> AssetMaterials;

		// Optional list of meshes to use to initialize SourceModels. If defined, only one array should be non-empty, and must be the same length as NumSourceModels.
		FSkeletalMeshAssetMeshes SourceMeshes;
	};

	/**
	 * Output information about a newly-created StaticMesh, returned by Create functions below.
	 * Some fields may be null, if the relevant function did not create that type of object
	 */
	struct FSkeletalMeshResults
	{
		/** USkeletalMesh asset that was created */
		USkeletalMesh* SkeletalMesh = nullptr;
	};


	/**
	 * Create a new UStaticMesh Asset based on the input Options
	 * @param Options defines configuration for the new UStaticMesh Asset
	 * @param ResultsOut new StaticMesh is returned here
	 * @return Ok, or error flag if some part of the creation process failed. On failure, no Asset is created.
	 */
	MODELINGCOMPONENTSEDITORONLY_API ECreateSkeletalMeshResult CreateSkeletalMeshAsset(
		const FSkeletalMeshAssetOptions& Options,
		FSkeletalMeshResults& ResultsOut);

}