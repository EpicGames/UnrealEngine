// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Engine/EngineTypes.h"   // FMeshNaniteSettings
#include "BodySetupEnums.h"

class UPackage;
class UStaticMesh;
class UStaticMeshComponent;
class UStaticMeshActor;
struct FMeshDescription;
class UMaterialInterface;

namespace UE
{
namespace AssetUtils
{

	using namespace UE::Geometry;

	/**
	 * Result enum returned by Create functions below to indicate succes/error conditions
	 */
	enum class ECreateStaticMeshResult
	{
		Ok = 0,
		InvalidPackage = 1,

		UnknownError = 100
	};

	/**
	 * Set of input meshes for StaticMesh Create() functions below.
	 * Only one of the arrays should be initialized, and it's size should be equal to 
	 * the number of LODs specified for the Asset.
	 */
	struct FStaticMeshAssetMeshes
	{
		TArray<const UE::Geometry::FDynamicMesh3*> DynamicMeshes;
		TArray<const FMeshDescription*> MeshDescriptions;
		TArray<FMeshDescription*> MoveMeshDescriptions;		// these will be MoveTemp()'d into the StaticMesh, so if initialized, this array will be invalid after Create call()
	};

	/**
	 * Options for new UStaticMesh asset created by Create() functions below.
	 */
	struct FStaticMeshAssetOptions
	{
		// this package will be used if it is not null, otherwise a new package will be created at NewAssetPath
		UPackage* UsePackage = nullptr;
		FString NewAssetPath;

		// number of SourceModels (ie LODs)
		int32 NumSourceModels = 1;
		// number of Material Slots desired on the Asset. At least one will always be created.
		int32 NumMaterialSlots = 1;

		// Controls whether or not the RecomputeNormals option will be enabled on the Asset
		bool bEnableRecomputeNormals = false;
		// Controls whether or not the RecomputeTangents option will be enabled on the Asset
		bool bEnableRecomputeTangents = true;

		// Whether to generate a nanite-enabled mesh
		bool bGenerateNaniteEnabledMesh = false;
		// Percentage of triangles to reduce down to for generating a coarse proxy mesh from the Nanite mesh. DEPRECATED, use NaniteSettings instead.
		float NaniteProxyTrianglePercent_DEPRECATED = 0;
		// Nanite settings to set on static mesh asset
		FMeshNaniteSettings NaniteSettings = FMeshNaniteSettings();

		// Whether ray tracing will be supported on this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance
		bool bSupportRayTracing = true;

		// Whether to allow distance field to be computed for this mesh. Disable this to save memory if the generated mesh will only be rendered in the distance
		bool bAllowDistanceField = true;

		// Whether to generate lightmap uvs for the generated mesh
		bool bGenerateLightmapUVs = false;

		// Whether to generate UVs with full floating point precision.
		bool bUseFullPrecisionUVs = false;
		
		// Controls whether the UBodySetup on the Asset will be created (generally should be true)
		bool bCreatePhysicsBody = true;
		// set asset collision type
		ECollisionTraceFlag CollisionType = ECollisionTraceFlag::CTF_UseDefault;
		// TODO: option for simple collision geo

		// Optional list of materials to initialize the Asset with. If defined, size must be the same as NumMaterialSlots
		TArray<UMaterialInterface*> AssetMaterials;

		// Optional list of meshes to use to initialize SourceModels. If defined, only one array should be non-empty, and must be the same length as NumSourceModels.
		FStaticMeshAssetMeshes SourceMeshes;

		// by default, PostEditChange() will be called to rebuild mesh, set true to skip this call
		bool bDeferPostEditChange = false;

		// Whether to build a reversed index buffer. Will double the memory usage of indices.
		bool bBuildReversedIndexBuffer = true;

		// Whether to attempt to re-create the original non-manifold mesh from the (optional) non-manifold index map on the dynamic mesh.
		bool bConvertBackToNonManifold = false;
	};

	/**
	 * Output information about a newly-created StaticMesh, returned by Create functions below.
	 * Some fields may be null, if the relevant function did not create that type of object
	 */
	struct FStaticMeshResults
	{
		/** UStaticMesh asset that was created */
		UStaticMesh* StaticMesh = nullptr;
		/** New Component that was created that references the Asset (or null, if no component is created) */
		UStaticMeshComponent* Component = nullptr;
		/** New Actor that was created for the Component (or null, if no component is created) */
		UStaticMeshActor* Actor = nullptr;
	};


	/**
	 * Create a new UStaticMesh Asset based on the input Options
	 * @param Options defines configuration for the new UStaticMesh Asset
	 * @param ResultsOut new StaticMesh is returned here
	 * @return Ok, or error flag if some part of the creation process failed. On failure, no Asset is created.
	 */
	MODELINGCOMPONENTSEDITORONLY_API ECreateStaticMeshResult CreateStaticMeshAsset(
		FStaticMeshAssetOptions& Options,
		FStaticMeshResults& ResultsOut);


}
}
