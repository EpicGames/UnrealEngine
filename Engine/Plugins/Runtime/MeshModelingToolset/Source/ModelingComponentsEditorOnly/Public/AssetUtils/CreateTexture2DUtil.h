// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"



namespace UE
{
namespace AssetUtils
{
	/**
	 * Result enum returned by Create() functions below to indicate success/error conditions
	 */
	enum class ECreateTexture2DResult
	{
		Ok = 0,
		InvalidPackage = 1,
		InvalidInputTexture = 2,
		NameError = 3,
		OverwriteTypeError = 4,

		UnknownError = 100
	};


	/**
	 * Options for new UTexture asset created by Create() functions below.
	 */
	struct FTexture2DAssetOptions
	{
		// this package will be used if it is not null, otherwise a new package will be created at NewAssetPath
		UPackage* UsePackage = nullptr;
		FString NewAssetPath;

		bool bDeferPostEditChange = false;

		// if NewAssetPath already exists, update the existing Texture2D.
		bool bOverwriteIfExists = false;
	};

	/**
	 * Output information about a newly-created UTexture2D, returned by Create functions below.
	 * Some fields may be null, if the relevant function did not create that type of object
	 */
	struct FTexture2DAssetResults
	{
		UTexture2D* Texture = nullptr;
	};

	/**
	 * Convert a temporary UTexture2D in the Transient package into a serialized Asset
	 * @param Options defines configuration for the new UTexture2D Asset
	 * @param ResultsOut new UTexture2D is returned here
	 * @return Ok, or error flag if some part of the creation process failed. On failure, no Asset is created.
	 */
	MODELINGCOMPONENTSEDITORONLY_API ECreateTexture2DResult SaveGeneratedTexture2DAsset(
		UTexture2D* GeneratedTexture,
		FTexture2DAssetOptions& Options,
		FTexture2DAssetResults& ResultsOut);

}
}

