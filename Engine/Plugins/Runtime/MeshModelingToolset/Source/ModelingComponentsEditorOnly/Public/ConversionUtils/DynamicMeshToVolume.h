// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FrameTypes.h"
#include "VectorTypes.h"

// NOTE: The current implementation of DynamicMeshToVolume is editor-only,
// and is therefore split here from VolumeToDynamicMesh. If it ever becomes
// safe for runtime, we should move it to the same place (or combine it in
// one file).

class AVolume;
class FProgressCancel;
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

namespace UE {
namespace Conversion {

using namespace UE::Geometry; 

struct FMeshToVolumeOptions
{
	/**
	 * When true, coplanar components with different group IDs will not be
	 * merged into a single volume polygon.
	 */
	bool bRespectGroupBoundaries = true;

	/** 
	 * If true, simplify the input mesh to the specified target triangle count.
	 * Volumes with many faces/vertices are very expensive, and all Volume processing
	 * happens on the game thread, so this should generally be used to prevent significant Editor hangs.
	 */
	bool bAutoSimplify = false;

	/** If bAutoSimplify is true, it will simplify the mesh to this MaxTriangles count */
	int32 MaxTriangles = 250;

	/** If true, attempt to clean degenerate triangles before converting to a volume, as degenerate triangles can cause BSP construction to fail. */
	bool bCleanDegenerate = true;

	/** If bCleanDegenerate is true, defines the minimum triangle area to keep */
	double MinTriangleArea = UE_DOUBLE_KINDA_SMALL_NUMBER;

	/** If bCleanDegenerate is true, defines the minimum triangle edge length to keep. */
	double MinEdgeLength = .01; // default value reflects how the BSP code tends to snap vertices
};

struct FDynamicMeshFace
{
	FFrame3d Plane;
	TArray<FVector3d> BoundaryLoop;
};


/**
 * Gets an array of face objects that can be used to convert a dynamic mesh to a volume, based on given FMeshToVolumeOptions.
 */
void MODELINGCOMPONENTSEDITORONLY_API GetPolygonFaces(const FDynamicMesh3& InputMesh, const FMeshToVolumeOptions& Options, 
	TArray<FDynamicMeshFace>& FacesOut, FProgressCancel* Progress = nullptr);

/**
 * Gets an array of face objects that can be used to convert a dynamic mesh to a volume. This version tries to
 * merge coplanar triangles into polygons.
 */
void MODELINGCOMPONENTSEDITORONLY_API GetPolygonFaces(const FDynamicMesh3& InputMesh, TArray<FDynamicMeshFace>& Faces,
	bool bRespectGroupBoundaries);
/**
 * Gets an array of face objects that can be used to convert a dynamic mesh to a volume. This version makes
 * each triangle its own face.
 */
void MODELINGCOMPONENTSEDITORONLY_API GetTriangleFaces(const FDynamicMesh3& InputMesh, TArray<FDynamicMeshFace>& Faces);

/**
 * Converts a dynamic mesh to a volume.
 */
void MODELINGCOMPONENTSEDITORONLY_API DynamicMeshToVolume(const FDynamicMesh3& InputMesh, AVolume* TargetVolume,
	const FMeshToVolumeOptions& Options = FMeshToVolumeOptions());
void MODELINGCOMPONENTSEDITORONLY_API DynamicMeshToVolume(const FDynamicMesh3& InputMesh, TArray<FDynamicMeshFace>& Faces, AVolume* TargetVolume);

}}//end namespace UE::Conversion