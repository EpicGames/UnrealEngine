// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingObjectsCreationAPI.h"

#include "EditorModelingObjectsCreationAPI.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class UInteractiveToolsContext;

/**
 * Implementation of UModelingObjectsCreationAPI suitable for use in UE Editor.
 * - CreateMeshObject() currently creates a StaticMesh Asset/Actor, a Volume Actor or a DynamicMesh Actor
 * - CreateTextureObject() currently creates a UTexture2D Asset
 * - CreateMaterialObject() currently creates a UMaterial Asset
 * 
 * This is intended to be registered in the ToolsContext ContextObjectStore.
 * Static utility functions ::Register() / ::Find() / ::Deregister() can be used to do this in a consistent way.
 * 
 * Several client-provided callbacks can be used to customize functionality (eg in Modeling Mode) 
 *  - GetNewAssetPathNameCallback is called to determine an asset path. This can be used to do
 *    things like pop up an interactive path-selection dialog, use project-defined paths, etc
 *  - OnModelingMeshCreated is broadcast for each new created mesh object
 *  - OnModelingTextureCreated is broadcast for each new created texture object
 *  - OnModelingMaterialCreated is broadcast for each new created material object
 */
UCLASS(MinimalAPI)
class UEditorModelingObjectsCreationAPI : public UModelingObjectsCreationAPI
{
	GENERATED_BODY()
public:

	// UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	UE_API virtual FCreateMeshObjectResult CreateMeshObject(const FCreateMeshObjectParams& CreateMeshParams) override;

	// UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	UE_API virtual FCreateTextureObjectResult CreateTextureObject(const FCreateTextureObjectParams& CreateTexParams) override;

	// UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	UE_API virtual FCreateMaterialObjectResult CreateMaterialObject(const FCreateMaterialObjectParams& CreateMaterialParams) override;

	// UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	UE_API virtual FCreateActorResult CreateNewActor(const FCreateActorParams& CreateActorParams) override;

	// UFUNCTION(BlueprintCallable, Category = "Modeling Objects")
	UE_API virtual FCreateComponentResult CreateNewComponentOnActor(const FCreateComponentParams& CreateComponentParams) override;


	//
	// Non-UFunction variants that support std::move operators
	//

	virtual bool HasMoveVariants() const { return true; }

	UE_API virtual FCreateMeshObjectResult CreateMeshObject(FCreateMeshObjectParams&& CreateMeshParams) override;

	UE_API virtual FCreateTextureObjectResult CreateTextureObject(FCreateTextureObjectParams&& CreateTexParams) override;

	UE_API virtual FCreateMaterialObjectResult CreateMaterialObject(FCreateMaterialObjectParams&& CreateMaterialParams) override;

	UE_API virtual FCreateActorResult CreateNewActor(FCreateActorParams&& CreateActorParams) override;

	UE_API virtual FCreateComponentResult CreateNewComponentOnActor(FCreateComponentParams&& CreateComponentParams) override;

	//
	// Callbacks that editor can hook into to handle asset creation
	//

	DECLARE_DELEGATE_RetVal_ThreeParams(FString, FGetAssetPathNameCallbackSignature, const FString& BaseName, const UWorld* TargetWorld, FString SuggestedFolder);
	FGetAssetPathNameCallbackSignature GetNewAssetPathNameCallback;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModelingMeshCreatedSignature, const FCreateMeshObjectResult& CreatedInfo);
	FModelingMeshCreatedSignature OnModelingMeshCreated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModelingTextureCreatedSignature, const FCreateTextureObjectResult& CreatedInfo);
	FModelingTextureCreatedSignature OnModelingTextureCreated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModelingMaterialCreatedSignature, const FCreateMaterialObjectResult& CreatedInfo);
	FModelingMaterialCreatedSignature OnModelingMaterialCreated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModelingActorCreatedSignature, const FCreateActorResult& CreatedInfo);
	FModelingActorCreatedSignature OnModelingActorCreated;

	DECLARE_MULTICAST_DELEGATE_OneParam(FModelingComponentCreatedSignature, const FCreateComponentResult& CreatedInfo);
	FModelingComponentCreatedSignature OnModelingComponentCreated;

	//
	// Utility functions to handle registration/unregistration
	//

	static UE_API UEditorModelingObjectsCreationAPI* Register(UInteractiveToolsContext* ToolsContext);
	static UE_API UEditorModelingObjectsCreationAPI* Find(UInteractiveToolsContext* ToolsContext);
	static UE_API bool Deregister(UInteractiveToolsContext* ToolsContext);



	//
	// internal implementations called by public functions
	//
	UE_API FCreateMeshObjectResult CreateStaticMeshAsset(FCreateMeshObjectParams&& CreateMeshParams);
	UE_API FCreateMeshObjectResult CreateVolume(FCreateMeshObjectParams&& CreateMeshParams);
	UE_API FCreateMeshObjectResult CreateDynamicMeshActor(FCreateMeshObjectParams&& CreateMeshParams);
	UE_API ECreateModelingObjectResult GetNewAssetPath(FString& OutNewAssetPath, const FString& BaseName, const UObject* StoreRelativeToObject, const UWorld* World);

	UE_API TArray<UMaterialInterface*> FilterMaterials(const TArray<UMaterialInterface*>& MaterialsIn);
};

#undef UE_API
