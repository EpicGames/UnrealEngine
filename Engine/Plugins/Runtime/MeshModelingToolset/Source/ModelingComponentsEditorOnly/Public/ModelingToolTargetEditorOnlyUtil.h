// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h"

class USkeletalMesh;
//
// UE::ToolTarget:: namespace contains utility/helper functions for interacting with UToolTargets.
// Generally these are meant to be used by UInteractiveTools to handle standard tasks that would
// otherwise require each Tool to figure out things like which ToolTargetInterface to cast to, etc.
// Using these functions ideally avoids all the boilerplate inherent in the ToolTarget system.
// 
// However, the cost is that it is not necessarily the most efficient, as each one of these functions
// may potentially do many repeated Cast<>'s internally. So, use sparingly, or cache the outputs.
//
namespace UE
{
namespace ToolTarget
{

/** @return available LODs on the Skeletal Mesh  */
MODELINGCOMPONENTSEDITORONLY_API TArray<EMeshLODIdentifier> GetAvailableLODs(const USkeletalMesh* SkeletalMesh, bool bSkipAutoGenerated);
	
}  // end namespace ToolTarget
}  // end namespace UE