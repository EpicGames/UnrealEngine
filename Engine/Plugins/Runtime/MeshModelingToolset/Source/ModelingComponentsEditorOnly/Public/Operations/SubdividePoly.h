// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingOperators.h"
#include "SubdividePoly.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

PREDECLARE_USE_GEOMETRY_CLASS(FGroupTopology);
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class FProgressCancel;

/**
 * Subdivision scheme
 */
UENUM()
enum class ESubdivisionScheme : uint8
{
	// Subdivides like Catmull-Clark, but does not smooth the result (i.e. the vertices remain
	// in their original planes).
	Bilinear,
	// Common subdivision scheme typically used with quad-dominant meshes. In our tools, it uses
	// the group topology to build a cage that serves as the starting point.
	CatmullClark UMETA(DisplayName = "Catmull-Clark"),
	// Subdivision scheme developed by Charles Loop that operates directly on triangle meshes, and
	// therefore does not use the group topology. 
	Loop
};

UENUM()
enum class ESubdivisionBoundaryScheme : uint8
{
	// Corners with only one adjoining face are smoothed along with the rest of the mesh boundary. For
	// example, a square patch will have rounded corners.
	SmoothCorners,
	// Corners with only one adjoining face are constrained to be passed through. For example, this
	// will keep the corners of a vertically deformed square patch sharp even as the rest of the patch
	// is vertically smoothed.
	SharpCorners,

	//~ Not currently a supported option, and might not be one that is worth supporting since the same
	//~ result can be achieved by deleting faces on the boundary after subdividing...
	// Points along the boundary are treated purely as control points with no faces extending to them. This
	// is not commonly used, except when trying to seamlessly join different patches by replicating vertices.
	//NoBoundaryFaces
};



/**
 * Method for computing normals on the output mesh
 */
UENUM()
enum class ESubdivisionOutputNormals : uint8
{
	// Copy normals to poly mesh then interpolate to new triangles as we subdivide.
	Interpolated,

	// Recompute normals after subdivision
	Generated,

	// Leave normals uninitialized on output mesh
	None
};

/**
 * Method for computing UVs on the output mesh
 */
UENUM()
enum class ESubdivisionOutputUVs : uint8
{
	// Copy UVs to poly mesh vertices then interpolate as we subdivide.
	Interpolated,

	// Leave UVs uninitialized on output mesh
	None
};


/**
 * Interprets an FGroupTopology object as a poly mesh and refines it to a given level. Topological subdivision 
 * can be computed separately from the interpolation of vertex positions. A typical workflow might be to first 
 * precompute the refined topology, then compute the interpolated positions (and other vertex values) as the original 
 * polymesh is deformed.
 */
class FSubdividePoly
{
public:

	UE_API FSubdividePoly(const FGroupTopology& InTopology,
				   const FDynamicMesh3& InOriginalMesh,
				   int InLevel);

	UE_API ~FSubdividePoly();

	enum class ETopologyCheckResult
	{
		Ok = 0,
		NoGroups = 1,
		InsufficientGroups = 2,
		UnboundedPolygroup = 3,
		MultiBoundaryPolygroup = 4,
		DegeneratePolygroup = 5
	};
	UE_API ETopologyCheckResult ValidateTopology();

	UE_API bool ComputeTopologySubdivision();

	UE_API bool ComputeSubdividedMesh(FDynamicMesh3& OutMesh);

	ESubdivisionScheme SubdivisionScheme = ESubdivisionScheme::CatmullClark;

	ESubdivisionBoundaryScheme BoundaryScheme = ESubdivisionBoundaryScheme::SharpCorners;

	ESubdivisionOutputNormals NormalComputationMethod = ESubdivisionOutputNormals::Generated;

	ESubdivisionOutputUVs UVComputationMethod = ESubdivisionOutputUVs::Interpolated;

	bool bNewPolyGroups = false;

	// wrapper around Opensubdiv::TopologyRefiner to avoid including OSD headers here. See cpp file.
	class RefinerImpl;

private:

	const FGroupTopology& GroupTopology;
	const FDynamicMesh3& OriginalMesh;
	int Level;

	TUniquePtr<RefinerImpl> Refiner;
};

#undef UE_API
