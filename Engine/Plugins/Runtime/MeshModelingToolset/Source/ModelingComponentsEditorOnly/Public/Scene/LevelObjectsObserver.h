// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class UWorld;
class AActor;

/**
 * FLevelObjectsObserver is a utility class that tracks the active-set of Actors
 * in a World, in the Editor, and emits events when the set changes. The goal is to
 * guarantee that:
 *   - OnActorAdded will be emitted when an Actor comes into existence
 *   - OnActorRemoved will be emitted when an Actor ceases to exist (possibly after GC, though)
 *   - these signals will be unique, ie once per Actor
 * 
 * This is a surprisingly complicated problem in the Editor due to the many ways
 * that Actors might be created or destroyed, in particular via Undo/Redo. 
 *
 * Note that it is not possible to guarantee that OnActorRemoved() fires before an Actor
 * is GC'd. So, listeners should maintain TWeakObjectPtr<>'s on Actors if they need
 * to know this information.
 * 
 */
class FLevelObjectsObserver : public FEditorUndoClient
{
public:
	/**
	 * Destructor.
	 */
	UE_API virtual ~FLevelObjectsObserver();

	/**
	 * Begin watching WorldIn. This will emit OnActorAdded for all Actors in the World.
	 */
	UE_API void Initialize(UWorld* WorldIn);

	/**
	 * Stop watching WorldIn. This will emit OnActorRemoved for all Actors that the Observer is currently aware of.
	 */
	UE_API void Shutdown();

	// FEditorUndoClient implementation
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;

	DECLARE_MULTICAST_DELEGATE_OneParam(FActorAddRemoveSignature, AActor*);

	/**
	 * OnActorAdded is emitted when a new Actor is detected
	 */
	FActorAddRemoveSignature OnActorAdded;
	/**
	 * OnActorRemoved is emitted when an existing Actor is destroyed.
	 * Note that the Actor pointer passed here may point to garbage, ie the Actor
	 * may have already been GC'd. Do not attempt to call functions on the pointer.
	 */
	FActorAddRemoveSignature OnActorRemoved;

protected:
	UWorld* World = nullptr;		// target World
	TSet<AActor*> Actors;			// active set of Actors

	FDelegateHandle OnActorDeletedHandle;
	FDelegateHandle OnActorAddedHandle;
	FDelegateHandle OnActorListChangedHandle;

	UE_API void OnUntrackedLevelChange();
	UE_API void HandleActorAddedEvent(AActor* Actor);
	UE_API void HandleActorDeletedEvent(AActor* Actor);
};

#undef UE_API
