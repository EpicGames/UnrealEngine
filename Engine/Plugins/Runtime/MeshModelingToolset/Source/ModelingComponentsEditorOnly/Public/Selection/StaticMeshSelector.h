// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Selection/GeometrySelector.h"
#include "Selection/DynamicMeshSelector.h"

#include "UObject/StrongObjectPtr.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API


class UDynamicMesh;
class UStaticMesh;
class UStaticMeshComponent;


class FStaticMeshSelector : public FBaseDynamicMeshSelector
{
public:
	using FBaseDynamicMeshSelector::Initialize;

	UE_API virtual bool Initialize(
		FGeometryIdentifier SourceGeometryIdentifier);

	//
	// IGeometrySelector API implementation
	//

	UE_API virtual void Shutdown() override;
	// disable sleep on StaticMesh Selector until we can properly track changes...
	virtual bool SupportsSleep() const override { return false; }
	//virtual bool Sleep();
	//virtual bool Restore();


	UE_API virtual bool IsLockable() const override;
	UE_API virtual bool IsLocked() const override;
	UE_API virtual void SetLockedState(bool bLocked) override;


	UE_API virtual IGeometrySelectionTransformer* InitializeTransformation(const FGeometrySelection& Selection) override;
	UE_API virtual void ShutdownTransformation(IGeometrySelectionTransformer* Transformer) override;

	UE_API virtual void UpdateAfterGeometryEdit(
		IToolsContextTransactionsAPI* TransactionsAPI,
		bool bInTransaction,
		TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChange,
		FText GeometryEditTransactionString) override;

protected:
	TWeakObjectPtr<UStaticMeshComponent> WeakStaticMeshComponent = nullptr;
	TWeakObjectPtr<UStaticMesh> WeakStaticMesh = nullptr;
	FDelegateHandle StaticMesh_OnMeshChangedHandle;

	// TODO: this is not a great design, it would be better if something external could own this mesh...
	TStrongObjectPtr<UDynamicMesh> LocalTargetMesh;
	UE_API void CopyFromStaticMesh();

	TSharedPtr<FBasicDynamicMeshSelectionTransformer> ActiveTransformer;
	UE_API void CommitMeshTransform();


public:
	static UE_API void SetAssetUnlockedOnCreation(UStaticMesh* StaticMesh);
	static UE_API void ResetUnlockedStaticMeshAssets();
};



/**
 * FStaticMeshComponentSelectorFactory constructs FStaticMeshSelector instances 
 * for UStaticMeshComponents
 */
class FStaticMeshComponentSelectorFactory : public IGeometrySelectorFactory
{
public:
	UE_API virtual bool CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const;
	UE_API virtual TUniquePtr<IGeometrySelector> BuildForTarget(FGeometryIdentifier TargetIdentifier) const;
};




#undef UE_API
