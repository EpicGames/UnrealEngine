// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Selection/GeometrySelector.h"
#include "Selection/DynamicMeshSelector.h"

#include "UObject/StrongObjectPtr.h"
#include "EditorUndoClient.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class UDynamicMesh;
class AVolume;
class UBrushComponent;


class FVolumeSelector : public FBaseDynamicMeshSelector, public FEditorUndoClient
{
public:
	using FBaseDynamicMeshSelector::Initialize;

	UE_API virtual bool Initialize(
		FGeometryIdentifier SourceGeometryIdentifier);

	//
	// IGeometrySelector API implementation
	//

	UE_API virtual void Shutdown() override;
	// disable sleep on VolumeSelector until we can properly track changes...
	virtual bool SupportsSleep() const override { return false; }
	//virtual bool Sleep();
	//virtual bool Restore();

	UE_API virtual bool IsLockable() const override;
	UE_API virtual bool IsLocked() const override;
	UE_API virtual void SetLockedState(bool bLocked) override;

	UE_API virtual IGeometrySelectionTransformer* InitializeTransformation(const FGeometrySelection& Selection) override;
	UE_API virtual void ShutdownTransformation(IGeometrySelectionTransformer* Transformer) override;

	UE_API virtual void UpdateAfterGeometryEdit(
		IToolsContextTransactionsAPI* TransactionsAPI,
		bool bInTransaction,
		TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChange,
		FText GeometryEditTransactionString) override;


	// FEditorUndoClient implementation
	UE_API virtual void PostUndo(bool bSuccess) override;
	UE_API virtual void PostRedo(bool bSuccess) override;

public:
	static UE_API void SetComponentUnlockedOnCreation(UBrushComponent* Component);
	static UE_API void ResetUnlockedBrushComponents();

protected:
	AVolume* ParentVolume = nullptr;
	UBrushComponent* BrushComponent = nullptr;

	// TODO: this is not a great design, it would be better if something external could own this mesh...
	TStrongObjectPtr<UDynamicMesh> LocalTargetMesh;

	UE_API void UpdateDynamicMeshFromVolume();

	TSharedPtr<FBasicDynamicMeshSelectionTransformer> ActiveTransformer;
	UE_API void CommitMeshTransform();
};




/**
 * FVolumeComponentSelectorFactory constructs FVolumeSelector instances 
 * for UBrushComponents
 */
class FBrushComponentSelectorFactory : public IGeometrySelectorFactory
{
public:
	UE_API virtual bool CanBuildForTarget(FGeometryIdentifier TargetIdentifier) const;
	UE_API virtual TUniquePtr<IGeometrySelector> BuildForTarget(FGeometryIdentifier TargetIdentifier) const;
};




#undef UE_API
