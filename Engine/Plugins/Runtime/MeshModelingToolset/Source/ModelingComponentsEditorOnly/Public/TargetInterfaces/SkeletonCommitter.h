// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "SkeletonCommitter.generated.h"


class USkeletonModifier;

UINTERFACE(MinimalAPI)
class USkeletonCommitter: public UInterface
{
	GENERATED_BODY()
};

class ISkeletonCommitter
{
	GENERATED_BODY()

public:
	virtual void SetupSkeletonModifier(USkeletonModifier* InModifier) = 0;
	virtual void CommitSkeletonModifier(USkeletonModifier* InModifier) = 0;
};