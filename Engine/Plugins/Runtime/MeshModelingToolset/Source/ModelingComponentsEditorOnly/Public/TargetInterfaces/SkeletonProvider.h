// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "SkeletonProvider.generated.h"


struct FReferenceSkeleton;

UINTERFACE(MinimalAPI)
class USkeletonProvider : public UInterface
{
	GENERATED_BODY()
};

class ISkeletonProvider
{
	GENERATED_BODY()

public:
	virtual FReferenceSkeleton GetSkeleton() const = 0;
};