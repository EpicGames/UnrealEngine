// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/DynamicMeshSource.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "TargetInterfaces/SkeletonCommitter.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"

#include "DynamicMeshComponentToolTarget.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class UDynamicMesh;

/**
 * A ToolTarget backed by a DynamicMeshComponent
 */
UCLASS(MinimalAPI, Transient)
class UDynamicMeshComponentToolTarget : 
	public UPrimitiveComponentToolTarget,
	public IMeshDescriptionCommitter, 
	public IMeshDescriptionProvider, 
	public IDynamicMeshProvider,
	public IDynamicMeshCommitter,
	public IMaterialProvider,
	public IPersistentDynamicMeshSource,
	public IPhysicsDataSource,
	public ISkeletonProvider,
	public ISkeletonCommitter
{
	GENERATED_BODY()

public:
	UE_API virtual bool IsValid() const override;


public:
	// IMeshDescriptionProvider implementation
	UE_API const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;

	// IMeshDescritpionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// IDynamicMeshProvider implementation
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;

	// IDynamicMeshCommitter implementation
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

	// IMaterialProvider implementation
	UE_API int32 GetNumMaterials() const override;
	UE_API UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;	

	// IPersistentDynamicMeshSource implementation
	UE_API virtual UDynamicMesh* GetDynamicMeshContainer() override;
	UE_API virtual void CommitDynamicMeshChange(TUniquePtr<FToolCommandChange> Change, const FText& ChangeMessage) override;
	UE_API virtual bool HasDynamicMeshComponent() const override;
	UE_API virtual UDynamicMeshComponent* GetDynamicMeshComponent() override;

	// IPhysicsDataSource implementation
	UE_API virtual UBodySetup* GetBodySetup() const override;
	UE_API virtual IInterface_CollisionDataProvider* GetComplexCollisionProvider() const override;

	// ISkeletonProvider implementation
	UE_API virtual FReferenceSkeleton GetSkeleton() const override;
	
	// ISkeletonCommitter implementation
	UE_API virtual void SetupSkeletonModifier(USkeletonModifier* InModifier) override;
	UE_API virtual void CommitSkeletonModifier(USkeletonModifier* InModifier) override;
	

	// Rest provided by parent class

protected:
	TUniquePtr<FMeshDescription> ConvertedMeshDescription;
	bool bHaveMeshDescription = false;

protected:
	friend class UDynamicMeshComponentToolTargetFactory;
};


/** Factory for UDynamicMeshComponentToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class UDynamicMeshComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};

#undef UE_API
