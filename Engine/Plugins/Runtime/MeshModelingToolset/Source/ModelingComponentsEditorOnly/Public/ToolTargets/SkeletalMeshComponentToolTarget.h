// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "TargetInterfaces/SkeletonCommitter.h"
#include "TargetInterfaces/SkeletonProvider.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"

#include "SkeletalMeshComponentToolTarget.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API


class USkeletalMesh;


/**
 * A tool target backed by a skeletal mesh component that can provide and take a mesh
 * description, without 'committer' interfaces for writing to the source skeletal mesh asset.
 */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshComponentReadOnlyToolTarget :
	public UPrimitiveComponentToolTarget,
	public IMeshDescriptionProvider,
	public IDynamicMeshProvider, 
	public IMaterialProvider,
	public ISkeletalMeshBackedTarget,
	public ISkeletonProvider
{
	GENERATED_BODY()

public:

	// UToolTarget implementation
	UE_API virtual bool IsValid() const override;

	// IMeshDescriptionProvider implementation
	UE_API virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;
	virtual bool SupportsLODs() const override { return true; }
	UE_API virtual TArray<EMeshLODIdentifier> GetAvailableLODs(bool bSkipAutoGenerated = true) const override;
	virtual EMeshLODIdentifier GetMeshDescriptionLOD() const override { return EditingLOD; }

	
	// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;

	// IDynamicMeshProvider
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh(const FGetMeshParameters& InGetMeshParams) override;

	// ISkeletalMeshBackedTarget implementation
	UE_API virtual USkeletalMesh* GetSkeletalMesh() const override;

	// ISkeletonProvider implementation
	UE_API virtual FReferenceSkeleton GetSkeleton() const override;
	
protected:
	// LOD to edit, default is to edit LOD0
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;	
	
	// So that the tool target factory can poke into Component.
	friend class USkeletalMeshComponentReadOnlyToolTargetFactory;
	friend class USkeletalMeshComponentToolTargetFactory;
};


/**
 * A tool target backed by a skeletal mesh component that can provide and take a mesh
 * description.
 */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshComponentToolTarget :
	public USkeletalMeshComponentReadOnlyToolTarget,
	public IMeshDescriptionCommitter,
	public IDynamicMeshCommitter,
	public ISkeletonCommitter
{
	GENERATED_BODY()

public:
	// IMeshDescriptionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// IDynamicMeshCommitter
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

	// ISkeletonCommitter
	UE_API virtual void SetupSkeletonModifier(USkeletonModifier* InModifier) override;
	UE_API virtual void CommitSkeletonModifier(USkeletonModifier* InModifier) override;
	
protected:
	// So that the tool target factory can poke into Component.
	friend class USkeletalMeshComponentToolTargetFactory;
};


/** Factory for USkeletalMeshComponentReadOnlyToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshComponentReadOnlyToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};


/** Factory for USkeletalMeshComponentToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;

	static UE_API bool CanWriteToSource(const UObject* SourceObject);
	
	virtual EMeshLODIdentifier GetActiveEditingLOD() const { return EditingLOD; }
	UE_API virtual void SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD);

protected:
	// LOD to edit, default is to edit LOD0
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;
};

#undef UE_API
