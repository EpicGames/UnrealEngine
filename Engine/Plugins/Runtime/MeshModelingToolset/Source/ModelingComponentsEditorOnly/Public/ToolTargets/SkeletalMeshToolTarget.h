// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/SkeletalMeshBackedTarget.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"

#include "SkeletalMeshToolTarget.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API


class USkeletalMesh;

/**
 * A tool target backed by a read-only skeletal mesh.
 * 
 * This is a special tool target that refers to the underlying asset (in this case a skeletal mesh), rather than indirectly through a component.
 * This type of target is used in cases, such as opening an asset through the content browser, when there is no component available.
 */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshReadOnlyToolTarget :
	public UToolTarget,
	public IMeshDescriptionProvider,
	public IDynamicMeshProvider,
	public IMaterialProvider,
	public ISkeletalMeshBackedTarget
{
	GENERATED_BODY()

public:
	// UToolTarget
	UE_API virtual bool IsValid() const override;

	// IMeshDescriptionProvider implementation
	UE_API virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;
	virtual bool SupportsLODs() const override { return true; }
	UE_API virtual TArray<EMeshLODIdentifier> GetAvailableLODs(bool bSkipAutoGenerated = true) const override;
	virtual EMeshLODIdentifier GetMeshDescriptionLOD() const override { return EditingLOD; }

	// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;	

	// IDynamicMeshProvider
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh(const FGetMeshParameters& InGetMeshParams) override;

	// ISkeletalMeshBackedTarget implementation
	UE_API virtual USkeletalMesh* GetSkeletalMesh() const override;

protected:
	TWeakObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	// The LOD to edit if requested to get/commit at EMeshLODIdentifier::Default or if no specific LOD is requested.
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;
	
	// So that the tool target factory can poke into Component.
	friend class USkeletalMeshReadOnlyToolTargetFactory;
	friend class USkeletalMeshToolTargetFactory;
	friend class USkeletalMeshComponentReadOnlyToolTarget;
	friend class USkeletalMeshComponentToolTarget;

	UE_DEPRECATED(5.3, "This function doesn't check anything skeletal mesh specific, only (badly) checks to see if the object is alive")
	static UE_API bool IsValid(const USkeletalMesh* USkeletalMesh);

	static UE_API TArray<EMeshLODIdentifier> GetAvailableLODs(const USkeletalMesh* SkeletalMesh, bool bSkipAutoGenerated);	
	static UE_API void GetMaterialSet(const USkeletalMesh* SkeletalMesh, FComponentMaterialSet& MaterialSetOut,
		bool bPreferAssetMaterials);
	static UE_API bool CommitMaterialSetUpdate(USkeletalMesh* SkeletalMesh,
		const FComponentMaterialSet& MaterialSet, bool bApplyToAsset);
	static UE_API int32 GetValidEditingLOD(const USkeletalMesh* SkeletalMesh, EMeshLODIdentifier RequestedEditingLOD);	
};


/**
 * A tool target backed by a skeletal mesh.
 */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshToolTarget :
	public USkeletalMeshReadOnlyToolTarget,
	public IMeshDescriptionCommitter,
	public IDynamicMeshCommitter
{
	GENERATED_BODY()

public:
	// IMeshDescriptionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// IDynamicMeshCommitter
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, 
		const FDynamicMeshCommitInfo& CommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

protected:
	// So that the tool target factory can poke into Component.
	friend class USkeletalMeshToolTargetFactory;
	friend class USkeletalMeshComponentToolTarget;

	static UE_API void CommitMeshDescription(USkeletalMesh* SkeletalMesh, const FCommitter& Committer, int32 LODIndex);
};


/** Factory for USkeletalMeshReadOnlyToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshReadOnlyToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};


/** Factory for USkeletalMeshToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class USkeletalMeshToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;

	static UE_API bool CanWriteToSource(const UObject* SourceObject);

	virtual EMeshLODIdentifier GetActiveEditingLOD() const { return EditingLOD; }
	UE_API virtual void SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD);

protected:
	// LOD to edit, default is to edit LOD0
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;
};

#undef UE_API
