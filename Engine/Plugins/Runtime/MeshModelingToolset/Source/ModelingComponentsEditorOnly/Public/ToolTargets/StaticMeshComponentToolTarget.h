// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "ToolTargets/StaticMeshToolTarget.h" // for FMeshDescriptionCache
#include "ComponentSourceInterfaces.h"  // for EMeshLODIdentifier

#include "StaticMeshComponentToolTarget.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class UStaticMesh;

/**
 * A tool target backed by a static mesh component that can provide and take a mesh
 * description, without 'committer' interfaces for writing to the source static mesh asset.
 */
UCLASS(MinimalAPI, Transient)
class UStaticMeshComponentReadOnlyToolTarget : public UPrimitiveComponentToolTarget,
	public IMeshDescriptionProvider, public IMaterialProvider, public IStaticMeshBackedTarget,
	public IDynamicMeshProvider, public IPhysicsDataSource
{
	GENERATED_BODY()

public:
	/**
	 * Configure active LOD to edit. Can only call this after Component is configured in base UPrimitiveComponentToolTarget.
	 * If requested LOD does not exist, fall back to one that does.
	 */
	UE_API virtual void SetEditingLOD(EMeshLODIdentifier RequestedEditingLOD);

	/** @return current editing LOD */
	virtual EMeshLODIdentifier GetEditingLOD() const { return EditingLOD; }

public:
	UE_API virtual bool IsValid() const override;


public:
	// IMeshDescriptionProvider implementation
	UE_API virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetMeshDescriptionCopy(const FGetMeshParameters& GetMeshParams) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;
	virtual bool SupportsLODs() const override { return true; }
	UE_API virtual TArray<EMeshLODIdentifier> GetAvailableLODs(bool bSkipAutoGenerated = true) const override;
	virtual EMeshLODIdentifier GetMeshDescriptionLOD() const override { return EditingLOD; }
	UE_API virtual TArray<int32> GetPolygonGroupToMaterialIndexMap() const override;



	// IDynamicMeshProvider
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh(const FGetMeshParameters& InGetMeshParams) override;


	// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;	

	// IStaticMeshBackedTarget
	UE_API virtual UStaticMesh* GetStaticMesh() const override;

	// IPhysicsDataSource
	UE_API virtual UBodySetup* GetBodySetup() const override;
	UE_API virtual IInterface_CollisionDataProvider* GetComplexCollisionProvider() const override;

	// Rest provided by parent class

public:

protected:
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;

	friend class UStaticMeshComponentToolTargetFactory;

private:
	UStaticMeshToolTarget::FMeshDescriptionCache CachedMeshDescriptions;
};

/**
 * A tool target backed by a static mesh component that can provide and take a mesh
 * description.
 */
UCLASS(MinimalAPI, Transient)
class UStaticMeshComponentToolTarget : 
	public UStaticMeshComponentReadOnlyToolTarget,
	public IMeshDescriptionCommitter, public IDynamicMeshCommitter
{
	GENERATED_BODY()

	// IDynamicMeshCommitter
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload
	
	// IMeshDescriptionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// Note: This method is on both the IMeshDescriptionCommitter and IMeshDescriptionProvider interface.
	// The UStaticMeshComponentReadOnlyToolTarget base class only implements the Provider; we lift it to here
	// so that the IMeshDescriptionCommitter interface will also find/call it.
	virtual TArray<int32> GetPolygonGroupToMaterialIndexMap() const override
	{
		return UStaticMeshComponentReadOnlyToolTarget::GetPolygonGroupToMaterialIndexMap();
	}
};


/** Factory for UStaticMeshComponentToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class UStaticMeshComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;

public:
	virtual EMeshLODIdentifier GetActiveEditingLOD() const { return EditingLOD; }
	UE_API virtual void SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD);

	static UE_API bool CanWriteToSource(const UObject* Source);

protected:
	// LOD to edit, default is to edit LOD0
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;

private:
	static UE_API const UStaticMesh* SourceToStaticMesh(const UObject* SourceObject);
};

#undef UE_API
