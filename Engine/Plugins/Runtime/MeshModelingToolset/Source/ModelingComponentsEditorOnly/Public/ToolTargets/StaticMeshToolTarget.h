// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/StaticMeshBackedTarget.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "ComponentSourceInterfaces.h"  // for EMeshLODIdentifier

#include "StaticMeshToolTarget.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class UStaticMesh;

/**
 * A tool target backed by a static mesh asset that can provide and take a mesh
 * description, without 'committer' interfaces for writing to the source static mesh asset.
 * 
 * This is a special tool target that refers to the underlying asset (in this case a static mesh), rather than indirectly through a component.
 * This type of target is used in cases, such as opening an asset through the content browser, when there is no component available.
 */
UCLASS(MinimalAPI, Transient)
class UStaticMeshReadOnlyToolTarget : 
	public UToolTarget,
	public IMeshDescriptionProvider, 
	public IMaterialProvider, 
	public IStaticMeshBackedTarget,
	public IDynamicMeshProvider
{
	GENERATED_BODY()

public:
	/**
	 * Configure active LOD to edit. Can only call this after Component is configured in base UPrimitiveComponentToolTarget.
	 * If requested LOD does not exist, fall back to one that does.
	 */
	UE_API virtual void SetEditingLOD(EMeshLODIdentifier RequestedEditingLOD);

	/** @return current editing LOD */
	virtual EMeshLODIdentifier GetEditingLOD() const { return EditingLOD; }

	// UToolTarget
	UE_API virtual bool IsValid() const override;

	// IMeshDescriptionProvider implementation
	UE_API virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;
	virtual bool SupportsLODs() const override { return true; }
	UE_API virtual TArray<EMeshLODIdentifier> GetAvailableLODs(bool bSkipAutoGenerated = true) const override;
	virtual EMeshLODIdentifier GetMeshDescriptionLOD() const override { return EditingLOD; }
	UE_API virtual TArray<int32> GetPolygonGroupToMaterialIndexMap() const override;

	// helper to get the material mapping returned by GetPolygonGroupToMaterialIndexMap for any static mesh
	static UE_API TArray<int32> MapSectionToMaterialID(const UStaticMesh* Mesh, EMeshLODIdentifier EditingLOD);


	// IDynamicMeshProvider
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh(const FGetMeshParameters& InGetMeshParams) override;


	// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override;	

	// IStaticMeshBackedTarget
	UE_API virtual UStaticMesh* GetStaticMesh() const override;

	// Rest provided by parent class


	// helper struct to cache mesh descriptions, so we can provide the const ptr GetMeshDescription API while still handling the Build Scale
	struct FCachedMeshDescription
	{
		FMeshDescription* Source;
		TUniquePtr<FMeshDescription> Copy;
	};
	using FMeshDescriptionCache = TMap<int32, FCachedMeshDescription>;

	// helper to get the mesh description with build scale applied, using/updating the cache as needed
	struct FGetMeshDescriptionOptions
	{
		bool bAllowRecomputeNormals = true;
		bool bAllowRecomputeTangents = true;
		static FGetMeshDescriptionOptions Default()
		{
			return FGetMeshDescriptionOptions{};
		}
	};
	static UE_API const FMeshDescription* GetMeshDescriptionWithScaleApplied(const UStaticMesh* Mesh, int32 UseLOD, FMeshDescriptionCache& Cache, const FGetMeshDescriptionOptions& Options = FGetMeshDescriptionOptions::Default());

protected:
	TWeakObjectPtr<UStaticMesh> StaticMesh = nullptr;

	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;

	friend class UStaticMeshToolTargetFactory;

	friend class UStaticMeshComponentReadOnlyToolTarget;

	friend class UStaticMeshComponentToolTargetFactory;

	// returns true if the mesh has a non-generated LOD of the requested level.  Interprets EMeshLODIdentifier::Default as LOD0
	static UE_API bool HasNonGeneratedLOD(const UStaticMesh* StaticMesh, EMeshLODIdentifier MeshLOD);
	static UE_API EMeshLODIdentifier GetValidEditingLOD(const UStaticMesh* StaticMesh, 
		EMeshLODIdentifier RequestedEditingLOD);
	static UE_API TArray<EMeshLODIdentifier> GetAvailableLODs(const UStaticMesh* StaticMesh, bool bSkipAutoGenerated);
	static UE_API void GetMaterialSet(const UStaticMesh* StaticMesh, 
		FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials);
	static UE_API bool CommitMaterialSetUpdate(UStaticMesh* SkeletalMesh, 
		const FComponentMaterialSet& MaterialSet, bool bApplyToAsset);

private:
	// In some cases we need to modify the mesh description that we pass to tools, e.g. to account for the BuildScale,
	// and in those cases we retain the storage here to cover the lifetime of the pointer returned by GetMeshDescription()
	FMeshDescriptionCache CachedMeshDescriptions;
};

/**
 * A tool target backed by a static mesh asset that can provide and take a mesh
 * description.
 *
 * This is a special tool target that refers to the underlying asset (in this case a static mesh), rather than indirectly through a component.
 * This type of target is used in cases, such as opening an asset through the content browser, when there is no component available.
 */
UCLASS(MinimalAPI, Transient)
class UStaticMeshToolTarget : 
	public UStaticMeshReadOnlyToolTarget,
	public IMeshDescriptionCommitter,
	public IDynamicMeshCommitter
{
	GENERATED_BODY()
	
public:
	// IMeshDescriptionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// IDynamicMeshCommitter
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo& CommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

	// Note: This method is on both the IMeshDescriptionCommitter and IMeshDescriptionProvider interface.
	// The UStaticMeshReadOnlyToolTarget base class only implements the Provider; we lift it to here
	// so that the IMeshDescriptionCommitter interface will also find/call it.
	virtual TArray<int32> GetPolygonGroupToMaterialIndexMap() const override
	{
		return UStaticMeshReadOnlyToolTarget::GetPolygonGroupToMaterialIndexMap();
	}

protected:
	friend class UStaticMeshComponentToolTarget;

	static UE_API void CommitMeshDescription(UStaticMesh* StaticMesh,
		const FCommitter& Committer, EMeshLODIdentifier EditingLODIn);
};


/** Factory for UStaticMeshToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class UStaticMeshToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;



public:
	virtual EMeshLODIdentifier GetActiveEditingLOD() const { return EditingLOD; }
	UE_API virtual void SetActiveEditingLOD(EMeshLODIdentifier NewEditingLOD);

	static UE_API bool CanWriteToSource(const UObject* Source);

protected:
	// LOD to edit, default is to edit LOD0
	EMeshLODIdentifier EditingLOD = EMeshLODIdentifier::LOD0;
};

#undef UE_API
