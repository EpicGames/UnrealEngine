// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ComponentSourceInterfaces.h"
#include "ConversionUtils/VolumeToDynamicMesh.h"
#include "MeshDescription.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

class FVolumeComponentTargetFactory : public FComponentTargetFactory
{
public:
	UE_API bool CanBuild(UActorComponent* Candidate) override;
	UE_API TUniquePtr<FPrimitiveComponentTarget> Build(UPrimitiveComponent* PrimitiveComponent) override;
};

/** Deprecated. Use the tool target instead. */
class FVolumeComponentTarget : public FPrimitiveComponentTarget
{
public:

	UE_API FVolumeComponentTarget(UPrimitiveComponent* Component);

	UE_API FMeshDescription* GetMesh() override;
	UE_API void CommitMesh(const FCommitter&) override;

	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bAssetMaterials) const override;
	virtual void CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override {};

	UE_API virtual bool HasSameSourceData(const FPrimitiveComponentTarget& OtherTarget) const override;

	const UE::Conversion::FVolumeToMeshOptions& GetVolumeToMeshOptions() { return VolumeToMeshOptions; }

protected:
	TUniquePtr<FMeshDescription> ConvertedMeshDescription;

	UE::Conversion::FVolumeToMeshOptions VolumeToMeshOptions;
};

#undef UE_API
