// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ConversionUtils/VolumeToDynamicMesh.h" // FVolumeToMeshOptions
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionCommitter.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "HAL/IConsoleManager.h"

#include "VolumeComponentToolTarget.generated.h"

#define UE_API MODELINGCOMPONENTSEDITORONLY_API

/**
 * The CVar "modeling.VolumeMaxTriCount" is used as a cap on triangles that the various Modeling Mode
 * Tools will allow an output AVolume to have. If this triangle count is exceeded, the mesh used to
 * create/update the AVolume will be auto-simplified. This is necessary because all AVolume process is
 * done on the game thread, and a large Volume (eg with 100k faces) will hang the editor for a long time
 * when it is created. The default is set to 500.
 */
extern MODELINGCOMPONENTSEDITORONLY_API TAutoConsoleVariable<int32> CVarModelingMaxVolumeTriangleCount;

/**
 * A tool target backed by AVolume
 */
UCLASS(MinimalAPI, Transient)
class UVolumeComponentToolTarget : public UPrimitiveComponentToolTarget,
	public IMaterialProvider, public IPhysicsDataSource,
	public IDynamicMeshCommitter, public IDynamicMeshProvider,
	public IMeshDescriptionCommitter, public IMeshDescriptionProvider
{
	GENERATED_BODY()

public:

	UE_API UVolumeComponentToolTarget();

	const UE::Conversion::FVolumeToMeshOptions& GetVolumeToMeshOptions() { return VolumeToMeshOptions; }

	// IDynamicMeshProvider implementation
	UE_API virtual UE::Geometry::FDynamicMesh3 GetDynamicMesh() override;

	// IDynamicMeshCommitter implementation
	UE_API virtual void CommitDynamicMesh(const UE::Geometry::FDynamicMesh3& Mesh, const FDynamicMeshCommitInfo&) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

	// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	// Ignores bPreferAssetMaterials
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& MaterialSetOut, bool bPreferAssetMaterials) const override;

	// Doesn't actually do anything for a volume
	virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& MaterialSet, bool bApplyToAsset) override { return false; }

	// IMeshDescriptionProvider implementation
	UE_API virtual const FMeshDescription* GetMeshDescription(const FGetMeshParameters& GetMeshParams = FGetMeshParameters()) override;
	UE_API virtual FMeshDescription GetEmptyMeshDescription() override;

	// IMeshDescritpionCommitter implementation
	UE_API virtual void CommitMeshDescription(const FCommitter& Committer, const FCommitMeshParameters& CommitMeshParams = FCommitMeshParameters()) override;
	using IMeshDescriptionCommitter::CommitMeshDescription; // unhide the other overload

	// IPhysicsDataSource implementation
	UE_API virtual UBodySetup* GetBodySetup() const override;
	// always returns null because volumes do not support IInterface_CollisionDataProvider
	virtual IInterface_CollisionDataProvider* GetComplexCollisionProvider() const override { return nullptr; }

	// Rest provided by parent class

protected:
	UE::Conversion::FVolumeToMeshOptions VolumeToMeshOptions;

	// This isn't for caching- we have to take ownership of the mesh description because it is
	// expected for things like a static mesh.
	TSharedPtr<FMeshDescription, ESPMode::ThreadSafe> ConvertedMeshDescription;

	friend class UVolumeComponentToolTargetFactory;
};

/** Factory for UVolumeComponentToolTarget to be used by the target manager. */
UCLASS(MinimalAPI, Transient)
class UVolumeComponentToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) override;
};

#undef UE_API
