// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelingOperators : ModuleRules
{
	public ModelingOperators(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"InteractiveToolsFramework",
				"GeometryCore",
				"DynamicMesh",
				"MeshConversion",
				"GeometryAlgorithms", // required for constrained Delaunay triangulation
				"SkeletalMeshDescription",
				"TextureUtilitiesCommon", // required for UDIM manipulation
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				// ... add private dependencies that you statically link with here ...
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}