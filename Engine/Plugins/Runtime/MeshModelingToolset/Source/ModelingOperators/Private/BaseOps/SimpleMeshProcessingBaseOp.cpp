// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseOps/SimpleMeshProcessingBaseOp.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"

using namespace UE::Geometry;

void FMeshBoundaryCache::Calculate(const FDynamicMesh3& Mesh)
{
	int32 NV = Mesh.MaxVertexID();

	// cache boundary verts info
	bIsBoundary.SetNum(NV);
	ParallelFor(NV, [&](int32 vid)
	{
		bIsBoundary[vid] = Mesh.IsBoundaryVertex(vid) && Mesh.IsReferencedVertex(vid);
	});
	for (int32 vid = 0; vid < NV; ++vid)
	{
		if (bIsBoundary[vid])
		{
			BoundaryVerts.Add(vid);
		}
	}
}


FSimpleMeshProcessingBaseOp::FSimpleMeshProcessingBaseOp(const FDynamicMesh3* Mesh) :
	FDynamicMeshOperator()
{
	// deep copy the src mesh into the result mesh.  This ResultMesh will be directly updated by the smoothing.
	ResultMesh->Copy(*Mesh);

	PositionBuffer.SetNum(ResultMesh->MaxVertexID());
	for (int vid : ResultMesh->VertexIndicesItr())
	{
		PositionBuffer[vid] = ResultMesh->GetVertex(vid);
	}
}

void FSimpleMeshProcessingBaseOp::SetTransform(const FTransformSRT3d& XForm)
{
	ResultTransform = XForm;
}



void FSimpleMeshProcessingBaseOp::UpdateResultMeshPositions()
{
	for (int32 vid : ResultMesh->VertexIndicesItr())
	{
		const FVector3d Pos = PositionBuffer[vid];
		ResultMesh->SetVertex(vid, Pos);
	}
}


void FSimpleMeshProcessingBaseOp::UpdateResultMeshNormals()
{
	if (ResultMesh->HasAttributes() && ResultMesh->Attributes()->PrimaryNormals() != nullptr)
	{
		FMeshNormals Normals(ResultMesh.Get());
		FDynamicMeshNormalOverlay* NormalOverlay = ResultMesh->Attributes()->PrimaryNormals();
		Normals.RecomputeOverlayNormals(NormalOverlay);
		Normals.CopyToOverlay(NormalOverlay);
	}
	else
	{
		FMeshNormals::QuickComputeVertexNormals(*ResultMesh);
	}
}


