// Copyright Epic Games, Inc. All Rights Reserved.

#include "BaseOps/VoxelBaseOp.h"

#include "CleaningOps/EditNormalsOp.h"

#include "DynamicMeshEditor.h"
#include "MeshSimplification.h"
#include "DynamicMesh/MeshNormals.h"
#include "Operations/RemoveOccludedTriangles.h"

using namespace UE::Geometry;

void FVoxelBaseOp::PostProcessResult(FProgressCancel* Progress, double MeshCellSize)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (bRemoveInternalSurfaces)
	{
		UE::MeshAutoRepair::RemoveInternalTriangles(*ResultMesh.Get(), true, EOcclusionTriangleSampling::Centroids, EOcclusionCalculationMode::FastWindingNumber, 0, .5, true);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	bool bFixNormals = bAutoSimplify;
	{
		FQEMSimplification Reducer(ResultMesh.Get());
		Reducer.Progress = Progress;
		Reducer.FastCollapsePass(MeshCellSize * .5, 3, true);

		if (bAutoSimplify)
		{
			const double MaxDisplacementSqr = SimplifyMaxErrorFactor * SimplifyMaxErrorFactor * MeshCellSize * MeshCellSize;
			Reducer.SimplifyToMaxError(MaxDisplacementSqr);
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (bFixNormals)
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OpResultMesh(ExtractResult().Release()); // moved the unique pointer
		OpResultMesh->EnableAttributes();

		// Recompute the normals
		FEditNormalsOp EditNormalsOp;
		EditNormalsOp.OriginalMesh = OpResultMesh; // the tool works on a deep copy of this mesh.
		EditNormalsOp.bFixInconsistentNormals = false;
		EditNormalsOp.bInvertNormals = false;
		EditNormalsOp.bRecomputeNormals = true;
		EditNormalsOp.NormalCalculationMethod = ENormalCalculationMethod::AreaAngleWeighting;
		EditNormalsOp.SplitNormalMethod = ESplitNormalMethod::FaceNormalThreshold;
		EditNormalsOp.bAllowSharpVertices = false;
		EditNormalsOp.NormalSplitThreshold = 60.f;

		EditNormalsOp.SetTransform(FTransform(ResultTransform));
		EditNormalsOp.CalculateResult(Progress);

		ResultMesh = EditNormalsOp.ExtractResult(); // return the edit normals operator copy to this tool.
	}
	else
	{
		// if nothing was simplified, just use per-vertex normals
		if (ResultMesh->HasAttributes())
		{
			FMeshNormals::InitializeOverlayToPerVertexNormals(ResultMesh->Attributes()->PrimaryNormals(), false);
		}
		else
		{
			FMeshNormals::QuickComputeVertexNormals(*ResultMesh.Get());
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (MinComponentVolume > 0 || MinComponentArea > 0)
	{
		FDynamicMeshEditor ComponentRemover(ResultMesh.Get());
		ComponentRemover.RemoveSmallComponents(MinComponentVolume, MinComponentArea);
	}
	
}