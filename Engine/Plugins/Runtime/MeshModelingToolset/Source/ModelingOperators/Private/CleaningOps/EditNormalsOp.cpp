// Copyright Epic Games, Inc. All Rights Reserved.

#include "CleaningOps/EditNormalsOp.h"

#include "DynamicMeshEditor.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "ToolContextInterfaces.h"

#include "DynamicMesh/MeshNormals.h"
#include "Operations/RepairOrientation.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditNormalsOp)

using namespace UE::Geometry;

void FEditNormalsOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FEditNormalsOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}
	bool bDiscardAttributes = false;
	ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);

	if (!ensureMsgf(ResultMesh->HasAttributes(), TEXT("Attributes not found on mesh? Conversion should always create them, so this operator should not need to do so.")))
	{
		ResultMesh->EnableAttributes();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Although the operator conceptually the same thing when run on the whole mesh or a subset represented by the input
	// selection, the implementations of most steps differ enough that it makes sense to split the code paths here
	if (EditTriangles.IsEmpty() && EditVertices.IsEmpty())
	{
		// TODO Use this case if EditVertices/EditTriangles imply the whole mesh
		CalculateResultWholeMesh(Progress);
	}
	else
	{
		CalculateResultSelection(Progress);
	}
}

void FEditNormalsOp::CalculateResultWholeMesh(FProgressCancel* Progress)
{
	// if you split normals you must always recompute as well
	bool bNeedsRecompute = bRecomputeNormals || SplitNormalMethod != ESplitNormalMethod::UseExistingTopology;

	if (bFixInconsistentNormals)
	{
		FMeshRepairOrientation Repair(ResultMesh.Get());
		Repair.OrientComponents();

		if (Progress && Progress->Cancelled())
		{
			return;
		}
		FDynamicMeshAABBTree3 Tree(ResultMesh.Get());
		Repair.SolveGlobalOrientation(&Tree);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (bInvertNormals)
	{
		for (int TID : ResultMesh->TriangleIndicesItr())
		{
			ResultMesh->ReverseTriOrientation(TID);
		}

		// also reverse the normal directions (but only if a recompute isn't going to do it for us below)
		if (!bNeedsRecompute)
		{
			FDynamicMeshNormalOverlay* Normals = ResultMesh->Attributes()->PrimaryNormals();
			for (int ElID : Normals->ElementIndicesItr())
			{
				auto El = Normals->GetElement(ElID);
				Normals->SetElement(ElID, -El);
			}
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	float NormalDotProdThreshold = FMathf::Cos(NormalSplitThreshold * FMathf::DegToRad);

	FMeshNormals FaceNormals(ResultMesh.Get());
	if (SplitNormalMethod != ESplitNormalMethod::UseExistingTopology)
	{
		if (SplitNormalMethod == ESplitNormalMethod::FaceNormalThreshold)
		{
			FaceNormals.ComputeTriangleNormals();
			const TArray<FVector3d>& Normals = FaceNormals.GetNormals();
			ResultMesh->Attributes()->PrimaryNormals()->CreateFromPredicate([&Normals, &NormalDotProdThreshold](int VID, int TA, int TB)
			{
				return Normals[TA].Dot(Normals[TB]) > NormalDotProdThreshold;
			}, 0);
		}
		else if (SplitNormalMethod == ESplitNormalMethod::PerTriangle)
		{
			FMeshNormals::InitializeMeshToPerTriangleNormals(ResultMesh.Get());
		}
		else if (SplitNormalMethod == ESplitNormalMethod::PerVertex)
		{
			FMeshNormals::InitializeOverlayToPerVertexNormals(ResultMesh->Attributes()->PrimaryNormals(), false);
		}
		else // SplitNormalMethod == ESplitNormalMethod::FaceGroupID
		{
			FPolygroupSet DefaultGroups(ResultMesh.Get());
			FPolygroupSet& UsePolygroups = (MeshPolygroups.IsValid()) ? *MeshPolygroups : DefaultGroups;

			ResultMesh->Attributes()->PrimaryNormals()->CreateFromPredicate([this, &UsePolygroups](int VID, int TA, int TB)
			{
				return UsePolygroups.GetTriangleGroup(TA) == UsePolygroups.GetTriangleGroup(TB);
			}, 0);
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	bool bAreaWeight = (NormalCalculationMethod == ENormalCalculationMethod::AreaWeighted || NormalCalculationMethod == ENormalCalculationMethod::AreaAngleWeighting);
	bool bAngleWeight = (NormalCalculationMethod == ENormalCalculationMethod::AngleWeighted || NormalCalculationMethod == ENormalCalculationMethod::AreaAngleWeighting);

	if (bNeedsRecompute)
	{
		FMeshNormals MeshNormals(ResultMesh.Get());
		MeshNormals.RecomputeOverlayNormals(ResultMesh->Attributes()->PrimaryNormals(), bAreaWeight, bAngleWeight);
		MeshNormals.CopyToOverlay(ResultMesh->Attributes()->PrimaryNormals(), false);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (SplitNormalMethod == ESplitNormalMethod::FaceNormalThreshold && bAllowSharpVertices)
	{
		ResultMesh->Attributes()->PrimaryNormals()->SplitVerticesWithPredicate(
			[this, &FaceNormals, &NormalDotProdThreshold](int ElementID, int TriID)
		{
			FVector3f ElNormal;
			ResultMesh->Attributes()->PrimaryNormals()->GetElement(ElementID, ElNormal);
			return ElNormal.Dot(FVector3f(FaceNormals.GetNormals()[TriID])) <= NormalDotProdThreshold;
		},
			[this, &FaceNormals](int ElementIdx, int TriID, float* FillVect)
		{
			FVector3f N(FaceNormals.GetNormals()[TriID]);
			FillVect[0] = N.X;
			FillVect[1] = N.Y;
			FillVect[2] = N.Z;
		}
		);
	}
}

void FEditNormalsOp::CalculateResultSelection(FProgressCancel* Progress)
{
	// if you split normals you must always recompute as well
	bool bNeedsRecompute = bRecomputeNormals || SplitNormalMethod != ESplitNormalMethod::UseExistingTopology;

	// TODO bFixInconsistentNormals is not yet implemented when there is a selection

	FDynamicMeshNormalOverlay& ResultNormals = *ResultMesh->Attributes()->PrimaryNormals();

	if (bInvertNormals)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		constexpr int MinEditVerticesForReverse = 2; // TODO Maybe the caller should be allowed to change this?
		static_assert(MinEditVerticesForReverse >= 1 && MinEditVerticesForReverse <= 3);

		for (int TID : EditTriangles)
		{
			const FIndex3i Vids = ResultMesh->GetTriangle(TID);

			int NumEditVertices = 0;
			NumEditVertices += (int)EditVertices.Contains(Vids.A);
			if (NumEditVertices >= MinEditVerticesForReverse)
			{
				ResultMesh->ReverseTriOrientation(TID);
				continue;
			}

			NumEditVertices += (int)EditVertices.Contains(Vids.B);
			if (NumEditVertices >= MinEditVerticesForReverse)
			{
				ResultMesh->ReverseTriOrientation(TID);
				continue;
			}

			NumEditVertices += (int)EditVertices.Contains(Vids.C);
			if (NumEditVertices >= MinEditVerticesForReverse)
			{
				ResultMesh->ReverseTriOrientation(TID);
				continue;
			}

			ensure(NumEditVertices >= 1); // Sanity check: each edit triangle must have at least one edit vertex
		}

		if (!bNeedsRecompute)
		{
			TArray<int> ElementTriangles;
			for (int ElID : ResultNormals.ElementIndicesItr())
			{
				if (EditVertices.Find(ResultNormals.GetParentVertex(ElID)))
				{
					ElementTriangles.Reset();
					ResultNormals.GetElementTriangles(ElID, ElementTriangles);
					for (int TID : ElementTriangles)
					{
						if (EditTriangles.Find(TID))
						{
							FVector3f El = ResultNormals.GetElement(ElID);
							ResultNormals.SetElement(ElID, -El);
							break;
						}
					}
				}
			}
		}
	}

	FDynamicMeshNormalOverlay Workspace(ResultMesh.Get());
	Workspace.Copy(*ResultMesh->Attributes()->PrimaryNormals());

	FMeshNormals FaceNormals(ResultMesh.Get());
	if (SplitNormalMethod != ESplitNormalMethod::UseExistingTopology)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		if (SplitNormalMethod == ESplitNormalMethod::FaceNormalThreshold)
		{
			FaceNormals.ComputeTriangleNormals();
			const TArray<FVector3d>& Normals = FaceNormals.GetNormals();

			const float NormalDotProdThreshold = FMathf::Cos(NormalSplitThreshold * FMathf::DegToRad);
			Workspace.CreateFromPredicate(
				[&Normals, &NormalDotProdThreshold](int VID, int TA, int TB)
			{
				return Normals[TA].Dot(Normals[TB]) > NormalDotProdThreshold;
			}, 0);
		}
		else if (SplitNormalMethod == ESplitNormalMethod::PerTriangle)
		{
			FMeshNormals::InitializeOverlayToPerTriangleNormals(&Workspace);
		}
		else if (SplitNormalMethod == ESplitNormalMethod::PerVertex)
		{
			FMeshNormals::InitializeOverlayToPerVertexNormals(&Workspace, false);
		}
		else
		{
			ensure(SplitNormalMethod == ESplitNormalMethod::FaceGroupID);

			FPolygroupSet DefaultGroups(ResultMesh.Get());
			FPolygroupSet& UsePolygroups = (MeshPolygroups.IsValid()) ? *MeshPolygroups : DefaultGroups;

			Workspace.CreateFromPredicate([this, &UsePolygroups](int VID, int TA, int TB)
			{
				return UsePolygroups.GetTriangleGroup(TA) == UsePolygroups.GetTriangleGroup(TB);
			}, 0);
		}
	}

	// TODO Consider disabling bRecomputeNormals option when we have a selection. At the moment if the user wants to
	// keep existing topology and just invert normals on a selection the result is different when bRecomputeNormals is
	// true/false (i.e., the normals change if the overlay is computed in the block below or in the !bNeedsRecompute
	// block above
	if (bNeedsRecompute)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		const bool bAreaWeight =
			NormalCalculationMethod == ENormalCalculationMethod::AreaWeighted ||
			NormalCalculationMethod == ENormalCalculationMethod::AreaAngleWeighting;
		const bool bAngleWeight =
			NormalCalculationMethod == ENormalCalculationMethod::AngleWeighted ||
			NormalCalculationMethod == ENormalCalculationMethod::AreaAngleWeighting;

		FMeshNormals MeshNormals(ResultMesh.Get());
		MeshNormals.RecomputeOverlayNormals(&Workspace, bAreaWeight, bAngleWeight);
		MeshNormals.CopyToOverlay(&Workspace, false);
	}

	if (SplitNormalMethod == ESplitNormalMethod::FaceNormalThreshold && bAllowSharpVertices)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		const float NormalDotProdThreshold = FMathf::Cos(NormalSplitThreshold * FMathf::DegToRad);
		Workspace.SplitVerticesWithPredicate(
			[this, &FaceNormals, &NormalDotProdThreshold, &Workspace](int ElementID, int TriID)
		{
			FVector3f ElNormal;
			Workspace.GetElement(ElementID, ElNormal);
			return ElNormal.Dot(FVector3f(FaceNormals.GetNormals()[TriID])) <= NormalDotProdThreshold;
		},
			[this, &FaceNormals](int ElementIdx, int TriID, float* FillVect)
		{
			FVector3f N(FaceNormals.GetNormals()[TriID]);
			FillVect[0] = N.X;
			FillVect[1] = N.Y;
			FillVect[2] = N.Z;
		}
		);
	}

	// Copy the subset of normals corresponding to the geometry selection to the Mesh overlays
	FDynamicMeshEditor Editor(ResultMesh.Get());
	Editor.AppendElementSubset(
		ResultMesh.Get(),
		EditTriangles,
		EditVertices,
		&Workspace,
		&ResultNormals);
}