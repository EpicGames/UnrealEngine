// Copyright Epic Games, Inc. All Rights Reserved.

#include "CleaningOps/HoleFillOp.h"
#include "Operations/SimpleHoleFiller.h"
#include "Operations/PlanarHoleFiller.h"
#include "Operations/MinimalHoleFiller.h"
#include "Operations/SmoothHoleFiller.h"
#include "Operations/HoleFillUtil.h"
#include "ConstrainedDelaunay2.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Selections/MeshConnectedComponents.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HoleFillOp)

using namespace UE::Geometry;

namespace 
{
	bool LoopIsValid(const FDynamicMesh3& Mesh, const FEdgeLoop& Loop)
	{
		if (Loop.Edges.Num() == 0) 
		{ 
			return false; 
		}

		for (int EdgeID : Loop.Edges)
		{
			if (!Mesh.IsBoundaryEdge(EdgeID))
			{
				return false;
			}
		}

		FEdgeLoop CheckLoop(&Mesh, Loop.Vertices, Loop.Edges);
		if (!CheckLoop.CheckValidity(EValidityCheckFailMode::ReturnOnly))
		{
			return false;
		}

		for (int VertexID : Loop.Vertices)
		{
			if (!Mesh.IsBoundaryVertex(VertexID))
			{
				// Not sure how this can happen if the edges are all valid...
				check(false);
			}
		}

		return true;
	}

	/// Look for a loop around a single isolated triangle. A triangle is isolated if its vertices are only
	/// incident on that triangle (i.e. triangles connected by a bowtie connection are not considered isolated.)
	bool LoopIsAnIsolatedTriangle(const FDynamicMesh3& Mesh, const FEdgeLoop& Loop, int& TriangleID )
	{
		if (Loop.Edges.Num() != 3)
		{
			return false;
		}

		check(Mesh.IsBoundaryEdge(Loop.Edges[0]));
		check(Mesh.IsBoundaryEdge(Loop.Edges[1]));
		check(Mesh.IsBoundaryEdge(Loop.Edges[2]));

		FDynamicMesh3::FEdge Edge0 = Mesh.GetEdge(Loop.Edges[0]);
		FDynamicMesh3::FEdge Edge1 = Mesh.GetEdge(Loop.Edges[1]);
		FDynamicMesh3::FEdge Edge2 = Mesh.GetEdge(Loop.Edges[2]);

		// Return true if all edges are incident on the same triangle...
		TriangleID = Edge0.Tri[0];
		if (TriangleID != Edge1.Tri[0]) { return false; }
		if (TriangleID != Edge2.Tri[0]) { return false; }

		// ...and the triangle's vertices are only connected to one triangle.
		const FIndex3i& Verts = Mesh.GetTriangle(TriangleID);
		return ((Mesh.GetVtxTriangleCount(Verts[0]) == 1) && 
				(Mesh.GetVtxTriangleCount(Verts[1]) == 1) && 
				(Mesh.GetVtxTriangleCount(Verts[2]) == 1));
	}

	TUniquePtr<FSmoothHoleFiller> MakeSmoothHoleFiller(FDynamicMesh3& Mesh, FEdgeLoop& Loop, const FSmoothFillOptions& Options)
	{
		TUniquePtr<FSmoothHoleFiller> Filler = MakeUnique<FSmoothHoleFiller>(Mesh, Loop);
		Filler->FillOptions = Options;
		return Filler;
	}

	
	// return index in the parent triangle (0, 1, 2) or -1 if not found
	int FindParentVertexInBaseTriangle(const FDynamicMeshUVOverlay& UVOverlay, int ElementID, int TriangleID)
	{
		const FDynamicMesh3* Mesh = UVOverlay.GetParentMesh();
		const FIndex3i BaseTriangle = Mesh->GetTriangle(TriangleID);
		int ParentVertex = UVOverlay.GetParentVertex(ElementID);
		return BaseTriangle.IndexOf(ParentVertex);
	}

	float Area(FDynamicMeshUVOverlay& UVOverlay, FIndex3i& Tri)
	{
		const FVector2f AB = UVOverlay.GetElement(Tri[1]) - UVOverlay.GetElement(Tri[0]);
		const FVector2f AC = UVOverlay.GetElement(Tri[2]) - UVOverlay.GetElement(Tri[0]);
		return 0.5f * FMathd::Abs(AB.X * AC.Y - AB.Y * AC.X);
	}

	// Given a single triangle adjacent to a newly added triangle in the base mesh, find the two elements that should 
	// go into a new triangle in the UV mesh (i.e. the shared edge between the adjacent triangle and the new UV triangle.) 
	// Also add a third UV element, taking the average UV coordinates of the shared edge.
	void SetUVTriangleFromAdjacentTriangle(FDynamicMeshUVOverlay& UVOverlay,
										   int AdjacentTriangle,
										   int NewTriangleID,
										   FIndex3i& NewTriangleElements)
	{
		const FDynamicMesh3* Mesh = UVOverlay.GetParentMesh();
		FIndex3i AdjacentTriangleElements = UVOverlay.GetTriangle(AdjacentTriangle);

		// Find valid elements on the adjacent triangle that point to the base triangle's vertices. There should be 2.
		TStaticArray<FVector2f, 2> EdgeElements;
		int FoundElements = 0;
		FVector2f OppositeVertexPosition;
		for (int AdjacentTriVertexIndex = 0; AdjacentTriVertexIndex < 3; ++AdjacentTriVertexIndex)
		{
			int IndexInBaseTriangle = FindParentVertexInBaseTriangle(UVOverlay, AdjacentTriangleElements[AdjacentTriVertexIndex], NewTriangleID);
			if (IndexInBaseTriangle >= 0)
			{
				NewTriangleElements[IndexInBaseTriangle] = AdjacentTriangleElements[AdjacentTriVertexIndex];
				EdgeElements[FoundElements] = UVOverlay.GetElement(AdjacentTriangleElements[AdjacentTriVertexIndex]);
				++FoundElements;
			}
			else
			{
				OppositeVertexPosition = UVOverlay.GetElement(AdjacentTriangleElements[AdjacentTriVertexIndex]);
			}
		}
		check(FoundElements == 2);

		// Now insert the new element
		FIndex3i TriangleVertices = Mesh->GetTriangle(NewTriangleID);
		FVector2f NewElement = 0.5f * (EdgeElements[0] + EdgeElements[1]);
		
		bool bIsDegen = ( Distance(EdgeElements[0], EdgeElements[1]) == 0.0f);
		if (!bIsDegen)
		{
			// push the new element off the edge slightly to avoid degenerate triangle (assuming the adjacent triangle 
			// wasn't degenerate before)
			FVector2f Delta = (NewElement - OppositeVertexPosition);
			if (Delta.Length() > 0.0)
			{
				NewElement += KINDA_SMALL_NUMBER * UE::Geometry::Normalized(Delta);
			}
			else
			{
				bIsDegen = true;
			}
		}

		int NewElementIndex = NewTriangleElements.IndexOf(FDynamicMesh3::InvalidID);
		NewTriangleElements[NewElementIndex] = UVOverlay.AppendElement(NewElement);
		UVOverlay.SetParentVertex(NewTriangleElements[NewElementIndex], TriangleVertices[NewElementIndex]);

		if (!bIsDegen)
		{
			ensure(Area(UVOverlay, NewTriangleElements) > 0.0);
		}
	}

} // unnamed namespace


bool FHoleFillOp::FillSingleTriangleHole(const FEdgeLoop& Loop, int32& NewGroupID)
{
	check(Loop.GetEdgeCount() == 3);
	check(Loop.GetVertexCount() == 3);

	NewGroupID = ResultMesh->AllocateTriangleGroup();

	FIndex3i Vertices{ Loop.Vertices[0], Loop.Vertices[1], Loop.Vertices[2] };

	// Find adjacent edges and triangles
	FIndex3i ExistingEdges;
	FIndex3i ExistingTriangles;
	for (int Nbr = 0; Nbr < 3; ++Nbr)
	{
		ExistingEdges[Nbr] = ResultMesh->FindEdge(Vertices[Nbr], Vertices[(Nbr + 1) % 3]);
		ensure(ExistingEdges[Nbr] != FDynamicMesh3::InvalidID);

		FIndex2i EdgeTris = ResultMesh->GetEdgeT(ExistingEdges[Nbr]);
		ensure(EdgeTris[0] != FDynamicMesh3::InvalidID && EdgeTris[1] == FDynamicMesh3::InvalidID);

		ExistingTriangles[Nbr] = EdgeTris[0];
	}

	FIndex3i ExistingTri0Vertices = ResultMesh->GetTriangle(ExistingTriangles[0]);

	// Check orientation of the loop
	if (IndexUtil::FindTriOrderedEdge(Vertices[0], Vertices[1], ExistingTri0Vertices) != FDynamicMesh3::InvalidID)
	{
		// Orientation matches an existing triangle, so reverse it
		Swap(Vertices[0], Vertices[1]);

		// keep ExistingEdges and ExistingTriangles aligned such that index i corresponds to the edge from vert i to (i+1)%3
		Swap(ExistingEdges[1], ExistingEdges[2]);
		Swap(ExistingTriangles[1], ExistingTriangles[2]);
	}

	int NewTriangleID = ResultMesh->AppendTriangle(Vertices, NewGroupID);
	if (NewTriangleID < 0)
	{
		return false;
	}

	// Pick the overlay elements to use based on neighbors, to minimize added seams. Returns false when neighbors don't share elements along the hole.
	auto GetOverlayElementsFromNeighbors = [this, &Vertices, &ExistingTriangles, &ExistingEdges]<typename OverlayType>(OverlayType* Overlay, FIndex3i& NewTriangleElements) -> bool
	{
		// We're going to try to assign the overlay elements such that the number of overlay seam edges is minimized. If at least 
		// two of our neighboring triangles have a shared overlay element at one of our vertices, then using the elements
		// of these two triangles will decrease the number of seam edges because we end up removing two seams and
		// adding at most one (or none if this was an enclosed hole in a single island).

		// For each of our vertices, get the elements associated with that vertex by the two neighbor triangles. The 
		// pair we store is (element from previous triangle, element from next triangle).
		FIndex2i VertElements[3];
		for (int NeighborIndex = 0; NeighborIndex < 3; ++NeighborIndex)
		{
			FIndex3i NeighborTriVids = ResultMesh->GetTriangle(ExistingTriangles[NeighborIndex]);
			FIndex3i NeighborTriElements = Overlay->GetTriangle(ExistingTriangles[NeighborIndex]);

			// Find our two vids in TriVids, get the corresponding elements in TriElements
			int VertIndex1 = NeighborIndex;
			int VertIndex2 = (VertIndex1 + 1) % 3;

			VertElements[VertIndex1].B = NeighborTriElements[IndexUtil::FindTriIndex(Vertices[VertIndex1], NeighborTriVids)];
			VertElements[VertIndex2].A = NeighborTriElements[IndexUtil::FindTriIndex(Vertices[VertIndex2], NeighborTriVids)];
		}

		/*
		 * Imagine going counterclockwise from the bottom left in the triangle below. A and B correspond
		 * to the element values in VertUVElements. When the two are equal the vertex has a shared element,
		 * and the adjacent elements need to be on the adjoining sides.
		 *
		 *     B.A
		 *     / \
		 *   A.---.B
		 *    B   A
		 */
		if (VertElements[0].A == VertElements[0].B)
		{
			NewTriangleElements = FIndex3i(VertElements[0].A, VertElements[1].A, VertElements[2].B);
		}
		else if (VertElements[1].A == VertElements[1].B)
		{
			NewTriangleElements = FIndex3i(VertElements[0].B, VertElements[1].A, VertElements[2].A);
		}
		else if (VertElements[2].A == VertElements[2].B)
		{
			NewTriangleElements = FIndex3i(VertElements[0].A, VertElements[1].B, VertElements[2].A);
		}
		else
		{
			return false; // failed to find a clean match, fall back to overlay-type-specific logic
		}
		return true;
	};

	if (ResultMesh->HasAttributes() && ResultMesh->Attributes()->PrimaryUV())
	{
		FDynamicMeshUVOverlay* UVOverlay = ResultMesh->Attributes()->PrimaryUV();

		// check we have a hole in the UV mesh as well (note that boundary edges of unset UV triangles aren't seen 
		// as seams by IsSeamEdge)
		checkSlow(UVOverlay->IsSeamEdge(ExistingEdges[0]) || !UVOverlay->IsSetTriangle(ExistingTriangles[0]));
		checkSlow(UVOverlay->IsSeamEdge(ExistingEdges[1]) || !UVOverlay->IsSetTriangle(ExistingTriangles[1]));
		checkSlow(UVOverlay->IsSeamEdge(ExistingEdges[2]) || !UVOverlay->IsSetTriangle(ExistingTriangles[2]));
		FIndex3i NewTriangleElements = FIndex3i::Invalid();

		if (!GetOverlayElementsFromNeighbors(UVOverlay, NewTriangleElements))
		{
			// Since we didn't have a matching element ID above, at least one of ExistingTriangles[0] or [1] must have set UVs --
			//  i.e., if they were both unset, then their element IDs would all be -1 and one of the tests for matching element IDs would pass
			// Pick an arbitrary set UV edge and add a new third UV element
			int32 NeighborTidToUse = UVOverlay->IsSetTriangle(ExistingTriangles[0]) ? ExistingTriangles[0] : ExistingTriangles[1];
			SetUVTriangleFromAdjacentTriangle(*UVOverlay, NeighborTidToUse, NewTriangleID, NewTriangleElements);
		}

		EMeshResult Result = EMeshResult::Ok;
		if (NewTriangleElements.A != IndexConstants::InvalidID)
		{
			Result = UVOverlay->SetTriangle(NewTriangleID, NewTriangleElements);
		}

		if (!ensure(Result == EMeshResult::Ok))
		{
			// TODO: undo changes ResultMesh and UVOverlay before returning? 
			// Or create a totally new triangle with unique UVs and pass that in as a fallback?
			return false;
		}
	}

	if (ResultMesh->HasAttributes() && ResultMesh->Attributes()->HasPrimaryColors())
	{
		FDynamicMeshColorOverlay* ColorOverlay = ResultMesh->Attributes()->PrimaryColors();
		FIndex3i NewTriangleElements = FIndex3i::Invalid();

		// try to fill the color overlay seamlessly from local info
		if (GetOverlayElementsFromNeighbors(ColorOverlay, NewTriangleElements))
		{
			if (NewTriangleElements.A != IndexConstants::InvalidID)
			{
				EMeshResult Result = ColorOverlay->SetTriangle(NewTriangleID, NewTriangleElements);
				ensure(Result == EMeshResult::Ok); // should always succeed, but if it doesn't we will just have some unset colors
			}
		}
		else
		{
			// fall back to a standard color overlay fill method, which should at least try to avoid a seam along one edge
			// (since it stomps IDs to match for each it finds with valid overlay elements, so should avoid seams at least on the last edge it processes)
			TStaticArray<int32, 3> ExistingEdgesArr{ ExistingEdges.A, ExistingEdges.B, ExistingEdges.C };
			HoleFillUtil::FillColorOverlay(*ResultMesh, ExistingEdgesArr, TArrayView<int32>(&NewTriangleID, 1));
		}
	}


	// And normals
	if (ResultMesh->HasAttributes())
	{
		FDynamicMeshEditor Editor(ResultMesh.Get());
		Editor.SetTriangleNormals({ NewTriangleID });
	}

	return true;
}


void FHoleFillOp::CalculateResult(FProgressCancel* Progress)
{
	NumFailedLoops = 0;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);

	if (Loops.Num() == 0)
	{
		return;
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	TSet<int32> NewGroupIDs;

	for (FEdgeLoop& Loop : Loops)
	{
		if (!LoopIsValid(*ResultMesh, Loop))
		{ 
			++NumFailedLoops;
			continue;
		}

		if (FillOptions.bRemoveIsolatedTriangles)
		{
			int IsolatedTriangleID;
			if (LoopIsAnIsolatedTriangle(*ResultMesh, Loop, IsolatedTriangleID))
			{
				ResultMesh->RemoveTriangle(IsolatedTriangleID);
				continue;
			}
		}

		if (FillOptions.bQuickFillSmallHoles && Loop.GetEdgeCount() == 3)
		{
			int32 PossibleNewGroupID;
			bool bFilledSmallHole = FillSingleTriangleHole(Loop, PossibleNewGroupID);

			if (bFilledSmallHole)
			{
				// TODO: If FillSmallHole fails, this newly created group ID won't be used. Is that okay?
				NewGroupIDs.Add(PossibleNewGroupID);
				continue;
			}
		}


		int32 NewGroupID = ResultMesh->AllocateTriangleGroup();
		NewGroupIDs.Add(NewGroupID);

		// Compute a best-fit plane of the boundary vertices
		TArray<FVector3d> VertexPositions;
		Loop.GetVertices(VertexPositions);
		FVector3d PlaneOrigin;
		FVector3d PlaneNormal;
		PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
		PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere

		// Now fill using the appropriate algorithm
		TUniquePtr<IHoleFiller> Filler;
		TArray<TArray<int>> VertexLoops;

		switch (FillType)
		{
		case EHoleFillOpFillType::TriangleFan:
			Filler = MakeUnique<FSimpleHoleFiller>(ResultMesh.Get(), Loop, FSimpleHoleFiller::EFillType::TriangleFan);
			break;
		case EHoleFillOpFillType::PolygonEarClipping:
			Filler = MakeUnique<FSimpleHoleFiller>(ResultMesh.Get(), Loop, FSimpleHoleFiller::EFillType::PolygonEarClipping);
			break;
		case EHoleFillOpFillType::Planar:
			VertexLoops.Add(Loop.Vertices);
			Filler = MakeUnique<FPlanarHoleFiller>(ResultMesh.Get(),
				&VertexLoops,
				ConstrainedDelaunayTriangulate<double>,
				PlaneOrigin,
				PlaneNormal);
			break;
		case EHoleFillOpFillType::Minimal:
			Filler = MakeUnique<FMinimalHoleFiller>(ResultMesh.Get(), Loop);
			break;
		case EHoleFillOpFillType::Smooth:
			Filler = MakeSmoothHoleFiller(*ResultMesh, Loop, SmoothFillOptions);			
			break;
		default:
			check(false);
		}

		check(Filler);
		bool bFilled = Filler->Fill(NewGroupID);

		if (!bFilled)
		{
			++NumFailedLoops;
			continue;
		}

		// Compute normals and UVs
		if (ResultMesh->HasAttributes())
		{
			FDynamicMeshEditor Editor(ResultMesh.Get());
			FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
			Editor.SetTriangleNormals(Filler->NewTriangles, (FVector3f)PlaneNormal);
			Editor.SetTriangleUVsFromProjection(Filler->NewTriangles, ProjectionFrame, MeshUVScaleFactor);
		}

		if (Progress && Progress->Cancelled())
		{
			return;
		}

	}	// for Loops

	// NewGroupIDs are assigned to triangles in the fill regions, which are the ones we want to select/highlight by
	// adding them to NewTriangles
	NewTriangles.Empty();
	for (int TriangleID : ResultMesh->TriangleIndicesItr())
	{
		int GroupID = ResultMesh->GetTriangleGroup(TriangleID);
		if (NewGroupIDs.Contains(GroupID))
		{
			NewTriangles.Emplace(TriangleID);
		}
	}
	
}


