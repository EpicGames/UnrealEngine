// Copyright Epic Games, Inc. All Rights Reserved.

#include "CleaningOps/RemeshMeshOp.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshAttributeUtil.h"
#include "Remesher.h"
#include "QueueRemesher.h"
#include "MeshConstraintsUtil.h"
#include "ProjectionTargets.h"
#include "DynamicMesh/MeshNormals.h"
#include "NormalFlowRemesher.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemeshMeshOp)

using namespace UE::Geometry;

TUniquePtr<FRemesher> FRemeshMeshOp::CreateRemesher(ERemeshType Type, FDynamicMesh3* TargetMesh)
{
	switch(Type)
	{
	case ERemeshType::Standard:
	{
		TUniquePtr<FQueueRemesher> QueueRemesher = MakeUnique<FQueueRemesher>(TargetMesh);
		QueueRemesher->MaxRemeshIterations = MaxRemeshIterations;
		QueueRemesher->MinActiveEdgeFraction = MinActiveEdgeFraction;
		return QueueRemesher;
	}
	case ERemeshType::FullPass:
	{
		return MakeUnique<FRemesher>(TargetMesh);
	}
	case ERemeshType::NormalFlow:
	{
		TUniquePtr<FNormalFlowRemesher> NormalFlowRemesher = MakeUnique<FNormalFlowRemesher>(TargetMesh);
		NormalFlowRemesher->MaxRemeshIterations = MaxRemeshIterations;
		NormalFlowRemesher->MinActiveEdgeFraction = 0;		// disable convergence check for NormalFlow remeshing
		NormalFlowRemesher->NumExtraProjectionIterations = ExtraProjectionIterations;
		NormalFlowRemesher->MaxTriangleCount = FMath::Max(0, TriangleCountHint);
		NormalFlowRemesher->FaceProjectionPassesPerRemeshIteration = FaceProjectionPassesPerRemeshIteration;
		NormalFlowRemesher->SurfaceProjectionSpeed = SurfaceProjectionSpeed;
		NormalFlowRemesher->NormalAlignmentSpeed = NormalAlignmentSpeed;
		NormalFlowRemesher->bSmoothInFillAreas = bSmoothInFillAreas;
		NormalFlowRemesher->FillAreaDistanceMultiplier = FillAreaDistanceMultiplier;
		NormalFlowRemesher->FillAreaSmoothMultiplier = FillAreaSmoothMultiplier;
		return NormalFlowRemesher;
	}
	default:
		checkf(false, TEXT("Encountered unexpected Remesh Type"));
		return nullptr;
	}

}

void FRemeshMeshOp::SetTransform(const FTransformSRT3d& Transform)
{
	ResultTransform = Transform;
}

void FRemeshMeshOp::SetUserSpecifiedConstraints(const FMeshConstraints& Constraints)
{
	UserSpecifiedConstraints = Constraints;
}

void FRemeshMeshOp::CalculateResult(FProgressCancel* Progress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RemeshMeshOp);

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	


	bool bHasDiscardedAttributes = bDiscardAttributes && !bPreserveSharpEdges && !bNeedsWeightMaps;
	ResultMesh->Copy(*OriginalMesh, true, true, true, !bHasDiscardedAttributes); // if we'll never need attributes, copying w/out them is fastest

	// helper to discard only attributes that we don't need
	auto SafeDiscardAttributes = [this, &bHasDiscardedAttributes](FDynamicMesh3& Mesh)
	{
		if (bHasDiscardedAttributes || !Mesh.HasAttributes())
		{
			return;
		}
		bHasDiscardedAttributes = true;
		if (bNeedsWeightMaps)
		{
			// manually turn off attributes aside from the weights
			Mesh.Attributes()->SetNumNormalLayers(0);
			Mesh.Attributes()->SetNumPolygroupLayers(0);
			Mesh.Attributes()->SetNumUVLayers(0);
			Mesh.Attributes()->DisableBones();
			Mesh.Attributes()->DiscardSculptLayers();
			Mesh.Attributes()->DisablePrimaryColors();
			Mesh.Attributes()->DisableMaterialID();
		}
		else
		{
			// discard the entire attribute set
			Mesh.DiscardAttributes();
		}
	};

	// if we copied w/ attributes but we just needed the weight maps, discard all else
	if (bDiscardAttributes && bNeedsWeightMaps && !bPreserveSharpEdges)
	{
		SafeDiscardAttributes(*ResultMesh);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	FDynamicMesh3* TargetMesh = ResultMesh.Get();

	TUniquePtr<FRemesher> Remesher = CreateRemesher(RemeshType, TargetMesh);

	Remesher->bEnableParallelProjection = this->bParallel;

	Remesher->bEnableSplits = bSplits;
	Remesher->bEnableFlips = bFlips;
	Remesher->bEnableCollapses = bCollapses;

	Remesher->SetTargetEdgeLength(TargetEdgeLength);

	Remesher->ProjectionMode = (bReproject) ? 
		FRemesher::ETargetProjectionMode::AfterRefinement : FRemesher::ETargetProjectionMode::NoProjection;

	Remesher->bEnableSmoothing = (SmoothingStrength > 0);
	Remesher->SmoothSpeedT = SmoothingStrength;
	// convert smooth type from UI enum to (currently 1:1) FRemesher enum
	Remesher->SmoothType = FRemesher::ESmoothTypes::Uniform;
	if (!bDiscardAttributes)
	{
		switch (SmoothingType)
		{
		case ERemeshSmoothingType::Uniform:
			Remesher->SmoothType = FRemesher::ESmoothTypes::Uniform;
			Remesher->FlipMetric = FRemesher::EFlipMetric::OptimalValence;
			break;
		case ERemeshSmoothingType::Cotangent:
			Remesher->SmoothType = FRemesher::ESmoothTypes::Cotan;
			Remesher->FlipMetric = FRemesher::EFlipMetric::MinEdgeLength;
			break;
		case ERemeshSmoothingType::MeanValue:
			Remesher->SmoothType = FRemesher::ESmoothTypes::MeanValue;
			Remesher->FlipMetric = FRemesher::EFlipMetric::MinEdgeLength;
			break;
		default:
			ensure(false);
		}
	}
	bool bIsUniformSmooth = (Remesher->SmoothType == FRemesher::ESmoothTypes::Uniform);

	Remesher->bPreventNormalFlips = bPreventNormalFlips;
	Remesher->bPreventTinyTriangles = bPreventTinyTriangles;

	Remesher->DEBUG_CHECK_LEVEL = 0;

	FMeshConstraints Constraints = UserSpecifiedConstraints;		// Initialize with externally supplied constraints

	constexpr bool bAllowSeamSplits = true;
	const bool bAllowSeamCollapse = !bPreserveSharpEdges;
	const bool bAllowSeamSmoothing = !bPreserveSharpEdges;

	FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints,
		*TargetMesh,
		MeshBoundaryConstraint,
		GroupBoundaryConstraint,
		MaterialBoundaryConstraint,
		bAllowSeamSplits,
		bAllowSeamSmoothing,
		bAllowSeamCollapse);

	if (bReprojectConstraints)
	{
		Constraints.ProjectionData.ProjectionCurves.Reset();

		if ( MeshBoundaryConstraint != EEdgeRefineFlags::NoConstraint )
		{
			FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(Constraints, FMeshConstraintsUtil::EBoundaryType::Mesh, *TargetMesh, BoundaryCornerAngleThreshold);
		}

		if (GroupBoundaryConstraint != EEdgeRefineFlags::NoConstraint)
		{
			FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(Constraints, FMeshConstraintsUtil::EBoundaryType::Group, *TargetMesh, BoundaryCornerAngleThreshold);
		}

		if (MaterialBoundaryConstraint != EEdgeRefineFlags::NoConstraint)
		{
			FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(Constraints, FMeshConstraintsUtil::EBoundaryType::MaterialID, *TargetMesh, BoundaryCornerAngleThreshold);
		}
	}

	Remesher->SetExternalConstraints(MoveTemp(Constraints));

	if (ProjectionTarget == nullptr)
	{
		check(ProjectionTargetSpatial == nullptr);
		ProjectionTarget = OriginalMesh.Get();
		ProjectionTargetSpatial = OriginalMeshSpatial.Get();
	}

	TUniquePtr<IProjectionTarget> ProjTarget;
	if (bUseWorldSpace)
	{
		ProjTarget = MakeUnique<FWorldSpaceProjectionTarget>(ProjectionTarget, ProjectionTargetSpatial, TargetMeshLocalToWorld, ToolMeshLocalToWorld);
	}
	else
	{
		ProjTarget = MakeUnique<FMeshProjectionTarget>(ProjectionTarget, ProjectionTargetSpatial);
	}
	Remesher->SetProjectionTarget(ProjTarget.Get());

	Remesher->CustomEdgeLengthScaleF = CustomEdgeLengthScaleF;

	Remesher->Progress = Progress;

	if (bDiscardAttributes && !bHasDiscardedAttributes)
	{
		SafeDiscardAttributes(*TargetMesh);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemeshMeshOp_Remesh);
		if (RemeshType == ERemeshType::FullPass)
		{
			// Run a fixed number of iterations
			for (int k = 0; k < RemeshIterations; ++k)
			{
				// If we are not uniform smoothing, then flips seem to often make things worse.
				// Possibly this is because without the tangential flow, we won't get to the nice tris.
				// In this case we are better off basically not flipping, and just letting collapses resolve things
				// regular-valence polygons - things stay "stuck". 
				// @todo try implementing edge-length flip criteria instead of valence-flip
				if (bIsUniformSmooth == false)
				{
					bool bUseFlipsThisPass = (k % 2 == 0 && k < RemeshIterations / 2);
					Remesher->bEnableFlips = bUseFlipsThisPass && bFlips;
				}

				Remesher->BasicRemeshPass();
			}
		}
		else if (RemeshType == ERemeshType::Standard || RemeshType == ERemeshType::NormalFlow)
		{
			// Run to convergence
			Remesher->BasicRemeshPass();
		}
		else 
		{
			check(!"Encountered unexpected Remesh Type");
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemeshMeshOp_Normals);
		if (!TargetMesh->HasAttributes())
		{
			FMeshNormals::QuickComputeVertexNormals(*TargetMesh);
		}
		else
		{
			FMeshNormals::QuickRecomputeOverlayNormals(*TargetMesh);
		}
	}

	if (bHasDiscardedAttributes && TargetMesh->HasAttributes())
	{
		// return the default single UV and normal layers, if we discarded to 0 above,
		// since we typically expect PrimaryUV and PrimaryNormals to be valid
		TargetMesh->Attributes()->SetNumUVLayers(1);
		TargetMesh->Attributes()->SetNumNormalLayers(1);
	}

	if ((bHasDiscardedAttributes || !TargetMesh->HasAttributes()) && bResultMustHaveAttributesEnabled)
	{
		TargetMesh->EnableAttributes();
		if (TargetMesh->HasVertexUVs())
		{
			CopyVertexUVsToOverlay(*TargetMesh, *TargetMesh->Attributes()->PrimaryUV());
		}
		if (TargetMesh->HasVertexNormals())
		{
			CopyVertexNormalsToOverlay(*TargetMesh, *TargetMesh->Attributes()->PrimaryNormals());
		}
	}
}



double FRemeshMeshOp::CalculateTargetEdgeLength(const FDynamicMesh3* Mesh, int TargetTriCount, double PrecomputedMeshArea)
{
	double InitialMeshArea = PrecomputedMeshArea;
	if (InitialMeshArea <= 0)
	{
		checkf(Mesh, TEXT("FRemeshMeshOp: Expected either a non-zero precomputed mesh area or a valid Mesh pointer"));
		InitialMeshArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(*Mesh).Y;
	}

	double TargetTriArea = InitialMeshArea / (double)TargetTriCount;
	double EdgeLen = TriangleUtil::EquilateralEdgeLengthForArea(TargetTriArea);
	return (double)FMath::RoundToInt(EdgeLen*100.0) / 100.0;
}
