// Copyright Epic Games, Inc. All Rights Reserved.

#include "CleaningOps/RemoveOccludedTrianglesOp.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"

#include "Operations/RemoveOccludedTriangles.h"
#include "Selections/MeshFaceSelection.h"
#include "Polygroups/PolygroupSet.h"
#include "MathUtil.h"

using namespace UE::Geometry;


void FRemoveOccludedTrianglesOp::SetTransform(const FTransform& Transform)
{
	ResultTransform = (FTransformSRT3d)Transform;
}

void FRemoveOccludedTrianglesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	bool bDiscardAttributes = false;
	ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
	
	auto ShrinkSelection = [](FDynamicMesh3& Mesh, TArray<int>& SelectedTris, int NumShrinks)
	{
		FMeshFaceSelection Selection(&Mesh);
		Selection.Select(SelectedTris);
		Selection.ContractBorderByOneRingNeighbours(NumShrinks);
		SelectedTris = Selection.AsArray();
	};

	auto SetNewGroupSelection = [](FDynamicMesh3& Mesh, const TArray<int>& SelectedTris, FName LayerName, bool bActiveGroupIsDefault)
	{
		// don't add any new groups if there's nothing to select
		if (SelectedTris.Num() == 0)
		{
			return FIndex2i::Invalid();
		}

		auto SetGroup = [](FDynamicMesh3& WriteMesh, UE::Geometry::FPolygroupSet& ActiveGroupSet, const TArray<int>& SelectedTris)
		{
			int32 JacketGroupID = ActiveGroupSet.AllocateNewGroupID();

			for (int TID : SelectedTris)
			{
				ActiveGroupSet.SetGroup(TID, JacketGroupID, WriteMesh);
			}

			return FIndex2i(JacketGroupID, ActiveGroupSet.GroupLayerIndex);
		};

		if (!bActiveGroupIsDefault)
		{
			UE::Geometry::FPolygroupSet ActiveGroupSet(&Mesh, LayerName);
			return SetGroup(Mesh, ActiveGroupSet, SelectedTris);
		}
		else
		{
			UE::Geometry::FPolygroupSet ActiveGroupSet(&Mesh);
			return SetGroup(Mesh, ActiveGroupSet, SelectedTris);
		}
	};

	TRemoveOccludedTriangles<FDynamicMesh3> Jacket(ResultMesh.Get());

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	Jacket.InsideMode = InsideMode;
	Jacket.TriangleSamplingMethod = TriangleSamplingMethod;
	Jacket.WindingIsoValue = WindingIsoValue;
	Jacket.NormalOffset = NormalOffset;
	Jacket.AddRandomRays = AddRandomRays;
	Jacket.AddTriangleSamples = AddTriangleSamples;
	// copy shared pointers to ray pointers for the jacketing algorithm (they w/ shorter lifespan so it's ok)
	TArray<FDynamicMeshAABBTree3*> RawOccluderTrees; RawOccluderTrees.SetNum(OccluderTrees.Num());
	TArray<TFastWindingTree<FDynamicMesh3>*> RawOccluderWindings; RawOccluderWindings.SetNum(OccluderWindings.Num());
	for (int32 TreeIdx = 0; TreeIdx < OccluderTrees.Num(); TreeIdx++)
	{
		RawOccluderTrees[TreeIdx] = OccluderTrees[TreeIdx].Get();
		RawOccluderWindings[TreeIdx] = OccluderWindings[TreeIdx].Get();
	}
	Jacket.Select(MeshTransforms, RawOccluderTrees, RawOccluderWindings, OccluderTransforms);
	if (ShrinkRemoval > 0)
	{
		ShrinkSelection(*ResultMesh.Get(), Jacket.RemovedT, ShrinkRemoval);
	}

	if (bSetTriangleGroupInsteadOfRemoving)
	{
		FIndex2i GroupIDAndLayerIndex = SetNewGroupSelection(*ResultMesh.Get(), Jacket.RemovedT, ActiveGroupLayer, bActiveGroupLayerIsDefault);
		CreatedGroupID = GroupIDAndLayerIndex.A;
		CreatedGroupLayerIndex = GroupIDAndLayerIndex.B;
	}
	else
	{
		Jacket.RemoveSelected();
	}

	if (MinTriCountConnectedComponent > 0 || MinAreaConnectedComponent > 0)
	{
		FDynamicMeshEditor Editor(ResultMesh.Get());
		const double MinVolume = -TMathUtilConstants<double>::MaxReal;
		Editor.RemoveSmallComponents(MinVolume, MinAreaConnectedComponent, MinTriCountConnectedComponent);
	}
}