// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/BooleanMeshesOp.h"

#include "Operations/MeshBoolean.h"
#include "MeshBoundaryLoops.h"
#include "Operations/MinimalHoleFiller.h"
#include "CompGeom/PolygonTriangulation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BooleanMeshesOp)

using namespace UE::Geometry;

void FBooleanMeshesOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FBooleanMeshesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}
	check(Meshes.Num() == 2 && Transforms.Num() == 2);
	
	int FirstIdx = 0;
	if ((!bTrimMode && CSGOperation == ECSGOperation::DifferenceBA) || (bTrimMode && TrimOperation == ETrimOperation::TrimB))
	{
		FirstIdx = 1;
	}
	int OtherIdx = 1 - FirstIdx;

	FMeshBoolean::EBooleanOp Op;
	// convert UI enum to algorithm enum
	if (bTrimMode)
	{
		switch (TrimSide)
		{
		case ETrimSide::RemoveInside:
			Op = FMeshBoolean::EBooleanOp::TrimInside;
			break;
		case ETrimSide::RemoveOutside:
			Op = FMeshBoolean::EBooleanOp::TrimOutside;
			break;
		default:
			check(false);
			Op = FMeshBoolean::EBooleanOp::TrimInside;
		}
	}
	else
	{
		switch (CSGOperation)
		{
		case ECSGOperation::DifferenceAB:
		case ECSGOperation::DifferenceBA:
			Op = FMeshBoolean::EBooleanOp::Difference;
			break;
		case ECSGOperation::Union:
			Op = FMeshBoolean::EBooleanOp::Union;
			break;
		case ECSGOperation::Intersect:
			Op = FMeshBoolean::EBooleanOp::Intersect;
			break;
		default:
			check(false); // all conversion cases should be implemented
			Op = FMeshBoolean::EBooleanOp::Union;
		}
	}

	FMeshBoolean MeshBoolean(Meshes[FirstIdx].Get(), Transforms[FirstIdx], Meshes[OtherIdx].Get(), Transforms[OtherIdx], ResultMesh.Get(), Op);
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	MeshBoolean.bPutResultInInputSpace = false;
	MeshBoolean.bSimplifyAlongNewEdges = bTryCollapseExtraEdges;
	MeshBoolean.WindingThreshold = WindingThreshold;
	MeshBoolean.Progress = Progress;
	MeshBoolean.Compute();
	ResultTransform = MeshBoolean.ResultTransform;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	CreatedBoundaryEdges = MeshBoolean.CreatedBoundaryEdges;

	// try to fill cracks/holes in boolean result
	if (CreatedBoundaryEdges.Num() > 0 && bAttemptFixHoles)
	{
		FMeshBoundaryLoops OpenBoundary(MeshBoolean.Result, false);
		TSet<int> ConsiderEdges(CreatedBoundaryEdges);
		OpenBoundary.EdgeFilterFunc = [&ConsiderEdges](int EID)
		{
			return ConsiderEdges.Contains(EID);
		};
		OpenBoundary.Compute();

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		const bool bNeedsNormalsAndUVs = MeshBoolean.Result->HasAttributes();
		// UV scale based on the mesh bounds, used for hole-filled regions.
		const double MeshUVScaleFactor = (1.0 / MeshBoolean.Result->GetBounds().MaxDim());
		for (FEdgeLoop& Loop : OpenBoundary.Loops)
		{
			const int32 NewGroupID = MeshBoolean.Result->AllocateTriangleGroup();
			
			FMinimalHoleFiller Filler(MeshBoolean.Result, Loop);
			const bool bFilled = Filler.Fill(NewGroupID);

			if (!bFilled)
			{
				continue;
			}
			
			if (bNeedsNormalsAndUVs)  // Compute normals and UVs
			{
				// Compute a best-fit plane of the boundary vertices
				FVector3d PlaneOrigin, PlaneNormal;
				{
					TArray<FVector3d> VertexPositions;
					Loop.GetVertices(VertexPositions);

					PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
					PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere
				}


				FDynamicMeshEditor Editor(ResultMesh.Get());
				FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
				Editor.SetTriangleNormals(Filler.NewTriangles, (FVector3f)PlaneNormal);
				Editor.SetTriangleUVsFromProjection(Filler.NewTriangles, ProjectionFrame, MeshUVScaleFactor);
			}
		}

		TArray<int32> UpdatedBoundaryEdges;
		for (int EID : CreatedBoundaryEdges)
		{
			if (MeshBoolean.Result->IsEdge(EID) && MeshBoolean.Result->IsBoundaryEdge(EID))
			{
				UpdatedBoundaryEdges.Add(EID);
			}
		}
		CreatedBoundaryEdges = MoveTemp(UpdatedBoundaryEdges);
	}
}
