// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/CubeGridBooleanOp.h"

#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMeshEditor.h"
#include "Generators/GridBoxMeshGenerator.h"
#include "Generators/MeshShapeGenerator.h"
#include "MeshSimplification.h"
#include "Operations/MeshBoolean.h"

using namespace UE::Geometry;

namespace CubeGridBooleanOpLocals
{

/**
 * Generator for creating ramps/pyramids/pyramid cutaways in the bounds of a box. The "welded"
 * refers to welding vertices along the Z axis to create those shapes. See the comment in
 * FCubeGridBooleanOp::FCornerInfo for more information.
 * 
 * The generator assigns the group IDs 0-3 to the sides (which become 1-4 in the FDynamicMesh3).
 */
class FWeldedMinimalBoxMeshGenerator : public FMeshShapeGenerator
{
public:
	FOrientedBox3d Box;
	FCubeGridBooleanOp::FCornerInfo CornerInfo;
	bool bCrosswiseDiagonal = false;

	// See comments in CubeGridBooleanOp.h about how these work
	TArray<int32, TFixedAllocator<6>> FaceUVOrientations;
	double HeightUVOffset = 0;

	// When the generated shape is subtracted from things, the resulting faces end up being seen from
	// to opposite direction, which will make the UVs look flipped. This bool allows us to pre-flip them.
	bool bFlipUVsForSubtraction = false;
		
	virtual FMeshShapeGenerator& Generate() override
	{
		// Fill any unset orientations with 0.
		FaceUVOrientations.SetNumZeroed(6);

		// Make sure all orientations are in [0,3]
		for (int& Orientation : FaceUVOrientations)
		{
			if (Orientation < 0)
			{
				Orientation = 4 - ((-Orientation) % 4);
			}
			Orientation %= 4;
		}

		// There is a particular odd case that we have to guard against. If one set of diagonal
		// corners is welded and the second set is unwelded, and if the chosen triangulation
		// diagonal connects the welded vertices, then the edge between these verts would normally
		// have four connected triangles, since both top and bottom use it. The two solutions to
		// this are (a) triangulate the bottom differently, which is fine since it's flat, or
		// (b) duplicate the welded verts, essentially building two pyramids with triangular bases.
		// While option b is a little more hassle, it seems like the one that may result in
		// cleaner geometry, so that is what we do. We do it by temporarily welding the raised
		// verts to create two independent concatenated pyramids.

		int FaceIndexWeldingToModify = -1;
		if (bCrosswiseDiagonal)
		{
			if (CornerInfo.WeldedAtBase[0] && CornerInfo.WeldedAtBase[2] 
				&& !CornerInfo.WeldedAtBase[1] && !CornerInfo.WeldedAtBase[3])
			{
				FaceIndexWeldingToModify = 1;
			}
			else if (CornerInfo.WeldedAtBase[1] && CornerInfo.WeldedAtBase[3]
				&& !CornerInfo.WeldedAtBase[0] && !CornerInfo.WeldedAtBase[2])
			{
				FaceIndexWeldingToModify = 0;
			}
		}

		if (FaceIndexWeldingToModify >= 0)
		{
			CornerInfo.WeldedAtBase[FaceIndexWeldingToModify] = true;
			AppendUsingCurrentSettings();
			Swap(CornerInfo.WeldedAtBase[FaceIndexWeldingToModify], CornerInfo.WeldedAtBase[FaceIndexWeldingToModify + 2]);
			AppendUsingCurrentSettings();
			CornerInfo.WeldedAtBase[FaceIndexWeldingToModify + 2] = false;
		}
		else
		{
			AppendUsingCurrentSettings();
		}

		return *this;
	}

protected:

	// This is basically the full generation function, but importantly it does all of the
	// tri/element allocations using append, which lets us use it more than once (see above)
	void AppendUsingCurrentSettings()
	{
		// Mapping from corner indices (like in TOrientedBox3::GetCorner())
		int CornerToVert[8] = { -1,-1,-1,-1,-1,-1,-1,-1 };

		int GroupID = 0;

		// These actually get reinitialized during each face traversal, and are out here as globals
		// to the lambdas we use (to avoid having to pass tons of args). They are indexed by 0-3 
		// counterclockwise from the bottom left corner, like IndexUtil::BoxFacesUV
		int UVIndices[4];
		int NormalIndices[4];
		int FaceCornerToVert[4];
		double FaceWidth;
		double FaceHeight;

		auto MakeUVsAndNormals = [this, &UVIndices, &NormalIndices, &FaceCornerToVert, &FaceWidth, &FaceHeight]
		(int FaceCornerIndex, const FVector3f& Normal, int ParentVert, float HeightOffset, int FaceOrientation) {
			
			/*
			 * FaceOrientation determines how the face is rotated in UV space. The assignment table ends up:
			 * 
			 *    FaceOrientation
			 * corner  0    1    2    3
			 *   0    0,0  h,0  w,h  0,w
			 *   1    w,0  h,w  0,h  0,0
			 *   2    w,h  0,w  0,0  h,0
			 *   3    0,h  0,0  w,0  h,w
			 * 
			 * Where HeightOffset needs to be added to the column where h is used (V for rotations 0 and 2, U for others)
			 * 
			 * From the table, we can see that we can index into IndexUtil::BoxFacesUV with an index that is rotated
			 * backwards from FaceCornerIndex FaceOrientation number of times to get multiplier that we can then multiply
			 * by w and h appropriately.
			 */

			int RotatedCornerIndex = (FaceCornerIndex + 4 - FaceOrientation) % 4;

			// If flipping for subtraction, the order of corners changes from 0, 1, 2, 3 (CCW) to 1, 0, 3, 2.
			if (bFlipUVsForSubtraction)
			{
				// Static analysis doesn't like this commented out version because it thinks that RotatedCornerIndex can
				// end up outside [0,3], which it can't. But we rephrase it below.
				//RotatedCornerIndex = (RotatedCornerIndex % 2) ? RotatedCornerIndex - 1 : RotatedCornerIndex + 1;

				static const int Adjustments[4] = { 1, -1, 1, -1 };
				RotatedCornerIndex += Adjustments[RotatedCornerIndex];
			}

			FVector2i UVBase = IndexUtil::BoxFacesUV[RotatedCornerIndex];

			FVector2f UV;
			if (FaceOrientation % 2 == 0)
			{
				UV.X = UVBase.X * FaceWidth;
				UV.Y = UVBase.Y * FaceHeight + HeightOffset;
			}
			else
			{
				UV.X = UVBase.X * FaceHeight + HeightOffset;
				UV.Y = UVBase.Y * FaceWidth;
			}

			UVIndices[FaceCornerIndex] = AppendUV(UV, ParentVert);
			
			NormalIndices[FaceCornerIndex] = AppendNormal(Normal, ParentVert);
			FaceCornerToVert[FaceCornerIndex] = ParentVert;
		};

		auto MakeTriangle = [this, &UVIndices, &NormalIndices, &FaceCornerToVert, &GroupID]
		(int FaceCorner1, int FaceCorner2, int FaceCorner3)
		{
			const int TriIndex = AppendTriangle(FaceCornerToVert[FaceCorner1], FaceCornerToVert[FaceCorner2], FaceCornerToVert[FaceCorner3]);
			SetTrianglePolygon(TriIndex, GroupID);
			SetTriangleUVs(TriIndex, UVIndices[FaceCorner1], UVIndices[FaceCorner2], UVIndices[FaceCorner3]);
			SetTriangleNormals(TriIndex, NormalIndices[FaceCorner1], NormalIndices[FaceCorner2], NormalIndices[FaceCorner3]);
		};

		// Ordered in the way we iterate across the sides of the box
		FVector3f SideFaceBoxSpaceNormals[4] = {
			(FVector3f)-Box.AxisX(),
			(FVector3f)-Box.AxisY(),
			(FVector3f)Box.AxisX(),
			(FVector3f)Box.AxisY(),
		};

		// Maps each left corner of the bottom face to the IndexUtil::BoxFaces index of the face that we'll construct.
		int BottomCornerToSideFaceIndex[4] = { 4, 3, 5, 2 };

		// Set up the sides
		for (int LeftCorner = 0; LeftCorner < 4; ++LeftCorner)
		{
			GroupID = LeftCorner;

			int FaceIndex = BottomCornerToSideFaceIndex[LeftCorner];
			int FaceUVOrientation = FaceUVOrientations[FaceIndex];

			// We're iterating over the bottom corners of the oriented box (see TOrientedBox3::GetCorner()),
			// but due to the way these are numbered, if we are looking at the side, the bottom two verts
			// actually decrease in corner index (wrapping around).
			int RightCorner = (LeftCorner + 3) % 4;
			if (CornerInfo.WeldedAtBase[LeftCorner] && CornerInfo.WeldedAtBase[RightCorner])
			{
				// This side is fully welded (i.e. doesn't exist)
				continue;
			}

			// Indices of the corners up the Z axis
			int LeftUp = LeftCorner + 4;
			int RightUp = RightCorner + 4;

			// Create the necessary verts if they don't exist yet
			if (CornerToVert[LeftCorner] < 0)
			{
				CornerToVert[LeftCorner] = AppendVertex(Box.GetCorner(LeftCorner));
			}
			if (CornerToVert[RightCorner] < 0)
			{
				CornerToVert[RightCorner] = AppendVertex(Box.GetCorner(RightCorner));
			}
			if (!CornerInfo.WeldedAtBase[LeftCorner] && CornerToVert[LeftUp] < 0)
			{
				CornerToVert[LeftUp] = AppendVertex(Box.GetCorner(LeftUp));
			}
			if (!CornerInfo.WeldedAtBase[RightCorner] && CornerToVert[RightUp] < 0)
			{
				CornerToVert[RightUp] = AppendVertex(Box.GetCorner(RightUp));
			}

			FaceWidth = Distance(Box.GetCorner(LeftCorner), Box.GetCorner(RightCorner));
			FaceHeight = Distance(Box.GetCorner(LeftCorner), Box.GetCorner(LeftUp));

			// The bottom two vertices will definitely need uv/normal elements
			MakeUVsAndNormals(0, SideFaceBoxSpaceNormals[LeftCorner], CornerToVert[LeftCorner], HeightUVOffset, FaceUVOrientation);
			MakeUVsAndNormals(1, SideFaceBoxSpaceNormals[LeftCorner], CornerToVert[RightCorner], HeightUVOffset, FaceUVOrientation);

			if (!CornerInfo.WeldedAtBase[LeftCorner])
			{
				MakeUVsAndNormals(3, SideFaceBoxSpaceNormals[LeftCorner], CornerToVert[LeftUp], HeightUVOffset, FaceUVOrientation);
				MakeTriangle(0, 1, 3);
				if (!CornerInfo.WeldedAtBase[RightCorner])
				{
					MakeUVsAndNormals(2, SideFaceBoxSpaceNormals[LeftCorner], CornerToVert[RightUp], HeightUVOffset, FaceUVOrientation);
					MakeTriangle(3, 1, 2);
				}
			}
			else
			{
				MakeUVsAndNormals(2, SideFaceBoxSpaceNormals[LeftCorner], CornerToVert[RightUp], HeightUVOffset, FaceUVOrientation);
				MakeTriangle(0, 1, 2);
			}
		}// end making side faces

		// 0-3 are reserved group id's for sides even if a side doesn't exist.
		GroupID = 4;

		// Figure out which direction we're going to triangulate the top and bottom. 
		// In the normal case, we place the diagonal such that a single raised/lowered
		// corner results in a pyramid with a square (rather than triangular) base, and
		// a checkerboard raised/lowered state results in a raised ridge. In the crosswise
		// case we flip the diagonal.
		int DiagFaceIdx1 = 0;
		if (CornerInfo.WeldedAtBase[1] != CornerInfo.WeldedAtBase[3] || 
			(!CornerInfo.WeldedAtBase[1] && CornerInfo.WeldedAtBase[0] && CornerInfo.WeldedAtBase[2]))
		{
			DiagFaceIdx1 = 1;
		}
		DiagFaceIdx1 = bCrosswiseDiagonal ? 1 - DiagFaceIdx1 : DiagFaceIdx1;

		// DiagFaceIdx1 is either 0 or 1 here and static analysis will complain if we do
		// mod 4 for some of the later statements. Throw in a check in case we ever change
		// the above.
		checkSlow(DiagFaceIdx1 == 0 || DiagFaceIdx1 == 1);
		int DiagFaceIdx2 = DiagFaceIdx1 + 2;
		int CCWFromDiag1 = DiagFaceIdx1 + 1;
		int CCWFromDiag2 = (DiagFaceIdx2 + 1) % 4;

		// We've iterated across the sides but it's still actually possible to end up not
		// allocating one of the vertices if we're making a rectangle-base pyramid.
		if (CornerToVert[DiagFaceIdx1] < 0)
		{
			CornerToVert[DiagFaceIdx1] = AppendVertex(Box.GetCorner(DiagFaceIdx1));
		}
		if (CornerToVert[DiagFaceIdx2] < 0)
		{
			CornerToVert[DiagFaceIdx2] = AppendVertex(Box.GetCorner(DiagFaceIdx2));
		}

		// Triangulate the bottom:
		// Prep for UVs
		FaceWidth = Distance(Box.GetCorner(0), Box.GetCorner(1));
		FaceHeight = Distance(Box.GetCorner(0), Box.GetCorner(3));
		int FaceUVOrientation = FaceUVOrientations[0];

		MakeUVsAndNormals(DiagFaceIdx1, (FVector3f)-Box.Frame.Z(), CornerToVert[DiagFaceIdx1], 0, FaceUVOrientation);
		MakeUVsAndNormals(DiagFaceIdx2, (FVector3f)-Box.Frame.Z(), CornerToVert[DiagFaceIdx2], 0, FaceUVOrientation);

		bool bDiagonalWelded = CornerInfo.WeldedAtBase[DiagFaceIdx1] && CornerInfo.WeldedAtBase[DiagFaceIdx2];
		if (!(bDiagonalWelded && CornerInfo.WeldedAtBase[CCWFromDiag1]))
		{
			MakeUVsAndNormals(CCWFromDiag1, (FVector3f)-Box.Frame.Z(), CornerToVert[CCWFromDiag1], 0, FaceUVOrientation);
			MakeTriangle(DiagFaceIdx1, CCWFromDiag1, DiagFaceIdx2);
		}
		if (!(bDiagonalWelded && CornerInfo.WeldedAtBase[CCWFromDiag2]))
		{
			MakeUVsAndNormals(CCWFromDiag2, (FVector3f)-Box.Frame.Z(), CornerToVert[CCWFromDiag2], 0, FaceUVOrientation);
			MakeTriangle(DiagFaceIdx1, DiagFaceIdx2, CCWFromDiag2);
		}
		++GroupID;

		// Triangulate the top. This is slightly different from everything else because
		// this face is not oriented to the box axes
		
		// Our ccw order of top corners
		int FaceIdxToOrigIdx[4] = { 5, 4, 7, 6 };

		// Given 0-3 index in ccw face ordering of top, is that vert welded?
		bool TopFaceIdxWelded[4] = { false, false, false, false };

		// Given 0-3 index in ccw face ordering of top, what is the position in space
		FVector3d CornerPositions[4];

		// Fill out the above and other structures by looking at welding of bottom verts
		for (int i = 0; i < 4; ++i)
		{
			int CornerIdx = FaceIdxToOrigIdx[i];
			if (CornerInfo.WeldedAtBase[CornerIdx - 4]) // downward index in box
			{
				CornerToVert[CornerIdx] = CornerToVert[CornerIdx - 4];
				CornerPositions[i] = Box.GetCorner(CornerIdx - 4);
				TopFaceIdxWelded[i] = true;
			}
			else
			{
				CornerPositions[i] = Box.GetCorner(CornerIdx);
			}
			FaceCornerToVert[i] = CornerToVert[CornerIdx];
		}

		// Prep for setting up UV's. We'll still get stretching when we're not planar, 
		// but that's probably not worth dealing with.
		FaceWidth = FMath::Max(
			FVector3d::Distance(CornerPositions[0], CornerPositions[1]), 
			FVector3d::Distance(CornerPositions[2], CornerPositions[3]));
		FaceHeight = FMath::Max(
			FVector3d::Distance(CornerPositions[0], CornerPositions[3]),
			FVector3d::Distance(CornerPositions[1], CornerPositions[2]));
		FaceUVOrientation = FaceUVOrientations[1];

		// We're looking at things from the opposite direction, so our diagonal needs to
		// connect the opposite face indices.
		checkSlow(DiagFaceIdx1 == 0 || DiagFaceIdx1 == 1);
		DiagFaceIdx1 = 1 - DiagFaceIdx1;
		DiagFaceIdx2 = DiagFaceIdx1 + 2;
		CCWFromDiag1 = DiagFaceIdx1 + 1;
		CCWFromDiag2 = (DiagFaceIdx2 + 1) % 4;

		bDiagonalWelded = TopFaceIdxWelded[DiagFaceIdx1] && TopFaceIdxWelded[DiagFaceIdx2];

		bool bIsPlanar = (TopFaceIdxWelded[0] == TopFaceIdxWelded[1]
			&& TopFaceIdxWelded[2] == TopFaceIdxWelded[3])
			|| (TopFaceIdxWelded[1] == TopFaceIdxWelded[2]
				&& TopFaceIdxWelded[3] == TopFaceIdxWelded[0]);

		bool bFirstTriExists = !(bDiagonalWelded && TopFaceIdxWelded[CCWFromDiag1]);
		if (bFirstTriExists)
		{
			FVector3f AToC(CornerPositions[DiagFaceIdx2] - CornerPositions[DiagFaceIdx1]);
			FVector3f AToB(CornerPositions[CCWFromDiag1] - CornerPositions[DiagFaceIdx1]);
			FVector3f Normal = AToC.Cross(AToB);
			Normal.Normalize();
				
			MakeUVsAndNormals(DiagFaceIdx1, Normal, FaceCornerToVert[DiagFaceIdx1], 0, FaceUVOrientation);
			MakeUVsAndNormals(CCWFromDiag1, Normal, FaceCornerToVert[CCWFromDiag1], 0, FaceUVOrientation);
			MakeUVsAndNormals(DiagFaceIdx2, Normal, FaceCornerToVert[DiagFaceIdx2], 0, FaceUVOrientation);

			MakeTriangle(DiagFaceIdx1, CCWFromDiag1, DiagFaceIdx2);

			if (!bIsPlanar)
			{
				++GroupID;
			}
		}

		if (!(bDiagonalWelded && TopFaceIdxWelded[CCWFromDiag2]))
		{
			if (bIsPlanar)
			{
				// Have to make this copy because we're not allowed to pass a reference to an element
				// in the container we're modifying.
				MakeUVsAndNormals(CCWFromDiag2, FVector3f(Normals[NormalIndices[DiagFaceIdx1]]), FaceCornerToVert[CCWFromDiag2], 0, FaceUVOrientation);
			}
			else
			{
				FVector3d AToC(CornerPositions[CCWFromDiag2] - CornerPositions[DiagFaceIdx1]);
				FVector3d AToB(CornerPositions[DiagFaceIdx2] - CornerPositions[DiagFaceIdx1]);
				FVector3d Normal = AToC.Cross(AToB);
				Normal.Normalize();

				if (bFirstTriExists)
				{
					NormalIndices[DiagFaceIdx1] = AppendNormal((FVector3f)Normal, FaceCornerToVert[DiagFaceIdx1]);
					NormalIndices[DiagFaceIdx2] = AppendNormal((FVector3f)Normal, FaceCornerToVert[DiagFaceIdx2]);
				}
				else
				{
					MakeUVsAndNormals(DiagFaceIdx1, (FVector3f)Normal, FaceCornerToVert[DiagFaceIdx1], 0, FaceUVOrientation);
					MakeUVsAndNormals(DiagFaceIdx2, (FVector3f)Normal, FaceCornerToVert[DiagFaceIdx2], 0, FaceUVOrientation);
				}
				MakeUVsAndNormals(CCWFromDiag2, (FVector3f)Normal, FaceCornerToVert[CCWFromDiag2], 0, FaceUVOrientation);
			}
			MakeTriangle(DiagFaceIdx1, DiagFaceIdx2, CCWFromDiag2);
		}// end triangulating second tri
	}
};

void GetTrisWithChangedPositionsAndConnectivity(
	const FDynamicMesh3& StartMeshIn, const FDynamicMesh3& EndMeshIn, TArray<int32>& TidsOut)
{
	// The boolean operations don't currently keep track of the triangles they modify. So instead,
	// we do our own pass and check vertex positions and triangles to see which changed.
	TSet<int32> ChangedVids;
	for (int32 Vid : StartMeshIn.VertexIndicesItr())
	{
		if (!EndMeshIn.IsVertex(Vid) || StartMeshIn.GetVertex(Vid) != EndMeshIn.GetVertex(Vid))
		{
			ChangedVids.Add(Vid);
		}
	}
	for (int32 Tid : StartMeshIn.TriangleIndicesItr())
	{
		if (!EndMeshIn.IsTriangle(Tid))
		{
			TidsOut.Add(Tid);
			continue;
		}

		FIndex3i TriVids = StartMeshIn.GetTriangle(Tid);
		if (TriVids != EndMeshIn.GetTriangle(Tid)
			|| ChangedVids.Contains(TriVids.A)
			|| ChangedVids.Contains(TriVids.B)
			|| ChangedVids.Contains(TriVids.C))
		{
			TidsOut.Add(Tid);
		}
	}
}//end FWeldedMinimalBoxMeshGenerator
}//end CubeGridBooleanOpLocals

void FCubeGridBooleanOp::CalculateResult(FProgressCancel* Progress)
{
	using namespace CubeGridBooleanOpLocals;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*InputMesh);
	if (!ResultMesh->HasTriangleGroups())
	{
		ResultMesh->EnableTriangleGroups();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	TUniquePtr<FDynamicMesh3> OpMesh;

	FWeldedMinimalBoxMeshGenerator Generator;
	Generator.Box = WorldBox;
	if (CornerInfo)
	{
		Generator.CornerInfo = *CornerInfo;
	}
	Generator.bCrosswiseDiagonal = bCrosswiseDiagonal;
	Generator.HeightUVOffset = OpMeshHeightUVOffset;
	Generator.bFlipUVsForSubtraction = bSubtract;
	Generator.FaceUVOrientations = FaceUVOrientations;

	OpMesh = MakeUnique<FDynamicMesh3>(&Generator.Generate());

	if (ensure(OpMesh->HasAttributes()))
	{
		// Apply the given material ID to the operator mesh if needed.
		OpMesh->Attributes()->EnableMaterialID();
		FDynamicMeshMaterialAttribute* MaterialIDs = OpMesh->Attributes()->GetMaterialID();
		for (int32 Tid : OpMesh->TriangleIndicesItr())
		{
			MaterialIDs->SetValue(Tid, OpMeshMaterialID);
		}

		// Rescale UV's
		FDynamicMeshEditor Editor(OpMesh.Get());
		float WorldUnitsInMetersFactor = .01f;
		float UVScaleToUse = bWorldSpaceUVs ? WorldUnitsInMetersFactor * UVScale : UVScale;
		Editor.RescaleAttributeUVs(UVScaleToUse, bWorldSpaceUVs);

		// Unreal stores V's as 1-V, so we probably want to apply this here so it aligns with everything
		// else. However this should happen after the scaling above.
		FDynamicMeshUVOverlay* Overlay = OpMesh->Attributes()->PrimaryUV();
		for (int32 ElementID : Overlay->ElementIndicesItr())
		{
			FVector2f UV = Overlay->GetElement(ElementID);
			Overlay->SetElement(ElementID, FVector2f(UV.X, 1 - UV.Y));
		}
	}

	if (!ResultMesh->HasAttributes())
	{
		ResultMesh->EnableAttributes();
	}
	if (!ResultMesh->Attributes()->HasMaterialID())
	{
		// This usually happens when we start with an empty mesh. Materials need to
		// be enabled or else we won't add in the OpMesh one during the boolean operation.
		ResultMesh->Attributes()->EnableMaterialID();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	
	// Perform the boolean operation.
	FMeshBoolean MeshBoolean(ResultMesh.Get(), InputTransform,
		OpMesh.Get(), FTransformSRT3d::Identity(), ResultMesh.Get(),
		bSubtract ? FMeshBoolean::EBooleanOp::Difference : FMeshBoolean::EBooleanOp::Union);

	MeshBoolean.bPutResultInInputSpace = true;
	MeshBoolean.bSimplifyAlongNewEdges = true;
	MeshBoolean.bPopulateSecondMeshGroupMap = true;
	MeshBoolean.Compute();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Adjust the transform
	if (bKeepInputTransform)
	{
		ResultTransform = InputTransform;
	}
	else
	{
		// Set the transform to be in the center.
		FVector3d Center = ResultMesh->GetBounds().Center();
		ResultTransform = FTransformSRT3d(Center);
	}
	MeshTransforms::ApplyTransformInverse(*ResultMesh, InputTransform, true);

	// TODO: Is it worth trying to fix holes like BooleanMeshesOp tries to?

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Prep to remap the side groups if necessary (or output the new side groups)
	TMap<int32, int32> GroupRemappingToApply;
	if (ensure(OpMeshSideGroups.Num() == 4))
	{
		for (int SideGroupIndex = 0; SideGroupIndex < 4; ++SideGroupIndex)
		{
			// We assign the sides groups 0-3 in the generator, but these get turned into 1-4
			// in the conversion to DynamicMesh.
			int32 OpMeshGroupID = SideGroupIndex + 1;
			const int* ResultGroup = MeshBoolean.SecondMeshGroupMap.FindTo(OpMeshGroupID);
			if (ResultGroup)
			{
				if (OpMeshSideGroups[SideGroupIndex] != IndexConstants::InvalidID)
				{
					GroupRemappingToApply.Add(*ResultGroup, OpMeshSideGroups[SideGroupIndex]);
				}
				else
				{
					OpMeshSideGroups[SideGroupIndex] = *ResultGroup;
				}
			}
		}
	}
	// Do the actual group remapping
	if (!GroupRemappingToApply.IsEmpty())
	{
		for (int32 Tid : ResultMesh->TriangleIndicesItr())
		{
			const int32* RemapGroup = GroupRemappingToApply.Find(ResultMesh->GetTriangleGroup(Tid));
			if (RemapGroup)
			{
				ResultMesh->SetTriangleGroup(Tid, *RemapGroup);;
			}
		}

		if (Progress && Progress->Cancelled())
		{
			return;
		}
	}

	if (bTrackChangedTids)
	{
		ChangedTids = MakeShared<TArray<int32>, ESPMode::ThreadSafe>();
		GetTrisWithChangedPositionsAndConnectivity(*InputMesh, *ResultMesh, *ChangedTids);
	}
}
