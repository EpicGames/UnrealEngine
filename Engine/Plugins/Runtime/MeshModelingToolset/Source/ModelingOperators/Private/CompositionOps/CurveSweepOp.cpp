// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/CurveSweepOp.h"

#include "ConstrainedDelaunay2.h"
#include "DynamicMeshEditor.h"
#include "MeshBoundaryLoops.h"
#include "CompGeom/PolygonTriangulation.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Operations/SimpleHoleFiller.h"
#include "Operations/PlanarHoleFiller.h"

using namespace UE::Geometry;

void FCurveSweepOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Make sure our inputs make sense.
	// (We could be a little less strict and allow things like a 1-sweep point capped
	// result to make double sided polygon, but those are likely to cause headaches later,
	// and generally represent a misuse of a tool)
	if (SweepCurve.Num() < (bSweepCurveIsClosed ? 3 : 2)
		|| ProfileCurve.Num() < (bProfileCurveIsClosed ? 3 : 2)
		|| ProfileVerticesToWeld.Num() >= ProfileCurve.Num())
	{
		ResultMesh->Clear();
		return;
	}

	// Use a mesh generator to do the work.
	FProfileSweepGenerator CurveSweeper;
	CurveSweeper.ProfileCurve = ProfileCurve;
	CurveSweeper.SweepCurve = SweepCurve;
	CurveSweeper.WeldedVertices = ProfileVerticesToWeld;
	CurveSweeper.bProfileCurveIsClosed = bProfileCurveIsClosed;
	CurveSweeper.bSweepCurveIsClosed = bSweepCurveIsClosed;

	CurveSweeper.bSharpNormals = bSharpNormals;
	CurveSweeper.UVScale = UVScale;
	CurveSweeper.bUVScaleRelativeWorld = bUVScaleRelativeWorld;
	CurveSweeper.UnitUVInWorldCoordinates = UnitUVInWorldCoordinates;
	CurveSweeper.UVOffset = UVOffset;
	CurveSweeper.bUVsSkipFullyWeldedEdges = bUVsSkipFullyWeldedEdges;
	CurveSweeper.PolygonGroupingMode = PolygonGroupingMode;
	CurveSweeper.QuadSplitMethod = QuadSplitMode;
	CurveSweeper.DiagonalTolerance = DiagonalTolerance;
	CurveSweeper.Generate();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(&CurveSweeper);

	// Cap the ends if needed
	if (bProfileCurveIsClosed && !bSweepCurveIsClosed && CapFillMode != ECapFillMode::None)
	{
		for (int CapIndex = 0; CapIndex < 2; ++CapIndex)
		{
			if (Progress && Progress->Cancelled())
			{
				return;
			}

			// Note that we rely on the sweep generator to tell us the start and end profile curves for
			// capping rather than looking for a boundary ourselves. This is because in cases where the
			// side of the profile curve is welded, there will actually be one open boundary rather than
			// two separate ones, yet we want to cap the two sides separately.
			// This does rely on the fact that the hole fillers we use only care about vertices, and don't
			// rely on there being edges between them. It also relies on the vertex ids in the DynamicMesh
			// matching the ones used in the generator. Without those two facts, the work of filling two
			// halves of an open boundary separately would be harder to do...

			// Put the profile curve into an edge loop
			TArray<int32> Edges;
			FEdgeLoop::VertexLoopToEdgeLoop(ResultMesh.Get(), CurveSweeper.EndProfiles[CapIndex], Edges);
			FEdgeLoop Loop(ResultMesh.Get(), CurveSweeper.EndProfiles[CapIndex], Edges);

			// The first cap actually needs to face in the opposite direction of the sweep to face outward.
			if (CapIndex == 0)
			{
				Loop.Reverse();
			}

			// Compute a best-fit plane of the vertices
			TArray<FVector3d> VertexPositions;
			Loop.GetVertices(VertexPositions);
			FVector3d PlaneOrigin;
			FVector3d PlaneNormal;
			PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
			PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere

			TUniquePtr<IHoleFiller> CapFiller;
			TArray<TArray<int>> VertexLoops;// Used for input if CapFillMode is Delaunay
			switch (CapFillMode)
			{
			case ECapFillMode::CenterFan:
				CapFiller = MakeUnique<FSimpleHoleFiller>(ResultMesh.Get(), Loop, FSimpleHoleFiller::EFillType::TriangleFan);
				break;
			case ECapFillMode::EarClipping:
				CapFiller = MakeUnique<FSimpleHoleFiller>(ResultMesh.Get(), Loop, FSimpleHoleFiller::EFillType::PolygonEarClipping);
				break;
			case ECapFillMode::Delaunay:
				VertexLoops.Add(Loop.Vertices);
				CapFiller = MakeUnique<FPlanarHoleFiller>(ResultMesh.Get(),
					&VertexLoops,
					ConstrainedDelaunayTriangulate<double>,
					PlaneOrigin,
					PlaneNormal);
				break;
			default:
				checkSlow(false);
			}

			int32 NewGroupID = ResultMesh->AllocateTriangleGroup();
			CapFiller->Fill(NewGroupID);

			// Compute normals and UVs
			if (ResultMesh->HasAttributes())
			{
				FDynamicMeshEditor Editor(ResultMesh.Get());

				// If we're using sharp normals, we don't need to do anything since we'll do a separate pass to 
				// generate them for the whole mesh. Otherwise, update normals for the cap.
				if (!bSharpNormals)
				{
					Editor.SetTriangleNormals(CapFiller->NewTriangles);
				}
					
				// Set UVs for the cap
				FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
				if (bUVScaleRelativeWorld)
				{
					FVector2f ScaleToUse(UVScale);
					ScaleToUse /= CurveSweeper.UnitUVInWorldCoordinates;
					Editor.SetTriangleUVsFromProjection(CapFiller->NewTriangles, ProjectionFrame, ScaleToUse, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						false); // don't normalize
				}
				else
				{
					Editor.SetTriangleUVsFromProjection(CapFiller->NewTriangles, ProjectionFrame, (FVector2f)UVScale, (FVector2f)UVOffset,
						0, // layer
						true,  // shift to origin
						true); // normalize first
				}
				
			}
		}// end for each loop
	}//end if capping

	if (bSharpNormals)
	{
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		// There is some wasted work in the sharp normal case: the sweep generator creates non-sharp normals unnecessarily,
		// and calculates triangle normals in the process that don't get saved for reuse here. The simplest fix is to
		// give the generator the option to not create normals, knowing that we'll do it here, but that feels crude.
		// The other option- calculating sharp normals in the generator itself- wouldn't use our existing code, since
		// the FMeshNormals code operates on a dynamic mesh.
		// If we decide that we really need the performance, we can do one of the above, but for now it seems not worth dirtying
		// our code.

		// Start by calculating triangle normals
		FMeshNormals NormalsUtility(ResultMesh.Get());
		NormalsUtility.ComputeTriangleNormals();
		const TArray<FVector3d>& Normals = NormalsUtility.GetNormals();

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		// Figure out which triangle vertices will share normals
		float NormalDotProdThreshold = FMathf::Cos(SharpNormalAngleTolerance * FMathf::DegToRad);
		ResultMesh->Attributes()->PrimaryNormals()->CreateFromPredicate([&Normals, &NormalDotProdThreshold](int VID, int TA, int TB)
			{
				return Normals[TA].Dot(Normals[TB]) > NormalDotProdThreshold;
			}, 0);

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		// Actually calculate the normals
		NormalsUtility.RecomputeOverlayNormals(ResultMesh->Attributes()->PrimaryNormals(), false, true);
		NormalsUtility.CopyToOverlay(ResultMesh->Attributes()->PrimaryNormals(), false);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Set the pivot of the mesh to the bounding box center
	FVector3d PivotLocation = ResultMesh->GetBounds(true).Center();
	MeshTransforms::Translate(*ResultMesh, -PivotLocation);
	ResultTransform = FTransformSRT3d::Identity();
	ResultTransform.SetTranslation(PivotLocation);
}

