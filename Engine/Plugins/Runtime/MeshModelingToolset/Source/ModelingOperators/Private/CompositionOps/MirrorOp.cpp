// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/MirrorOp.h"

#include "Operations/MeshPlaneCut.h"

using namespace UE::Geometry;

void FMirrorOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FMirrorOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);
	
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Crop if we need to.
	TArray<int32> EdgesToWeld;
	if (bCropFirst)
	{
		// Note: there is some work duplication, because inside, both mirroring and cutting with a plane
		// compute a signed distance from the plane. To share these results, we would need to edit the
		// cutter to keep them updated as it removes and adds vertices. We are not currently doing that,
		// but we could implement it.

		FMeshPlaneCut Cutter(ResultMesh.Get(), LocalPlaneOrigin, LocalPlaneNormal);
		Cutter.PlaneTolerance = PlaneTolerance;
		Cutter.bSimplifyAlongNewEdges = bSimplifyAlongNewEdges;
		Cutter.Cut();

		if (Progress && Progress->Cancelled())
		{
			return;
		}
	}

	// Set up the mirror operation
	FMeshMirror Mirrorer(ResultMesh.Get(), LocalPlaneOrigin, LocalPlaneNormal);
	Mirrorer.bWeldAlongPlane = bWeldAlongPlane;
	Mirrorer.WeldNormalMode = WeldNormalMode;
	Mirrorer.bAllowBowtieVertexCreation = bAllowBowtieVertexCreation;
	Mirrorer.PlaneTolerance = PlaneTolerance;

	// Run the operation
	if (bAppendToOriginal)
	{
		Mirrorer.MirrorAndAppend(Progress);
	}
	else
	{
		Mirrorer.Mirror(Progress);
	}
}
