// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/SelfUnionMeshesOp.h"

#include "Operations/MeshSelfUnion.h"
#include "MeshBoundaryLoops.h"
#include "Operations/MinimalHoleFiller.h"

using namespace UE::Geometry;

void FSelfUnionMeshesOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FSelfUnionMeshesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	*ResultMesh = *CombinedMesh;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	FMeshSelfUnion Union(ResultMesh.Get());
	Union.WindingThreshold = WindingNumberThreshold;
	Union.bTrimFlaps = bTrimFlaps;
	Union.bSimplifyAlongNewEdges = bTryCollapseExtraEdges;
	bool bSuccess = Union.Compute();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	CreatedBoundaryEdges = Union.CreatedBoundaryEdges;

	if (!bSuccess && bAttemptFixHoles)
	{
		FMeshBoundaryLoops OpenBoundary(ResultMesh.Get(), false);
		TSet<int> ConsiderEdges(CreatedBoundaryEdges);
		OpenBoundary.EdgeFilterFunc = [&ConsiderEdges](int EID)
		{
			return ConsiderEdges.Contains(EID);
		};
		OpenBoundary.Compute();

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		for (FEdgeLoop& Loop : OpenBoundary.Loops)
		{
			FMinimalHoleFiller Filler(ResultMesh.Get(), Loop);
			Filler.Fill();
		}

		TArray<int32> UpdatedBoundaryEdges;
		for (int EID : Union.CreatedBoundaryEdges)
		{
			if (ResultMesh->IsEdge(EID) && ResultMesh->IsBoundaryEdge(EID))
			{
				UpdatedBoundaryEdges.Add(EID);
			}
		}
		CreatedBoundaryEdges = MoveTemp(UpdatedBoundaryEdges);
	}

}