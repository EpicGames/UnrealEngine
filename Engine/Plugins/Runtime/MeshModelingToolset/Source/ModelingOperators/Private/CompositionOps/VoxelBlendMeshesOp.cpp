// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/VoxelBlendMeshesOp.h"
#include "CleaningOps/EditNormalsOp.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/MeshNormals.h"
#include "Operations/ExtrudeMesh.h"
#include "Operations/RemoveOccludedTriangles.h"

#include "Implicit/Solidify.h"
#include "Implicit/Blend.h"

using namespace UE::Geometry;

void FVoxelBlendMeshesOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FVoxelBlendMeshesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (!ensure(Transforms.Num() == Meshes.Num()))
	{
		return;
	}

	TImplicitBlend<FDynamicMesh3> ImplicitBlend;
	ImplicitBlend.bSubtract = bSubtract;

	FVector3d AverageTranslation = GetAverageTranslation<FTransform>(Transforms);
	ResultTransform = FTransformSRT3d(AverageTranslation);

	TArray<FDynamicMesh3> TransformedMeshes; TransformedMeshes.Reserve(Meshes.Num());
	FAxisAlignedBox3d CombinedBounds = FAxisAlignedBox3d::Empty();
	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		TransformedMeshes.Emplace(*Meshes[MeshIdx]);
		if (TransformedMeshes[MeshIdx].TriangleCount() == 0)
		{
			continue;
		}
		
		FTransformSRT3d ToApply = Transforms[MeshIdx];
		ToApply.SetTranslation(ToApply.GetTranslation() - AverageTranslation);
		MeshTransforms::ApplyTransform(TransformedMeshes[MeshIdx], ToApply, true);

		if (bVoxWrap)
		{
			if (ThickenShells > 0 && !TransformedMeshes[MeshIdx].IsClosed())
			{
				// thickness should be at least a cell wide so we don't end up deleting a bunch of the input surface
				double CellSize = TransformedMeshes[MeshIdx].GetBounds(true).MaxDim() / InputVoxelCount;
				double SafeThickness = FMathd::Max(CellSize * 2, ThickenShells);

				FMeshNormals::QuickComputeVertexNormals(TransformedMeshes[MeshIdx]);
				FExtrudeMesh Extrude(&TransformedMeshes[MeshIdx]);
				Extrude.bSkipClosedComponents = true;
				Extrude.DefaultExtrudeDistance = -SafeThickness;
				Extrude.IsPositiveOffset = false;
				Extrude.Apply();
			}

			FDynamicMeshAABBTree3 Spatial(&TransformedMeshes[MeshIdx]);
			TFastWindingTree<FDynamicMesh3> Winding(&Spatial);
			TImplicitSolidify<FDynamicMesh3> Solidify(&TransformedMeshes[MeshIdx], &Spatial, &Winding);
			Solidify.SetCellSizeAndExtendBounds(Spatial.GetBoundingBox(), 0, InputVoxelCount);
			Solidify.CancelF = [&Progress]()
			{
				return Progress && Progress->Cancelled();
			};
			TransformedMeshes[MeshIdx].Copy(&Solidify.Generate());
			
			if (Progress && Progress->Cancelled()) {
				return;
			};

			if (bRemoveInternalsAfterVoxWrap)
			{
				UE::MeshAutoRepair::RemoveInternalTriangles(TransformedMeshes[MeshIdx], true, EOcclusionTriangleSampling::Centroids, EOcclusionCalculationMode::FastWindingNumber, 0, .5, true);
			}
		}

		if (TransformedMeshes[MeshIdx].TriangleCount() == 0)
		{
			continue;
		}
		ImplicitBlend.Sources.Add(&TransformedMeshes[MeshIdx]);
		FAxisAlignedBox3d& SourceBounds = ImplicitBlend.SourceBounds.Add_GetRef(TransformedMeshes[MeshIdx].GetBounds(true));
		CombinedBounds.Contain(SourceBounds);
	}

	if (ImplicitBlend.Sources.Num() == 0)
	{
		return;
	}

	ImplicitBlend.SetCellSizesAndFalloff(CombinedBounds, BlendFalloff, InputVoxelCount, OutputVoxelCount);
	ImplicitBlend.BlendPower = BlendPower;
	ImplicitBlend.CancelF = [&Progress]()
	{
		return Progress && Progress->Cancelled();
	};

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(&ImplicitBlend.Generate());
	
	PostProcessResult(Progress, ImplicitBlend.MeshCellSize);
}