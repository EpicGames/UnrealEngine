// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/VoxelMorphologyMeshesOp.h"
#include "CleaningOps/EditNormalsOp.h"

#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Operations/RemoveOccludedTriangles.h"
#include "Operations/ExtrudeMesh.h"

#include "Generators/MarchingCubes.h"
#include "Implicit/Morphology.h"
#include "Implicit/Solidify.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VoxelMorphologyMeshesOp)

using namespace UE::Geometry;

void FVoxelMorphologyMeshesOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FVoxelMorphologyMeshesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	TImplicitMorphology<FDynamicMesh3> ImplicitMorphology;
	switch (Operation)
	{
	case EMorphologyOperation::Dilate:
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Dilate;
		break;

	case EMorphologyOperation::Contract:
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Contract;
		break;

	case EMorphologyOperation::Open:
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Open;
		break;

	case EMorphologyOperation::Close:
		ImplicitMorphology.MorphologyOp = TImplicitMorphology<FDynamicMesh3>::EMorphologyOp::Close;
		break;

	default:
		check(false);
	}

	if (!ensure(Transforms.Num() == Meshes.Num()))
	{
		return;
	}

	FDynamicMesh3 CombinedMesh;
	FVector3d AverageTranslation = GetAverageTranslation<FTransformSRT3d>(Transforms);
	ResultTransform = FTransformSRT3d(AverageTranslation);

	// append all meshes (transformed but without attributes)
	FDynamicMeshEditor AppendEditor(&CombinedMesh);
	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		FTransformSRT3d MeshTransform = Transforms[MeshIdx];
		bool bReverseOrientation = MeshTransform.GetDeterminant() < 0;
		FMeshIndexMappings IndexMaps;
		AppendEditor.AppendMesh(Meshes[MeshIdx].Get(), IndexMaps,
			[MeshTransform, AverageTranslation](int VID, const FVector3d& Pos)
			{
				return MeshTransform.TransformPosition(Pos) - AverageTranslation;
			}, nullptr
		);
		if (bReverseOrientation)
		{
			for (int TID : Meshes[MeshIdx]->TriangleIndicesItr())
			{
				CombinedMesh.ReverseTriOrientation(IndexMaps.GetNewTriangle(TID));
			}
		}
	}

	if (CombinedMesh.TriangleCount() == 0)
	{
		return;
	}

	if (bVoxWrapInput && ThickenShells > 0)
	{
		// positive offsets should be at least a cell wide so we don't end up deleting a bunch of the input surface
		double CellSize = CombinedMesh.GetBounds(true).MaxDim() / InputVoxelCount;
		double SafeThickness = FMathd::Max(CellSize * 2, ThickenShells);

		FMeshNormals::QuickComputeVertexNormals(CombinedMesh);
		FExtrudeMesh Extrude(&CombinedMesh);
		Extrude.bSkipClosedComponents = true;
		Extrude.DefaultExtrudeDistance = -SafeThickness;
		Extrude.IsPositiveOffset = false;
		Extrude.Apply();
	}
	
	ImplicitMorphology.Source = &CombinedMesh;
	FDynamicMeshAABBTree3 Spatial(&CombinedMesh, true);

	if (bVoxWrapInput)
	{
		TFastWindingTree<FDynamicMesh3> Winding(&Spatial);
		TImplicitSolidify<FDynamicMesh3> Solidify(&CombinedMesh, &Spatial, &Winding);
		Solidify.SetCellSizeAndExtendBounds(Spatial.GetBoundingBox(), 0, InputVoxelCount);
		Solidify.CancelF = [&Progress]()
		{
			return Progress && Progress->Cancelled();
		};
		CombinedMesh.Copy(&Solidify.Generate());
		
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		if (bRemoveInternalsAfterVoxWrap)
		{
			UE::MeshAutoRepair::RemoveInternalTriangles(CombinedMesh, true, EOcclusionTriangleSampling::Centroids, EOcclusionCalculationMode::FastWindingNumber, 0, .5, true);
		}

		Spatial.Build(); // rebuild w/ updated mesh
	}

	if (CombinedMesh.TriangleCount() == 0)
	{
		return;
	}

	ImplicitMorphology.SourceSpatial = &Spatial;
	ImplicitMorphology.SetCellSizesAndDistance(CombinedMesh.GetBounds(true), Distance, InputVoxelCount, OutputVoxelCount);
	ImplicitMorphology.CancelF = [&Progress]()
	{
		return Progress && Progress->Cancelled();
	};

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(&ImplicitMorphology.Generate());

	PostProcessResult(Progress, ImplicitMorphology.MeshCellSize);
}
