// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/VoxelSolidifyMeshesOp.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshEditor.h"
#include "Operations/ExtrudeMesh.h"
#include "Spatial/FastWinding.h"
#include "Generators/MarchingCubes.h"
#include "DynamicMesh/MeshNormals.h"

#include "Implicit/Solidify.h"

using namespace UE::Geometry;

void FVoxelSolidifyMeshesOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FVoxelSolidifyMeshesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (!ensure(Transforms.Num() == Meshes.Num()))
	{
		return;
	}

	FDynamicMesh3 CombinedMesh;

	FVector3d AverageTranslation = GetAverageTranslation<FTransformSRT3d>(Transforms);
	ResultTransform = FTransformSRT3d(AverageTranslation);

	// append all meshes (transformed but without attributes)
	FDynamicMeshEditor AppendEditor(&CombinedMesh);
	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		FTransformSRT3d MeshTransform = Transforms[MeshIdx];
		bool bReverseOrientation = MeshTransform.GetDeterminant() < 0;
		FMeshIndexMappings IndexMaps;
		AppendEditor.AppendMesh(Meshes[MeshIdx].Get(), IndexMaps,
			[MeshTransform, AverageTranslation](int VID, const FVector3d& Pos)
			{
				return MeshTransform.TransformPosition(Pos) - AverageTranslation;
			}, nullptr
		);
		if (bReverseOrientation)
		{
			for (int TID : Meshes[MeshIdx]->TriangleIndicesItr())
			{
				CombinedMesh.ReverseTriOrientation(IndexMaps.GetNewTriangle(TID));
			}
		}
	}

	if (CombinedMesh.TriangleCount() == 0)
	{
		return;
	}

	if (bApplyThickenShells)
	{
		// thickness should be at least a cell wide so we don't end up deleting a bunch of the input surface
		double CellSize = CombinedMesh.GetBounds(true).MaxDim() / InputVoxelCount;
		double SafeThickness = FMathd::Max(CellSize * 2, ThickenShells);

		FMeshNormals::QuickComputeVertexNormals(CombinedMesh);
		FExtrudeMesh Extrude(&CombinedMesh);
		Extrude.bSkipClosedComponents = true;
		Extrude.DefaultExtrudeDistance = -SafeThickness;
		Extrude.IsPositiveOffset = false;
		Extrude.Apply();
	}

	FDynamicMeshAABBTree3 Spatial(&CombinedMesh);
	TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial);


	TImplicitSolidify<FDynamicMesh3> Solidify(&CombinedMesh, &Spatial, &FastWinding);
	Solidify.SetCellSizeAndExtendBounds(Spatial.GetBoundingBox(), ExtendBounds, OutputVoxelCount);
	Solidify.WindingThreshold = WindingThreshold;
	Solidify.SurfaceSearchSteps = SurfaceSearchSteps;
	Solidify.bSolidAtBoundaries = bSolidAtBoundaries;
	Solidify.ExtendBounds = ExtendBounds;
	Solidify.CancelF = [&Progress]()
	{
		return Progress && Progress->Cancelled();
	};

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(&Solidify.Generate());
	
	PostProcessResult(Progress, Solidify.MeshCellSize);
}