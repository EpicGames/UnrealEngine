// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveOps/GenerateCrossSectionOp.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "ConstrainedDelaunay2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateCrossSectionOp)

using namespace UE::Geometry;

void FGenerateCrossSectionOp::SetTransform(const FTransformSRT3d& Transform)
{
	ResultTransform = Transform;
}

void FGenerateCrossSectionOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}
	ResultMesh->Copy(*InputMesh, true, true, true, true);

	MeshCutter = MakeUnique<FMeshPlaneCut>(ResultMesh.Get(), LocalPlaneOrigin, LocalPlaneNormal);

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	MeshCutter->UVScaleFactor = 1.0;
	MeshCutter->bSimplifyAlongNewEdges = true;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	MeshCutter->Cut();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}
}

TArray<TArray<FVector3d>> FGenerateCrossSectionOp::GetCutLoops() const
{
	if (!MeshCutter)
	{
		return TArray<TArray<FVector3d>>();
	}

	if (MeshCutter->OpenBoundaries.Num() == 0)
	{
		return TArray<TArray<FVector3d>>();
	}

	TArray<TArray<FVector3d>> Boundaries;

	// It's possible this is greater than 1, but since we're only ever using the cutter once, it should always be exactly equal to one.
	ensure(MeshCutter->OpenBoundaries.Num() == 1);

	if (MeshCutter->OpenBoundaries[0].CutLoopsFailed)
	{
		return TArray<TArray<FVector3d>>();
	}

	Boundaries.SetNum(MeshCutter->OpenBoundaries[0].CutLoops.Num());

	for (int BoundaryIndex = 0; BoundaryIndex < MeshCutter->OpenBoundaries[0].CutLoops.Num(); ++BoundaryIndex)
	{
		MeshCutter->OpenBoundaries[0].CutLoops[BoundaryIndex].GetVertices(Boundaries[BoundaryIndex]);
	}

	return Boundaries;
}

TArray<TArray<FVector3d>> FGenerateCrossSectionOp::GetCutSpans() const
{
	if (!MeshCutter)
	{
		return TArray<TArray<FVector3d>>();
	}

	if (MeshCutter->OpenBoundaries.Num() == 0)
	{
		return TArray<TArray<FVector3d>>();
	}

	TArray<TArray<FVector3d>> Boundaries;

	// It's possible this is greater than 1, but since we're only ever using the cutter once, it should always be exactly equal to one.
	ensure(MeshCutter->OpenBoundaries.Num() == 1);

	if (MeshCutter->OpenBoundaries[0].CutLoopsFailed)
	{
		return TArray<TArray<FVector3d>>();
	}

	Boundaries.SetNum(MeshCutter->OpenBoundaries[0].CutSpans.Num());

	for (int BoundaryIndex = 0; BoundaryIndex < MeshCutter->OpenBoundaries[0].CutSpans.Num(); ++BoundaryIndex)
	{
		FPolyline3d Polyline;
		MeshCutter->OpenBoundaries[0].CutSpans[BoundaryIndex].GetPolyline(Polyline);
		Boundaries[BoundaryIndex] = Polyline.GetVertices();
	}

	return Boundaries;
}


TUniquePtr<UE::Geometry::FDynamicMeshOperator> UGenerateCrossSectionOpFactory::MakeNewOperator()
{
	TUniquePtr<FGenerateCrossSectionOp> GenerateCrossSectionOp = MakeUnique<FGenerateCrossSectionOp>();

	GenerateCrossSectionOp->InputMesh = OriginalMesh;
	GenerateCrossSectionOp->LocalPlaneOrigin = LocalPlaneOrigin;
	GenerateCrossSectionOp->LocalPlaneNormal = LocalPlaneNormal;
	GenerateCrossSectionOp->bSimplifyAlongNewEdges = bSimplifyAlongNewEdges;
	GenerateCrossSectionOp->SetTransform(TargetTransform);

	return GenerateCrossSectionOp;
}
