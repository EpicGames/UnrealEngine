// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveOps/TriangulateCurvesOp.h"

#include "Util/ProgressCancel.h"
#include "BoxTypes.h"
#include "CompGeom/PolygonTriangulation.h"
#include "CompGeom/Delaunay2.h"
#include "Operations/ExtrudeMesh.h"
#include "DynamicMesh/MeshNormals.h"
#include "Curve/GeneralPolygon2.h"

#include "Curve/PolygonOffsetUtils.h"
#include "Curve/PolygonIntersectionUtils.h"

#include "Components/SplineComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TriangulateCurvesOp)

using namespace UE::Geometry;


void FTriangulateCurvesOp::AddSpline(USplineComponent* Spline, double ErrorTolerance)
{
	FCurvePath& Path = Paths.Emplace_GetRef();
	Spline->ConvertSplineToPolyLine(ESplineCoordinateSpace::Type::World, ErrorTolerance * ErrorTolerance, Path.Vertices);
	Path.bClosed = Spline->IsClosedLoop();
	// Spline conversion creates a duplicate end/start point, which we don't need for triangulation
	if (Path.bClosed && Path.Vertices.Num() > 1)
	{
		Path.Vertices.Pop();
	}
	if (Paths.Num() == 1)
	{
		FirstPathTransform = Spline->GetComponentTransform();
	}
}

void FTriangulateCurvesOp::AddWorldCurve(TArrayView<const FVector3d> WorldSpaceVertices, bool bClosed, const FTransform& ReferenceTransform)
{
	FCurvePath& Path = Paths.Emplace_GetRef();
	Path.bClosed = bClosed;
	// Make sure we don't add a duplicate end/start point on closed curves
	if (bClosed && WorldSpaceVertices.Num() > 1 && WorldSpaceVertices[0] == WorldSpaceVertices.Last())
	{
		Path.Vertices.Append(WorldSpaceVertices.Slice(0, WorldSpaceVertices.Num() - 1));
	}
	else
	{
		Path.Vertices.Append(WorldSpaceVertices);
	}
	if (Paths.Num() == 1)
	{
		FirstPathTransform = ReferenceTransform;
	}
}

void FTriangulateCurvesOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->EnableAttributes();
	ResultMesh->EnableTriangleGroups();
	ResultMesh->EnableVertexNormals(FVector3f::ZAxisVector);

	SetResultTransform(FTransformSRT3d(FirstPathTransform));

	FAxisAlignedBox3d InputBounds;
	for (const FCurvePath& Path : Paths)
	{
		InputBounds.Contain(Path.Vertices);
	}
	double UVScaleDenom = bWorldSpaceUVScale ? 100.0 : InputBounds.MaxDim();
	double UseUVScaleFactor = UVScaleFactor / UVScaleDenom;

	auto AppendTriangles = [this, UseUVScaleFactor](FDynamicMesh3& Mesh, const TArray<FVector3d>& Vertices, const TArray<FIndex3i>& Tris, int32 GroupID, FVector3d PlaneOrigin, FVector3d Normal, bool bFlip) -> void
	{
		if (Tris.IsEmpty())
		{
			return;
		}

		FFrame3d ProjectionFrame(ResultTransform.GetTranslation(), Normal);
		FVector3f LocalNormal = (FVector3f)ResultTransform.InverseTransformNormal(Normal);

		checkSlow(Mesh.IsCompact());
		int32 VertStart = Mesh.MaxVertexID();
		int32 TriStart = Mesh.MaxTriangleID();
		for (const FVector3d V : Vertices)
		{
			int32 VID = Mesh.AppendVertex(ResultTransform.InverseTransformPosition(V));
			Mesh.SetVertexNormal(VID, LocalNormal);
			int32 NormalEID = Mesh.Attributes()->PrimaryNormals()->AppendElement(LocalNormal);
			int32 UVEID = Mesh.Attributes()->PrimaryUV()->AppendElement(FVector2f(ProjectionFrame.ToPlaneUV(V) * UseUVScaleFactor));
			// since we always add vertices and overlay elements at the same time, the IDs must match up
			checkSlow(VID == NormalEID && NormalEID == UVEID);
		}
		for (const FIndex3i& T : Tris)
		{
			FIndex3i NewTri(T.A + VertStart, T.B + VertStart, T.C + VertStart);
			if (bFlip)
			{
				Swap(NewTri.B, NewTri.C);
			}
			int32 NewTID = Mesh.AppendTriangle(NewTri, GroupID);
			Mesh.Attributes()->PrimaryNormals()->SetTriangle(NewTID, NewTri);
			Mesh.Attributes()->PrimaryUV()->SetTriangle(NewTID, NewTri);
		}
	};

	// Non-flat case must go through the simple triangulation path
	if (FlattenMethod == EFlattenCurveMethod::DoNotFlatten)
	{
		for (int32 PathIdx = 0; PathIdx < Paths.Num(); ++PathIdx)
		{
			if (Paths[PathIdx].Vertices.Num() < 3)
			{
				continue;
			}
			
			FVector3d Normal, PlanePoint;
			PolygonTriangulation::ComputePolygonPlane<double>(Paths[PathIdx].Vertices, Normal, PlanePoint);
			if (bFlipResult)
			{
				Normal = -Normal;
			}
			TArray<FIndex3i> Tris;
			PolygonTriangulation::TriangulateSimplePolygon<double>(Paths[PathIdx].Vertices, Tris, false);
			AppendTriangles(*ResultMesh, Paths[PathIdx].Vertices, Tris, PathIdx, PlanePoint, Normal, bFlipResult);
		}

		ApplyThickness(UseUVScaleFactor);
		return;
	}

	FVector3d SharedFrameOrigin = InputBounds.Center();
	FVector3d SharedFrameNormal = FVector::UnitZ();
	if (FlattenMethod == EFlattenCurveMethod::AlongX)
	{
		SharedFrameNormal = FVector::UnitX();
	}
	else if (FlattenMethod == EFlattenCurveMethod::AlongY)
	{
		SharedFrameNormal = FVector::UnitY();
	}
	else if (FlattenMethod == EFlattenCurveMethod::ToBestFitPlane && CombineMethod != ECombineCurvesMethod::LeaveSeparate)
	{
		// Take an unweighted average of the polygon normals as the shared plane normal
		// TODO: consider area-weighting these or instead doing a plane fit directly to the combined vertices
		FVector3d PlanePoint;
		FVector3d AccumulatedNormal;
		PolygonTriangulation::ComputePolygonPlane<double>(Paths[0].Vertices, AccumulatedNormal, PlanePoint);
		for (int32 PathIdx = 1; PathIdx < Paths.Num(); ++PathIdx)
		{
			FVector3d PolygonNormal;
			PolygonTriangulation::ComputePolygonPlane<double>(Paths[PathIdx].Vertices, PolygonNormal, PlanePoint);
			if (PolygonNormal.Dot(AccumulatedNormal) < 0)
			{
				PolygonNormal = -PolygonNormal;
			}
			AccumulatedNormal += PolygonNormal;
		}
		SharedFrameNormal = Normalized(AccumulatedNormal);
	}

	auto ConvertOffsetOpenEndType = [](EOpenCurveEndShapes OpenEnd) -> EPolygonOffsetEndType
	{
		switch (OpenEnd)
		{
		case EOpenCurveEndShapes::Square:
			return EPolygonOffsetEndType::Square;
		case EOpenCurveEndShapes::Round:
			return EPolygonOffsetEndType::Round;
		case EOpenCurveEndShapes::Butt:
			return EPolygonOffsetEndType::Butt;
		}
		checkNoEntry();
		return EPolygonOffsetEndType::Square;
	};

	auto ConvertOffsetJoinType = [](EOffsetJoinMethod OffsetJoin) -> EPolygonOffsetJoinType
	{
		switch (OffsetJoin)
		{
		case EOffsetJoinMethod::Square:
			return EPolygonOffsetJoinType::Square;
		case EOffsetJoinMethod::Miter:
			return EPolygonOffsetJoinType::Miter;
		case EOffsetJoinMethod::Round:
			return EPolygonOffsetJoinType::Round;
		}
		checkNoEntry();
		return EPolygonOffsetJoinType::Square;
	};

	auto ProcessPaths = [&/*TODO FIX*/](FVector3d Normal, FVector3d PlanePoint, TArrayView<const FCurvePath> ToProcess, int32 GroupID) -> int32
	{
		if (bFlipResult)
		{
			Normal = -Normal;
		}

		TArray<UE::Geometry::FGeneralPolygon2d> PolygonsGroup[2]; // Second group for Intersect/Difference/ExclusiveOr
		const bool bNeedsSecondGroup = CombineMethod == ECombineCurvesMethod::Intersect || CombineMethod == ECombineCurvesMethod::Difference || CombineMethod == ECombineCurvesMethod::ExclusiveOr;
		auto GetPolygonsGroup = [&PolygonsGroup, bNeedsSecondGroup](int32 CurveIdx) ->TArray<UE::Geometry::FGeneralPolygon2d>&
		{
			return bNeedsSecondGroup && CurveIdx > 0 ? PolygonsGroup[1] : PolygonsGroup[0];
		};

		// Array of projected points (to be reused)
		TArray<FVector2D> Projected;
		const FFrame3d Frame(PlanePoint, Normal);
		FVector3d FrameX = Frame.GetAxis(0);
		FVector3d FrameY = Frame.GetAxis(1);
		FVector3d FrameZ = Frame.GetAxis(2);
		auto SetProjected = [Frame, &Projected](const FCurvePath& CurvePath) -> void
		{
			Projected.Reset(CurvePath.Vertices.Num());
			for (const FVector& Pt : CurvePath.Vertices)
			{
				Projected.Add(Frame.ToPlaneUV(Pt));
			}
		};
		// Array of general polygons (to be reused);
		TArray<UE::Geometry::FGeneralPolygon2d> PolygonsToAdd;

		// If open paths are to be offset, use this to convert them to polygons
		bool bHasProcessedOpenCurves = EOffsetOpenCurvesMethod::Offset == OffsetOpenMethod;
		// Note: If curve offset is 0, the open curves do not contribute polygons to the output
		if (bHasProcessedOpenCurves && CurveOffset != 0.0)
		{
			for (int32 CurveIdx = 0; CurveIdx < ToProcess.Num(); ++CurveIdx)
			{
				const FCurvePath& CurvePath = ToProcess[CurveIdx];
				if (!CurvePath.bClosed)
				{
					SetProjected(CurvePath);
					UE::Geometry::FOffsetPolygon2d OffsetPolygon;
					OffsetPolygon.Polygons.Add(Projected);
					OffsetPolygon.Offset = CurveOffset;
					OffsetPolygon.MiterLimit = MiterLimit;
					OffsetPolygon.JoinType = ConvertOffsetJoinType(OffsetJoinMethod);
					OffsetPolygon.EndType = ConvertOffsetOpenEndType(OpenEndShape);
					if (OffsetPolygon.ComputeResult())
					{
						GetPolygonsGroup(CurveIdx).Append(OffsetPolygon.Result);
					}
				}
			}
		}

		// Closed-path handling
		for (int32 CurveIdx = 0; CurveIdx < ToProcess.Num(); ++CurveIdx)
		{
			const FCurvePath& CurvePath = ToProcess[CurveIdx];
			if (bHasProcessedOpenCurves && !CurvePath.bClosed)
			{
				continue;
			}
			checkSlow(CurvePath.bClosed || OffsetOpenMethod == EOffsetOpenCurvesMethod::TreatAsClosed);

			SetProjected(CurvePath);
			UE::Geometry::FGeneralPolygon2d PolyToProcess(Projected);
			TArrayView<const FGeneralPolygon2d> InitialPolygons(&PolyToProcess, 1);
			// Always pre-process with a union to make orientation consistent
			PolygonsToAdd.Reset();
			PolygonsUnion(InitialPolygons, PolygonsToAdd, true);
			if (OffsetClosedMethod != EOffsetClosedCurvesMethod::DoNotOffset)
			{
				PolygonsOffset(
					CurveOffset,
					PolygonsToAdd, PolygonsToAdd,
					true,
					MiterLimit,
					ConvertOffsetJoinType(OffsetJoinMethod),
					OffsetClosedMethod == EOffsetClosedCurvesMethod::OffsetOuterSide ? EPolygonOffsetEndType::Polygon : EPolygonOffsetEndType::Joined);
			}
			GetPolygonsGroup(CurveIdx).Append(PolygonsToAdd);
		}

		if (CombineMethod == ECombineCurvesMethod::Union)
		{
			checkSlow(!bNeedsSecondGroup);
			PolygonsUnion(PolygonsGroup[0], PolygonsGroup[0], true);
		}
		else if (CombineMethod == ECombineCurvesMethod::Intersect)
		{
			PolygonsIntersection(PolygonsGroup[0], PolygonsGroup[1], PolygonsGroup[0]);
		}
		else if (CombineMethod == ECombineCurvesMethod::Difference)
		{
			PolygonsDifference(PolygonsGroup[0], PolygonsGroup[1], PolygonsGroup[0]);
		}
		else if (CombineMethod == ECombineCurvesMethod::ExclusiveOr)
		{
			PolygonsExclusiveOr(PolygonsGroup[0], PolygonsGroup[1], PolygonsGroup[0]);
		}

		// Triangulate and append each polygon separately
		for (int32 PolyIdx = 0; PolyIdx < PolygonsGroup[0].Num(); ++PolyIdx)
		{
			UE::Geometry::FDelaunay2 Delaunay;
			Delaunay.bAutomaticallyFixEdgesToDuplicateVertices = true;
			TArray<FIndex3i> Triangles;
			TArray<FVector2d> Vertices;
			TArray<FVector3d> Vertices3d;
			Delaunay.Triangulate(PolygonsGroup[0][PolyIdx], &Triangles, &Vertices, true);
			Vertices3d.SetNum(Vertices.Num());
			for (int32 VIdx = 0; VIdx < Vertices.Num(); ++VIdx)
			{
				Vertices3d[VIdx] = Frame.FromPlaneUV(Vertices[VIdx]);
			}
			AppendTriangles(*ResultMesh, Vertices3d, Triangles, GroupID++, PlanePoint, Normal, true);
		}

		return GroupID;
	};

	if (CombineMethod == ECombineCurvesMethod::LeaveSeparate)
	{
		int32 GroupIdx = 1;
		for (int32 CurveIdx = 0; CurveIdx < Paths.Num(); ++CurveIdx)
		{
			FVector3d PlanePoint;
			FVector3d Normal;
			PolygonTriangulation::ComputePolygonPlane<double>(Paths[CurveIdx].Vertices, Normal, PlanePoint);
			FVector3d ProjectedFrameOrigin = FVector3d::PointPlaneProject(SharedFrameOrigin, FPlane(PlanePoint, Normal));
			GroupIdx = ProcessPaths(Normal, ProjectedFrameOrigin, TArrayView<const FCurvePath>(&Paths[CurveIdx], 1), GroupIdx);
		}
	}
	else
	{
		ProcessPaths(SharedFrameNormal, SharedFrameOrigin, Paths, 1);
	}

	ApplyThickness(UseUVScaleFactor);

	if (Progress && Progress->Cancelled())
	{
		return;
	}
}

void FTriangulateCurvesOp::ApplyThickness(double UseUVScaleFactor)
{
	if (Thickness > 0)
	{
		FExtrudeMesh ExtrudeMesh(ResultMesh.Get());
		ExtrudeMesh.DefaultExtrudeDistance = Thickness;
		ExtrudeMesh.UVScaleFactor = UseUVScaleFactor;
		ExtrudeMesh.Apply();
	}
}
