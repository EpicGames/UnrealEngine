// Copyright Epic Games, Inc. All Rights Reserved.

#include "CuttingOps/EdgeLoopInsertionOp.h"

#include "Util/ProgressCancel.h"

using namespace UE::Geometry;

void FEdgeLoopInsertionOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FEdgeLoopInsertionOp::GetLoopEdgeLocations(TArray<TPair<FVector3d, FVector3d>>& EndPointPairsOut) const
{
	EndPointPairsOut.Reset();
	for (int32 Eid : LoopEids)
	{
		TPair<FVector3d, FVector3d> Endpoints;
		ResultMesh->GetEdgeV(Eid, Endpoints.Key, Endpoints.Value);
		EndPointPairsOut.Add(MoveTemp(Endpoints));
	}
}

void FEdgeLoopInsertionOp::CalculateResult(FProgressCancel* Progress)
{
	bSucceeded = false;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);
	ResultTopology = MakeShared<FGroupTopology, ESPMode::ThreadSafe>();
	*ResultTopology = *OriginalTopology;
	ResultTopology->RetargetOnClonedMesh(ResultMesh.Get());
	ChangedTids = MakeShared<TSet<int32>, ESPMode::ThreadSafe>();
	LoopEids.Reset();
	ProblemGroupEdgeIDs.Reset();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (InputLengths.Num() > 0)
	{
		FGroupEdgeInserter::FEdgeLoopInsertionParams Params;
		Params.Mesh = ResultMesh.Get();
		Params.Topology = ResultTopology.Get();
		Params.GroupEdgeID = GroupEdgeID;
		Params.Mode = Mode;
		Params.SortedInputLengths = &InputLengths;
		Params.bInputsAreProportions = bInputsAreProportions;
		Params.StartCornerID = StartCornerID;
		Params.VertexTolerance = VertexTolerance;

		FGroupEdgeInserter Inserter;
		FGroupEdgeInserter::FOptionalOutputParams OptionalOut;
		OptionalOut.NewEidsOut = &LoopEids;
		OptionalOut.ChangedTidsOut = ChangedTids.Get();
		OptionalOut.ProblemGroupEdgeIDsOut = &ProblemGroupEdgeIDs;
		bSucceeded = Inserter.InsertEdgeLoops(Params, OptionalOut, Progress);
	}
}
