// Copyright Epic Games, Inc. All Rights Reserved.

#include "CuttingOps/GroupEdgeInsertionOp.h"
#include "MeshRegionBoundaryLoops.h"

#include "Util/ProgressCancel.h"

using namespace UE::Geometry;

void FGroupEdgeInsertionOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FGroupEdgeInsertionOp::GetEdgeLocations(TArray<TPair<FVector3d, FVector3d>>& EndPointPairsOut) const
{
	EndPointPairsOut.Reset();
	for (int32 Eid : Eids)
	{
		TPair<FVector3d, FVector3d> Endpoints;
		ResultMesh->GetEdgeV(Eid, Endpoints.Key, Endpoints.Value);
		EndPointPairsOut.Add(MoveTemp(Endpoints));
	}
}

void FGroupEdgeInsertionOp::CalculateResult(FProgressCancel* Progress)
{
	bSucceeded = false;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);
	ResultTopology = MakeShared<FGroupTopology, ESPMode::ThreadSafe>();
	*ResultTopology = *OriginalTopology;
	ResultTopology->RetargetOnClonedMesh(ResultMesh.Get());
	ChangedTids = MakeShared<TSet<int32>, ESPMode::ThreadSafe>();

	if (bShowingBaseMesh || (Progress && Progress->Cancelled()))
	{
		return;
	}

	FGroupEdgeInserter Inserter;
	FGroupEdgeInserter::FGroupEdgeInsertionParams Params;
	Params.Mesh = ResultMesh.Get();
	Params.Topology = ResultTopology.Get();
	Params.Mode = Mode;
	Params.VertexTolerance = VertexTolerance;
	Params.StartPoint = StartPoint;
	Params.EndPoint = EndPoint;
	Params.GroupID = CommonGroupID;
	Params.GroupBoundaryIndex = CommonBoundaryIndex;

	FGroupEdgeInserter::FOptionalOutputParams OptionalOut;
	OptionalOut.NewEidsOut = &Eids;
	OptionalOut.ChangedTidsOut = ChangedTids.Get();

	bSucceeded = Inserter.InsertGroupEdge(Params, OptionalOut, Progress);
}
