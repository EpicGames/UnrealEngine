// Copyright Epic Games, Inc. All Rights Reserved.

#include "CuttingOps/PlaneCutOp.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "Operations/MeshPlaneCut.h"
#include "ConstrainedDelaunay2.h"

using namespace UE::Geometry;

const FName FPlaneCutOp::ObjectIndexAttribute = "ObjectIndexAttribute";

void FPlaneCutOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}

void FPlaneCutOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}
	ResultMesh->Copy(*OriginalMesh, true, true, true, true);

	CutPlaneLocalThickness = FMath::Abs(CutPlaneLocalThickness); // sanitize thickness param
	
	FMeshPlaneCut Cut(ResultMesh.Get(), LocalPlaneOrigin, LocalPlaneNormal);

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	Cut.UVScaleFactor = UVScaleFactor;
	Cut.bSimplifyAlongNewEdges = bSimplifyAlongNewEdges;


	if (Progress && Progress->Cancelled())
	{
		return;
	}
	if (bKeepBothHalves)
	{
		int MaxSubObjectID = -1;
		TDynamicMeshScalarTriangleAttribute<int>* SubObjectAttrib = static_cast<TDynamicMeshScalarTriangleAttribute<int>*>(ResultMesh->Attributes()->GetAttachedAttribute(ObjectIndexAttribute));
		for (int TID : ResultMesh->TriangleIndicesItr())
		{
			MaxSubObjectID = FMath::Max(MaxSubObjectID, SubObjectAttrib->GetValue(TID));
		}
		if (CutPlaneLocalThickness <= Cut.PlaneTolerance)
		{
			Cut.CutWithoutDelete(true, 0, SubObjectAttrib, MaxSubObjectID+1);
		}
		else // for a 'thick plane' we need two cuts
		{
			Cut.PlaneOrigin -= Cut.PlaneNormal * CutPlaneLocalThickness;
			Cut.CutWithoutDelete(true, 0, SubObjectAttrib, MaxSubObjectID + 1, true, false);

			int SecondCutMaxID = MaxSubObjectID;
			for (int TID : ResultMesh->TriangleIndicesItr())
			{
				SecondCutMaxID = FMath::Max(SecondCutMaxID, SubObjectAttrib->GetValue(TID));
			}
			Cut.PlaneOrigin += Cut.PlaneNormal * (2.0 * CutPlaneLocalThickness);
			Cut.CutWithoutDelete(true, 0, SubObjectAttrib, SecondCutMaxID + 1, false, true);

			for (int TID : ResultMesh->TriangleIndicesItr())
			{
				int SubObjectID = SubObjectAttrib->GetValue(TID);
				if (SubObjectID > MaxSubObjectID && SubObjectID <= SecondCutMaxID)
				{
					ResultMesh->RemoveTriangle(TID);
				}
			}
		}

		
	}
	else
	{
		Cut.Cut();
	}


	if (Progress && Progress->Cancelled())
	{
		return;
	}
	if (bFillCutHole)
	{
		Cut.HoleFill(ConstrainedDelaunayTriangulate<double>, bFillSpans);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	if (bFillCutHole && bKeepBothHalves)
	{
		TDynamicMeshScalarTriangleAttribute<int>* SubObjectAttrib = static_cast<TDynamicMeshScalarTriangleAttribute<int>*>(ResultMesh->Attributes()->GetAttachedAttribute(ObjectIndexAttribute));
		Cut.TransferTriangleLabelsToHoleFillTriangles(SubObjectAttrib);
	}
}