// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformationOps/ExtrudeOp.h"

#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicSubmesh3.h"
#include "Operations/OffsetMeshRegion.h"
#include "Operations/MeshBoolean.h"
#include "Selections/MeshConnectedComponents.h"

#include "Util/ProgressCancel.h"

using namespace UE::Geometry;

namespace ExtrudeOpLocals
{
	// Take the selected triangles and change their groups to new ones. This makes it
	// easier to track the groups into the result in a boolean extrude, and it also
	// makes sure that the extrude doesn't create disconnected groups (if you do it
	// on just some of the triangles in a group)
	void UpdateSelectionGroups(FDynamicMesh3& Mesh, const TArray<int32> TriangleSelectionIn, TArray<int32>& SelectionGidsOut)
	{
		if (Mesh.HasTriangleGroups())
		{
			TMap<int32, int32> GroupMap;
			for (int32 Tid : TriangleSelectionIn)
			{
				int32 Gid = Mesh.GetTriangleGroup(Tid);
				int32* NewGidPointer = GroupMap.Find(Gid);
				if (NewGidPointer)
				{
					Mesh.SetTriangleGroup(Tid, *NewGidPointer);
				}
				else
				{
					int32 NewGid = Mesh.AllocateTriangleGroup();
					Mesh.SetTriangleGroup(Tid, NewGid);
					GroupMap.Add(Gid, NewGid);

					SelectionGidsOut.Add(NewGid);
				}
			}
		}
	}
}

void FExtrudeOp::CalculateResult(FProgressCancel* Progress)
{
	if (FMath::Abs(ExtrudeDistance) < KINDA_SMALL_NUMBER
		|| (Progress && Progress->Cancelled()))
	{
		return;
	}

	// Reset result
	ResultMesh->Copy(*OriginalMesh, true, true, true, true);

	if (ExtrudeMode == EExtrudeMode::Boolean)
	{
		BooleanExtrude(Progress);
	}
	else
	{
		MoveAndStitchExtrude(Progress);
	}
}

bool FExtrudeOp::BooleanExtrude(FProgressCancel* Progress)
{
	using namespace ExtrudeOpLocals;

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	TArray<int32> SelectionGids;
	UpdateSelectionGroups(*ResultMesh, TriangleSelection, SelectionGids);

	// Grab just the patch to be extruded.
	TUniquePtr<FDynamicSubmesh3> Submesh = MakeUnique<FDynamicSubmesh3>(ResultMesh.Get(), TriangleSelection);

	Submesh->MapGroupsToSubmesh(SelectionGids);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Extrude the patch:
	FOffsetMeshRegion Extruder(&Submesh->GetSubmesh());

	Extruder.Triangles.Reserve(TriangleSelection.Num());
	for (int32 Tid : Submesh->GetSubmesh().TriangleIndicesItr())
	{
		Extruder.Triangles.Add(Tid);
	}

	// We need a custom function here so that we can set the side groups based on the original mesh, not the submesh
	Extruder.LoopEdgesShouldHaveSameGroup = [&Submesh, this](int32 Eid1, int32 Eid2) {

		// Unfortunately, we can't just use Submesh->MapEdgeToBaseMesh() for the eids here because
		// that does the mapping via the vertices, and as the extruder extrudes separate full
		// regions, it ends up splitting bowties, which breaks that correspondence. 
		// However, the triangles of the next regions to be extruded don't change, so we can get
		// the corresponding edges via tid and edge sub index (in the triangle edge triplet).
		const FDynamicMesh3& ExtrusionMesh = Submesh->GetSubmesh();
		int32 Tid1 = ExtrusionMesh.GetEdgeT(Eid1).A;
		int32 SubIdx1 = IndexUtil::FindTriIndex(Eid1, ExtrusionMesh.GetTriEdges(Tid1));
		int32 EidToUse1 = ResultMesh->GetTriEdge(Submesh->MapTriangleToBaseMesh(Tid1), SubIdx1);

		int32 Tid2 = ExtrusionMesh.GetEdgeT(Eid2).A;
		int32 SubIdx2 = IndexUtil::FindTriIndex(Eid2, ExtrusionMesh.GetTriEdges(Tid2));
		int32 EidToUse2 = ResultMesh->GetTriEdge(Submesh->MapTriangleToBaseMesh(Tid2), SubIdx2);

		return FOffsetMeshRegion::EdgesSeparateSameGroupsAndAreColinearAtBorder(
			ResultMesh.Get(), EidToUse1, EidToUse2, bUseColinearityForSettingBorderGroups);
	};

	Extruder.UVScaleFactor = UVScaleFactor;
	if (DirectionMode == EDirectionMode::SingleDirection)
	{
		Extruder.OffsetPositionFunc = [this](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID)
		{
			return Pos + ExtrudeDistance * MeshSpaceExtrudeDirection;
		};
	}
	else
	{
		Extruder.OffsetPositionFunc = [this](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID) {
			return Pos + ExtrudeDistance * VertexVector;
		};
	}

	Extruder.ExtrusionVectorType =
		DirectionMode == EDirectionMode::VertexNormals ? FOffsetMeshRegion::EVertexExtrusionVectorType::VertexNormal
		: DirectionMode == EDirectionMode::SelectedTriangleNormals ? FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage
		: DirectionMode == EDirectionMode::SelectedTriangleNormalsEven ? FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted
		: FOffsetMeshRegion::EVertexExtrusionVectorType::Zero;

	Extruder.MaxScaleForAdjustingTriNormalsOffset = MaxScaleForAdjustingTriNormalsOffset;

	Extruder.bIsPositiveOffset = (ExtrudeDistance > 0);
	Extruder.bOffsetFullComponentsAsSolids = true;
	Extruder.Apply();

	FMeshNormals::QuickComputeVertexNormalsForTriangles(*ResultMesh, Extruder.AllModifiedAndNewTriangles);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// For whole-component extrusions, the boolean operation that we are about to perform typically
	// results in the removal of the original triangles, resulting in an open shell. If we want
	// the shell to be a closed solid, we actually just want the extruded portion in the boolean,
	// with no participation from the original triangles. So we actually have to delete the whole-component
	// triangles before combining with the extruded piece(s)
	if (bShellsToSolids)
	{
		FMeshConnectedComponents SelectionComponents(ResultMesh.Get());
		SelectionComponents.FindTrianglesConnectedToSeeds(TriangleSelection);

		TSet<int32> TriangleSelectionSet(TriangleSelection);
		for (FMeshConnectedComponents::FComponent& Component : SelectionComponents.Components)
		{
			bool bWholeComponentIsSelected = true;
			for (int32 Tid : Component.Indices)
			{
				if (!TriangleSelectionSet.Contains(Tid))
				{
					bWholeComponentIsSelected = false;
					break;
				}
			}

			// Perform the deletion if needed
			if (bWholeComponentIsSelected)
			{
				for (int32 Tid : Component.Indices)
				{
					ResultMesh->RemoveTriangle(Tid);
				}
			}
		}//end iterating through selection components
	}

	// Now perform a boolean operation with our result.
	FMeshBoolean::EBooleanOp Op = ExtrudeDistance > 0 ? FMeshBoolean::EBooleanOp::Union : FMeshBoolean::EBooleanOp::Difference;
	FMeshBoolean MeshBoolean(ResultMesh.Get(), FTransformSRT3d::Identity(), &Submesh->GetSubmesh(), FTransformSRT3d::Identity(), ResultMesh.Get(), Op);
	MeshBoolean.bPutResultInInputSpace = true;
	MeshBoolean.bSimplifyAlongNewEdges = true;
	MeshBoolean.bPopulateSecondMeshGroupMap = true;
	MeshBoolean.Progress = Progress;
	bool bSuccess = MeshBoolean.Compute();

	// Get the group ID's of the extruded face, if they still exist (they may not if we cut entirely through)
	ExtrudedFaceNewGids.Empty();
	for (int32 Gid : SelectionGids)
	{
		const int32* NewGid = MeshBoolean.SecondMeshGroupMap.FindTo(Gid);
		if (NewGid)
		{
			ExtrudedFaceNewGids.Add(*NewGid);
		}
	}

	return bSuccess;
}

bool FExtrudeOp::MoveAndStitchExtrude(FProgressCancel* Progress)
{
	using namespace ExtrudeOpLocals;

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	TArray<int32> SelectionGids;
	UpdateSelectionGroups(*ResultMesh, TriangleSelection, SelectionGids);

	FOffsetMeshRegion Extruder(ResultMesh.Get());
	Extruder.UseVersion = FOffsetMeshRegion::EVersion::Legacy;		// to reconsider...
	Extruder.Triangles = TriangleSelection;

	Extruder.LoopEdgesShouldHaveSameGroup = [this](int32 Eid1, int32 Eid2) {
		return FOffsetMeshRegion::EdgesSeparateSameGroupsAndAreColinearAtBorder(
			ResultMesh.Get(), Eid1, Eid2, bUseColinearityForSettingBorderGroups);
	};

	Extruder.UVScaleFactor = UVScaleFactor;
	if (DirectionMode == EDirectionMode::SingleDirection)
	{
		Extruder.OffsetPositionFunc = [this](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID)
		{
			return Pos + ExtrudeDistance * MeshSpaceExtrudeDirection;
		};
	}
	else
	{
		Extruder.OffsetPositionFunc = [this](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID) {
			return Pos + ExtrudeDistance * VertexVector;
		};
	}

	Extruder.ExtrusionVectorType =
		DirectionMode == EDirectionMode::VertexNormals ? FOffsetMeshRegion::EVertexExtrusionVectorType::VertexNormal
		: DirectionMode == EDirectionMode::SelectedTriangleNormals ? FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage
		: DirectionMode == EDirectionMode::SelectedTriangleNormalsEven ? FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted
		: FOffsetMeshRegion::EVertexExtrusionVectorType::Zero;

	Extruder.MaxScaleForAdjustingTriNormalsOffset = MaxScaleForAdjustingTriNormalsOffset;

	Extruder.bIsPositiveOffset = (ExtrudeDistance > 0);
	Extruder.bOffsetFullComponentsAsSolids = bShellsToSolids;
	Extruder.Apply();

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	FMeshNormals::QuickComputeVertexNormalsForTriangles(*ResultMesh, Extruder.AllModifiedAndNewTriangles);

	ExtrudedFaceNewGids = SelectionGids;

	return true;
}
