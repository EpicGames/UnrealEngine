// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeformationOps/LatticeDeformerOp.h"
#include "DynamicSubmesh3.h"
#include "Operations/FFDLattice.h"

using namespace UE::Geometry;

void FLatticeDeformerOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	TArray<FVector3d> DeformedPositions;
	FLatticeExecutionInfo ExecutionInfo = FLatticeExecutionInfo();
	ExecutionInfo.bParallel = true;
	
	const bool bUsingSubmesh = Submesh != nullptr;
	
	// retrieves the deformed vertex positions of the applicable mesh - either the whole mesh or a submesh, depending on selection
	Lattice->GetDeformedMeshVertexPositions(LatticeControlPoints, DeformedPositions, InterpolationType, ExecutionInfo, Progress);
	
	if (bUsingSubmesh)
	{
		check(Submesh->GetSubmesh().MaxVertexID() == DeformedPositions.Num())
	}
	else
	{
		check(ResultMesh->MaxVertexID() == DeformedPositions.Num());
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (bUsingSubmesh)
	{
		for (const int SubVID : Submesh->GetSubmesh().VertexIndicesItr())
		{
			// retrieve the index in the Base Mesh which maps to the (deformed) submesh
			const int BaseMeshVertexID = Submesh->MapVertexToBaseMesh(SubVID);
			// sets the deformed position of the submesh in the result mesh
			ResultMesh->SetVertex(BaseMeshVertexID, DeformedPositions[SubVID]);
		}
		for (const int BaseVID : ResultMesh->VertexIndicesItr())
		{
			// for all vertices in the base mesh that are NOT in the submesh, they keep their original (transformed) position
			if (Submesh->MapVertexToSubmesh(BaseVID) == INDEX_NONE)
			{
				ResultMesh->SetVertex(BaseVID, WorldTransform.TransformPosition(ResultMesh->GetVertex(BaseVID)));
			}
		}
	}
	else
	{
		// if deforming entire mesh, all the mesh's new positions will be in DeformedPositions
		for (const int VID : ResultMesh->VertexIndicesItr())
		{
			if (Lattice->VertexHasValidEmbedding(VID))
			{
				ResultMesh->SetVertex(VID, DeformedPositions[VID]);
			}
		}
	}

	if (bDeformNormals)
	{
		auto GetDeformedNormals = [this, &ExecutionInfo, &Progress](const FDynamicMesh3& Mesh, TArray<FVector3f>& DeformedNormals)
		{
			if (Mesh.HasAttributes())
			{
				const FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
				check(NormalOverlay != nullptr);

				Lattice->GetRotatedOverlayNormals(LatticeControlPoints,
												  NormalOverlay,
												  DeformedNormals,
												  InterpolationType,
												  ExecutionInfo,
												  Progress);
			}
			else if (Mesh.HasVertexNormals())
			{
				TArray<FVector3f> OriginalNormals;
				OriginalNormals.SetNum(Mesh.MaxVertexID());
				for (int VertexID : Mesh.VertexIndicesItr())
				{
					OriginalNormals[VertexID] = Mesh.GetVertexNormal(VertexID);
				}

				Lattice->GetRotatedMeshVertexNormals(LatticeControlPoints,
													 OriginalNormals,
													 DeformedNormals,
													 InterpolationType,
													 ExecutionInfo,
													 Progress);
			}
		};

		const FDynamicMesh3& DeformNormalMesh = bUsingSubmesh ? Submesh->GetSubmesh() : *ResultMesh;
		TArray<FVector3f> DeformedNormals;
		GetDeformedNormals(DeformNormalMesh, DeformedNormals);

		if (DeformedNormals.IsEmpty())
		{
			return;
		}

		if (Progress && Progress->Cancelled())
		{
			return;
		}
		
		if (bUsingSubmesh)
		{
			if (ResultMesh->HasAttributes())
			{
				const FDynamicMeshNormalOverlay* SubOverlay = Submesh->GetSubmesh().Attributes()->PrimaryNormals();
				check(SubOverlay != nullptr);

				FDynamicMeshNormalOverlay* ResultOverlay = ResultMesh->Attributes()->PrimaryNormals();
				check(ResultOverlay != nullptr);

				for (const int SubElementID : SubOverlay->ElementIndicesItr())
				{
					const int BaseElementID = Submesh->MapNormalToBaseMesh(0, SubElementID);
					ResultOverlay->SetElement(BaseElementID, DeformedNormals[SubElementID]);
				}
			}
			else if (ResultMesh->HasVertexNormals())
			{
				for (const int SubVID : Submesh->GetSubmesh().VertexIndicesItr())
				{
					const int BaseVertexID = Submesh->MapVertexToBaseMesh(SubVID);
					ResultMesh->SetVertexNormal(BaseVertexID, DeformedNormals[SubVID]);
				}
			}
		}
		else
		{
			if (ResultMesh->HasAttributes())
			{
				FDynamicMeshNormalOverlay* NormalOverlay = ResultMesh->Attributes()->PrimaryNormals();
				check(NormalOverlay != nullptr);
			
				for (int ElementID : NormalOverlay->ElementIndicesItr())
				{
					NormalOverlay->SetElement(ElementID, DeformedNormals[ElementID]);
				}
			}
			else if (ResultMesh->HasVertexNormals())
			{
				for (int vid : ResultMesh->VertexIndicesItr())
				{
					ResultMesh->SetVertexNormal(vid, DeformedNormals[vid]);
				}
			}
		}
	}
}

FLatticeDeformerOp::FLatticeDeformerOp(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InOriginalMesh,
									   TSharedPtr<FFFDLattice, ESPMode::ThreadSafe> InLattice,
									   const TArray<FVector3d>& InLatticeControlPoints,
									   ELatticeInterpolation InInterpolationType,
									   bool bInDeformNormals) :
	Lattice(InLattice),
	OriginalMesh(InOriginalMesh),
	LatticeControlPoints(InLatticeControlPoints),
	InterpolationType(InInterpolationType),
	bDeformNormals(bInDeformNormals)
{}

FLatticeDeformerOp::FLatticeDeformerOp(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InOriginalMesh,
									   TSharedPtr<FDynamicSubmesh3, ESPMode::ThreadSafe> InSubmesh,
									   FTransform3d InTransform,
									   TSharedPtr<FFFDLattice, ESPMode::ThreadSafe> InLattice,
									   const TArray<FVector3d>& InLatticeControlPoints,
									   ELatticeInterpolation InInterpolationType,
									   bool bInDeformNormals) :
	Lattice(InLattice),
	OriginalMesh(InOriginalMesh),
	Submesh(InSubmesh),
	WorldTransform(InTransform),
	LatticeControlPoints(InLatticeControlPoints),
	InterpolationType(InInterpolationType),
	bDeformNormals(bInDeformNormals)
{}
