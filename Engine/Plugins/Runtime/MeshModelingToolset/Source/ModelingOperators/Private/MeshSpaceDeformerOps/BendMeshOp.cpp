// Copyright Epic Games, Inc. All Rights Reserved.
#include "SpaceDeformerOps/BendMeshOp.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;

//Bends along the Y-axis
void FBendMeshOp::CalculateResult(FProgressCancel* Progress)
{
	FMeshSpaceDeformerOp::CalculateResult(Progress);

	if (!OriginalMesh || (Progress && Progress->Cancelled()))
	{
		return;
	}

	// You can think of the bend as follows: pick a range along the Z axis, 
	// and pick a rotation point with its Z coordinate within that range.
	// Take the mesh region in that range and compress it entirely to the
	// rotation center Z. Then rotate the points out from here, choosing 
	// each angle of rotation based on the original Z difference of the 
	// point from the rotation center Z, but clamped to min/max.

	// The math below follows the paper "Global and Local Deformations Of Solid Primitives"
	// by Alan H. Barr, that uses a right-handed, y-up coordinate system and bends
	// a line parallel to the y axis. We translate our system to make things follow 
	// along smoother.

	// Matrix from gizmo space (z-up) to a y-up space
	FMatrix ToYUp(EForceInit::ForceInitToZero);
	ToYUp.M[0][0] =  1.f;
	ToYUp.M[1][2] =  1.f;
	ToYUp.M[2][1] = -1.f;
	ToYUp.M[3][3] =  1.f; 
	
	// Full transform to y-up in gizmo space
	FMatrix ObjectToYUpGizmo(EForceInit::ForceInitToZero);
	{
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				for (int k = 0; k < 4; ++k)
				{
					ObjectToYUpGizmo.M[i][j] += ToYUp.M[i][k] * ObjectToGizmo.M[k][j];
				}
			}
		}
	}

	float Det = ObjectToYUpGizmo.Determinant();

	// Check to see that the transform is invertible. The space deformation happens
	// on the transformed mesh, and we'd need to be able to map the new positions back
	// into the untransformed mesh.
	if (FMath::Abs(Det) < KINDA_SMALL_NUMBER)
	{
		return;
	}

	FMatrix YUpGizmoToObject = ObjectToYUpGizmo.Inverse();
	
	// We want to bend towards the positive Z axis, which corresponds to a progressively
	// more clockwise rotation of points as one moves up on the Y axis. Hence the negative
	// in the expression below.
	const double DegreesToRadians = 0.017453292519943295769236907684886; // Pi / 180
	const double TotalBendAngleRadians = -BendDegrees * DegreesToRadians;

	// Check to see that we're actually trying to bend some amount.
	if (FMath::Abs(TotalBendAngleRadians) < KINDA_SMALL_NUMBER)
	{
		return;
	}
	
	// bounds in Op space
	const double YMin = LowerBoundsInterval;
	const double YMax = UpperBoundsInterval;
	const double BentLength = YMax - YMin;

	bool bSharpBend = (BentLength == 0);

	// Y0 is our rotation center Y, the level onto which everything is compressed before being
	// rotated out.
	const double Y0 = bLockBottom ? YMin : 0;

	// K is the amount we rotate per unit step up Y in the bent region. Ik, its inverse,
	// is actually our rotation center Z, because Ik is BentLength/TotalBendAngleRadians, 
	// so it is the radius of the arc.
	// When BentLength == 0, K doesn't actually matter- we don't use it.
	const double K = bSharpBend ? 1.0 : TotalBendAngleRadians / BentLength;
	const double Ik = 1.0 / K;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Deal with the normals before we change the vertex positions (which determine the rotations)
	if (ResultMesh->HasAttributes())
	{
		FDynamicMeshNormalOverlay* Normals = ResultMesh->Attributes()->PrimaryNormals();
		ParallelFor(Normals->MaxElementID(), [this, Normals, &ObjectToYUpGizmo, &YUpGizmoToObject, YMin, YMax, Y0, K, bSharpBend, TotalBendAngleRadians](int32 ElID)
		{
			if (!Normals->IsElement(ElID))
			{
				return;
			}

			// get the vertex
			auto VertexID = Normals->GetParentVertex(ElID);
			const FVector3d& SrcPos = ResultMesh->GetVertex(VertexID);

			FVector3d SrcNormal(Normals->GetElement(ElID));

			const double SrcPos4[4] = { SrcPos[0], SrcPos[1], SrcPos[2], 1.0 };

			// Position in gizmo space
			double GizmoPos4[4] = { 0., 0., 0., 0. };
			for (int i = 0; i < 4; ++i)
			{
				for (int j = 0; j < 4; ++j)
				{
					GizmoPos4[i] += ObjectToYUpGizmo.M[i][j] * SrcPos4[j];
				}
			}

			FVector3d RotatedNormal(0, 0, 0);
			{
				// Rotate normal to gizmo space.
				for (int i = 0; i < 3; ++i)
				{
					for (int j = 0; j < 3; ++j)
					{
						RotatedNormal[i] += YUpGizmoToObject.M[j][i] * SrcNormal[j];
					}
				}
			}

			const double YHat = FMath::Clamp(GizmoPos4[1], YMin, YMax);

			// In the normal case, the bend angle is proportional to the clamped value's
			// position in the bend range. In the sharp bend case, however, where the bent
			// range is 0-sized, there is a jump from 0 to TotalBendAngleRadians at Y0. 
			const double Theta = bSharpBend && GizmoPos4[1] > Y0 ? TotalBendAngleRadians : K * (YHat - Y0);

			const double S0 = TMathUtil<double>::Sin(Theta);
			const double C0 = TMathUtil<double>::Cos(Theta);

			{
				// The below expression for the normal transformation comes from the paper, but can be derived
				// from the inverse of the Jacobian of the transformation that we're applying. We could actually
				// divide by rate of expansion to keep the normal the same length, but it's cleaner just to set
				// a normalized vector at the end.

				double KHat = (!bSharpBend && YHat == GizmoPos4[1]) ? K : 0;

				double RateOfExpansion = (1. - KHat * GizmoPos4[2]);
				FVector3d DstNormal(0., 0., 0.);
				DstNormal[0] = RateOfExpansion * RotatedNormal[0];
				DstNormal[1] =                          C0 * RotatedNormal[1] - S0 * RateOfExpansion * RotatedNormal[2];
				DstNormal[2] =                          S0 * RotatedNormal[1] + C0 * RateOfExpansion * RotatedNormal[2];

				// rotate back to mesh space.
				RotatedNormal = FVector3d(0, 0, 0);
				for (int i = 0; i < 3; ++i)
				{
					for (int j = 0; j < 3; ++j)
					{
						RotatedNormal[i] += ObjectToYUpGizmo.M[j][i] * DstNormal[j];
					}
				}

			}

			Normals->SetElement(ElID, FVector3f(Normalized(RotatedNormal)));
		});
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Adjust the positions.
	ParallelFor(ResultMesh->MaxVertexID(), [this, &ObjectToYUpGizmo, &YUpGizmoToObject, YMin, YMax, Y0, K, Ik, bSharpBend, TotalBendAngleRadians](int32 VertexID)
	{
		if (!ResultMesh->IsVertex(VertexID))
		{
			return;
		}

		// The approach we use follows the paper, and uses the variable names used there. However
		// it may be useful to see some equivalent code that may make the operations more obvious. 
		// This code works on the original Z-up frame and bends along the Z axis instead of Y, so
		// you would swap Y and Z variables to make it work in the Y-up frame:
		/*
		double ClampedZ = FMath::Clamp(GizmoSpacePosition.Z, BentMinZ, BentMaxZ);
		double ZDifferenceFromClampedRegion = GizmoSpacePosition.Z - ClampedZ;

		double AngleToRotate = ArcAngle * (ClampedZ - RotationCenterZ) / BentLength; // assumes BentLength != 0

		FVector2d YZToRotate(GizmoSpacePosition.Y, RotationCenterZ + ZDifferenceFromClampedRegion);

		FVector2d RotationCenterYZ(RotationCenterY, RotationCenterZ);
		FMatrix2d RotationMatrix = FMatrix2d::RotationRad(AngleToRotate);

		FVector2d RotatedYZ = RotationMatrix * (YZToRotate - RotationCenterYZ) + RotationCenterYZ;

		FVector3d NewGizmoSpacePosition = FVector3d(GizmoSpacePosition.X, RotatedYZ[0], RotatedYZ[1]);
		*/

		const FVector3d SrcPos = ResultMesh->GetVertex(VertexID);
		const double SrcPos4[4] = { SrcPos[0], SrcPos[1], SrcPos[2], 1.0};

		// Position in gizmo space
		double GizmoPos4[4] = { 0., 0., 0., 0. };
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				GizmoPos4[i] += ObjectToYUpGizmo.M[i][j] * SrcPos4[j];
			}
		}

		const double YHat = FMath::Clamp(GizmoPos4[1], YMin, YMax);

		// In the usual case, the bend angle is proportional to the clamped value's
		// position in the bend range. In the sharp bend case, however, where the bent
		// range is 0-sized, there is a jump from 0 to TotalBendAngleRadians at Y0. 
		const double Theta = bSharpBend && GizmoPos4[1] > Y0 ? TotalBendAngleRadians : K * (YHat - Y0);

		const double S0 = TMathUtil<double>::Sin(Theta);
		const double C0 = TMathUtil<double>::Cos(Theta);

		const double ZRelativeRotationCenter = GizmoPos4[2] - Ik;

		double Y = -S0 * ZRelativeRotationCenter + Y0;
		double Z =  C0 * ZRelativeRotationCenter + Ik;

		if (GizmoPos4[1] > YMax)
		{
			const double YDiff = GizmoPos4[1] - YMax;
			Y += C0 * YDiff;
			Z += S0 * YDiff;
		}
		else if (GizmoPos4[1] < YMin)
		{
			const double YDiff = GizmoPos4[1] - YMin;
			Y += C0 * YDiff;
			Z += S0 * YDiff;
		}

		GizmoPos4[1] = Y;
		GizmoPos4[2] = Z;

		// Position in Obj Space
		double DstPos4[4] = { 0., 0., 0., 0. };
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				DstPos4[i] += YUpGizmoToObject.M[i][j] * GizmoPos4[j];
			}
		}

		ResultMesh->SetVertex(VertexID, FVector3d(DstPos4[0], DstPos4[1], DstPos4[2]));
	});
}