// Copyright Epic Games, Inc. All Rights Reserved.
#include "SpaceDeformerOps/FlareMeshOp.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;

//Some simple non-linear interpolation functions to play with.

template <class T>
T coserp(float percent, const T& value1, const T& value2)
{
	return T(0.5) * (cos(percent*PI) * (value1 - value2) + value1 + value2); 
}

template <class T>
double inverseCoserp(const T& valueBetween, const T& value1, const T& value2)
{
	return acos((T(2) * valueBetween - value1 - value2) / (value1 - value2)) / PI;
}

template <class T>
T sinerp(float percent, const T& value1, const T& value2)
{
	return T(0.5) * (sin(percent * PI) * (value1 - value2) + value1 + value2);   
}

template <class T>
double  inverseSinerp(const T& valueBetween, const T& value1, const T& value2)
{
	return asin((T(2.0) * valueBetween - value1 - value2) / (value1 - value2)) / PI;
}


//Flares along the Z axis
void FFlareMeshOp::CalculateResult(FProgressCancel* Progress)
{
	FMeshSpaceDeformerOp::CalculateResult(Progress);

	if (!OriginalMesh || (Progress && Progress->Cancelled()))
	{
		return;
	}

	float Det = ObjectToGizmo.Determinant();

	// Check if the transform is nearly singular
	// this could happen if the scale on the object to world transform has a very small component.
	if (FMath::Abs(Det) < 1.e-4)
	{
		return;
	}

	FMatrix GizmoToObject = ObjectToGizmo.Inverse();


	const double ZMin = LowerBoundsInterval;
	const double ZMax =  UpperBoundsInterval;

	const double FlareRatioX = FlarePercentX / 100.;
	const double FlareRatioY = FlarePercentY / 100.;

	if (ResultMesh->HasAttributes())
	{
		// Fix the normals first if they exist.

		FDynamicMeshNormalOverlay* Normals = ResultMesh->Attributes()->PrimaryNormals();
		ParallelFor(Normals->MaxElementID(), [this, Normals, &GizmoToObject, ZMin, ZMax, FlareRatioX, FlareRatioY](int32 ElID)
		{
			if (!Normals->IsElement(ElID))
			{
				return;
			}

			// get the vertex
			auto VertexID = Normals->GetParentVertex(ElID);
			const FVector3d& SrcPos = ResultMesh->GetVertex(VertexID);

			FVector3f SrcNormalF = Normals->GetElement(ElID);
			FVector3d SrcNormal; 
			SrcNormal[0] = SrcNormalF[0]; SrcNormal[1] = SrcNormalF[1]; SrcNormal[2] = SrcNormalF[2];


			const double SrcPos4[4] = { SrcPos[0], SrcPos[1], SrcPos[2], 1.0 };

			// Position in gizmo space
			double GizmoPos4[4] = { 0., 0., 0., 0. };
			for (int i = 0; i < 4; ++i)
			{
				for (int j = 0; j < 4; ++j)
				{
					GizmoPos4[i] += ObjectToGizmo.M[i][j] * SrcPos4[j];
				}
			}

			FVector3d RotatedNormal(0, 0, 0);
			{
				// Rotate normal to gizmo space.
				for (int i = 0; i < 3; ++i)
				{
					for (int j = 0; j < 3; ++j)
					{
						RotatedNormal[i] += GizmoToObject.M[j][i] * SrcNormal[j];
					}
				}
			}
			
			// transform normal.  Do this before changing GizmoPos
			// To get the normal, note that the positions are transformed like this:
			// X = Rx * x
			// Y = Ry * y
			// Z = z
			// The jacobian is:
			// Rx  0  x*DRx
			// 0  Ry  y*DRy
			// 0   0   1
			// Where DRx is dRx/dz and DRy is dRy/dz
			// Then take the transpose of the inverse of the Jacobian and multiply by the determinant (or don't, since we don't
			// care about the length, but it's cleaner if you do).
			double Rx = 0., Ry = 0., DRx = 0., DRy = 0.;
			const double T = FMath::Clamp((GizmoPos4[2] - ZMin) / (ZMax - ZMin), 0.0, 1.0);
			
			switch (FlareType) // evaluate the requested curve and its slope 
			{
				case EFlareType::LinearFlare :
				{
					const double LinearDisplacement = (T < 0.5) ? 2. * T : 2. * (1. - T);
					Rx = 1. + FlareRatioX * LinearDisplacement;
					Ry = 1. + FlareRatioY * LinearDisplacement;

					double DLinearDz = (T < 0.5) ? 2. / (ZMax - ZMin) : -2. / (ZMax - ZMin);

					// hack: make normal transitions between regions of discontinuous surface slope (critical points) less noticable 
					{
						const double HalfSmoothDist = 0.05;
						if (FMath::Abs(T - 0.5) < HalfSmoothDist)
						{
							double Alpha = (T - 0.5 + HalfSmoothDist) / (2. * HalfSmoothDist);
							DLinearDz = FMath::Lerp(2. / (ZMax - ZMin), -2. / (ZMax - ZMin), Alpha);
						}
						if (T < HalfSmoothDist)
						{
							double Alpha = (T + HalfSmoothDist) / (2. * HalfSmoothDist);
							DLinearDz = FMath::Lerp(0. , 2. / (ZMax - ZMin), Alpha);
						}
						if (1. - T < HalfSmoothDist)
						{
							double Alpha = (T - 1. + HalfSmoothDist) / (2. * HalfSmoothDist);
							DLinearDz = FMath::Lerp(-2. / (ZMax - ZMin), 0., Alpha);
						}
					}

					DRx = FlareRatioX * DLinearDz;
					DRy = FlareRatioY * DLinearDz;
				}
				break;

				case EFlareType::SinFlare :
				{
					const double SinPiT = FMath::Sin(PI * T);
					const double CosPiT = FMath::Cos(PI * T);
					
					Rx = 1. + FlareRatioX * SinPiT;
					Ry = 1. + FlareRatioY * SinPiT;

					DRx = FlareRatioX * CosPiT * (PI / (ZMax - ZMin));
					DRy = FlareRatioY * CosPiT * (PI / (ZMax - ZMin));

				}
				break;

				case EFlareType::SinSqrFlare :
				{
					const double SinPiT = FMath::Sin(PI * T);
					const double Sin2PiT = FMath::Sin(2 * PI * T);

					Rx = 1. + FlareRatioX * SinPiT * SinPiT;
					Ry = 1. + FlareRatioY * SinPiT * SinPiT;

					DRx = FlareRatioX * Sin2PiT * (PI / (ZMax - ZMin));
					DRy = FlareRatioY * Sin2PiT * (PI / (ZMax - ZMin));
				}
				break;

				default :
				{
					// should never reach this
					check(0);
				}
			}
			
			if (GizmoPos4[2] > ZMax || GizmoPos4[2] < ZMin)
			{
				DRx = DRy = 0.f;
			}
			
			FVector3d DstNormal(0., 0., 0.);
			DstNormal[0] = Ry * RotatedNormal[0];
			DstNormal[1] = Rx * RotatedNormal[1];
			DstNormal[2] = -Ry * DRx * GizmoPos4[0] * RotatedNormal[0] - Rx * DRy * GizmoPos4[1] * RotatedNormal[1] + Rx * Ry * RotatedNormal[2];

			// rotate back to mesh space.
			RotatedNormal = FVector3d(0, 0, 0);
			for (int i = 0; i < 3; ++i)
			{
				for (int j = 0; j < 3; ++j)
				{
					RotatedNormal[i] += ObjectToGizmo.M[j][i] * DstNormal[j];
				}
			}

			

			Normals->SetElement(ElID, FVector3f(Normalized(RotatedNormal)));
		});
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	ParallelFor(ResultMesh->MaxVertexID(), [this, &GizmoToObject, ZMin, ZMax, FlareRatioX, FlareRatioY](int32 VertexID)
	{
		if (!ResultMesh->IsVertex(VertexID))
		{
			return;
		}
		
		const FVector3d SrcPos = ResultMesh->GetVertex(VertexID);

		const double SrcPos4[4] = { SrcPos[0], SrcPos[1], SrcPos[2], 1.0 };

		// Position in gizmo space
		double GizmoPos4[4] = { 0., 0., 0., 0. };
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				GizmoPos4[i] += ObjectToGizmo.M[i][j] * SrcPos4[j];
			}
		}

		
		// Parameterize curve between ZMin and ZMax to go between 0, 1
		double T = FMath::Clamp( (GizmoPos4[2]- ZMin) / (ZMax - ZMin), 0.0, 1.0);
		double Rx = 0., Ry = 0.;
		switch (FlareType)
		{
			case EFlareType::LinearFlare :
			{
				const double LinearDisplacement = (T < 0.5) ? 2. * T : 2. * (1. - T);
				Rx = 1. + FlareRatioX * LinearDisplacement;
				Ry = 1. + FlareRatioY * LinearDisplacement;
			}
			break;

			case EFlareType::SinFlare :
			{
				const double SinPiT = FMath::Sin(PI * T);
				Rx = 1. + FlareRatioX * SinPiT;
				Ry = 1. + FlareRatioY * SinPiT;
			}
			break;

			case EFlareType::SinSqrFlare :
			{
				const double SinPiT = FMath::Sin(PI * T);
				Rx = 1. + FlareRatioX * SinPiT * SinPiT;
				Ry = 1. + FlareRatioY * SinPiT * SinPiT;
			}
			break;

			default :
			{
				// should never..
				check(0);
			}
		}
		
		// 2d scale x,y values.
		GizmoPos4[0] *= Rx;
		GizmoPos4[1] *= Ry;


		double DstPos4[4] = { 0., 0., 0., 0. };
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				DstPos4[i] += GizmoToObject.M[i][j] * GizmoPos4[j];
			}
		}

		// set the position
		ResultMesh->SetVertex(VertexID, FVector3d(DstPos4[0], DstPos4[1], DstPos4[2]));
	});

}