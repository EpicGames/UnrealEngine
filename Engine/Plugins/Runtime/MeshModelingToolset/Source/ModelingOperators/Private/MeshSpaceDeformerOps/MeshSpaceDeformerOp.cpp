// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpaceDeformerOps/MeshSpaceDeformerOp.h"
#include "DynamicMesh/DynamicMesh3.h"

using namespace UE::Geometry;

void FMeshSpaceDeformerOp::SetTransform(const FTransformSRT3d& Transform)
{
	ResultTransform = Transform;
}

void FMeshSpaceDeformerOp::CalculateResult(FProgressCancel* Progress)
{
	if ((Progress && Progress->Cancelled()) || !OriginalMesh)
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	FMatrix WorldToGizmo(EForceInit::ForceInitToZero);
	{
		// Matrix that rotates to the gizmo frame 
		const FMatrix3d ToGizmo(GizmoFrame.GetAxis(0),
			GizmoFrame.GetAxis(1),
			GizmoFrame.GetAxis(2),
			true /* row constructor*/);

		// Gizmo To World is rotation followed by translation of the form
		// world_vec = FromGizmo * local_vec + GizmoCenter
		// so the inverse will be
		// local_vec = FromGizmo^(-1) * world_vec - FromGizmo^(-1) * GizmoCenter
		// 

		// Copy the rotation
		for (int i = 0; i < 3; ++i)
		{
			for (int j = 0; j < 3; ++j)
			{
				WorldToGizmo.M[i][j] = ToGizmo(i, j);
			}
		}
		// Add the translation 
		FVector3d Translation = ToGizmo * GizmoFrame.Origin;
		WorldToGizmo.M[0][3] = -Translation[0];
		WorldToGizmo.M[1][3] = -Translation[1];
		WorldToGizmo.M[2][3] = -Translation[2];
		WorldToGizmo.M[3][3] = 1.f;
	}
	// Get the transform as a matrix.  We need to transpose this due
	// to the vec * Matrix  vs Matrix * vec usage in engine.

	FMatrix ObjectToWorld = FTransform(ResultTransform).ToMatrixWithScale().GetTransposed();

	ObjectToGizmo = FMatrix(EForceInit::ForceInitToZero);
	{
		// ObjectToGizmo = WorldToGizmo * ObjectToWorld
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				for (int k = 0; k < 4; ++k)
				{
					ObjectToGizmo.M[i][j] += WorldToGizmo.M[i][k] * ObjectToWorld.M[k][j];
				}
			}
		}
	}

	// The rest is up to the subclass.
}
