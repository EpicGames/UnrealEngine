// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpaceDeformerOps/TwistMeshOp.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

using namespace UE::Geometry;

//Twists along the Z axis
void FTwistMeshOp::CalculateResult(FProgressCancel* Progress)
{
	FMeshSpaceDeformerOp::CalculateResult(Progress);

	if (!OriginalMesh || (Progress && Progress->Cancelled()))
	{
		return;
	}

	float Det = ObjectToGizmo.Determinant();

	// Check if the transform is nearly singular
	// this could happen if the scale on the object to world transform has a very small component.
	if (FMath::Abs(Det) < 1.e-4)
	{
		return;
	}

	// note the transpose of GizmoToObject should be used to transform normals into Gizmo Space.
	// the transpose of OjbectToGizmo should be used to transform normals back to Object Space.  

	FMatrix GizmoToObject = ObjectToGizmo.Inverse();

	


	const double ZMin = LowerBoundsInterval;
	const double ZMax =  UpperBoundsInterval;


	const double DegreesToRadians = 0.017453292519943295769236907684886; // Pi / 180
	const double ThetaRadians = DegreesToRadians * TwistDegrees;

	if (ResultMesh->HasAttributes())
	{
		// Fix the normals first if they exist.

		FDynamicMeshNormalOverlay* Normals = ResultMesh->Attributes()->PrimaryNormals();
		ParallelFor(Normals->MaxElementID(), [this, Normals, &GizmoToObject, ZMin, ZMax, ThetaRadians](int32 ElID)
		{
			if (!Normals->IsElement(ElID))
			{
				return;
			}

			// get the vertex
			auto VertexID = Normals->GetParentVertex(ElID);
			const FVector3d& SrcPos = ResultMesh->GetVertex(VertexID);

			FVector3f SrcNormalF = Normals->GetElement(ElID);
			FVector3d SrcNormal;
			SrcNormal[0] = SrcNormalF[0]; SrcNormal[1] = SrcNormalF[1]; SrcNormal[2] = SrcNormalF[2];



			const double SrcPos4[4] = { SrcPos[0], SrcPos[1], SrcPos[2], 1.0 };

			// Position in gizmo space
			double GizmoPos4[4] = { 0., 0., 0., 0. };
			for (int i = 0; i < 4; ++i)
			{
				for (int j = 0; j < 4; ++j)
				{
					GizmoPos4[i] += ObjectToGizmo.M[i][j] * SrcPos4[j];
				}
			}

			FVector3d RotatedNormal(0, 0, 0);
			{
				// Rotate normal to gizmo space.
				for (int i = 0; i < 3; ++i)
				{
					for (int j = 0; j < 3; ++j)
					{
						RotatedNormal[i] += GizmoToObject.M[j][i] * SrcNormal[j];
					}
				}
			}


			double T = FMath::Clamp((GizmoPos4[2] - ZMin) / (ZMax - ZMin), 0.0, 1.0) 
				- (bLockBottom ? 0 : 0.5);
			double Theta = ThetaRadians * T;



			const double C0 = FMath::Cos(Theta);
			const double S0 = FMath::Sin(Theta);

			// transform normal.  Do this before changing GizmoPos

			{

				// rotate with twist deformation
				double DthetaDZ = ThetaRadians / (ZMax - ZMin);
				if (GizmoPos4[2] > ZMax || GizmoPos4[2] < ZMin) DthetaDZ = 0.;

				FVector3d DstNormal(0., 0., 0.);
				DstNormal[0] = C0 * RotatedNormal[0] - S0 * RotatedNormal[1];
				DstNormal[1] = S0 * RotatedNormal[0] + C0 * RotatedNormal[1];
				DstNormal[2] = DthetaDZ * (GizmoPos4[1] * RotatedNormal[0] - GizmoPos4[0] * RotatedNormal[1]) + RotatedNormal[2];

				// rotate back to mesh space.
				RotatedNormal = FVector3d(0, 0, 0);
				for (int i = 0; i < 3; ++i)
				{
					for (int j = 0; j < 3; ++j)
					{
						RotatedNormal[i] += ObjectToGizmo.M[j][i] * DstNormal[j];
					}
				}
				
			}

			Normals->SetElement(ElID, FVector3f(Normalized(RotatedNormal)));
		});
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// now fix the vertex positions
	ParallelFor(ResultMesh->MaxVertexID(), [this, &GizmoToObject, ZMin, ZMax, ThetaRadians](int32 VertexID)
	{
		if (!ResultMesh->IsVertex(VertexID))
		{
			return;
		}

		const FVector3d SrcPos = ResultMesh->GetVertex(VertexID);
		const double SrcPos4[4] = { SrcPos[0], SrcPos[1], SrcPos[2], 1.0 };

		// Position in gizmo space
		double GizmoPos4[4] = { 0., 0., 0., 0. };
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				GizmoPos4[i] += ObjectToGizmo.M[i][j] * SrcPos4[j];
			}
		}

		double T = FMath::Clamp( (GizmoPos4[2] - ZMin) / ( ZMax - ZMin ), 0.0, 1.0) 
			- (bLockBottom ? 0 : 0.5);
		double Theta = ThetaRadians * T;

		

		const double C0 = FMath::Cos(Theta);
		const double S0 = FMath::Sin(Theta);
	

		// apply 2d rotation.
		const double X =  C0 * GizmoPos4[0]  - S0 * GizmoPos4[1];
		const double Y =  S0 * GizmoPos4[0]  + C0 * GizmoPos4[1];
		
		GizmoPos4[0] = X;
		GizmoPos4[1] = Y;

		

		// Position in Obj Space
		double DstPos4[4] = { 0., 0., 0., 0. };
		for (int i = 0; i < 4; ++i)
		{
			for (int j = 0; j < 4; ++j)
			{
				DstPos4[i] += GizmoToObject.M[i][j] * GizmoPos4[j];
			}
		}

		// set the position
		ResultMesh->SetVertex(VertexID, FVector3d(DstPos4[0], DstPos4[1], DstPos4[2]));
		
	});
}