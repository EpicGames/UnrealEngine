// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizationOps/RecomputeUVsOp.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "Selections/MeshConnectedComponents.h"
#include "Parameterization/MeshLocalParam.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/PatchBasedMeshUVGenerator.h"
#include "Properties/RecomputeUVsProperties.h"
#include "Parameterization/MeshUDIMClassifier.h"

#include "DynamicSubmesh3.h"

#include "Async/ParallelFor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecomputeUVsOp)

using namespace UE::Geometry;


#define LOCTEXT_NAMESPACE "RecomputeUVsOp"


namespace UE
{
	namespace Geometry
	{
		struct TileConnectedComponents
		{
			TileConnectedComponents(const FMeshConnectedComponents& ConnectedComponentsIn, const FVector2i& TileIn)
				: ConnectedComponents(ConnectedComponentsIn),
				Tile(TileIn)
			{
				for (int32 ComponentIndex = 0; ComponentIndex < ConnectedComponents.Num(); ++ComponentIndex)
				{
					TileTids.Append(ConnectedComponents[ComponentIndex].Indices);
				}
			}

			TArray<int32> TileTids;
			FMeshConnectedComponents ConnectedComponents;
			FVector2i Tile;
		};
	}
}

void FRecomputeUVsOp::NormalizeUVAreas(const FDynamicMesh3& Mesh, FDynamicMeshUVOverlay* Overlay, const TileConnectedComponents& TileComponents, float GlobalScale)
{
	int32 NumComponents = TileComponents.ConnectedComponents.Num();

	ParallelFor(NumComponents, [&](int32 k)
		{
			const TArray<int>& Triangles = TileComponents.ConnectedComponents[k].Indices;
			TSet<int> UVElements;
			UVElements.Reserve(Triangles.Num() * 3);
			double AreaUV = 0;
			double Area3D = 0;
			FVector3d Triangle3D[3];
			FVector2d TriangleUV[3];
			FAxisAlignedBox2d BoundsUV = FAxisAlignedBox2d::Empty();

			for (int tid : Triangles)
			{
				FIndex3i TriElements = Overlay->GetTriangle(tid);
				if (!TriElements.Contains(FDynamicMesh3::InvalidID))
				{
					for (int j = 0; j < 3; ++j)
					{
						TriangleUV[j] = FVector2d(Overlay->GetElement(TriElements[j]));
						BoundsUV.Contain(TriangleUV[j]);
						Triangle3D[j] = Mesh.GetVertex(Overlay->GetParentVertex(TriElements[j]));
						UVElements.Add(TriElements[j]);
					}
					AreaUV += VectorUtil::Area(TriangleUV[0], TriangleUV[1], TriangleUV[2]);
					Area3D += VectorUtil::Area(Triangle3D[0], Triangle3D[1], Triangle3D[2]);
				}
			}

			double LinearScale = (AreaUV > 0.00001) ? (FMathd::Sqrt(Area3D) / FMathd::Sqrt(AreaUV)) : 1.0;
			LinearScale = LinearScale * GlobalScale;
			FVector2d ComponentOrigin = BoundsUV.Center();

			for (int elemid : UVElements)
			{
				FVector2d UV = FVector2d(Overlay->GetElement(elemid));
				UV = (UV - ComponentOrigin) * LinearScale;
				Overlay->SetElement(elemid, FVector2f(UV));
			}
		});
}


void FRecomputeUVsOp::CalculateResult(FProgressCancel* Progress)
{
	NewResultInfo = FGeometryResult(EGeometryResultType::InProgress);
	bool bOK = false;

	if (IsValid())
	{
		if (bMergingOptimization)
		{
			bOK = CalculateResult_RegionOptimization(Progress);
		}
		else
		{
			bOK = CalculateResult_Basic(Progress);
		}
	}

	NewResultInfo.SetSuccess(bOK, Progress);
	SetResultInfo(NewResultInfo);

}

void FRecomputeUVsOp::CollectIslandComponentsPerTile(const FDynamicMeshUVOverlay& UVOverlay, TArray< TileConnectedComponents >& ComponentsPerTile, bool& bUseExistingUVTopology)
{
	bUseExistingUVTopology = false;
	ComponentsPerTile.Empty();

	TArray<FVector2i> Tiles;
	TArray<TUniquePtr<TArray<int32>>> TileTids;

	if (bUDIMsEnabled)
	{
		TOptional<TArray<int32>> SelectionArray;
		if (Selection.IsSet())
		{
			SelectionArray = Selection.GetValue().Array();
		}
		FDynamicMeshUDIMClassifier TileClassifier(&UVOverlay, SelectionArray);

		Tiles = TileClassifier.ActiveTiles();
		for (const FVector2i& Tile : Tiles)
		{
			TileTids.Emplace(MakeUnique<TArray<int32>>(TileClassifier.TidsForTile(Tile)));
		}
	}
	else
	{
		if (Selection.IsSet())
		{
			Tiles.Add({ 0,0 });
			TileTids.Emplace(MakeUnique<TArray<int32>>(Selection.GetValue().Array()));
		}
		else
		{
			Tiles.Add({ 0,0 });
			TileTids.Add(nullptr);
		}
	}

	for (int32 TileIndex = 0; TileIndex < Tiles.Num(); ++TileIndex)
	{
		FMeshConnectedComponents ConnectedComponents(ResultMesh.Get());
		if (IslandMode == ERecomputeUVsIslandMode::PolyGroups)
		{
			if (InputGroups != nullptr)
			{
				if (TileTids[TileIndex])
				{
					ConnectedComponents.FindConnectedTriangles(*TileTids[TileIndex], [this](int32 CurTri, int32 NbrTri) {
						return InputGroups->GetTriangleGroup(CurTri) == InputGroups->GetTriangleGroup(NbrTri);
						});
				}
				else
				{
					ConnectedComponents.FindConnectedTriangles([this](int32 CurTri, int32 NbrTri) {
						return InputGroups->GetTriangleGroup(CurTri) == InputGroups->GetTriangleGroup(NbrTri);
						});
				}
			}
			else
			{
				if (TileTids[TileIndex])
				{
					ConnectedComponents.FindConnectedTriangles(*TileTids[TileIndex], [this](int32 CurTri, int32 NbrTri) {
						return ResultMesh->GetTriangleGroup(CurTri) == ResultMesh->GetTriangleGroup(NbrTri);
						});
				}
				else
				{
					ConnectedComponents.FindConnectedTriangles([this](int32 CurTri, int32 NbrTri) {
						return ResultMesh->GetTriangleGroup(CurTri) == ResultMesh->GetTriangleGroup(NbrTri);
						});
				}
			}
		}
		else
		{
			if (TileTids[TileIndex])
			{
				ConnectedComponents.FindConnectedTriangles(*TileTids[TileIndex], [&](int32 Triangle0, int32 Triangle1) {
					return UVOverlay.AreTrianglesConnected(Triangle0, Triangle1);
					});
			}
			else
			{
				ConnectedComponents.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1) {
					return UVOverlay.AreTrianglesConnected(Triangle0, Triangle1);
					});
			}

			bUseExistingUVTopology = true;
		}

		if (!ConnectedComponents.Components.IsEmpty())
		{
			ComponentsPerTile.Emplace(ConnectedComponents, Tiles[TileIndex]);
		}
	}
}

bool FRecomputeUVsOp::CalculateResult_Basic(FProgressCancel* Progress)
{
	if (!InputMesh.IsValid())
	{
		return false;
	}

	ResultMesh = MakeUnique<FDynamicMesh3>(*InputMesh);

	FDynamicMesh3* BaseMesh = ResultMesh.Get();
	FDynamicMeshUVEditor UVEditor(BaseMesh, UVLayer, true);
	FDynamicMeshUVOverlay* UseOverlay = UVEditor.GetOverlay();
	if (ensure(UseOverlay != nullptr) == false)
	{
		return false;
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// find group-connected-components
	bool bUseExistingUVTopology = false;
	TArray< TileConnectedComponents > ComponentsPerTile;
	CollectIslandComponentsPerTile(*UseOverlay, ComponentsPerTile, bUseExistingUVTopology);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}


	for (TileConnectedComponents& TileComponents : ComponentsPerTile)
	{
		int32 NumComponents = TileComponents.ConnectedComponents.Num();
		TArray<bool> bComponentSolved;
		bComponentSolved.Init(false, NumComponents);
		int32 SuccessCount = 0;
		TArray<FAxisAlignedBox2f> PerComponentBoundingBoxes;
		PerComponentBoundingBoxes.SetNum(NumComponents);

		// TODO: the solves here could be done in parallel if we pre-allocated the island element IDs
		for (int32 k = 0; k < NumComponents; ++k)
		{
			const TArray<int32>& ComponentTris = TileComponents.ConnectedComponents[k].Indices;

			FAxisAlignedBox2f& UVBounds = PerComponentBoundingBoxes[k];
			if (bPackToOriginalBounds)
			{
				for (int32 tid : ComponentTris)
				{
					if (UseOverlay->IsSetTriangle(tid))
					{
						FIndex3i UVTri = UseOverlay->GetTriangle(tid);
						FVector2f U = UseOverlay->GetElement(UVTri.A);
						FVector2f V = UseOverlay->GetElement(UVTri.B);
						FVector2f W = UseOverlay->GetElement(UVTri.C);
						UVBounds.Contain(U); UVBounds.Contain(V); UVBounds.Contain(W);
					}
				}
			}

			bComponentSolved[k] = false;
			switch (UnwrapType)
			{
			case ERecomputeUVsUnwrapType::ExpMap:
			{
				FDynamicMeshUVEditor::FExpMapOptions Options;
				Options.NormalSmoothingRounds = this->NormalSmoothingRounds;
				Options.NormalSmoothingAlpha = this->NormalSmoothingAlpha;
				bComponentSolved[k] = UVEditor.SetTriangleUVsFromExpMap(ComponentTris, Options);
			}
			break;

			case ERecomputeUVsUnwrapType::ConformalFreeBoundary:
				bComponentSolved[k] = UVEditor.SetTriangleUVsFromFreeBoundaryConformal(ComponentTris, bUseExistingUVTopology);
				if (bComponentSolved[k])
				{
					UVEditor.ScaleUVAreaTo3DArea(ComponentTris, true);
				}
				break;
			case ERecomputeUVsUnwrapType::SpectralConformal:
			{
				bComponentSolved[k] = UVEditor.SetTriangleUVsFromFreeBoundarySpectralConformal(ComponentTris, bUseExistingUVTopology, bPreserveIrregularity);
				if (bComponentSolved[k])
				{
					UVEditor.ScaleUVAreaTo3DArea(ComponentTris, true);
				}
				break;
			}
			}

			if (bComponentSolved[k])
			{
				SuccessCount++;
			}

			if (Progress && Progress->Cancelled())
			{
				return false;
			}
		}


		if (bAutoRotate)
		{
			ParallelFor(NumComponents, [&](int32 k)
				{
					if (Progress && Progress->Cancelled())
					{
						return;
					}
					if (bComponentSolved[k])
					{
						const TArray<int32>& ComponentTris = TileComponents.ConnectedComponents[k].Indices;
						UVEditor.AutoOrientUVArea(ComponentTris);
					};
				});
		}
		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		if (bPackToOriginalBounds)
		{
			ParallelFor(NumComponents, [&](int32 k)
				{
					if (Progress && Progress->Cancelled())
					{
						return;
					}
					if (bComponentSolved[k])
					{
						const TArray<int32>& ComponentTris = TileComponents.ConnectedComponents[k].Indices;
						const FAxisAlignedBox2f& UVBounds = PerComponentBoundingBoxes[k];
						UVEditor.ScaleUVAreaToBoundingBox(ComponentTris, UVBounds, true, true);
					}
				});
		}
		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		if (bNormalizeAreas)
		{
			// todo should be a DynamicUVEditor function?
			NormalizeUVAreas(*ResultMesh, UseOverlay, TileComponents, AreaScaling);
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		if (bPackUVs && ensure(TileComponents.TileTids.Num() > 0))
		{
			bool bPackingSuccess = UVEditor.UDIMPack(PackingTextureResolution, PackingGutterWidth, TileComponents.Tile, &TileComponents.TileTids);
			if (!bPackingSuccess)
			{
				NewResultInfo.AddError(FGeometryError(0, LOCTEXT("IslandPackingFailed", "Failed to pack UV islands")));
				return false;
			}
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

	}

	return true;
}






bool FRecomputeUVsOp::CalculateResult_RegionOptimization(FProgressCancel* Progress)
{
	if (!InputMesh.IsValid())
	{
		return false;
	}

	ResultMesh = MakeUnique<FDynamicMesh3>(*InputMesh);

	FDynamicMesh3* BaseMesh = ResultMesh.Get();
	FDynamicMeshUVEditor UVEditor(BaseMesh, UVLayer, true);
	FDynamicMeshUVOverlay* UseOverlay = UVEditor.GetOverlay();
	if (ensure(UseOverlay != nullptr) == false)
	{
		return false;
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// find group-connected-components
	bool bUseExistingUVTopology = false;
	TArray< TileConnectedComponents > ComponentsPerTile;
	CollectIslandComponentsPerTile(*UseOverlay, ComponentsPerTile, bUseExistingUVTopology);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	for (TileConnectedComponents& TileComponents : ComponentsPerTile)
	{
		FPatchBasedMeshUVGenerator UVGenerator;
		UVGenerator.MergingThreshold = this->MergingThreshold;
		UVGenerator.CompactnessThreshold = this->CompactnessThreshold;
		UVGenerator.MaxNormalDeviationDeg = this->MaxNormalDeviationDeg;
		UVGenerator.NormalSmoothingRounds = this->NormalSmoothingRounds;
		UVGenerator.NormalSmoothingAlpha = this->NormalSmoothingAlpha;

		TArray<TArray<int32>> UVIslandTriangleSets;
		bool bIslandsOK = UVGenerator.ComputeIslandsByRegionMerging(*BaseMesh, *UseOverlay, TileComponents.ConnectedComponents, UVIslandTriangleSets, Progress);

		if (bIslandsOK == false)
		{
			NewResultInfo.AddError(FGeometryError(0, LOCTEXT("IslandMergingFailed", "Failed to merge UV islands")));
			return false;
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		TArray<bool> bIslandUVsValid;
		int32 NumSolvesFailed = UVGenerator.ComputeUVsFromTriangleSets(*BaseMesh, *UseOverlay, UVIslandTriangleSets, bIslandUVsValid, Progress);
		if (NumSolvesFailed > 0)
		{
			NewResultInfo.AddWarning(FGeometryWarning(0, LOCTEXT("PartialSolvesFailed", "Failed to compute UVs for some UV islands")));
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		if (bAutoRotate)
		{
			ParallelFor(UVIslandTriangleSets.Num(), [&](int32 k)
				{
					if (Progress && Progress->Cancelled())
					{
						return;
					}
					if (bIslandUVsValid[k])
					{
						UVEditor.AutoOrientUVArea(UVIslandTriangleSets[k]);
					}
				});
		}
		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		if (bNormalizeAreas)
		{
			// todo should be a DynamicUVEditor function?
			NormalizeUVAreas(*ResultMesh, UseOverlay, TileComponents, AreaScaling);
		}


		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		if (bPackUVs)
		{
			bool bPackingSuccess = UVEditor.UDIMPack(PackingTextureResolution, PackingGutterWidth, TileComponents.Tile, &TileComponents.TileTids);
			if (!bPackingSuccess)
			{
				NewResultInfo.AddError(FGeometryError(0, LOCTEXT("IslandPackingFailed", "Failed to pack UV islands")));
				return false;
			}
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}
	}

	return true;
}

bool FRecomputeUVsOp::IsValid() const
{
	if (bPackToOriginalBounds && (bMergingOptimization || IslandMode == ERecomputeUVsIslandMode::PolyGroups))
	{
		// It is not a valid state to be packing into original UV island bounds if islands are being merged or re-generated from polygroups
		return false;
	}

	return true;
}


/**
 * Factory
 */


TUniquePtr<FDynamicMeshOperator> URecomputeUVsOpFactory::MakeNewOperator()
{
	TUniquePtr<FRecomputeUVsOp> RecomputeUVsOp = MakeUnique<FRecomputeUVsOp>();

	RecomputeUVsOp->InputMesh = OriginalMesh;
	RecomputeUVsOp->InputGroups = InputGroups;
	RecomputeUVsOp->UVLayer = GetSelectedUVChannel();

	RecomputeUVsOp->IslandMode = static_cast<ERecomputeUVsIslandMode>(Settings->IslandGeneration);

	switch (Settings->UnwrapType)
	{
	case ERecomputeUVsPropertiesUnwrapType::ExpMap:
		RecomputeUVsOp->UnwrapType = ERecomputeUVsUnwrapType::ExpMap;
		RecomputeUVsOp->bMergingOptimization = false;
		break;
	case ERecomputeUVsPropertiesUnwrapType::Conformal:
		RecomputeUVsOp->UnwrapType = ERecomputeUVsUnwrapType::ConformalFreeBoundary;
		RecomputeUVsOp->bMergingOptimization = false;
		break;
	case ERecomputeUVsPropertiesUnwrapType::IslandMerging:
		RecomputeUVsOp->UnwrapType = ERecomputeUVsUnwrapType::ExpMap;
		RecomputeUVsOp->bMergingOptimization = true;
		break;
	case ERecomputeUVsPropertiesUnwrapType::SpectralConformal:
		RecomputeUVsOp->UnwrapType = ERecomputeUVsUnwrapType::SpectralConformal;
		RecomputeUVsOp->bMergingOptimization = false;
		break;
	}

	RecomputeUVsOp->bAutoRotate = (Settings->AutoRotation == ERecomputeUVsToolOrientationMode::MinBounds);

	RecomputeUVsOp->NormalSmoothingRounds = Settings->SmoothingSteps;
	RecomputeUVsOp->NormalSmoothingAlpha = Settings->SmoothingAlpha;

	RecomputeUVsOp->bPreserveIrregularity = Settings->bPreserveIrregularity;

	RecomputeUVsOp->MergingThreshold = Settings->MergingDistortionThreshold;
	RecomputeUVsOp->MaxNormalDeviationDeg = Settings->MergingAngleThreshold;

	RecomputeUVsOp->bPackUVs = false;
	RecomputeUVsOp->bNormalizeAreas = false;
	RecomputeUVsOp->bUDIMsEnabled = Settings->bEnableUDIMLayout;

	switch (Settings->LayoutType)
	{
	case ERecomputeUVsPropertiesLayoutType::None:
		break;
	case ERecomputeUVsPropertiesLayoutType::Repack:
		RecomputeUVsOp->bPackUVs = true;
		RecomputeUVsOp->PackingTextureResolution = Settings->TextureResolution;
		break;
	case ERecomputeUVsPropertiesLayoutType::NormalizeToExistingBounds:
		RecomputeUVsOp->bPackToOriginalBounds = true;
		RecomputeUVsOp->bPackUVs = false;
		RecomputeUVsOp->bNormalizeAreas = false;
		break;
	case ERecomputeUVsPropertiesLayoutType::NormalizeToBounds:
		RecomputeUVsOp->bNormalizeAreas = true;
		RecomputeUVsOp->AreaScaling = Settings->NormalizeScale / static_cast<float>(OriginalMesh->GetBounds().MaxDim());
		break;
	case ERecomputeUVsPropertiesLayoutType::NormalizeToWorld:
		RecomputeUVsOp->bNormalizeAreas = true;
		RecomputeUVsOp->AreaScaling = Settings->NormalizeScale;
	}

	RecomputeUVsOp->SetTransform(TargetTransform);
	RecomputeUVsOp->Selection = Selection;

	return RecomputeUVsOp;
}


#undef LOCTEXT_NAMESPACE

