// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizationOps/TexelDensityOp.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Parameterization/MeshUVPacking.h"
#include "Properties/UVLayoutProperties.h"
#include "Selections/MeshConnectedComponents.h"
#include "Parameterization/MeshUDIMClassifier.h"
#include "UDIMUtilities.h"
#include "Parameterization/UVMetrics.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(TexelDensityOp)

using namespace UE::Geometry;

namespace UVTexelDensityOpLocals
{
	FVector2f ExternalUVToInternalUV(const FVector2f& UV)
	{
		return FVector2f(UV.X, 1 - UV.Y);
	}

	FVector2f InternalUVToExternalUV(const FVector2f& UV)
	{
		return FVector2f(UV.X, 1 - UV.Y);
	}

	struct TileConnectedComponents
	{
		TileConnectedComponents(const FMeshConnectedComponents& ConnectedComponentsIn, const FVector2i& TileIn)
			: ConnectedComponents(ConnectedComponentsIn),
		Tile(TileIn)
		{
			for (int32 ComponentIndex = 0; ComponentIndex < ConnectedComponents.Num(); ++ComponentIndex)
			{
				TileTids.Append(ConnectedComponents[ComponentIndex].Indices);
			}
		}

		TArray<int32> TileTids;
		FMeshConnectedComponents ConnectedComponents;
		FVector2i Tile;
	};

	void CollectIslandComponentsPerTile(const FDynamicMesh3& Mesh, const FDynamicMeshUVOverlay& UVOverlay, TOptional<TSet<int32>>& Selection,
		                                TArray< TileConnectedComponents >& ComponentsPerTile, bool bUDIMsEnabled)
	{
		ComponentsPerTile.Empty();

		TArray<FVector2i> Tiles;
		TArray<TUniquePtr<TArray<int32>>> TileTids;

		if (bUDIMsEnabled)
		{
			TOptional<TArray<int32>> SelectionArray;
			if (Selection.IsSet())
			{
				SelectionArray = Selection.GetValue().Array();
			}
			FDynamicMeshUDIMClassifier TileClassifier(&UVOverlay, SelectionArray);

			Tiles = TileClassifier.ActiveTiles();
			for (const FVector2i& Tile : Tiles)
			{
				TileTids.Emplace(MakeUnique<TArray<int32>>(TileClassifier.TidsForTile(Tile)));
			}
		}
		else
		{
			if (Selection.IsSet())
			{
				Tiles.Add({ 0,0 });
				TileTids.Emplace(MakeUnique<TArray<int32>>(Selection.GetValue().Array()));
			}
			else
			{
				Tiles.Add({ 0,0 });
				TileTids.Emplace(MakeUnique<TArray<int32>>());
				TileTids[0]->Reserve(Mesh.TriangleCount());
				for (int32 Tid : Mesh.TriangleIndicesItr())
				{
					TileTids[0]->Add(Tid);
				}
			}
		}

		for (int32 TileIndex = 0; TileIndex < Tiles.Num(); ++TileIndex)
		{
			FMeshConnectedComponents ConnectedComponents(&Mesh);
			if (TileTids[TileIndex])
			{
				ConnectedComponents.FindConnectedTriangles(*TileTids[TileIndex], [&](int32 Triangle0, int32 Triangle1) {
					return UVOverlay.AreTrianglesConnected(Triangle0, Triangle1);
				});
			}
			else
			{
				ConnectedComponents.FindConnectedTriangles([&](int32 Triangle0, int32 Triangle1) {
					return UVOverlay.AreTrianglesConnected(Triangle0, Triangle1);
				});
			}

			if (!ConnectedComponents.Components.IsEmpty())
			{
				ComponentsPerTile.Emplace(ConnectedComponents, Tiles[TileIndex]);
			}
		}
	}
}

bool UUVEditorTexelDensitySettings::InSamplingMode() const
{
	return true;
}


void FUVEditorTexelDensityOp::SetTransform(const FTransformSRT3d& Transform)
{
	ResultTransform = Transform;
}


void FUVEditorTexelDensityOp::ScaleMeshSubRegionByDensity(FDynamicMeshUVOverlay* UVLayer, const TArray<int32>& Tids, TSet<int32>& UVElements, int32 TileResolution)
{
	double AverageTidTexelDensity = 0.0;
	int SetTriangleCount = 0;
	for (int Tid : Tids)
	{
		if (ResultMesh->Attributes()->GetUVLayer(UVLayerIndex)->IsSetTriangle(Tid))
		{
			AverageTidTexelDensity += FUVMetrics::TexelDensity(*ResultMesh, UVLayerIndex, Tid, TileResolution);
			SetTriangleCount++;
		}
	}
	if (SetTriangleCount)
	{
		AverageTidTexelDensity /= SetTriangleCount;
	}

	double TargetTexelDensity = (double)TargetPixelCountMeasurement / TargetWorldSpaceMeasurement;
	double RequiredGlobalScaleFactor = TargetTexelDensity / AverageTidTexelDensity;

	for (int ElementID : UVElements)
	{
		FVector2f UV = UVTexelDensityOpLocals::InternalUVToExternalUV(UVLayer->GetElement(ElementID));
		UV = (UV * RequiredGlobalScaleFactor);
		UVLayer->SetElement(ElementID, UVTexelDensityOpLocals::ExternalUVToInternalUV(UV));
	}
}



void FUVEditorTexelDensityOp::CalculateResult(FProgressCancel* Progress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVLayoutOp_CalculateResult);

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	ResultMesh->Copy(*OriginalMesh, true, true, true, true);

	if (!ensureMsgf(ResultMesh->HasAttributes(), TEXT("Attributes not found on mesh? Conversion should always create them, so this operator should not need to do so.")))
	{
		ResultMesh->EnableAttributes();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}


	int UVLayerInput = UVLayerIndex;
	FDynamicMeshUVOverlay* UseUVLayer = ResultMesh->Attributes()->GetUVLayer(UVLayerInput);


	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (bMaintainOriginatingUDIM)
	{
		TArray<UVTexelDensityOpLocals::TileConnectedComponents> TileComponents;
		UVTexelDensityOpLocals::CollectIslandComponentsPerTile(*ResultMesh, *ResultMesh->Attributes()->GetUVLayer(UVLayerIndex), Selection, TileComponents, true);

		for (UVTexelDensityOpLocals::TileConnectedComponents Tile : TileComponents)
		{
			FVector2i TileIndex = Tile.Tile;
			float TileResolution = TextureResolution;
			if (TextureResolutionPerUDIM.IsSet()
				&& TileIndex.X >= 0 && TileIndex.X <= UE::TextureUtilitiesCommon::UDIM_Max_X
				&& TileIndex.Y >= 0 && TileIndex.Y <= UE::TextureUtilitiesCommon::UDIM_Max_Y)
			{
				const int32 TileID = UE::TextureUtilitiesCommon::GetUDIMIndex(TileIndex.X, TileIndex.Y);
				TileResolution = TextureResolutionPerUDIM.GetValue().FindOrAdd(TileID, this->TextureResolution);
			}

			TSet<int32> ElementsToMove;
			ElementsToMove.Reserve(Tile.TileTids.Num() * 3);
			for (int Tid : Tile.TileTids)
			{
				FIndex3i Elements = UseUVLayer->GetTriangle(Tid);
				ElementsToMove.Add(Elements[0]);
				ElementsToMove.Add(Elements[1]);
				ElementsToMove.Add(Elements[2]);
			}

			for (int32 Element : ElementsToMove)
			{
				FVector2f UV = UVTexelDensityOpLocals::InternalUVToExternalUV(UseUVLayer->GetElement(Element));
				UV = UV - FVector2f(TileIndex);
				UseUVLayer->SetElement(Element, UVTexelDensityOpLocals::ExternalUVToInternalUV(UV));
			}

			// TODO: There is a second connected components call inside the packer that might be unnessessary. Could be a future optimization.
			FDynamicMeshUVPacker Packer(UseUVLayer, MakeUnique<TArray<int32>>(Tile.TileTids));
			Packer.TextureResolution = TileResolution;
			ExecutePacker(Packer);
			if (Progress && Progress->Cancelled())
			{
				return;
			}

			if (TexelDensityMode == EUVTexelDensityOpModes::ScaleGlobal)
			{
				ScaleMeshSubRegionByDensity(UseUVLayer, Tile.TileTids, ElementsToMove, TileResolution);
			}

			if (TexelDensityMode == EUVTexelDensityOpModes::ScaleIslands)
			{
				for (int32 ComponentIndex = 0; ComponentIndex < Tile.ConnectedComponents.Components.Num(); ComponentIndex++)
				{
					const TArray<int32>& ComponentTids = Tile.ConnectedComponents.Components[ComponentIndex].Indices;

					TSet<int32> ComponentElementsToMove;
					ComponentElementsToMove.Reserve(ComponentTids.Num() * 3);
					for (int Tid : ComponentTids)
					{
						FIndex3i Elements = UseUVLayer->GetTriangle(Tid);
						ComponentElementsToMove.Add(Elements[0]);
						ComponentElementsToMove.Add(Elements[1]);
						ComponentElementsToMove.Add(Elements[2]);
					}

					ScaleMeshSubRegionByDensity(UseUVLayer, ComponentTids, ComponentElementsToMove, TileResolution);
				}
			}

			for (int32 Element : ElementsToMove)
			{
				FVector2f UV = UVTexelDensityOpLocals::InternalUVToExternalUV(UseUVLayer->GetElement(Element));
				UV = UV + FVector2f(TileIndex);
				UseUVLayer->SetElement(Element, UVTexelDensityOpLocals::ExternalUVToInternalUV(UV));
			}
		}
	}
	else
	{
		TArray<UVTexelDensityOpLocals::TileConnectedComponents> TileComponents;
		UVTexelDensityOpLocals::CollectIslandComponentsPerTile(*ResultMesh, *ResultMesh->Attributes()->GetUVLayer(UVLayerIndex), Selection, TileComponents, false);

		if(TileComponents.Num() == 0)
		{
			return;
		}

		TSet<int32> ElementsToMove;
		ElementsToMove.Reserve(TileComponents[0].TileTids.Num() * 3);
		for (int Tid : TileComponents[0].TileTids)
		{
			FIndex3i Elements = UseUVLayer->GetTriangle(Tid);
			ElementsToMove.Add(Elements[0]);
			ElementsToMove.Add(Elements[1]);
			ElementsToMove.Add(Elements[2]);
		}

		FDynamicMeshUVPacker Packer(UseUVLayer, MakeUnique<TArray<int32>>(TileComponents[0].TileTids));
		Packer.TextureResolution = TextureResolution;
		ExecutePacker(Packer);
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		if (TexelDensityMode == EUVTexelDensityOpModes::ScaleGlobal)
		{
			ScaleMeshSubRegionByDensity(UseUVLayer, TileComponents[0].TileTids, ElementsToMove, TextureResolution);
		}

		if (TexelDensityMode == EUVTexelDensityOpModes::ScaleIslands)
		{
			for (int32 ComponentIndex = 0; ComponentIndex < TileComponents[0].ConnectedComponents.Components.Num(); ComponentIndex++)
			{
				const TArray<int32>& ComponentTids = TileComponents[0].ConnectedComponents.Components[ComponentIndex].Indices;

				TSet<int32> ComponentElementsToMove;
				ComponentElementsToMove.Reserve(ComponentTids.Num() * 3);
				for (int Tid : ComponentTids)
				{
					FIndex3i Elements = UseUVLayer->GetTriangle(Tid);
					ComponentElementsToMove.Add(Elements[0]);
					ComponentElementsToMove.Add(Elements[1]);
					ComponentElementsToMove.Add(Elements[2]);
				}

				ScaleMeshSubRegionByDensity(UseUVLayer, ComponentTids, ComponentElementsToMove, TextureResolution);
			}
		}

	}


}

void FUVEditorTexelDensityOp::ExecutePacker(FDynamicMeshUVPacker& Packer)
{
	if (TexelDensityMode == EUVTexelDensityOpModes::Normalize)
	{
		Packer.bScaleIslandsByWorldSpaceTexelRatio = true;
		if (Packer.StandardPack() == false)
		{
			// failed... what to do?
			return;
		}
	}
}

TUniquePtr<FDynamicMeshOperator> UUVTexelDensityOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FUVEditorTexelDensityOp> Op = MakeUnique<FUVEditorTexelDensityOp>();

	Op->OriginalMesh = OriginalMesh;

	switch (Settings->TexelDensityMode)
	{
	case ETexelDensityToolMode::ApplyToIslands:
		Op->TexelDensityMode = EUVTexelDensityOpModes::ScaleIslands;
		break;
	case ETexelDensityToolMode::ApplyToWhole:
		Op->TexelDensityMode = EUVTexelDensityOpModes::ScaleGlobal;
		break;
	case ETexelDensityToolMode::Normalize:
		Op->TexelDensityMode = EUVTexelDensityOpModes::Normalize;
		break;
	default:
		ensure(false);
		break;
	}

	Op->TextureResolution = Settings->TextureResolution;
	Op->TargetWorldSpaceMeasurement = Settings->TargetWorldUnits;
	Op->TargetPixelCountMeasurement = Settings->TargetPixelCount;

	Op->UVLayerIndex = GetSelectedUVChannel();
	Op->TextureResolution = Settings->TextureResolution;
	Op->SetTransform(TargetTransform);
	Op->bMaintainOriginatingUDIM = Settings->bEnableUDIMLayout;
	Op->Selection = Selection;
	Op->TextureResolutionPerUDIM = TextureResolutionPerUDIM;

	return Op;
}
