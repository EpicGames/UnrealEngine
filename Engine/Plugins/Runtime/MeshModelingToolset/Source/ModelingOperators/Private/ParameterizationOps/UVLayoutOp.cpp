// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizationOps/UVLayoutOp.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Parameterization/MeshUVPacking.h"
#include "Properties/UVLayoutProperties.h"
#include "Selections/MeshConnectedComponents.h"
#include "Parameterization/MeshUDIMClassifier.h"
#include "UDIMUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVLayoutOp)

using namespace UE::Geometry;

namespace UVLayoutOpLocals
{
	FVector2f ExternalUVToInternalUV(const FVector2f& UV)
	{
		return FVector2f(UV.X, 1 - UV.Y);
	}

	FVector2f InternalUVToExternalUV(const FVector2f& UV)
	{
		return FVector2f(UV.X, 1 - UV.Y);
	}
}


void FUVLayoutOp::SetTransform(const FTransformSRT3d& Transform) 
{
	ResultTransform = Transform;
}


void FUVLayoutOp::CalculateResult(FProgressCancel* Progress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UVLayoutOp_CalculateResult);
	
	if (Progress && Progress->Cancelled())
	{
		return;
	}
	ResultMesh->Copy(*OriginalMesh, true, true, true, true);

	if (!ensureMsgf(ResultMesh->HasAttributes(), TEXT("Attributes not found on mesh? Conversion should always create them, so this operator should not need to do so.")))
	{
		ResultMesh->EnableAttributes();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}


	int UVLayerInput = UVLayerIndex;
	FDynamicMeshUVOverlay* UseUVLayer = ResultMesh->Attributes()->GetUVLayer(UVLayerInput);
	

	bool bWillRepackIslands = (UVLayoutMode != EUVLayoutOpLayoutModes::TransformOnly);

	// split bowties so that we can process islands independently
	if (bWillRepackIslands || bAlwaysSplitBowties)
	{
		UseUVLayer->SplitBowties();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (bMaintainOriginatingUDIM)
	{		
		TOptional<TArray<int32>> SelectionArray;
		if (Selection.IsSet())
		{
			SelectionArray = Selection.GetValue().Array();
		}
		FDynamicMeshUDIMClassifier TileClassifier(UseUVLayer, SelectionArray);

		TArray<FVector2i> Tiles = TileClassifier.ActiveTiles();

		for (FVector2i TileIndex : Tiles)
		{
			TUniquePtr<TArray<int32>> TileTids;
			TileTids = MakeUnique<TArray<int32>>(TileClassifier.TidsForTile(TileIndex));
			float TileResolution = TextureResolution;
			if (TextureResolutionPerUDIM.IsSet()
				&& TileIndex.X >= 0 && TileIndex.X <= UE::TextureUtilitiesCommon::UDIM_Max_X
				&& TileIndex.Y >= 0 && TileIndex.Y <= UE::TextureUtilitiesCommon::UDIM_Max_Y)
			{
				const int32 TileID = UE::TextureUtilitiesCommon::GetUDIMIndex(TileIndex.X, TileIndex.Y);
				TileResolution = TextureResolutionPerUDIM.GetValue().FindOrAdd(TileID, this->TextureResolution);
			}

			// Do this first, so we don't need to keep the TileTids around after moving it into the packer.
			TSet<int32> ElementsToMove;
			ElementsToMove.Reserve(TileTids->Num() * 3);
			for (int Tid : *TileTids)
			{
				FIndex3i Elements = UseUVLayer->GetTriangle(Tid);
				ElementsToMove.Add(Elements[0]);
				ElementsToMove.Add(Elements[1]);
				ElementsToMove.Add(Elements[2]);
			}

			// TODO: There is a second connected components call inside the packer that might be unnessessary. Could be a future optimization.
			FDynamicMeshUVPacker Packer(UseUVLayer, MoveTemp(TileTids) );
			Packer.TextureResolution = TileResolution;					
			ExecutePacker(Packer);
			if (Progress && Progress->Cancelled())
			{
				return;
			}

			for (int32 Element : ElementsToMove)
			{
				FVector2f UV = UVLayoutOpLocals::InternalUVToExternalUV(UseUVLayer->GetElement(Element));
				UV = UV + FVector2f(TileIndex);
				UseUVLayer->SetElement(Element, UVLayoutOpLocals::ExternalUVToInternalUV(UV));
			}			
		}
	}
	else
	{
		TUniquePtr<TArray<int32>> TidsToLayout = nullptr;
		if (Selection.IsSet())
		{
			TidsToLayout = MakeUnique<TArray<int32>>();
			TSet<int32>& SelectionSet = Selection.GetValue();
			TidsToLayout->Reserve(SelectionSet.Num());
			for(int Tid : SelectionSet)
			{
				if (OriginalMesh->IsTriangle(Tid))
				{
					TidsToLayout->Add(Tid);
				}
			}
		}
		
		FDynamicMeshUVPacker Packer(UseUVLayer, MoveTemp(TidsToLayout) );
		Packer.TextureResolution = this->TextureResolution;
		ExecutePacker(Packer);
		if (Progress && Progress->Cancelled())
		{
			return;
		}
	}

	if (UVScaleFactor != 1.0 || UVTranslation != FVector2f::Zero() )
	{
		for (int ElementID : UseUVLayer->ElementIndicesItr())
		{
			FVector2f UV = UVLayoutOpLocals::InternalUVToExternalUV(UseUVLayer->GetElement(ElementID));
			UV = (UV * UVScaleFactor) + UVTranslation;
			UseUVLayer->SetElement(ElementID, UVLayoutOpLocals::ExternalUVToInternalUV(UV));
		}
	}

}

void FUVLayoutOp::ExecutePacker(FDynamicMeshUVPacker& Packer)
{	
	Packer.GutterSize = this->GutterSize;
	Packer.bAllowFlips = this->bAllowFlips;
	Packer.bPreserveScale = this->bPreserveScale;
	Packer.bPreserveRotation = this->bPreserveRotation;

	if (UVLayoutMode == EUVLayoutOpLayoutModes::RepackToUnitRect)
	{
		if (Packer.StandardPack() == false)
		{
			// failed... what to do?
			return;
		}
	}
	else if (UVLayoutMode == EUVLayoutOpLayoutModes::StackInUnitRect)
	{
		if (Packer.StackPack() == false)
		{
			// failed... what to do?
			return;
		}
	}
	else if (UVLayoutMode == EUVLayoutOpLayoutModes::Normalize)
	{
		Packer.bScaleIslandsByWorldSpaceTexelRatio = true;
		if (Packer.StandardPack() == false)
		{
			// failed... what to do?
			return;
		}
	}
}

TUniquePtr<FDynamicMeshOperator> UUVLayoutOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FUVLayoutOp> Op = MakeUnique<FUVLayoutOp>();

	Op->OriginalMesh = OriginalMesh;

	switch (Settings->LayoutType)
	{
	case EUVLayoutType::Transform:
		Op->UVLayoutMode = EUVLayoutOpLayoutModes::TransformOnly;
		break;
	case EUVLayoutType::Stack:
		Op->UVLayoutMode = EUVLayoutOpLayoutModes::StackInUnitRect;
		break;
	case EUVLayoutType::Repack:
		Op->UVLayoutMode = EUVLayoutOpLayoutModes::RepackToUnitRect;
		break;
	case EUVLayoutType::Normalize:
		Op->UVLayoutMode = EUVLayoutOpLayoutModes::Normalize;
		break;
	}

	Op->UVLayerIndex = GetSelectedUVChannel();
	Op->TextureResolution = Settings->TextureResolution;
	Op->bPreserveScale = Settings->bPreserveScale;
	Op->bPreserveRotation = Settings->bPreserveRotation;	
	Op->bAllowFlips = Settings->bAllowFlips;
	Op->UVScaleFactor = Settings->Scale;
	Op->UVTranslation = FVector2f(Settings->Translation);
	Op->SetTransform(TargetTransform);
	Op->bMaintainOriginatingUDIM = Settings->bEnableUDIMLayout;
	Op->Selection = Selection;
	Op->TextureResolutionPerUDIM = TextureResolutionPerUDIM;

	return Op;
}
