// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizationOps/UVProjectionOp.h"

#include "DynamicMesh/MeshNormals.h"
#include "DynamicMeshEditor.h"
#include "Parameterization/DynamicMeshUVEditor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UVProjectionOp)

using namespace UE::Geometry;


void FUVProjectionOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}
	*ResultMesh = *OriginalMesh;
	if (!ensureMsgf(ResultMesh->HasAttributes(), TEXT("Attributes not found on mesh? Conversion should always create them, so this operator should not need to do so.")))
	{
		ResultMesh->EnableAttributes();
	}
	
	if (ProjectionMethod == EUVProjectionMethod::Plane)
	{
		CalculateResult_Plane(Progress);
	}
	if (ProjectionMethod == EUVProjectionMethod::ExpMap)
	{
		CalculateResult_ExpMap(Progress);
	}
	else if (ProjectionMethod == EUVProjectionMethod::Box)
	{
		CalculateResult_Box(Progress);
	}
	else if (ProjectionMethod == EUVProjectionMethod::Cylinder)
	{
		CalculateResult_Cylinder(Progress);
	}
	else
	{
		return;
	}
}


static void ApplyUVTransforms(
	FDynamicMeshUVEditor& UVEditor, const TArray<int32>& UVElements, 
	FVector2f UVOrigin,
	float UVRotationAngleDeg,
	FVector2f UVScale,
	FVector2f UVTranslate)
{
	FMatrix2f RotationMatrix = FMatrix2f::RotationDeg(UVRotationAngleDeg);
	UVEditor.TransformUVElements(UVElements, [&](const FVector2f& UV)
	{
		FVector2f UVScaleRotate = (RotationMatrix * UV) * UVScale;
		return (UVScaleRotate + UVOrigin) + UVTranslate;
	});
}


void FUVProjectionOp::CalculateResult_Box(FProgressCancel* Progress)
{
	FDynamicMeshUVEditor UVEditor(ResultMesh.Get(), UseUVLayer, true);

	FUVEditResult EditResult;
	if (!TriangleROI || TriangleROI->Num() == 0)
	{
		TArray<int32> Triangles;
		for (int32 tid : ResultMesh->TriangleIndicesItr())
		{
			Triangles.Add(tid);
		}
		UVEditor.SetTriangleUVsFromBoxProjection(Triangles, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionBox.Frame, 2*ProjectionBox.Extents, MinRegionSize, &EditResult);
	}
	else
	{
		UVEditor.SetTriangleUVsFromBoxProjection(*TriangleROI, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionBox.Frame, 2*ProjectionBox.Extents, MinRegionSize, &EditResult);
		//FCompactMaps CompactMaps;
		//UVEditor.GetOverlay()->CompactInPlace(CompactMaps);
	}

	ApplyUVTransforms(UVEditor, EditResult.NewUVElements, UVOrigin, UVRotationAngleDeg, UVScale, UVTranslate);
}


void FUVProjectionOp::CalculateResult_Plane(FProgressCancel* Progress)
{
	FFrame3d ProjectionFrame(ProjectionBox.Frame);
	FVector2d Dimensions(2 * ProjectionBox.Extents.X, 2 * ProjectionBox.Extents.Y);

	FDynamicMeshUVEditor UVEditor(ResultMesh.Get(), UseUVLayer, true);

	FUVEditResult EditResult;
	if (!TriangleROI || TriangleROI->Num() == 0)
	{
		TArray<int32> Triangles;
		for (int32 tid : ResultMesh->TriangleIndicesItr())
		{
			Triangles.Add(tid);
		}
		UVEditor.SetTriangleUVsFromPlanarProjection(Triangles, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionFrame, Dimensions, &EditResult);
	}
	else
	{
		UVEditor.SetTriangleUVsFromPlanarProjection(*TriangleROI, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionFrame, Dimensions, &EditResult);
		//FCompactMaps CompactMaps;
		//UVEditor.GetOverlay()->CompactInPlace(CompactMaps);
	}

	ApplyUVTransforms(UVEditor, EditResult.NewUVElements, UVOrigin, UVRotationAngleDeg, UVScale, UVTranslate);
}


void FUVProjectionOp::CalculateResult_ExpMap(FProgressCancel* Progress)
{
	FFrame3d ProjectionFrame(ProjectionBox.Frame);
	FVector2d Dimensions(2 * ProjectionBox.Extents.X, 2 * ProjectionBox.Extents.Y);

	FDynamicMeshUVEditor UVEditor(ResultMesh.Get(), UseUVLayer, true);

	FUVEditResult EditResult;
	if (!TriangleROI || TriangleROI->Num() == 0)
	{
		TArray<int32> Triangles;
		for (int32 tid : ResultMesh->TriangleIndicesItr())
		{
			Triangles.Add(tid);
		}
		UVEditor.SetTriangleUVsFromExpMap(Triangles, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionFrame, Dimensions, SmoothingRounds, SmoothingAlpha, BlendWeight, &EditResult);
	}
	else
	{
		UVEditor.SetTriangleUVsFromExpMap(*TriangleROI, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionFrame, Dimensions, SmoothingRounds, SmoothingAlpha, BlendWeight, &EditResult);
		//FCompactMaps CompactMaps;
		//UVEditor.GetOverlay()->CompactInPlace(CompactMaps);
	}

	ApplyUVTransforms(UVEditor, EditResult.NewUVElements, UVOrigin, UVRotationAngleDeg, UVScale, UVTranslate);
}


void FUVProjectionOp::CalculateResult_Cylinder(FProgressCancel* Progress)
{
	FDynamicMeshUVEditor UVEditor(ResultMesh.Get(), UseUVLayer, true);

	FUVEditResult EditResult;
	if (!TriangleROI || TriangleROI->Num() == 0)
	{
		TArray<int32> Triangles;
		for (int32 tid : ResultMesh->TriangleIndicesItr())
		{
			Triangles.Add(tid);
		}
		UVEditor.SetTriangleUVsFromCylinderProjection(Triangles, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionBox.Frame, 2 * ProjectionBox.Extents, CylinderSplitAngle, &EditResult);
	}
	else
	{
		UVEditor.SetTriangleUVsFromCylinderProjection(*TriangleROI, [&](const FVector3d P) { return MeshToProjectionSpace.TransformPosition(P); },
			ProjectionBox.Frame, 2 * ProjectionBox.Extents, CylinderSplitAngle, &EditResult);
		//FCompactMaps CompactMaps;
		//UVEditor.GetOverlay()->CompactInPlace(CompactMaps);
	}

	ApplyUVTransforms(UVEditor, EditResult.NewUVElements, UVOrigin, UVRotationAngleDeg, UVScale, UVTranslate);
}

