// Copyright Epic Games, Inc. All Rights Reserved.

#include "PolyModelingOps/LinearExtrusionOp.h"
#include "Operations/OffsetMeshRegion.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/MeshNormals.h"
#include "Util/ProgressCancel.h"

using namespace UE::Geometry;


void FLinearExtrusionOp::CalculateResult(FProgressCancel* Progress)
{
	bool bMeshValid = OriginalMeshShared->AccessSharedObject([&](const FDynamicMesh3& Mesh)
	{
		*ResultMesh = Mesh;
	});

	if (bMeshValid)
	{
		CalculateResultInPlace(*ResultMesh, Progress);
	}
}



bool FLinearExtrusionOp::CalculateResultInPlace(FDynamicMesh3& EditMesh, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	FOffsetMeshRegion Extruder(&EditMesh);

	Extruder.Triangles = TriangleSelection;
	Extruder.NumSubdivisions = NumSubdivisions;

	if ( bInferGroupsFromNeighbours )
	{
		Extruder.LoopEdgesShouldHaveSameGroup = [this, &EditMesh](int32 EdgeID1, int32 EdgeID2) 
		{
			return FOffsetMeshRegion::EdgesSeparateSameGroupsAndAreColinearAtBorder(
				&EditMesh, EdgeID1, EdgeID2, bUseColinearityForSettingBorderGroups);
		};
	}
	else
	{
		Extruder.LoopEdgesShouldHaveSameGroup = [this, &EditMesh](int32 EdgeID1, int32 EdgeID2) { return true; };
	}
	Extruder.bGroupPerSubdivision = bNewGroupPerSubdivision;
	Extruder.bSingleGroupPerArea = !bRemapExtrudeGroups;
	Extruder.CreaseAngleThresholdDeg = CreaseAngleThresholdDeg;

	Extruder.UVScaleFactor = UVScaleFactor;
	Extruder.bUVIslandPerGroup = bUVIslandPerGroup;

	Extruder.SetMaterialID = SetMaterialID;
	Extruder.bInferMaterialID = bInferMaterialID;

	Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::Zero;
	Extruder.OffsetPositionFunc = [this](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID)
	{
		FVector3d RelativePos = StartFrame.ToFramePoint(Pos);
		if (this->RegionModifierMode == ESelectionShapeModifierMode::FlattenToPlane)
		{
			RelativePos.Z = 0;
		}
		RelativePos *= Scale;
		FVector3d NewPos = ToFrame.FromFramePoint(RelativePos);

		if (this->RegionModifierMode == ESelectionShapeModifierMode::RaycastToPlane)
		{
			FVector3d Direction = Normalized(ToFrame.Origin - StartFrame.Origin);
			if (ToFrame.RayPlaneIntersection(Pos, Direction, 2, NewPos) == false)
			{
				if (ToFrame.RayPlaneIntersection(Pos, -Direction, 2, NewPos) == false)
				{
					NewPos = Pos;
				}
			}
			FRay3d Ray(Pos, Direction);
			double RayT = Ray.GetParameter(NewPos);
			if (FMathd::Abs(RayT) > RaycastMaxDistance)
			{
				NewPos = Ray.PointAt(RaycastMaxDistance * FMathd::Sign(RayT));
			}
		}

		return NewPos;
	};

	Extruder.bIsPositiveOffset = (ToFrame.Origin - StartFrame.Origin).Dot(StartFrame.Z()) > 0;
	Extruder.bOffsetFullComponentsAsSolids = bShellsToSolids;

	Extruder.Apply();

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (bRecomputeNormals)
	{
		FMeshNormals::RecomputeOverlayTriNormals(EditMesh, Extruder.AllModifiedAndNewTriangles);
	}


	return true;
}

