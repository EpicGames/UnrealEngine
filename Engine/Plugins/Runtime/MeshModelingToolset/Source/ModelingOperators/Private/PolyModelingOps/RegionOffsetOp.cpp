// Copyright Epic Games, Inc. All Rights Reserved.

#include "PolyModelingOps/RegionOffsetOp.h"
#include "Operations/OffsetMeshRegion.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/MeshNormals.h"
#include "Util/ProgressCancel.h"

using namespace UE::Geometry;


void FRegionOffsetOp::CalculateResult(FProgressCancel* Progress)
{
	bool bMeshValid = OriginalMeshShared->AccessSharedObject([&](const FDynamicMesh3& Mesh)
	{
		*ResultMesh = Mesh;
	});

	if (bMeshValid)
	{
		CalculateResultInPlace(*ResultMesh, Progress);
	}
}



bool FRegionOffsetOp::CalculateResultInPlace(FDynamicMesh3& EditMesh, FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if ( FMathd::Abs(OffsetDistance) < FMathd::ZeroTolerance )
	{
		return true;
	}

	FOffsetMeshRegion Extruder(&EditMesh);

	Extruder.Triangles = TriangleSelection;
	Extruder.NumSubdivisions = NumSubdivisions;

	if ( bInferGroupsFromNeighbours )
	{
		Extruder.LoopEdgesShouldHaveSameGroup = [this, &EditMesh](int32 EdgeID1, int32 EdgeID2) 
		{
			return FOffsetMeshRegion::EdgesSeparateSameGroupsAndAreColinearAtBorder(
				&EditMesh, EdgeID1, EdgeID2, bUseColinearityForSettingBorderGroups);
		};
	}
	else
	{
		Extruder.LoopEdgesShouldHaveSameGroup = [this, &EditMesh](int32 EdgeID1, int32 EdgeID2) { return true; };
	}
	Extruder.bGroupPerSubdivision = bNewGroupPerSubdivision;
	Extruder.bSingleGroupPerArea = !bRemapExtrudeGroups;
	Extruder.CreaseAngleThresholdDeg = CreaseAngleThresholdDeg;

	Extruder.UVScaleFactor = UVScaleFactor;
	Extruder.bUVIslandPerGroup = bUVIslandPerGroup;

	Extruder.SetMaterialID = SetMaterialID;
	Extruder.bInferMaterialID = bInferMaterialID;

	Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::VertexNormal;
	if (OffsetMode == EOffsetComputationMode::FaceNormals)
	{
		Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAverage;
	} 
	else if (OffsetMode == EOffsetComputationMode::ApproximateConstantThickness)
	{
		Extruder.ExtrusionVectorType = FOffsetMeshRegion::EVertexExtrusionVectorType::SelectionTriNormalsAngleWeightedAdjusted;
	} 

	Extruder.OffsetPositionFunc = [this](const FVector3d& Pos, const FVector3d& VertexVector, int32 VertexID) {
		return Pos + OffsetDistance * VertexVector;
	};

	Extruder.bIsPositiveOffset = (OffsetDistance > 0);
	Extruder.bOffsetFullComponentsAsSolids = bShellsToSolids;

	Extruder.Apply();

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (bRecomputeNormals)
	{
		FMeshNormals::RecomputeOverlayTriNormals(EditMesh, Extruder.AllModifiedAndNewTriangles);
	}


	return true;
}

