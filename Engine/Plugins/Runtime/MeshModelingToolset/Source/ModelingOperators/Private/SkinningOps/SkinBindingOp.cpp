// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinningOps/SkinBindingOp.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "ReferenceSkeleton.h"


namespace UE::Geometry
{

void FSkinBindingOp::SetTransformHierarchyFromReferenceSkeleton(
	const FReferenceSkeleton& InRefSkeleton
	)
{
	// Only use non-virtual bones, since virtual bones cannot be bound to the skin.
	const TArray<FMeshBoneInfo>& BoneInfo = InRefSkeleton.GetRawRefBoneInfo();
	const TArray<FTransform>& BonePose = InRefSkeleton.GetRawRefBonePose();
		
	BoneTransformHierarchy.Reset(BoneInfo.Num());
		
	for (int32 Index = 0; Index < BoneInfo.Num(); Index++)
	{
		SkinBinding::FBonePoseInfo Info{ .LocalTransform = BonePose[Index], .Name = BoneInfo[Index].Name, .ParentIndex = BoneInfo[Index].ParentIndex };
		BoneTransformHierarchy.Add(Info);
	}
}


void FSkinBindingOp::CalculateResult(FProgressCancel* InProgress)
{
	if (InProgress && InProgress->Cancelled())
	{
		return;
	}

	ResultMesh->Copy(*OriginalMesh, true, true, true, true);
		
	if (InProgress && InProgress->Cancelled())
	{
		return;
	}

	SkinBinding::FBindSettings Settings
	{
		.Stiffness = Stiffness,
		.MaxInfluences = MaxInfluences,
		.VoxelResolution = VoxelResolution,
		.BindType = BindType,
	};
	SkinBinding::CreateSkinWeights(*ResultMesh, BoneTransformHierarchy, ProfileName, Settings);
}


} // namespace UE::Geometry
