// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothingOps/CotanSmoothingOp.h"
#include "Solvers/MeshSmoothing.h"
#include "Solvers/ConstrainedMeshSmoother.h"
#include "DynamicMesh/MeshNormals.h"

using namespace UE::Geometry;

FCotanSmoothingOp::FCotanSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn) :
	FSmoothingOpBase(Mesh, OptionsIn)
{
}

FCotanSmoothingOp::FCotanSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn, const FDynamicSubmesh3& Submesh) :
	FSmoothingOpBase(Mesh, OptionsIn, Submesh)
{
}

double FCotanSmoothingOp::GetSmoothPower(int32 VertexID, bool bIsBoundary)
{
	double UsePower = SmoothOptions.SmoothPower;
	if (SmoothOptions.bUseWeightMap)
	{
		double t = FMathd::Clamp(SmoothOptions.WeightMap->GetValue(VertexID), 0.0, 1.0);
		UsePower = FMathd::Lerp(SmoothOptions.WeightMapMinMultiplier * UsePower, UsePower, t);
	}
	return UsePower;
}


void FCotanSmoothingOp::CalculateResult(FProgressCancel* Progress)
{
	// Update the values in the position buffer with smoothed positions.
	Smooth(Progress);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Copy the results back into the result mesh and update normals
	UpdateResultMesh();

}

void FCotanSmoothingOp::Smooth(FProgressCancel* Progress)
{
	ELaplacianWeightScheme UseScheme = ELaplacianWeightScheme::ClampedCotangent;
	if (SmoothOptions.bUniform)
	{
		UseScheme = ELaplacianWeightScheme::Uniform;
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Smoother = UE::MeshDeformation::ConstructConstrainedMeshSmoother(
		UseScheme, *ResultMesh);

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (SmoothOptions.SmoothPower < 0.0001)
	{
		for (int32 vid : ResultMesh->VertexIndicesItr())
		{
			PositionBuffer[vid] = ResultMesh->GetVertex(vid);
		}
	}
	else if (SmoothOptions.SmoothPower > 10000 )
	{
		Smoother->Deform(PositionBuffer);
	}
	else
	{
		for (int32 vid : ResultMesh->VertexIndicesItr())
		{
			FVector3d Position = ResultMesh->GetVertex(vid);
			double VertPower = GetSmoothPower(vid, false);
			double Weight = (VertPower < FMathf::ZeroTolerance) ? 999999.0 : (1.0 / VertPower);

			if (SmoothOptions.NormalOffset != 0)
			{
				check(SmoothOptions.BaseNormals.IsValid());
				Position += SmoothOptions.NormalOffset * SmoothOptions.BaseNormals->GetNormals()[vid];
			}

			Smoother->AddConstraint(vid, Weight, Position, false);
		}

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		Smoother->Deform(PositionBuffer);
	}

}
