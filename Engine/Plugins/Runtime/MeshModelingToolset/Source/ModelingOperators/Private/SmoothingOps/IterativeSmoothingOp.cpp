// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothingOps/IterativeSmoothingOp.h"
#include "Solvers/MeshSmoothing.h"

using namespace UE::Geometry;

FIterativeSmoothingOp::FIterativeSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn) :
	FSmoothingOpBase(Mesh, OptionsIn)
{
}

FIterativeSmoothingOp::FIterativeSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn,  const FDynamicSubmesh3& Submesh) :
	FSmoothingOpBase(Mesh, OptionsIn, Submesh)
{
}


void FIterativeSmoothingOp::CalculateResult(FProgressCancel* Progress)
{
	// Update the values in the position buffer with smoothed positions.
	if (SmoothOptions.bUseImplicit)
	{
		if (SmoothOptions.bUniform)
		{
			Smooth_MeanValue(Progress);
		}
		else
		{
			Smooth_Implicit_Cotan(Progress);
		}
	}
	else
	{
		Smooth_Forward(SmoothOptions.bUniform, Progress);
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Copy the results back into the result mesh and update normals
	UpdateResultMesh();

}


double FIterativeSmoothingOp::GetSmoothAlpha(int32 VertexID, bool bIsBoundary)
{
	double UseAlpha = (bIsBoundary) ? SmoothOptions.BoundarySmoothAlpha : SmoothOptions.SmoothAlpha;
	if (SmoothOptions.bUseWeightMap)
	{
		double t = FMathd::Clamp(SmoothOptions.WeightMap->GetValue(VertexID), 0.0, 1.0);
		UseAlpha = FMathd::Lerp(SmoothOptions.WeightMapMinMultiplier * UseAlpha, UseAlpha, t);
	}
	return UseAlpha;
}


void FIterativeSmoothingOp::Smooth_Implicit_Cotan(FProgressCancel* Progress)
{
	double Intensity = 1.;
	UE::MeshDeformation::ComputeSmoothing_BiHarmonic(ELaplacianWeightScheme::ClampedCotangent, 
		*ResultMesh, SmoothOptions.SmoothAlpha, Intensity, SmoothOptions.Iterations, PositionBuffer, Progress);
}


void FIterativeSmoothingOp::Smooth_MeanValue(FProgressCancel* Progress)
{
	double Intensity = 1.;
	UE::MeshDeformation::ComputeSmoothing_BiHarmonic(ELaplacianWeightScheme::MeanValue,
		*ResultMesh, SmoothOptions.SmoothAlpha, Intensity, SmoothOptions.Iterations, PositionBuffer, Progress);
}


void FIterativeSmoothingOp::Smooth_Forward(bool bUniform, FProgressCancel* Progress)
{
	UE::MeshDeformation::ComputeSmoothing_Forward(bUniform, SmoothOptions.bSmoothBoundary,
		*ResultMesh, [this](int VID, bool bBoundary) { return GetSmoothAlpha(VID, bBoundary); }, SmoothOptions.Iterations, PositionBuffer, Progress);
}
