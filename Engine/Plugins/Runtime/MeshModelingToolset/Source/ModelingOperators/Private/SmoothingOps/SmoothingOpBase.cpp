// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothingOps/SmoothingOpBase.h"

#include "DynamicSubmesh3.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"

using namespace UE::Geometry;

FSmoothingOpBase::FSmoothingOpBase(const FDynamicMesh3* Mesh, const FOptions& OptionsIn) :
	FDynamicMeshOperator(),
	SmoothOptions(OptionsIn)
{
	// deep copy the src mesh into the result mesh.  This ResultMesh will be directly updated by the smoothing.
	ResultMesh->Copy(*Mesh);
	
	PositionBuffer.SetNum(ResultMesh->MaxVertexID());

	for (int VID : ResultMesh->VertexIndicesItr())
	{
		PositionBuffer[VID] = ResultMesh->GetVertex(VID);
	}
}

FSmoothingOpBase::FSmoothingOpBase(const FDynamicMesh3* Mesh, const FOptions& OptionsIn, const FDynamicSubmesh3& Submesh) :
	FDynamicMeshOperator(),
	SmoothOptions(OptionsIn)
{
	// copy the src mesh into the SavedMesh so that positions of unselected vertices are already saved in ResultMesh
	SavedMesh = MakeUnique<FDynamicMesh3>(*Mesh);
	
	ResultMesh = MakeUnique<FDynamicMesh3>(Submesh.GetSubmesh());
	const int32 NumVertsToSmooth = ResultMesh->MaxVertexID(); // the number of vertices in the submesh, which is the region to be smoothed
	PositionBuffer.SetNum(NumVertsToSmooth);
	SmoothedToOriginalMap.SetNum(NumVertsToSmooth);

	for (int SubmeshVID : ResultMesh->VertexIndicesItr())
	{
		const FVector3d VertexPosition = ResultMesh->GetVertex(SubmeshVID);
		PositionBuffer[SubmeshVID] = VertexPosition;
		SmoothedToOriginalMap[SubmeshVID] = Submesh.MapVertexToBaseMesh(SubmeshVID);
	}
}

void FSmoothingOpBase::SetTransform(const FTransformSRT3d& XForm)
{
	ResultTransform = XForm;
}


void FSmoothingOpBase::UpdateResultMesh()
{
	// if SavedMesh was populated, we are operating on a submesh
	if (SavedMesh != nullptr)
	{
		for (int32 VID : SavedMesh->VertexIndicesItr())
		{
			// if vertex is in submesh, it was smoothed and has new position
			if (SmoothedToOriginalMap.Contains(VID))
			{
				int32 SubmeshIndex = SmoothedToOriginalMap.Find(VID);
				const FVector3d Pos = PositionBuffer[SubmeshIndex];
				SavedMesh->SetVertex(VID, Pos);
			}
			// else, can keep vertex's original position as initially stored in SavedMesh, no need to reassign
		}
		ResultMesh = MoveTemp(SavedMesh);
	}
	// otherwise, the entire mesh is smoothed
	else
	{
		// move all the vertices to their new location
		for (int32 VID : ResultMesh->VertexIndicesItr())
		{
			const FVector3d Pos = PositionBuffer[VID];
			ResultMesh->SetVertex(VID, Pos);
		}
	}

	// recalculate normals
	if (ResultMesh->HasAttributes())
	{
		FMeshNormals Normals(ResultMesh.Get());
		FDynamicMeshNormalOverlay* NormalOverlay = ResultMesh->Attributes()->PrimaryNormals();
		Normals.RecomputeOverlayNormals(NormalOverlay);
		Normals.CopyToOverlay(NormalOverlay);
	}
	else
	{
		FMeshNormals::QuickComputeVertexNormals(*ResultMesh);
	}
}
