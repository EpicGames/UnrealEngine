// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingTaskTypes.h"


//
// @todo nothing in this file is specific to modeling operations...these templates are general-purpose?
// 

namespace UE
{
namespace Geometry
{

/**
 * TModelingOpTask is an FAbortableBackgroundTask that executes a modeling operator of template type OpType.
 * OpType must implement a function with signature void CalculateResult(FProgressCancel*)
 * 
 * After work completes, ExtractOperator() can be used to recover the internal OpType instance, to get access to the completed work.
 * 
 * See TBackgroundModelingComputeSource for example usage (however this class can be used by itself)
 */
template<typename OpType>
class TModelingOpTask : public FAbortableBackgroundTask
{
	friend class FAsyncTask<TModelingOpTask<OpType>>;

public:

	TModelingOpTask(TUniquePtr<OpType> OperatorIn) :
		Operator(MoveTemp(OperatorIn))
	{}

	/**
	 * @return the contained computation Operator
	 */
	TUniquePtr<OpType> ExtractOperator()
	{
		return MoveTemp(Operator);
	}

protected:
	TUniquePtr<OpType> Operator;

	// 	FAbortableBackgroundTask API
	void DoWork()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ModelingOpTask_DoWork);
		
		if (Operator)
		{
			Operator->CalculateResult(GetProgress());
		}
	}

	// FAsyncTask framework required function
	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(TModelingOpTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};






/**
 * This status is returned by TBackgroundModelingComputeSource to indicate what 
 * state a background computation is in
 */
enum class EBackgroundComputeTaskStatus
{
	/** Computation of a result has finished and is waiting to be returned */
	ValidResultAvailable,

	/**
	 * We have a result available, but a recompute has been requested. The status is not yet 
	 * InProgress only because there is a delay of CancelActiveOpDelaySeconds between the
	 * request and operation restart, and we have not yet restarted.
	 */
	 DirtyResultAvailable,

	/** Last active computation was canceled and nothing new has happend yet*/
	Aborted,
	/** Computation is currently running */
	InProgress,
	/** Not running active computation, and last result has already been returned, so no new results to report */
	NotComputing
};




/**
 * TBackgroundModelingComputeSource is a container that can be used to repeatedly execute
 * a background computation. The assumption is that this background computation may need to
 * be canceled and restarted, ie if input parameters change due to user input/actions.
 * 
 * Clients of the template must provide an OpType, which is the operation to execute,
 * and a OpTypeFactory, which creates OpType instances on demand. The APIs that must
 * be provided in these types are minimal:
 *   OpTypeFactory:
 *      - TUniquePtr<OpType> MakeNewOperator()
 *   OpType:
 *      - void CalculateResult(FProgressCancel*)
 * 
 * The Client cancels the active computation and spawns a new one by calling NotifyActiveComputeInvalidated(). 
 * This does not immediately terminate the computation, it waits for a delay of .CancelActiveOpDelaySeconds
 * for this active compute to finish. This both (1) allows for partial updates to appear at UI levels
 * if the compute is fast enough and (2) avoids constantly respawning new computes as the user (for example)
 * drags a slider parameter.
 * 
 * However as a result of this delay the Client must Tick() this class regularly.
 * 
 * CheckStatus() can be used to determine if the computation has finished, in which case
 * a new result is available. ExtractResult() will return this result.
 * 
 * Note that a cancelled computation does not necessarily immediately terminate even after the timeout.
 * This requires that the OpType implementation test the provided ProgressCancel instance frequently.
 * When the Operator is "cancelled" it is moved to a separate task that waits for the owning FAsyncTask
 * to finish and then deletes it (and the contained Operator) 
 */
template<typename OpType, typename OpTypeFactory>
class TBackgroundModelingComputeSource
{
protected:
	OpTypeFactory* OperatorSource = nullptr;
	FAsyncTaskExecuterWithAbort<TModelingOpTask<OpType>>* ActiveBackgroundTask = nullptr;

	enum class EBackgroundComputeTaskState
	{
		NotActive = 0,
		ComputingResult = 1,
		WaitingToCancel = 2
	};

	// internal state flag
	EBackgroundComputeTaskState TaskState;

	double AccumTime = 0;
	double LastStartTime = 0;
	mutable double LastEndTime = 0; // mutable because it must be set in CheckStatus, which is const
	double LastInvalidateTime = 0;

	TSharedPtr<std::atomic<int>> ActiveTaskCount;

	void StartNewCompute();

public:
	TBackgroundModelingComputeSource(OpTypeFactory* OperatorSourceIn)
		: OperatorSource(OperatorSourceIn)
	{
		TaskState = EBackgroundComputeTaskState::NotActive;
		ActiveTaskCount = MakeShared<std::atomic<int>>(0);
	}

	~TBackgroundModelingComputeSource() = default;

	/**
	 * Tick the active computation. Client must call this frequently with valid DeltaTime
	 * parameter, as cancellation/restart cycles are based on time delay specified by CancelActiveOpDelaySeconds
	 */
	void Tick(float DeltaTime);

	/**
	 * Cancel the active computation immediately and do not start a new one
	 */
	void CancelActiveCompute();

	/**
	 * Cancel the active computation if one is running, after a delay of CancelActiveOpDelaySeconds.
	 * Then start a new one.
	 */
	void NotifyActiveComputeInvalidated();

	struct FStatus
	{
		EBackgroundComputeTaskStatus TaskStatus = EBackgroundComputeTaskStatus::NotComputing;

		// if TaskStatus == ValidResultAvailable then this is the time spent computing the (valid) result
		// if TaskStatus == DirtyResultAvailable then this is the time spent computing the (dirty) result
		// if TaskStatus == Aborted              then this is the time spent computing the result before the task was aborted
		// if TaskStatus == InProgress           then this is the time spent computing the result so far
		// if TaskStatus == NotComputing         then this is the time spent not computing anything
		double ElapsedTime = -1;
	};

	/**
	 * Return status of the active background computation.
	 */
	FStatus CheckStatus() const;

	// If waiting for background tasks, returns true, and sets NumTasks to the number of active tasks
	bool IsWaitingForBackgroundTasks(int32& NumTasks) const
	{
		if (bWaitingForActiveTasksToFinish)
		{
			NumTasks = *ActiveTaskCount;
			return NumTasks >= MaxActiveTaskCount;
		}
		return false;
	}

	/**
	 * @return The last computed Operator. This may only be called once, the caller then owns the Operator.
	 */
	TUniquePtr<OpType> ExtractResult();

	/**
	 * @return duration in seconds of current computation
	 */
	double GetElapsedComputeTime() const {
		FStatus Status = CheckStatus();
		if (Status.TaskStatus == EBackgroundComputeTaskStatus::InProgress)
		{
			return Status.ElapsedTime;
		}
		return 0.;
	}

public:
	/** Default wait delay for cancel/restart cycle */
	double CancelActiveOpDelaySeconds = 0.5;
	/** Maximum number of active tasks to allow to run in the background until we wait to launch more */
	int32 MaxActiveTaskCount = 5;

private:
	bool bWaitingForActiveTasksToFinish = false;
};




//
// TBackgroundModelingComputeSource template implementation
// 



template<typename OpType, typename OpTypeFactory>
void TBackgroundModelingComputeSource<OpType, OpTypeFactory>::Tick(float DeltaTime)
{
	AccumTime += (double)DeltaTime;

	if (TaskState == EBackgroundComputeTaskState::WaitingToCancel)
	{
		if ((AccumTime - LastInvalidateTime) > CancelActiveOpDelaySeconds)
		{
			CancelActiveCompute();

			int Active = *ActiveTaskCount;
			if (Active < MaxActiveTaskCount)
			{
				bWaitingForActiveTasksToFinish = false;
				StartNewCompute();
			}
			else
			{
				bWaitingForActiveTasksToFinish = true;
				// failed to start new task, return to 'waiting to cancel' state
				TaskState = EBackgroundComputeTaskState::WaitingToCancel;
			}
		}
	}
}



template<typename OpType, typename OpTypeFactory>
void TBackgroundModelingComputeSource<OpType, OpTypeFactory>::CancelActiveCompute()
{
	if (ActiveBackgroundTask != nullptr)
	{
		ActiveBackgroundTask->CancelAndDelete();
		ActiveBackgroundTask = nullptr;
	}
	TaskState = EBackgroundComputeTaskState::NotActive;
}

template<typename OpType, typename OpTypeFactory>
void TBackgroundModelingComputeSource<OpType, OpTypeFactory>::StartNewCompute()
{
	check(ActiveBackgroundTask == nullptr);

	(*ActiveTaskCount)++;
	TUniquePtr<OpType> NewOp = OperatorSource->MakeNewOperator();
	ActiveBackgroundTask = new FAsyncTaskExecuterWithAbort<TModelingOpTask<OpType> >(MoveTemp(NewOp));
	ActiveBackgroundTask->TaskCounter = ActiveTaskCount;
	ActiveBackgroundTask->StartBackgroundTask();

	LastStartTime = AccumTime;
	TaskState = EBackgroundComputeTaskState::ComputingResult;
}


template<typename OpType, typename OpTypeFactory>
void TBackgroundModelingComputeSource<OpType, OpTypeFactory>::NotifyActiveComputeInvalidated()
{
	// switch to waiting-to-cancel state
	if (TaskState == EBackgroundComputeTaskState::ComputingResult)
	{
		LastInvalidateTime = AccumTime;
		TaskState = EBackgroundComputeTaskState::WaitingToCancel;
	}

	// if compute is not actively running, start a new one
	if (TaskState == EBackgroundComputeTaskState::NotActive)
	{
		StartNewCompute();
		return;
	}

}



template<typename OpType, typename OpTypeFactory>
typename TBackgroundModelingComputeSource<OpType, OpTypeFactory>::FStatus
TBackgroundModelingComputeSource<OpType, OpTypeFactory>::CheckStatus() const
{
	FStatus Status;

	if (bWaitingForActiveTasksToFinish)
	{
		Status.TaskStatus = EBackgroundComputeTaskStatus::InProgress;
		Status.ElapsedTime = AccumTime - LastStartTime;
	}
	else if (ActiveBackgroundTask == nullptr)
	{
		Status.TaskStatus = EBackgroundComputeTaskStatus::NotComputing;
		Status.ElapsedTime = AccumTime - LastInvalidateTime;
	}
	else if (!ActiveBackgroundTask->IsDone())
	{
		Status.TaskStatus = EBackgroundComputeTaskStatus::InProgress;
		Status.ElapsedTime = AccumTime - LastStartTime;
	}
	else if (ActiveBackgroundTask->GetTask().IsAborted())
	{
		Status.TaskStatus = EBackgroundComputeTaskStatus::Aborted;
		Status.ElapsedTime = LastInvalidateTime - LastStartTime;
	}
	else
	{
		if (TaskState == EBackgroundComputeTaskState::ComputingResult)
		{
			LastEndTime = AccumTime;
		}

		// Task is done and not aborted, but we may be waiting to start a new one
		Status.TaskStatus = (TaskState == EBackgroundComputeTaskState::WaitingToCancel ?
			EBackgroundComputeTaskStatus::DirtyResultAvailable :
			EBackgroundComputeTaskStatus::ValidResultAvailable);
		Status.ElapsedTime = LastEndTime - LastStartTime;
	}

	return Status;
}



template<typename OpType, typename OpTypeFactory>
TUniquePtr<OpType> TBackgroundModelingComputeSource<OpType, OpTypeFactory>::ExtractResult()
{
	check(ActiveBackgroundTask != nullptr && ActiveBackgroundTask->IsDone());
	TUniquePtr<OpType> Result = ActiveBackgroundTask->GetTask().ExtractOperator();

	delete ActiveBackgroundTask;
	ActiveBackgroundTask = nullptr;

	//  don't over-write a waiting-to-cancel state
	if (TaskState != EBackgroundComputeTaskState::WaitingToCancel)
	{
		TaskState = EBackgroundComputeTaskState::NotActive;
	}
	else
	{
		StartNewCompute();
	}

	return Result;
}



} // end namespace UE::Geometry
} // end namespace UE
