// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "VectorTypes.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FDynamicMesh3;

class FMeshBoundaryCache
{
public:
	TArray<bool> bIsBoundary;
	TArray<int32> BoundaryVerts;
	UE_API void Calculate(const FDynamicMesh3& Mesh);
};


/**
 * Dynamic Mesh Operator that has a separate PositionBuffer that contains copy of input
 * vertex positions. These can be modified by operator and then UpdateResultMeshPositions() can
 * be copied to update ResultMesh. 
 *
 * UpdateResultMeshNormals() recalculates normals
 */
class FSimpleMeshProcessingBaseOp : public FDynamicMeshOperator
{
public:
	UE_API FSimpleMeshProcessingBaseOp(const FDynamicMesh3* Mesh);

	virtual ~FSimpleMeshProcessingBaseOp() = default;

	// set ability on protected transform.
	UE_API void SetTransform(const FTransformSRT3d& XForm);

	// copy the PositionBuffer locations back to the ResultMesh
	UE_API void UpdateResultMeshPositions();

	// recompute noramls of ResultMesh
	UE_API void UpdateResultMeshNormals();

protected:
	TArray<FVector3d> PositionBuffer;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
