// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FVoxelBaseOp : public FDynamicMeshOperator
{
public:
	virtual ~FVoxelBaseOp() = default;

	// inputs
	double MinComponentVolume = 0.0, MinComponentArea = 0.0;

	bool bAutoSimplify = true;
	double SimplifyMaxErrorFactor = 1.0;

	int OutputVoxelCount = 1024;
	int InputVoxelCount = 1024;

	bool bRemoveInternalSurfaces = false;

	template<typename TransformType>
	static FVector3d GetAverageTranslation(TArrayView<const TransformType> Transforms)
	{
		if (Transforms.IsEmpty())
		{
			return FVector3d::ZeroVector;
		}

		FVector3d Avg(0, 0, 0);
		for (const FTransformSRT3d& Transform : Transforms)
		{
			Avg += Transform.GetTranslation();
		}
		return Avg / double(Transforms.Num());
	}

	UE_API virtual void PostProcessResult(FProgressCancel* Progress, double MeshCellSize);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
