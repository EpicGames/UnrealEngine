// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "Polygroups/PolygroupSet.h"

#include "EditNormalsOp.generated.h"

#define UE_API MODELINGOPERATORS_API


UENUM()
enum class ENormalCalculationMethod : uint8
{
	/** Use triangle area to weight how much a triangle's normal contributes its vertices' normals */
	AreaWeighted,
	/** Use the angle of a triangle at a vertex to weight how much that triangle's normal contributes to that vertex's normal */
	AngleWeighted,
	/** Multiply area and angle weights together for a combined weight controlling how much a triangle's normal contributes to its vertices' normals */
	AreaAngleWeighting
};


UENUM()
enum class ESplitNormalMethod : uint8
{
	/** Keep the existing split-normals structure on the mesh */
	UseExistingTopology,
	/** Recompute split-normals by grouping faces around each vertex based on an angle threshold */
	FaceNormalThreshold,
	/** Recompute split-normals by grouping faces around each vertex that share a face/polygroup */
	FaceGroupID,
	/** Set each triangle-vertex to have the face normal of that triangle's plane */
	PerTriangle,
	/** Set each vertex to have a fully shared normal, i.e. no split normals  */
	PerVertex
};

namespace UE
{
namespace Geometry
{

class FEditNormalsOp : public FDynamicMeshOperator
{
public:
	virtual ~FEditNormalsOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> MeshPolygroups;

	// These are indices into OriginalMesh. If both are non-empty only edit the corresponding elements
	// in the normal overlay otherwise operate on the whole normal overlay
	TSet<int> EditTriangles;
	TSet<int> EditVertices;

	bool bFixInconsistentNormals;
	bool bInvertNormals;
	bool bRecomputeNormals;
	ENormalCalculationMethod NormalCalculationMethod;
	ESplitNormalMethod SplitNormalMethod;
	bool bAllowSharpVertices;
	float NormalSplitThreshold;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

private:
	UE_API void CalculateResultWholeMesh(FProgressCancel* Progress);
	UE_API void CalculateResultSelection(FProgressCancel* Progress);
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
