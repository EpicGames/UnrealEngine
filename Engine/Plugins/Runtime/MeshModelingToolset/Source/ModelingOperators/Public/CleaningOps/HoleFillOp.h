// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelingOperators.h"
#include "EdgeLoop.h"
#include "Operations/SmoothHoleFiller.h"
#include "HoleFillOp.generated.h"

#define UE_API MODELINGOPERATORS_API

UENUM()
enum class EHoleFillOpFillType : uint8
{
	/** Fill with a fan of triangles connected to a new central vertex. */
	TriangleFan UMETA(DisplayName = "TriangleFan"),

	/** Incrementally triangulate the hole boundary without introducing new interior vertices. */
	PolygonEarClipping UMETA(DisplayName = "PolygonEarClipping"),

	/** Choose a best-fit plane, project the boundary vertices to the plane, and use 2D Delaunay triangulation. */
	Planar UMETA(DisplayName = "Planar"),

	/** Fill with a triangulation which attempts to minimize Gaussian curvature and not introduce new interior vertices. */
	Minimal UMETA(DisplayName = "Minimal"),

	/** Fill hole with a simple triangulation, then alternate between smoothing and remeshing. Optionally include the
	    triangles around the hole in the smoothing/remeshing. */
	Smooth UMETA(DisplayName = "Smooth")
};


namespace UE
{
namespace Geometry
{

struct FFillOptions
{
	/** Clean up triangles that have no neighbors */
	bool bRemoveIsolatedTriangles = false;

	/** Identify and quickly fill single-triangle holes */
	bool bQuickFillSmallHoles = false;
};

class FHoleFillOp : public FDynamicMeshOperator
{
public:

	// inputs
	
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;		// Ownership shared with Tool

	EHoleFillOpFillType FillType = EHoleFillOpFillType::Minimal;
	double MeshUVScaleFactor = 1.0;
	TArray<UE::Geometry::FEdgeLoop> Loops;

	FFillOptions FillOptions;
	UE::Geometry::FSmoothFillOptions SmoothFillOptions;

	// output
	TArray<int32> NewTriangles;
	int NumFailedLoops = 0;

	// FDynamicMeshOperator implementation
	UE_API void CalculateResult(FProgressCancel* Progress) override;

	UE_API bool FillSingleTriangleHole(const FEdgeLoop& Loop, int32& NewGroupID);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
