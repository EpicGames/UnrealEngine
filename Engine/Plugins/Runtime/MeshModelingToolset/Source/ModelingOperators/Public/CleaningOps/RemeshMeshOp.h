// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "MeshConstraints.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#include "RemeshMeshOp.generated.h"

#define UE_API MODELINGOPERATORS_API

/** Remeshing modes */
UENUM()
enum class ERemeshType : uint8
{
	/** One pass over the entire mesh, then remesh only changed edges */
	Standard = 0 UMETA(DisplayName = "Standard"),

	/** Multiple full passes over the entire mesh */
	FullPass = 1 UMETA(DisplayName = "Full Pass"),

	/** One pass over the entire mesh, then remesh only changed edges. Use Normal flow to align triangles with input.*/
	NormalFlow = 2 UMETA(DisplayName = "Normal Flow"),

};

/** Smoothing modes */
UENUM()
enum class ERemeshSmoothingType : uint8
{
	/** Uniform Smoothing */
	Uniform = 0 UMETA(DisplayName = "Uniform"),

	/** Cotangent Smoothing */
	Cotangent = 1 UMETA(DisplayName = "Shape Preserving"),

	/** Mean Value Smoothing */
	MeanValue = 2 UMETA(DisplayName = "Mixed")
};

namespace UE
{
namespace Geometry
{

class FRemesher;

class FRemeshMeshOp : public FDynamicMeshOperator
{
public:
	virtual ~FRemeshMeshOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial;

	ERemeshType RemeshType = ERemeshType::Standard;

	int RemeshIterations = 20;
	int MaxRemeshIterations = 20;
	double MinActiveEdgeFraction = 0.01;		// terminate remeshing if modified edge count in last pass is below this fraction of total edges
	int ExtraProjectionIterations = 5;
	int TriangleCountHint = 0;
	float SmoothingStrength = 0.25f;
	double TargetEdgeLength = 1.0f;
	ERemeshSmoothingType SmoothingType;
	bool bDiscardAttributes = false;
	// Note: if bDiscardAttributes and bNeedsWeightMaps are both true, the weight map attributes will be kept through remeshing (but other attributes will be discarded)
	bool bNeedsWeightMaps = false;
	bool bPreserveSharpEdges = true;
	bool bFlips = true;
	bool bSplits = true;
	bool bCollapses = true;
	bool bReproject = true;
	bool bPreventNormalFlips = true;
	bool bPreventTinyTriangles = true;
	bool bReprojectConstraints = false;
	double BoundaryCornerAngleThreshold = 45.0;

	// When true, result will have attributes object regardless of whether attributes 
	// were discarded or present initially.
	bool bResultMustHaveAttributesEnabled = false;
	EEdgeRefineFlags MeshBoundaryConstraint, GroupBoundaryConstraint, MaterialBoundaryConstraint;

	FDynamicMesh3* ProjectionTarget = nullptr;
	FDynamicMeshAABBTree3* ProjectionTargetSpatial = nullptr;

	FTransformSRT3d TargetMeshLocalToWorld = FTransformSRT3d::Identity();
	FTransformSRT3d ToolMeshLocalToWorld = FTransformSRT3d::Identity();
	bool bUseWorldSpace = false;
	bool bParallel = true;

	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomEdgeLengthScaleF;

	// Normal flow only:

	/// During each call to RemeshIteration, do this many passes of face-aligned projection
	int FaceProjectionPassesPerRemeshIteration = 1;

	/// drag on surface projection
	double SurfaceProjectionSpeed = 0.2;

	/// drag on normal alignment
	double NormalAlignmentSpeed = 0.2;

	/// Control whether or not we want to apply mesh smoothing in "free" areas that have not projected to target surface.
	/// This smoothing is on applied in the ExtraProjections Iterations
	bool bSmoothInFillAreas = true;

	/// This is used as a multiplier on MaxEdgeLength to determine when we identify points as being in "free" areas
	float FillAreaDistanceMultiplier = 0.25;

	/// This is used as a multiplier on the Remesher smoothing rate, applied to points identified as being in "free" areas
	float FillAreaSmoothMultiplier = 0.25;

	// End normal flow only


	/// Add arbitrary mesh constraints besides the boundary/seam constraints that are computed by this op
	UE_API void SetUserSpecifiedConstraints(const FMeshConstraints& Constraints);

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


public:
	static UE_API double CalculateTargetEdgeLength(const FDynamicMesh3* Mesh, int TargetTriCount, double PrecomputedMeshArea = 0.0);


protected:

	UE_API virtual TUniquePtr<FRemesher> CreateRemesher(ERemeshType Type, FDynamicMesh3* TargetMesh);

private:

	FMeshConstraints UserSpecifiedConstraints;

};

} // end namespace UE::Geometry
} // end namespace UE


#undef UE_API
