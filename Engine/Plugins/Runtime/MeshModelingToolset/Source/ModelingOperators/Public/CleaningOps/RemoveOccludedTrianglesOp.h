// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Operations/RemoveOccludedTriangles.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FRemoveOccludedTrianglesOp : public FDynamicMeshOperator
{
public:
	virtual ~FRemoveOccludedTrianglesOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	
	TArray<TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe>> OccluderTrees;
	TArray<TSharedPtr<UE::Geometry::TFastWindingTree<FDynamicMesh3>, ESPMode::ThreadSafe>> OccluderWindings;
	TArray<FTransformSRT3d> OccluderTransforms;

	TArray<FTransformSRT3d> MeshTransforms;

	UE::Geometry::EOcclusionTriangleSampling TriangleSamplingMethod =
		UE::Geometry::EOcclusionTriangleSampling::Centroids;

	// we nudge points out by this amount to try to counteract numerical issues
	double NormalOffset = FMathd::ZeroTolerance;

	// use this as winding isovalue for WindingNumber mode
	double WindingIsoValue = 0.5;

	// random rays to add beyond +/- major axes, for raycast sampling
	int AddRandomRays = 0;

	// add triangle samples per triangle (in addition to TriangleSamplingMethod)
	int AddTriangleSamples = 0;

	// once triangles to remove are identified, do iterations of boundary erosion, ie contract selection by boundary vertex one-rings
	int ShrinkRemoval = 0;

	double MinAreaConnectedComponent = 0;

	int MinTriCountConnectedComponent = 0;

	// if true, we will set triangle group IDs for occluded triangles, rather than deleting the triangles
	bool bSetTriangleGroupInsteadOfRemoving = false;

	// name of the group layer to use if we are setting triangle groups instead of removing occluded triangles
	FName ActiveGroupLayer;

	// if true, the ActiveGroupLayer is the name of the "default" layer, so we'll use the built-in group IDs
	bool bActiveGroupLayerIsDefault = true;

	UE::Geometry::EOcclusionCalculationMode InsideMode =
		UE::Geometry::EOcclusionCalculationMode::FastWindingNumber;


	// outputs, used when bSetTriangleGroupInsteadOfRemoving is true
	int CreatedGroupID = -1;
	int CreatedGroupLayerIndex = -1;


	UE_API void SetTransform(const FTransform& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
