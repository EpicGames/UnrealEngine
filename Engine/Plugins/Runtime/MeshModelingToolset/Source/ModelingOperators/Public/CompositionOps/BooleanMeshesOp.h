// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"


#include "BooleanMeshesOp.generated.h"

#define UE_API MODELINGOPERATORS_API


/** CSG operation types */
UENUM()
enum class ECSGOperation : uint8
{
	/** Subtract the second object from the first object */
	DifferenceAB = 0 UMETA(DisplayName = "Difference A - B"),

	/** Subtract the first object from the second object */
	DifferenceBA = 1 UMETA(DisplayName = "Difference B - A"),

	/** Intersection of the two objects, i.e. where both objects overlap */
	Intersect = 2 UMETA(DisplayName = "Intersection"),

	/** Union of the two objects, i.e. merger of both objects  */
	Union = 3 UMETA(DisplayName = "Union"),
};

UENUM()
enum class ETrimOperation : uint8
{
	/** Remove geometry from the first object using the second object */
	TrimA = 0 UMETA(DisplayName = "Trim A"),

	/** Remove geometry from the second object using the first object */
	TrimB = 1 UMETA(DisplayName = "Trim B"),
};


UENUM()
enum class ETrimSide : uint8
{
	RemoveInside = 0,
	RemoveOutside = 1,
};


namespace UE
{
namespace Geometry
{

class FBooleanMeshesOp : public FDynamicMeshOperator
{
public:
	virtual ~FBooleanMeshesOp() override = default;

	// inputs
	ECSGOperation CSGOperation;
	ETrimOperation TrimOperation;
	ETrimSide TrimSide;
	bool bTrimMode = false; // if true, do a trim operation instead of a boolean
	TArray<TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>> Meshes;
	TArray<FTransformSRT3d> Transforms; // 1:1 with Meshes
	bool bAttemptFixHoles = false;
	double WindingThreshold = 0.5;

	/** If true, try to do edge-collapses along cut edges to remove unnecessary edges inserted by cut */
	bool bTryCollapseExtraEdges = false;
	/** Angle threshold in degrees used for testing if two triangles should be considered coplanar, or two lines collinear */
	float TryCollapseExtraEdgesPlanarThresh = 0.01f;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

	// IDs of any newly-created boundary edges in the result mesh
	const TArray<int>& GetCreatedBoundaryEdges() const
	{
		return CreatedBoundaryEdges;
	}

private:
	TArray<int> CreatedBoundaryEdges;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
