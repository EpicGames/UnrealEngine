// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IndexTypes.h"
#include "ModelingOperators.h"
#include "OrientedBoxTypes.h"

#define UE_API MODELINGOPERATORS_API

class FProgressCancel;

namespace UE {
namespace Geometry {

/**
 * Operator meant to be used with UCubeGridTool that adds or subtracts a box to/from
 * an input mesh, with options to weld the box's corners to make ramps/corners
 */
class FCubeGridBooleanOp : public FDynamicMeshOperator
{

public:

	virtual ~FCubeGridBooleanOp() = default;

	// Inputs:

	// The mesh to add to or subtract from
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;
	FTransformSRT3d InputTransform = FTransformSRT3d::Identity();

	// Box from which to generate the second mesh
	FOrientedBox3d WorldBox;

	// Optional: information to weld corners along the Z axis to create ramps/pyramids/pyramid cutaways
	struct FCornerInfo
	{
		// "Base" refers to the 0-3 indexed corners in FOrientedBox3d, and the flags here are
		// true if the vertex is welded to its Z axis neighbor. The lower side on the Z axis 
		// is the enclosed area for the purposes of an operation. 
		bool WeldedAtBase[4] = { false, false, false, false };
	};
	TSharedPtr<FCornerInfo> CornerInfo;

	// Determines whether the box is added or subtracted from the mesh.
	bool bSubtract = false;

	// When true, mesh is constructed such that ResultTransform is InputTransform. When false,
	// ResultTransform is based off of the centroid of the result.
	bool bKeepInputTransform = false;

	// Only relevant when CornerInfo is used. When true, diagonal will generally prefer to 
	// lie flat across the non-planar top. This determines, for instance, whether a pulled
	// corner results in a pyramid with three faces (if true) or four (if false).
	bool bCrosswiseDiagonal = false;

	// The material ID to give to triangles in the operator mesh.
	int32 OpMeshMaterialID = 0;

	/** 
	 * For each face of the op box (indexed as in IndexUtil::BoxFaces), determines the orientation
	 * in which the UV's should be assigned, which can be imagined as number of counterclockwise rotations 
	 * of the face in the UV plane while keeping the corner labels the same.
	 * 
	 * For instance, the corner UV's for orientation 0 are (0,0), (width, 0), (width, height), (0, height),
	 * whereas for orientation 1 they are (height, 0), (height, width), (0, width), (0,0)
	 * 
	 * This is used to orient UV's in a uniform direction while extruding/pushing in different directions
	 * in the cube grid.
	 */
	TArray<int32, TFixedAllocator<6>> FaceUVOrientations;

	/**
	 * When setting UV's on the box, the component affected by height is adjusted by this offset. This is used
	 * to avoid "restarting" UV's when performing multiple operations with the same selection.
	 */
	double OpMeshHeightUVOffset = 0;

	float UVScale = 1;
	bool bWorldSpaceUVs = true;

	// Both Input and Output:
	// When not set to InvalidID, these will control the setting of the side groups in the operator
	// mesh. This allows for the sides to keep the same groups with each step when selection is unchanged.
	TArray<int32, TFixedAllocator<4>> OpMeshSideGroups = {
		IndexConstants::InvalidID, 
		IndexConstants::InvalidID, 
		IndexConstants::InvalidID, 
		IndexConstants::InvalidID
	};

	// Outputs:

	// Will be reset to a container storing tids whose positions or connectivity changed. Only filled
	// if bTrackChangedTids is true.
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> ChangedTids;
	bool bTrackChangedTids = false;

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};


}} // end namespace UE::Geometry

#undef UE_API
