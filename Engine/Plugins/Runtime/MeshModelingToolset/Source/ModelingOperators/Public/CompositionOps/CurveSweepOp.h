// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Generators/SweepGenerator.h" // EProfileSweepPolygonGrouping
#include "ModelingOperators.h"
#include "Util/ProgressCancel.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

/*
 * Operation for sweeping a profile curve along a sweep curve to create a mesh.
 */
class FCurveSweepOp : public FDynamicMeshOperator
{
public:
	enum class ECapFillMode
	{
		None,
		Delaunay,
		EarClipping,
		CenterFan
	};

	virtual ~FCurveSweepOp() = default;

	// Inputs
	TArray<FVector3d> ProfileCurve;
	TArray<FFrame3d> SweepCurve;
	TSet<int32> ProfileVerticesToWeld;

	// If true, the last profile curve point will be considered connected to the first.
	bool bProfileCurveIsClosed = false;

	// If true, the last sweep point will be considered connected to the first.
	bool bSweepCurveIsClosed = false;

	// Whether adjacent triangles should share averaged normals or have their own (to give sharpness)
	bool bSharpNormals = true;

	// When using sharp normals, the degree difference that adjacent triangles can have in their normals for
	// them to be considered "coplanar" and therefore share normals.
	double SharpNormalAngleTolerance = 0.1;

	// What kind of cap to create
	ECapFillMode CapFillMode = ECapFillMode::Delaunay;

	// Whether fully welded edges (welded vertex to welded vertex) in the profile curve should affect
	// the UV layout, since such edges don't generate triangles.
	bool bUVsSkipFullyWeldedEdges = true;

	// Generated UV's will be multiplied by these values.
	FVector2d UVScale = FVector2d(1, 1);

	// These values will be added to the generated UV's after applying UVScale.
	FVector2d UVOffset = FVector2d(0, 0);

	// If true, UVs are scaled to keep a consistent scale across differently sized geometry
	bool bUVScaleRelativeWorld = false;

	// When bUVScaleRelativeWorld is true, the size in world coordinates of 1 UV coordinate
	double UnitUVInWorldCoordinates = 100;

	EProfileSweepPolygonGrouping PolygonGroupingMode = EProfileSweepPolygonGrouping::PerFace;

	EProfileSweepQuadSplit QuadSplitMode = EProfileSweepQuadSplit::ShortestDiagonal;

	// When QuadSplitMode is ShortestDiagonal, biases one of the diagonals so that symmetric
	// quads are split uniformly. The tolerance is a proportion allowable difference.
	double DiagonalTolerance = 0.01;

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
