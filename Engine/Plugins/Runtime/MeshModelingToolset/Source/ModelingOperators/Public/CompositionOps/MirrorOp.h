// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingOperators.h"
#include "Util/ProgressCancel.h"
#include "Operations/MeshMirror.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FMirrorOp : public FDynamicMeshOperator
{
public:
	virtual ~FMirrorOp() = default;

	// Inputs
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	FVector3d LocalPlaneOrigin;
	FVector3d LocalPlaneNormal;

	/** Tolerance to use when bCropFirst or bWeldAlongPlane is true. */
	double PlaneTolerance = FMathf::ZeroTolerance * 10.0;;

	/** Whether to crop the result first along the plane. */
	bool bCropFirst = true;

	/** Whether to locally simplify the new edges created when cropping along the plane. Only relevant if bCropFirst is true. */
	bool bSimplifyAlongNewEdges = true;

	/** If true, the mirrored portion is appended to the original. If false, the result is just the mirrored portion. */
	bool bAppendToOriginal = true;

	/** Whether vertices on the mirror plane should be welded. Only relevant if bAppendToOriginal is true. */
	bool bWeldAlongPlane = true;
	
	/** The normal compute method for welded vertices along the mirror plane. */
	EMeshMirrorNormalMode WeldNormalMode = EMeshMirrorNormalMode::MirrorNormals;

	/** Whether, when welding, new bowtie vertex creation should be allowed. */
	bool bAllowBowtieVertexCreation = false;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
