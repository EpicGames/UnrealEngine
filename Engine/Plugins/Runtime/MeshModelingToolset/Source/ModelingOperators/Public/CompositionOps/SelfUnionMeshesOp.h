// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FSelfUnionMeshesOp : public FDynamicMeshOperator
{
public:
	virtual ~FSelfUnionMeshesOp() = default;

	// inputs
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> CombinedMesh;
	bool bAttemptFixHoles = false;
	double WindingNumberThreshold = .5;
	bool bTrimFlaps = false;

	/** If true, try to do edge-collapses along cut edges to remove unnecessary edges inserted by cut */
	bool bTryCollapseExtraEdges = false;
	/** Angle threshold in degrees used for testing if two triangles should be considered coplanar, or two lines collinear */
	float TryCollapseExtraEdgesPlanarThresh = 0.01f;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

	// IDs of any newly-created boundary edges in the result mesh
	TArray<int> GetCreatedBoundaryEdges() const
	{
		return CreatedBoundaryEdges;
	}

private:
	TArray<int> CreatedBoundaryEdges;
};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
