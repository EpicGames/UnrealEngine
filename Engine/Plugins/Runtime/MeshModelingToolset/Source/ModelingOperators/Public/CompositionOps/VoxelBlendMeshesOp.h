// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#include "BaseOps/VoxelBaseOp.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FVoxelBlendMeshesOp : public FVoxelBaseOp
{
public:
	virtual ~FVoxelBlendMeshesOp() = default;

	// inputs
	TArray<TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>> Meshes;
	TArray<FTransform> Transforms; // 1:1 with Meshes

	double BlendFalloff = 10;
	double BlendPower = 2;

	bool bVoxWrap = false;
	bool bRemoveInternalsAfterVoxWrap = false;
	double ThickenShells = 0.0;

	bool bSubtract = false;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
