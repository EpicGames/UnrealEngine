// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#include "BaseOps/VoxelBaseOp.h"

#include "VoxelMorphologyMeshesOp.generated.h"

#define UE_API MODELINGOPERATORS_API


/** Morphology operation types */
UENUM()
enum class EMorphologyOperation : uint8
{
	/** Expand the shapes outward */
	Dilate = 0,

	/** Shrink the shapes inward */
	Contract = 1,

	/** Dilate and then contract, to delete small negative features (sharp inner corners, small holes) */
	Close = 2,

	/** Contract and then dilate, to delete small positive features (sharp outer corners, small isolated pieces) */
	Open = 3

};

namespace UE
{
namespace Geometry
{

class FVoxelMorphologyMeshesOp : public FVoxelBaseOp
{
public:
	virtual ~FVoxelMorphologyMeshesOp() = default;

	// inputs
	TArray<TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>> Meshes;
	TArray<FTransformSRT3d> Transforms; // 1:1 with Meshes

	double Distance = 1.0;
	EMorphologyOperation Operation;

	bool bVoxWrapInput = false;
	bool bRemoveInternalsAfterVoxWrap = false;
	double ThickenShells = 0.0;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
