// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#include "BaseOps/VoxelBaseOp.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FVoxelSolidifyMeshesOp : public FVoxelBaseOp
{
public:
	virtual ~FVoxelSolidifyMeshesOp() = default;

	// inputs
	TArray<TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>> Meshes;
	TArray<FTransformSRT3d> Transforms; // 1:1 with Meshes

	double WindingThreshold = .5;
	double ExtendBounds = 1;
	bool bSolidAtBoundaries = true;
	int SurfaceSearchSteps = 3;

	bool bApplyThickenShells = false;
	double ThickenShells = 5;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

};


} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
