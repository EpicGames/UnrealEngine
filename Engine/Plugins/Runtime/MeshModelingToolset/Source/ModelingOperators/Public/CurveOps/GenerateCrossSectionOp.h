// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "Operations/MeshPlaneCut.h"

#include "GenerateCrossSectionOp.generated.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
	namespace Geometry
	{

		class FGenerateCrossSectionOp : public FDynamicMeshOperator
		{
		public:
			virtual ~FGenerateCrossSectionOp() = default;

			// inputs
			FVector3d LocalPlaneOrigin, LocalPlaneNormal;
			bool bSimplifyAlongNewEdges = true;
			TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;

			UE_API void SetTransform(const FTransformSRT3d& Transform);

			//
			// FDynamicMeshOperator implementation
			// 

			UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

			// Outputs

			UE_API TArray<TArray<FVector3d>> GetCutLoops() const;
			UE_API TArray<TArray<FVector3d>> GetCutSpans() const;

		private:

			TUniquePtr<FMeshPlaneCut> MeshCutter;
		};

	} // end namespace UE::Geometry
} // end namespace UE



UCLASS(MinimalAPI)
class UGenerateCrossSectionOpFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;
		
	FVector3d LocalPlaneOrigin, LocalPlaneNormal;
	bool bSimplifyAlongNewEdges = true;

	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	UE::Geometry::FTransformSRT3d TargetTransform;
};

#undef UE_API
