// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingOperators.h"

#include "TriangulateCurvesOp.generated.h"

#define UE_API MODELINGOPERATORS_API

class USplineComponent;

UENUM()
enum class EFlattenCurveMethod : uint8
{
	// Do not flatten the curves before triangulations
	DoNotFlatten,
	// Fit planes to the curves, and flatten the curves by projection to their plane
	ToBestFitPlane,
	// Flatten by projection along the X axis
	AlongX,
	// Flatten by projection along the Y axis
	AlongY,
	// Flatten by projection along the Z axis
	AlongZ
};

UENUM()
enum class ECombineCurvesMethod : uint8
{
	// Triangulate each curve separately
	LeaveSeparate,
	// Triangulate the union of the curve polygons -- the space covered by any of the polygons
	Union,
	// Triangulate the intersection of the curve polygons -- the space covered by all of the polygons
	Intersect,
	// Triangulate the difference of the first curve polygon minus the remaining curve polygons
	Difference,
	// Triangulate the exclusive-or of the curve polygons -- the space covered by the first polygon, or the remaining polygons, but not both
	ExclusiveOr
};

UENUM()
enum class EOffsetClosedCurvesMethod : uint8
{
	// Do not offset the closed curves
	DoNotOffset,
	// Offset the outside of the closed curves -- growing or shrinking the solid shape
	OffsetOuterSide,
	// Offset both sides of the closed curves -- creating hollow shapes that follow the curves with Curve Offset width
	OffsetBothSides
};

UENUM()
enum class EOffsetOpenCurvesMethod : uint8
{
	// Treat open curves as if they were closed -- connecting the last point back to the first
	TreatAsClosed,
	// Offset the open curves, creating shapes following the curves with Curve Offset width
	Offset
};

UENUM()
enum class EOffsetJoinMethod : uint8
{
	// Cut off corners between offset edges with square shapes
	Square,
	// Miter corners between offset edges, extending the neighboring curve edges straight to their intersection point, unless that point is farther than the miter limit distance
	Miter,
	// Smoothly join corners between offset edges with circular paths
	Round
};

UENUM()
enum class EOpenCurveEndShapes : uint8
{
	// Close the ends of open paths with square end caps
	Square,
	// Close the ends of open paths with round end caps
	Round,
	// Close the ends of open paths abruptly with no end caps
	Butt
};

namespace UE {
namespace Geometry {

class FDynamicMesh3;

/**
 * FTriangulateCurvesOp triangulates polygons/paths generated from USplineComponent inputs.
 */
class FTriangulateCurvesOp : public FDynamicMeshOperator
{
public:
	virtual ~FTriangulateCurvesOp() = default;

	//
	// Inputs
	//

	// Sample a spline with the given ErrorTolerance and add it as a curve
	UE_API void AddSpline(USplineComponent* Spline, double ErrorTolerance);

	// Add a curve in world space
	// @param WorldSpaceVertices The vertices of the curve, in world space
	// @param bClosed Whether the curve is closed, i.e. should include an edge from the last vertex back to the first vertex
	// @param ReferenceTransform The transform that would take the curve vertices from local space to world space
	// Note: The first curve's Reference Transform is used as the local reference frame for the triangulation, and will be the operator's Result Transform
	UE_API void AddWorldCurve(TArrayView<const FVector3d> WorldSpaceVertices, bool bClosed, const FTransform& ReferenceTransform);

	//
	// Parameters
	//

	// TODO: Add more options, e.g. for flattening the input curves, setting triangulation method, and additional processing of the curves (e.g., union, offset, etc)

	// Scaling applied to the default UV values
	double UVScaleFactor = 1.0;

	// If true, UVs will be consistently scaled relative to the world space, otherwise UVs will be relative to the mesh bounds.
	bool bWorldSpaceUVScale = false;

	// If > 0, thicken the result mesh to make a solid
	double Thickness = 0.0;

	EFlattenCurveMethod FlattenMethod = EFlattenCurveMethod::DoNotFlatten;

	// Note: Combining and offsetting curves only works when curves are flattened; curves will be left separate and non-offset if FlattenMethod is DoNotFlatten

	ECombineCurvesMethod CombineMethod = ECombineCurvesMethod::LeaveSeparate;

	EOffsetClosedCurvesMethod OffsetClosedMethod = EOffsetClosedCurvesMethod::DoNotOffset;
	EOffsetOpenCurvesMethod OffsetOpenMethod = EOffsetOpenCurvesMethod::TreatAsClosed;
	EOffsetJoinMethod OffsetJoinMethod = EOffsetJoinMethod::Square;
	EOpenCurveEndShapes OpenEndShape = EOpenCurveEndShapes::Square;
	double MiterLimit = 1.0;
	double CurveOffset = 1.0;

	bool bFlipResult = false;

	//
	// FDynamicMeshOperator interface 
	//
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

private:

	struct FCurvePath
	{
		bool bClosed = false;
		TArray<FVector3d> Vertices;
	};

	// Paths for all splines, in world space
	TArray<FCurvePath> Paths;

	// Local to World transform of the first path
	FTransform FirstPathTransform;

	UE_API void ApplyThickness(double UseUVScaleFactor);
};

}} // end UE::Geometry

#undef UE_API
