// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingOperators.h"
#include "Operations/GroupEdgeInserter.h"

#define UE_API MODELINGOPERATORS_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FEdgeLoopInsertionOp : public FDynamicMeshOperator
{
public:
	virtual ~FEdgeLoopInsertionOp() = default;

	// Inputs:
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<const FGroupTopology, ESPMode::ThreadSafe> OriginalTopology;
	FGroupEdgeInserter::EInsertionMode Mode;
	double VertexTolerance; // TODO: Add some defaults
	int32 GroupEdgeID = FDynamicMesh3::InvalidID;
	TArray<double> InputLengths;
	bool bInputsAreProportions = false;
	int32 StartCornerID = FDynamicMesh3::InvalidID;

	// Outputs:
	// Edge IDs in the ResultMesh corresponding to the loop.
	TSet<int32> LoopEids; 

	// IDs of triangles in the OriginalMesh that were changed or deleted.
	TSharedPtr<TSet<int32>, ESPMode::ThreadSafe> ChangedTids;

	// IDs of group edges in OriginalTopology that surround non-quad-like regions
	// that stopped the loop.
	TSet<int32> ProblemGroupEdgeIDs;

	TSharedPtr<FGroupTopology, ESPMode::ThreadSafe> ResultTopology;
	bool bSucceeded = false;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	/** 
	 * Converts LoopEids into pairs of endpoints, since ResultMesh is inaccessible without
	 * extracting it. Can be used to render the added loops.
	 * Clears EndPointPairsOut before use.
	 */
	UE_API void GetLoopEdgeLocations(TArray<TPair<FVector3d, FVector3d>>& EndPointPairsOut) const;

	// FDynamicMeshOperator implementation
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
