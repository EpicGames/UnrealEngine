// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GroupTopology.h"
#include "ModelingOperators.h"
#include "Operations/GroupEdgeInserter.h"

#define UE_API MODELINGOPERATORS_API

class FProgressCancel;

namespace UE
{
namespace Geometry
{

class FGroupEdgeInsertionOp : public FDynamicMeshOperator
{
public:
	virtual ~FGroupEdgeInsertionOp() = default;

	// Inputs:
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<const FGroupTopology, ESPMode::ThreadSafe> OriginalTopology;
	FGroupEdgeInserter::EInsertionMode Mode;
	double VertexTolerance = KINDA_SMALL_NUMBER * 10;

	FGroupEdgeInserter::FGroupEdgeSplitPoint StartPoint;
	FGroupEdgeInserter::FGroupEdgeSplitPoint EndPoint;

	int32 CommonGroupID = FDynamicMesh3::InvalidID;
	int32 CommonBoundaryIndex = 0;

	// This overrides the behavior to revert to the original mesh when CalculateResult is called.
	bool bShowingBaseMesh = false;

	// Outputs:

	// Edge ID's in the ResultMesh that together make up the new group edge.
	TSet<int32> Eids; 

	// Triangle ID's in the OriginalMesh that got changed or deleted in the operation.
	TSharedPtr<TSet<int32>, ESPMode::ThreadSafe> ChangedTids;

	TSharedPtr<FGroupTopology, ESPMode::ThreadSafe> ResultTopology;
	bool bSucceeded = false;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	/** 
	 * Converts GroupEdgeEids into pairs of endpoints, since ResultMesh is inaccessible without
	 * extracting it. Can be used to render the added loops.
	 * Clears EndPointPairsOut before use.
	 */
	UE_API void GetEdgeLocations(TArray<TPair<FVector3d, FVector3d>>& EndPointPairsOut) const;

	// FDynamicMeshOperator implementation
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
