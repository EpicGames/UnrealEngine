// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#define UE_API MODELINGOPERATORS_API


namespace UE
{
namespace Geometry
{


class FPlaneCutOp : public FDynamicMeshOperator
{
public:
	virtual ~FPlaneCutOp() = default;

	// inputs
	FVector3d LocalPlaneOrigin, LocalPlaneNormal;
	bool bFillCutHole = true;
	bool bFillSpans = false;
	bool bKeepBothHalves = false;
	bool bSimplifyAlongNewEdges = true;
	double CutPlaneLocalThickness = 0; // plane thickness in the local space of the mesh
	double UVScaleFactor = 0;
	static UE_API const FName ObjectIndexAttribute;
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
