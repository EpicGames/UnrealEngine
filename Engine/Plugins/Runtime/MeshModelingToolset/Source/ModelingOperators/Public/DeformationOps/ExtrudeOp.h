// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ModelingOperators.h"

#define UE_API MODELINGOPERATORS_API

namespace UE {
namespace Geometry {

class FDynamicMesh3;
class FDynamicMeshChangeTracker;

class FExtrudeOp : public FDynamicMeshOperator
{
public:
	enum class EExtrudeMode
	{
		// Performs extrusion by shifting the selected faces as stitching them
		// to the border left behind.
		MoveAndStitch,

		// Performs the extrusion by extruding selected faces into a closed mesh
		// first, and then performing a boolean with the original mesh. This allows
		// the extrusion to cut holes through the mesh or to bridge sections.
		Boolean
	};

	enum class EDirectionMode
	{
		// Take the angle-weighed average of the selected triangles around each
		// extruded vertex to determine vertex movement direction.
		SelectedTriangleNormals = 0,

		// Like Selected Triangle Normals, but also adjusts the distances moved in
		// an attempt to keep triangles parallel to their original facing.
		SelectedTriangleNormalsEven = 1,

		// Vertex normals, regardless of selection.
		VertexNormals = 2,

		// Extrude all triangles in the same direction regardless of their facing.
		SingleDirection = 3,
	};


	virtual ~FExtrudeOp() = default;

	// Inputs:
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TArray<int32> TriangleSelection;
	double ExtrudeDistance = 0;
	EExtrudeMode ExtrudeMode = EExtrudeMode::MoveAndStitch;
	EDirectionMode DirectionMode = EDirectionMode::SelectedTriangleNormalsEven;
	double MaxScaleForAdjustingTriNormalsOffset = 4.0;
	// Only used if DirectionMode is SingleDirection
	FVector3d MeshSpaceExtrudeDirection;
	// Used when setting groups for the sides when the extrusion includes a mesh border. When true,
	// different groups are added based on colinearity of the border edges.
	bool bUseColinearityForSettingBorderGroups = true;
	float UVScaleFactor = 1.0f;
	
	/** Controls whether extruding an entire open-border patch should create a solid or an open shell */
	bool bShellsToSolids = true;

	// Outputs
	TArray<int32> ExtrudedFaceNewGids;

	void SetTransform(const FTransformSRT3d& Transform)
	{
		ResultTransform = Transform;
	}

	// FDynamicMeshOperator implementation 
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

protected:
	UE_API virtual bool BooleanExtrude(FProgressCancel* Progress);
	UE_API virtual bool MoveAndStitchExtrude(FProgressCancel* Progress);
};

}} // end UE::Geometry

#undef UE_API
