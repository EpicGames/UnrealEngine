// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseOps/SimpleMeshProcessingBaseOp.h"
#include "DynamicMeshEditor.h"
#include "Operations/JoinMeshLoops.h"
#include "Operations/FFDLattice.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FLatticeDeformerOp : public FDynamicMeshOperator
{
public:

	UE_API FLatticeDeformerOp(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InOriginalMesh,
					   TSharedPtr<UE::Geometry::FFFDLattice, ESPMode::ThreadSafe> InLattice,
					   const TArray<FVector3d>& InLatticeControlPoints,
					   UE::Geometry::ELatticeInterpolation InInterpolationType,
					   bool bInDeformNormals);

	UE_API FLatticeDeformerOp(TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InOriginalMesh,
					   TSharedPtr<FDynamicSubmesh3, ESPMode::ThreadSafe> InSubmesh,
					   FTransform3d InWorldTransform,
					   TSharedPtr<UE::Geometry::FFFDLattice, ESPMode::ThreadSafe> InLattice,
					   const TArray<FVector3d>& InLatticeControlPoints,
					   UE::Geometry::ELatticeInterpolation InInterpolationType,
					   bool bInDeformNormals);

	// FDynamicMeshOperator implementation
	UE_API void CalculateResult(FProgressCancel* Progress) override;

protected:

	// Inputs
	const TSharedPtr<const UE::Geometry::FFFDLattice, ESPMode::ThreadSafe> Lattice;
	const TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	const TSharedPtr<const FDynamicSubmesh3, ESPMode::ThreadSafe> Submesh;
	const FTransform3d WorldTransform;
	const TArray<FVector3d> LatticeControlPoints;
	UE::Geometry::ELatticeInterpolation InterpolationType;
	bool bDeformNormals = false;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
