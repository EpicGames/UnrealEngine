// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseOps/SimpleMeshProcessingBaseOp.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshWeights.h"
#include "MeshBoundaryLoops.h"
#include "Operations/JoinMeshLoops.h"
#include "Solvers/ConstrainedMeshDeformer.h"
#include "WeightMapTypes.h"



namespace UE
{
namespace Geometry
{

/**
 * Base class for Mesh Offset operations (ie displacement along some kind of normal)
 * Optionally duplicates input mesh and joins the two together, either by welding along
 * shared border or creating quad-strip join.
 */
class FMeshOffsetBaseOp : public FSimpleMeshProcessingBaseOp
{

public:
	FMeshOffsetBaseOp(const FDynamicMesh3* Mesh) : FSimpleMeshProcessingBaseOp(Mesh) {}


	// subclasses must implement this function, it should update ResultMesh
	virtual void CalculateOffset(FProgressCancel* Progress) = 0;

	// positive range of offset distances (default offset will be max value unless there is a weightmap)
	FInterval1d OffsetRange = FInterval1d(0.0, 1.0);
	// sign of offset, should be +1 or -1
	double OffsetSign = 1.0;

	TSharedPtr<FMeshNormals> BaseNormals;
	FMeshBoundaryCache BoundaryCache;

	TSharedPtr<UE::Geometry::FMeshBoundaryLoops> BoundaryLoops;
	bool bFixedBoundary = false;

	bool bCreateShell = false;

	TSharedPtr<UE::Geometry::FIndexedWeightMap1f> WeightMap;
	bool bUseWeightMap = false;

	/** @return signed offset distance at vertex, transformed by weight map iv applicable */
	double GetOffsetDistance(int32 VertexID, double DefaultValue = 1.0) const
	{
		double Weight = (bUseWeightMap) ? WeightMap->GetValue(VertexID) : DefaultValue;
		Weight = FMathd::Clamp(Weight, 0.0, 1.0);
		return OffsetSign * OffsetRange.Interpolate(Weight);
	}


	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		BoundaryCache.Calculate(*ResultMesh);

		if (bCreateShell)
		{
			check(BoundaryLoops.IsValid());

			FDynamicMesh3 InnerMesh;
			InnerMesh.Copy(*ResultMesh);
			bool bIsPositiveOffset = (OffsetSign >= 0);
			if (bIsPositiveOffset)
			{
				InnerMesh.ReverseOrientation();
			}

			CalculateOffset(Progress);
			UpdateResultMeshPositions();

			if (!bIsPositiveOffset)
			{
				ResultMesh->ReverseOrientation();
			}

			UE::Geometry::FDynamicMeshEditor Editor(ResultMesh.Get());
			UE::Geometry::FMeshIndexMappings MeshMap;
			Editor.AppendMesh(&InnerMesh, MeshMap);

			bool bWeldStitch = bFixedBoundary;

			// join the boundary loops by weld or stitch
			for (UE::Geometry::FEdgeLoop& BaseLoop : BoundaryLoops->Loops)
			{
				int32 LoopCount = BaseLoop.GetVertexCount();
				TArray<int32> OffsetLoop;
				OffsetLoop.SetNum(LoopCount);
				for (int k = 0; k < LoopCount; ++k)
				{
					OffsetLoop[k] = MeshMap.GetNewVertex(BaseLoop.Vertices[k]);
				}

				TArray<int32>& LoopA = (bIsPositiveOffset) ? BaseLoop.Vertices : OffsetLoop;
				TArray<int32>& LoopB = (bIsPositiveOffset) ? OffsetLoop : BaseLoop.Vertices;

				if (bWeldStitch)
				{
					Editor.WeldVertexLoops(LoopA, LoopB);
				}
				else
				{
					UE::Geometry::FJoinMeshLoops Join(ResultMesh.Get(), LoopA, LoopB);
					Join.Apply();
				}
			}
			UpdateResultMeshNormals();
		}
		else
		{
			CalculateOffset(Progress);
			UpdateResultMeshPositions();
			UpdateResultMeshNormals();
		}

	}

};






class FIterativeOffsetMeshOp : public FMeshOffsetBaseOp
{
public:
	FIterativeOffsetMeshOp(const FDynamicMesh3* Mesh) : FMeshOffsetBaseOp(Mesh) {}

	int32 Steps = 1;
	double SmoothAlpha = 0.1;
	bool bReprojectSmooth = false;
	double BoundaryAlpha = 0.2;		// should not be > 0.9

	virtual void CalculateOffset(FProgressCancel* Progress) override
	{
		if (Steps > 1 && SmoothAlpha > 0)
		{
			Offset_Smoothed(Progress);
		}
		else
		{
			Offset(Progress);
		}
	}

	void Offset(FProgressCancel* Progress)
	{
		check(BaseNormals.IsValid());

		int32 UseSteps = FMath::Max(1, Steps);
		if (UseSteps == 1)
		{
			for (int32 vid : ResultMesh->VertexIndicesItr())
			{
				double Dist = GetOffsetDistance(vid);

				FVector3d OffsetPosition = ResultMesh->GetVertex(vid);
				if (bFixedBoundary == false || BoundaryCache.bIsBoundary[vid] == false)
				{
					OffsetPosition += Dist * BaseNormals->GetNormals()[vid];
				}
				PositionBuffer[vid] = OffsetPosition;
			}
		}
		else
		{
			FMeshNormals StepNormals(ResultMesh.Get());
			for (int32 k = 0; k < UseSteps; ++k)
			{
				FMeshNormals& UseNormals = (k == 0) ? *BaseNormals : StepNormals;

				// do offset step
				for (int32 vid : ResultMesh->VertexIndicesItr())
				{
					double Dist = GetOffsetDistance(vid);
					double OffsetPerStep = Dist / (double)UseSteps;

					FVector3d OffsetPosition = ResultMesh->GetVertex(vid);
					if (bFixedBoundary == false || BoundaryCache.bIsBoundary[vid] == false)
					{
						OffsetPosition += OffsetPerStep * UseNormals.GetNormals()[vid];
					}
					PositionBuffer[vid] = OffsetPosition;
				}

				// bake positions and re-calc normals
				for (int32 vid : ResultMesh->VertexIndicesItr())
				{
					ResultMesh->SetVertex(vid, PositionBuffer[vid]);
				}
				StepNormals.ComputeVertexNormals();
			}

			for (int32 vid : ResultMesh->VertexIndicesItr())
			{
				PositionBuffer[vid] = ResultMesh->GetVertex(vid);
			}
		}
	}


	void Offset_Smoothed(FProgressCancel* Progress)
	{
		check(BaseNormals.IsValid());
		int32 UseSteps = FMath::Max(1, Steps);
		TArray<FVector3d> SmoothedBuffer = PositionBuffer;

		bool bHasSmoothing = (SmoothAlpha > 0);
		FDynamicMesh3 ProjectMesh;
		FDynamicMeshAABBTree3 Spatial;
		if (bReprojectSmooth && bHasSmoothing)
		{
			ProjectMesh.Copy(*ResultMesh, false, false, false, false);
			Spatial.SetMesh(&ProjectMesh, true);
		}

		FMeshNormals StepNormals(ResultMesh.Get());
		for (int32 k = 0; k < UseSteps; ++k)
		{
			FMeshNormals& UseNormals = (k == 0) ? *BaseNormals : StepNormals;

			// do offset step
			int32 NumVertices = ResultMesh->MaxVertexID();
			for (int32 vid : ResultMesh->VertexIndicesItr())
			{
				double Dist = GetOffsetDistance(vid);
				double OffsetPerStep = Dist / (double)UseSteps;

				FVector3d OffsetPosition = ResultMesh->GetVertex(vid);
				if (bFixedBoundary == false || BoundaryCache.bIsBoundary[vid] == false)
				{
					OffsetPosition += OffsetPerStep * UseNormals.GetNormals()[vid];
				}
				PositionBuffer[vid] = OffsetPosition;
			}

			if (bReprojectSmooth && bHasSmoothing)
			{
				for (int32 vid : ResultMesh->VertexIndicesItr())
				{
					ProjectMesh.SetVertex(vid, PositionBuffer[vid]);
				}
				Spatial.Build();
			}

			// do smooth step
			if (bHasSmoothing)
			{
				ParallelFor(NumVertices, [&](int32 vid)
				{
					if (ResultMesh->IsReferencedVertex(vid) == false || BoundaryCache.bIsBoundary[vid])
					{
						SmoothedBuffer[vid] = PositionBuffer[vid];
						return;
					}
					FVector3d Centroid = FMeshWeights::UniformCentroid(*ResultMesh, vid, [&](int32 nbrvid) { return PositionBuffer[nbrvid]; });
					SmoothedBuffer[vid] = UE::Geometry::Lerp(PositionBuffer[vid], Centroid, SmoothAlpha);
					if (bReprojectSmooth)
					{
						SmoothedBuffer[vid] = Spatial.FindNearestPoint(SmoothedBuffer[vid]);
					}
				} /*, EParallelForFlags::ForceSingleThread*/);

				if (bFixedBoundary == false)
				{
					ParallelFor(BoundaryCache.BoundaryVerts.Num(), [&](int32 idx)
					{
						int32 vid = BoundaryCache.BoundaryVerts[idx];
						FVector3d Centroid = FMeshWeights::FilteredUniformCentroid(*ResultMesh, vid,
							[&](int32 nbrvid) { return PositionBuffer[nbrvid]; },
							[&](int32 nbrvid) { return BoundaryCache.bIsBoundary[nbrvid]; });
						SmoothedBuffer[vid] = UE::Geometry::Lerp(PositionBuffer[vid], Centroid, BoundaryAlpha);
					});
				}

				for (int32 vid : ResultMesh->VertexIndicesItr())
				{
					PositionBuffer[vid] = SmoothedBuffer[vid];
				}
			}

			// bake positions and re-calc normals
			for (int32 vid : ResultMesh->VertexIndicesItr())
			{
				ResultMesh->SetVertex(vid, PositionBuffer[vid]);
			}
			StepNormals.ComputeVertexNormals();
		}

		for (int32 vid : ResultMesh->VertexIndicesItr())
		{
			PositionBuffer[vid] = ResultMesh->GetVertex(vid);
		}
	}

};







class FLaplacianOffsetMeshOp : public FMeshOffsetBaseOp
{
public:
	FLaplacianOffsetMeshOp(const FDynamicMesh3* Mesh) : FMeshOffsetBaseOp(Mesh) {}

	bool bUniformWeights = true;
	double Softness = 0;

	virtual void CalculateOffset(FProgressCancel* Progress) override
	{
		Offset();
	}

	void Offset()
	{
		check(BaseNormals.IsValid());
		ELaplacianWeightScheme UseScheme = (bUniformWeights) ?
			ELaplacianWeightScheme::Uniform : ELaplacianWeightScheme::ClampedCotangent;

		TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Scaler = UE::MeshDeformation::ConstructConstrainedMeshDeformer(
			UseScheme, *ResultMesh);

		double Weight = FMathd::Max(Softness, 0.001) / 100.0;

		for (int32 vid : ResultMesh->VertexIndicesItr())
		{
			double Dist = GetOffsetDistance(vid);

			FVector3d OffsetPosition = ResultMesh->GetVertex(vid);
			OffsetPosition += Dist * BaseNormals->GetNormals()[vid];

			Scaler->AddConstraint(vid, Weight, OffsetPosition, false);
		}

		Scaler->Deform(PositionBuffer);
	}
};

} // end namespace UE::Geometry
} // end namespace UE
