// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "TransformTypes.h"
#include "Util/ProgressCancel.h"

namespace UE
{
namespace Geometry
{

/**
 * FDynamicMeshOperator is a base interface for operator implementations that can produce 
 * a FDynamicMesh3 with associated FTransformSRT3d. This class exists so we can write generic code 
 * that works with any type of mesh operation of this style. 
 */
class FDynamicMeshOperator
{
protected:
	TUniquePtr<FDynamicMesh3> ResultMesh;
	FTransformSRT3d ResultTransform;
	FGeometryResult ResultInfo;

public:
	FDynamicMeshOperator()
	{
		ResultMesh = MakeUnique<FDynamicMesh3>();
		ResultTransform = FTransformSRT3d::Identity();
	}
	virtual ~FDynamicMeshOperator() = default;

	/**
	 * Set the output transform
	 */
	virtual void SetResultTransform(const FTransformSRT3d& Transform)
	{
		ResultTransform = Transform;
	}

	/**
	 * Set the output information
	 */
	virtual void SetResultInfo(const FGeometryResult& Info)
	{
		ResultInfo = Info;
	}

	/**
	 * @return ownership of the internal mesh that CalculateResult() produced
	 */
	TUniquePtr<FDynamicMesh3> ExtractResult()
	{
		return MoveTemp(ResultMesh);
	}

	/**
	 * @return the transform applied to the mesh produced by CalculateResult()
	 */
	const FTransformSRT3d& GetResultTransform() const
	{
		return ResultTransform;
	}

	/**
	 * @return the result information returned by CalculateResult()
	 */
	const FGeometryResult& GetResultInfo() const
	{
		return ResultInfo;
	}

	/**
	 * Calculate the result of the operator. This will populate the internal Mesh and Transform.
	 * @param Progress implementors can use this object to report progress and determine if they should halt and terminate expensive computations
	 */
	virtual void CalculateResult(FProgressCancel* Progress) = 0;
};





/**
 * A IDynamicMeshOperatorFactory is a base interface to a factory that
 * creates FDynamicMeshOperators
 */
class IDynamicMeshOperatorFactory
{
public:
	virtual ~IDynamicMeshOperatorFactory() = default;

	virtual TUniquePtr<FDynamicMeshOperator> MakeNewOperator() = 0;
};








/**
 * TGenericDataOperator is a base interface for operator implementations that can produce an
 * object of arbitrary data type. Ownership is unique, ie the Operator calculates the result
 * and then the caller takes it via ExtractResult()
 */
template<typename ResultType>
class TGenericDataOperator
{
protected:
	TUniquePtr<ResultType> Result;

public:

	using ResultTypeName = ResultType;
	
	TGenericDataOperator(bool bCreateInitialObject = true)
	{
		if (bCreateInitialObject)
		{
			Result = MakeUnique<ResultType>();
		}
	}

	virtual ~TGenericDataOperator() = default;

	/**
	 * Set the result of the Operator (generally called by CalculateResult() implementation)
	 */
	void SetResult(TUniquePtr<ResultType>&& ResultIn)
	{
		Result = MoveTemp(ResultIn);
	}

	/**
	 * @return ownership of the internal data that CalculateResult() produced
	 */
	TUniquePtr<ResultType> ExtractResult()
	{
		return MoveTemp(Result);
	}

	/**
	 * Calculate the result of the operator. This must populate the internal Result data
	 * @param Progress implementors can use this object to report progress and determine if they should halt and terminate expensive computations
	 */
	virtual void CalculateResult(FProgressCancel* Progress) = 0;
};





/**
 * A IDynamicMeshOperatorFactory is a base interface to a factory that
 * creates TGenericDataOperator instances that create the given ResultType
 */
template<typename ResultType>
class IGenericDataOperatorFactory
{
public:
	virtual ~IGenericDataOperatorFactory() = default;

	virtual TUniquePtr<TGenericDataOperator<ResultType>> MakeNewOperator() = 0;
};


} // end namespace UE::Geometry
} // end namespace UE
