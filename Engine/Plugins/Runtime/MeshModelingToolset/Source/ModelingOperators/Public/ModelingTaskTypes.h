// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/AsyncWork.h"
#include "Util/ProgressCancel.h"

#include <atomic>


namespace UE
{
namespace Geometry
{


/**
 * TDeleterTask is a task that takes ownership of another FAsyncTask
 * that is (presumably) still executing some operation. The DeleterTask waits
 * for this FAsyncTask to complete, and then deletes it.
 *
 * TDeleterTask should be launched as a background task via FAutoDeleteAsyncTask,
 * see FAsyncTaskExecuterWithAbort::CancelAndDelete for an example.
 */
template<typename DeleteTaskType>
class TDeleterTask : public FNonAbandonableTask
{
	friend class FAsyncTask<TDeleterTask<DeleteTaskType>>;
public:
	TDeleterTask(FAsyncTask<DeleteTaskType>* TaskIn)
		: Task(TaskIn)
	{}

	/** The task we will delete, when it completes */
	FAsyncTask<DeleteTaskType>* Task;

	void DoWork()
	{
		// wait for task to complete
		Task->EnsureCompletion();
		// should always be true if EnsureCompletion() returned
		check(Task->IsDone());
		// delete it
		delete Task;
	}


	// required for task system
	inline TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(TDeleterTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};


/**
 * FAsyncTaskExecuterWithAbort is an extension of FAsyncTask that adds a bAbort flag.
 * This flag is passed to the target TTask via a new function that it must
 * implemented, SetAbortSource().
 * 
 * If the task-executing source wishes to cancel the contained task, then
 * it calls CancelAndDelete(). This sets the abort flag to true, and spanws
 * a TDeleterTask that will take ownership of the now-discarded background
 * task, watch for it to be complete, and delete it, then delete itself.
 * 
 * So after calling CancelAndDelete() the task-executing source may discard
 * any reference to the original FAsyncTask (but not delete it!)
 * 
 * @warning keep in mind that the target TTask may not terminate immediately (hence all the machinery). It may continue to use resources/CPU for some time.
 */
template<typename TTask>
class FAsyncTaskExecuterWithAbort : public FAsyncTask<TTask>
{
public:
	/** Set to true to abort the child task. CancelAndDelete() should be used in most cases, instead of changing bAbort directly. */
	bool bAbort = false;

	// Optional counter, if set will be decremented on task destruction
	TSharedPtr<std::atomic<int>> TaskCounter;

	template <typename Arg0Type, typename... ArgTypes>
	FAsyncTaskExecuterWithAbort(Arg0Type&& Arg0, ArgTypes&&... Args)
		: FAsyncTask<TTask>(Forward<Arg0Type>(Arg0), Forward<ArgTypes>(Args)...)

	{
		FAsyncTask<TTask>::GetTask().SetAbortSource(&bAbort);
	}

	virtual ~FAsyncTaskExecuterWithAbort()
	{
		if (TaskCounter.Get())
		{
			--(*TaskCounter);
		}
	}

	/**
	 * Tells the child FAbandonableTask to terminate itself, via the bAbort flag
	 * passed in SetAbortSource, and then starts a TDeleterTask that waits for
	 * completion to delete this task.
	 */
	void CancelAndDelete()
	{
		bAbort = true;
		(new FAutoDeleteAsyncTask<TDeleterTask<TTask>>(this))->StartBackgroundTask();
	}
};

// An extension of FAsyncTaskExecuterWithAbort that requires the task have a GetProgress() function
// Intended to be used with FAbortableBackgroundTask
// Provides a PollProgress method to report out task progress
template<typename TTask>
class FAsyncTaskExecuterWithProgressCancel : public FAsyncTaskExecuterWithAbort<TTask>
{
protected:
	FProgressCancel* Progress = nullptr;
	float LastProgressFrac;
	FText LastProgressMessage;

public:
	using FAsyncTaskExecuterWithAbort<TTask>::bAbort;

	template <typename Arg0Type, typename... ArgTypes>
	FAsyncTaskExecuterWithProgressCancel(Arg0Type&& Arg0, ArgTypes&&... Args)
		: FAsyncTaskExecuterWithAbort<TTask>(Forward<Arg0Type>(Arg0), Forward<ArgTypes>(Args)...)

	{
		FAsyncTask<TTask>::GetTask().SetAbortSource(&bAbort);
		Progress = FAsyncTask<TTask>::GetTask().GetProgress();
	}

	bool PollProgress(float& OutProgressFrac, FText& OutProgressMessage)
	{
		if (!Progress)
		{
			return false;
		}
		OutProgressFrac = Progress->GetProgress();
		bool bNewProgress = false;
		if (OutProgressFrac != LastProgressFrac)
		{
			OutProgressMessage = Progress->GetProgressMessage();
			LastProgressMessage = OutProgressMessage;
			LastProgressFrac = OutProgressFrac;
			return true;
		}
		else if (OutProgressFrac == 0.0f)
		{
			// at progress == 0, allow message to update w/out progress advancing
			// Note: Not polling the message at progress > 0 allows us to take the message lock less frequently,
			// but does mean we could miss messages that happen w/out a corresponding progress change.
			// Because these messages should be short-lived, this trade-off seems ok, but we can revisit this
			// if the missed progress messages become a problem.
			OutProgressMessage = Progress->GetProgressMessage();
			if (!OutProgressMessage.IdenticalTo(LastProgressMessage))
			{
				LastProgressMessage = OutProgressMessage;
				return true;
			}
		}
		return false;
	}
};


/**
 * FAbortableBackgroundTask is a FNonAbandonableTask intended for long-running
 * background computations that might need to be interrupted, such as expensive operations
 * driven by a UI tool that the user may wish to cancel.
 * 
 * This class is intended to be used with FAsyncTaskExecuterWithAbort. In that case the 
 * SetAbortSource() function will be automatically called/configured. 
 * 
 * A FProgressCancel object can be returned which will allow expensive computations
 * to check the value of this internal flag.
  */
class FAbortableBackgroundTask : public FNonAbandonableTask
{
private:
	/** pointer to a bool owned somewhere else. If that bool becomes true, this task should cancel further computation */
	bool* bExternalAbortFlag = nullptr;
	/** Internal ProgressCancel instance that can be passed to expensive compute functions, etc */
	FProgressCancel Progress;

public:

	FAbortableBackgroundTask()
	{
		Progress.CancelF = [this]() { return (bExternalAbortFlag != nullptr) ? *bExternalAbortFlag : false; };
	}

	/** Set the abort source flag. */
	void SetAbortSource(bool* bAbortFlagLocation)
	{
		bExternalAbortFlag = bAbortFlagLocation;
	}

	/** @return true if the task should stop computation */
	bool IsAborted() const
	{
		return (bExternalAbortFlag != nullptr) ? *bExternalAbortFlag : false;
	}

	/** @return pointer to internal progress object which can be passed to expensive child computations */
	FProgressCancel* GetProgress()
	{
		return &Progress;
	}
};



} // end namespace UE::Geometry
} // end namespace UE


