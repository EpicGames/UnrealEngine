// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Polygroups/PolygroupSet.h"

#include "RecomputeUVsOp.generated.h"

#define UE_API MODELINGOPERATORS_API

class URecomputeUVsToolProperties;

namespace UE
{
namespace Geometry
{

	struct TileConnectedComponents;

enum class ERecomputeUVsUnwrapType
{
	ExpMap = 0,
	ConformalFreeBoundary = 1,
	SpectralConformal = 2
};



enum class ERecomputeUVsIslandMode
{
	PolyGroups = 0,
	UVIslands = 1
};


class FRecomputeUVsOp : public FDynamicMeshOperator
{
public:
	virtual ~FRecomputeUVsOp() = default;

	//
	// Inputs
	// 

	// source mesh
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;

	// source groups (optional)
	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> InputGroups;

	// orientation control
	bool bAutoRotate = true;

	// Individual packing
	bool bPackToOriginalBounds = false;

	// area scaling
	bool bNormalizeAreas = true;
	float AreaScaling = 1.0;

	// UV layer
	int32 UVLayer = 0;

	// Atlas Packing parameters
	bool bPackUVs = true;
	int32 PackingTextureResolution = 512;
	float PackingGutterWidth = 1.0f;

	ERecomputeUVsUnwrapType UnwrapType = ERecomputeUVsUnwrapType::ExpMap;
	ERecomputeUVsIslandMode IslandMode = ERecomputeUVsIslandMode::PolyGroups;

	//
	// Spectral Conformal Map Options 
	//
	bool bPreserveIrregularity = false;

	// 
	// ExpMap Options
	//
	int32 NormalSmoothingRounds = 0;
	double NormalSmoothingAlpha = 0.25;

	//
	// Patch Merging options
	//
	bool bMergingOptimization = false;
	double MergingThreshold = 1.5;
	double CompactnessThreshold = 9999999.0;		// effectively disabled as it usually is not a good idea
	double MaxNormalDeviationDeg = 45.0;

	// UDIM options	
	bool bUDIMsEnabled = false;

	// Selection
	TOptional<TSet<int32>> Selection;

	// set ability on protected transform.
	void SetTransform(const FTransformSRT3d& XForm)
	{
		ResultTransform = XForm;
	}

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


protected:

	/*
	* Check if operation is valid to do work. It may not be valid if there are conflicting or inconsistent options passed to it from the user.
	*/
	UE_API bool IsValid() const;

	FGeometryResult NewResultInfo;

	UE_API void NormalizeUVAreas(const FDynamicMesh3& Mesh, FDynamicMeshUVOverlay* Overlay, const TileConnectedComponents& TileComponents, float GlobalScale = 1.0f);

	UE_API void CollectIslandComponentsPerTile(const FDynamicMeshUVOverlay& UVOverlay, TArray< TileConnectedComponents >& ComponentsPerTile, bool& UseExistingUVTopology);

	UE_API virtual bool CalculateResult_Basic(FProgressCancel* Progress);
	UE_API virtual bool CalculateResult_RegionOptimization(FProgressCancel* Progress);
};

} // end namespace UE::Geometry
} // end namespace UE


/**
 * Can be hooked up to a UMeshOpPreviewWithBackgroundCompute to perform UV solving operations.
 * 
 * Inherits from UObject so that it can hold a strong pointer to the settings UObject, which
 * needs to be a UObject to be displayed in the details panel.
 */
UCLASS(MinimalAPI)
class URecomputeUVsOpFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:

	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<URecomputeUVsToolProperties> Settings = nullptr;

	TOptional<TSet<int32>> Selection;

	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> InputGroups;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TUniqueFunction<int32()> GetSelectedUVChannel = []() { return 0; };
	UE::Geometry::FTransformSRT3d TargetTransform;
};

#undef UE_API
