// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "InteractiveTool.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "TexelDensityOp.generated.h"

#define UE_API MODELINGOPERATORS_API

UENUM()
enum class ETexelDensityToolMode
{
	ApplyToIslands,
	ApplyToWhole,
	Normalize
};

UCLASS(MinimalAPI)
class UUVEditorTexelDensitySettings : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UFUNCTION()
	UE_API virtual bool InSamplingMode() const;

	UPROPERTY(EditAnywhere, Category = "TexelDensity", meta = (DisplayName = "Scale Mode", EditCondition = "!InSamplingMode" ))
	ETexelDensityToolMode TexelDensityMode = ETexelDensityToolMode::ApplyToIslands;

	UPROPERTY(EditAnywhere, Category = "TexelDensity", meta = (DisplayName = "World Units", EditCondition = "TexelDensityMode != ETexelDensityToolMode::Normalize"))
	float TargetWorldUnits = 100;

	UPROPERTY(EditAnywhere, Category = "TexelDensity", meta = (DisplayName = "Pixels", EditCondition = "TexelDensityMode != ETexelDensityToolMode::Normalize && !InSamplingMode"))
	float TargetPixelCount = 1024;

	UPROPERTY(EditAnywhere, Category = "TexelDensity", meta = (DisplayName = "Texture Dimensions", EditCondition = "TexelDensityMode != ETexelDensityToolMode::Normalize && !InSamplingMode", LinearDeltaSensitivity = 1000, Delta = 64, SliderExponent = 1, UIMin = 2, UIMax = 16384))
	float TextureResolution = 1024;

	UPROPERTY(EditAnywhere, Category = "TexelDensity", meta = (DisplayName = "Use UDIMs", EditCondition = "TexelDensityMode != ETexelDensityToolMode::Normalize && !InSamplingMode", TransientToolProperty))
	bool bEnableUDIMLayout = false;
};


namespace UE
{
	namespace Geometry
	{
		class FDynamicMesh3;
		class FDynamicMeshUVPacker;

		enum class EUVTexelDensityOpModes
		{
			ScaleGlobal = 0,
			ScaleIslands = 1,
			Normalize = 2
		};

		class FUVEditorTexelDensityOp : public FDynamicMeshOperator
		{
		public:
			virtual ~FUVEditorTexelDensityOp() = default;

			// inputs
			TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

			EUVTexelDensityOpModes TexelDensityMode = EUVTexelDensityOpModes::ScaleGlobal;

			int UVLayerIndex = 0;
			int TextureResolution = 1024;
			int TargetWorldSpaceMeasurement = 100;
			int TargetPixelCountMeasurement = 1024;

			bool bMaintainOriginatingUDIM = false;
			TOptional<TSet<int32>> Selection;
			TOptional<TMap<int32, int32>> TextureResolutionPerUDIM;
			FVector2f UVTranslation = FVector2f::Zero();

			UE_API void SetTransform(const FTransformSRT3d& Transform);

			//
			// FDynamicMeshOperator implementation
			// 

			UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

		protected:
			UE_API void ExecutePacker(FDynamicMeshUVPacker& Packer);			
			UE_API void ScaleMeshSubRegionByDensity(FDynamicMeshUVOverlay* UVLayer, const TArray<int32>& Tids, TSet<int32>& UVElements, int32 TileResolution);


		};

	} // end namespace UE::Geometry
} // end namespace UE

/**
 * Can be hooked up to a UMeshOpPreviewWithBackgroundCompute to perform UV Layout operations.
 *
 * Inherits from UObject so that it can hold a strong pointer to the settings UObject, which
 * needs to be a UObject to be displayed in the details panel.
 */
UCLASS(MinimalAPI)
class UUVTexelDensityOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UUVEditorTexelDensitySettings> Settings;

	TOptional<TSet<int32>> Selection;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TUniqueFunction<int32()> GetSelectedUVChannel = []() { return 0; };
	FTransform TargetTransform;
	TOptional<TMap<int32, int32>> TextureResolutionPerUDIM;
};

#undef UE_API
