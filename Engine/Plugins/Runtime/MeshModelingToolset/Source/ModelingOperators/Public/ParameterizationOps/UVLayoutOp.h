// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#include "UVLayoutOp.generated.h"

#define UE_API MODELINGOPERATORS_API

class UUVLayoutProperties;

namespace UE
{
namespace Geometry
{
	class FDynamicMesh3;
	class FDynamicMeshUVPacker;

enum class EUVLayoutOpLayoutModes
{
	TransformOnly = 0,
	RepackToUnitRect = 1,
	StackInUnitRect = 2,
	Normalize = 3
};

class FUVLayoutOp : public FDynamicMeshOperator
{
public:
	virtual ~FUVLayoutOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	EUVLayoutOpLayoutModes UVLayoutMode = EUVLayoutOpLayoutModes::RepackToUnitRect;

	int UVLayerIndex = 0;
	int TextureResolution = 128;
	bool bPreserveScale = false;
	bool bPreserveRotation = false;
	bool bAllowFlips = false;
	bool bAlwaysSplitBowties = true;
	float UVScaleFactor = 1.0;
	float GutterSize = 1.0;
	bool bMaintainOriginatingUDIM = false;
	TOptional<TSet<int32>> Selection;
	TOptional<TMap<int32, int32>> TextureResolutionPerUDIM;
	FVector2f UVTranslation = FVector2f::Zero();

	UE_API void SetTransform(const FTransformSRT3d& Transform);

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

protected:
	UE_API void ExecutePacker(FDynamicMeshUVPacker& Packer);
};

} // end namespace UE::Geometry
} // end namespace UE

/**
 * Can be hooked up to a UMeshOpPreviewWithBackgroundCompute to perform UV Layout operations.
 *
 * Inherits from UObject so that it can hold a strong pointer to the settings UObject, which
 * needs to be a UObject to be displayed in the details panel.
 */
UCLASS(MinimalAPI)
class UUVLayoutOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UUVLayoutProperties> Settings;
	TOptional<TSet<int32>> Selection;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TUniqueFunction<int32()> GetSelectedUVChannel = []() { return 0; };
	FTransform TargetTransform;
	TOptional<TMap<int32, int32>> TextureResolutionPerUDIM;
};

#undef UE_API
