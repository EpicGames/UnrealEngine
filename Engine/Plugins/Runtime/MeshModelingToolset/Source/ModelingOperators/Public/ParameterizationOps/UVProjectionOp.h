// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "OrientedBoxTypes.h"

#include "UVProjectionOp.generated.h"

#define UE_API MODELINGOPERATORS_API

UENUM()
enum class EUVProjectionMethod : uint8
{
	/** Calculate UVs by assigning triangles to faces of a box, and then apply per-box-face planar projection */
	Box,
	/** Calculate UVs by assigning triangles to a cylinder using radial projection or cylinder end caps using planar projection */
	Cylinder,
	/** Calculate UVs by projecting to a plane */
	Plane,
	/** Calculate UVs by Exponential Map projection centered at the surface point nearest to the center of the input plane */
	ExpMap UMETA(DisplayName = "ExpMap")
};

namespace UE
{
namespace Geometry
{

class FUVProjectionOp : public FDynamicMeshOperator
{
public:
	virtual ~FUVProjectionOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<TArray<int32>, ESPMode::ThreadSafe> TriangleROI;
	int32 UseUVLayer = 0;

	// transform from Mesh into space of Projection Geometry
	FTransformSRT3d MeshToProjectionSpace;

	EUVProjectionMethod ProjectionMethod = EUVProjectionMethod::Box;
	FOrientedBox3d ProjectionBox;
	int32 MinRegionSize = 2;
	float CylinderSplitAngle = 45.0f;

	// ExpMap parameters for modifying normals
	double BlendWeight = 0.0;
	int32 SmoothingRounds = 0;
	double SmoothingAlpha = 0.5;

	// position to use as UV origin
	FVector2f UVOrigin = FVector2f(0.5f,0.5f);
	// rotation applied to computed UVs
	float UVRotationAngleDeg = 0.0;
	// scale applied to computed UVs
	FVector2f UVScale = FVector2f::One();
	// translation applied to computed UVs
	FVector2f UVTranslate = FVector2f::Zero();


	//
	// FDynamicMeshOperator implementation
	// 
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


	UE_API virtual void CalculateResult_Box(FProgressCancel* Progress);
	UE_API virtual void CalculateResult_Plane(FProgressCancel* Progress);
	UE_API virtual void CalculateResult_ExpMap(FProgressCancel* Progress);
	UE_API virtual void CalculateResult_Cylinder(FProgressCancel* Progress);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
