// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "ModelingOperators.h"

#define UE_API MODELINGOPERATORS_API


namespace UE {
namespace Geometry {

class FDynamicMesh3;

/**
 * RegionOffsetOp extrudes a set of triangles of a mesh in a linear direction,
 * with various options for how to process that extrusion. 
 */
class FRegionOffsetOp : public FDynamicMeshOperator
{
public:
	virtual ~FRegionOffsetOp() = default;

	//
	// Inputs
	//

	/** Handle to the source DynamicMesh */
	TSharedPtr<FSharedConstDynamicMesh3> OriginalMeshShared;

	/** Triangles of the OriginalMesh that should be extruded */
	TArray<int32> TriangleSelection;

	//
	// Parameters
	//

	enum class EOffsetComputationMode
	{
		/** */
		VertexNormals = 0,
		/** */
		FaceNormals = 1,
		/**  */
		ApproximateConstantThickness = 2
	};


	double OffsetDistance = 1.0;

	EOffsetComputationMode OffsetMode = EOffsetComputationMode::ApproximateConstantThickness;


	/** If bShellsToSolids is true, patches surrounded by mesh boundaries will be extruded into closed solids instead of open shells */
	bool bShellsToSolids = true;

	/** Number of subdivisions along the extrusion "tubes" */
	int32 NumSubdivisions = 0;

	/** 
	 * Assign new groups to regions of the extrusion "tube" based on the existing adjacent groups before extrusion. Each (disconnected) 
	 * unique pair of group IDs gets a new unique group ID on the extrusion. If disabled, the entire extrusion tube will be assigned
	 * a single new group ID
	 */
	bool bInferGroupsFromNeighbours = true;
	
	/** If true, assign a separate group for each subdivision, otherwise all subdivisions have the same group */
	bool bNewGroupPerSubdivision = true;
	
	/** assign new GroupIDs to each input GroupID in the extrude area. If false, disconnected extrude areas get single new group IDs */
	bool bRemapExtrudeGroups = true;

	/** if the opening angle at an edge of the extrusion tube is > this threshold, "cut" any group that crosses the edge into two groups */
	double CreaseAngleThresholdDeg = 180.0;

	/** Scaling applied to the default UV values */
	double UVScaleFactor = 1.0f;

	/** If true, assign a new UV island for each new group, otherwise each disconnected extrusion tube is unwrapped to a single UV island*/
	bool bUVIslandPerGroup = true;

	/** Constant Material ID to set along the extrusion tube */
	int SetMaterialID = 0;

	/** If true, Material IDs around the border of the extrude area are propagated "down" the extrusion tube */
	bool bInferMaterialID = true;

	/** Determine whether normals should be recomputed */
	bool bRecomputeNormals = true;

	// (this parameter may not be relevant and perhaps should be removed...)
	// Used when setting groups for the sides when the extrusion includes a mesh border. When true,
	// different groups are added based on colinearity of the border edges.
	bool bUseColinearityForSettingBorderGroups = true;


	//
	// Outputs
	//


	/**
	 * Compute result applied directly to EditMesh
	 * @return true on success, false on error or if Progress is cancelled
	 */
	UE_API virtual bool CalculateResultInPlace(FDynamicMesh3& EditMesh, FProgressCancel* Progress);


	//
	// FDynamicMeshOperator interface 
	//
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


};

}} // end UE::Geometry

#undef UE_API
