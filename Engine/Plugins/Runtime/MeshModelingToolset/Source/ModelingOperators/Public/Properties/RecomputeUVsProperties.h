// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InteractiveTool.h"

#include "RecomputeUVsProperties.generated.h"


UENUM()
enum class ERecomputeUVsPropertiesUnwrapType
{
	// Values must match UE::Geometry::ERecomputeUVsUnwrapType

	/** ExpMap UV flattening is very fast but has limited abilities to reduce stretching and distortion */
	ExpMap = 0 UMETA(DisplayName = "ExpMap"),
	/** Conformal UV flattening is increasingly expensive on large islands but reduces distortion */
	Conformal = 1,
	/** Compared to the default Conformal method does not pin two vertices along the boundary which reduces the distortion but is more expensive to compute. */
	SpectralConformal = 2,
	/** UV islands will be merged into larger islands if it does not increase stretching and distortion beyond defined limits */
	IslandMerging = 3
};


UENUM()
enum class ERecomputeUVsPropertiesIslandMode
{
	// Values must match UE::Geometry::ERecomputeUVsIslandMode

	/** Use PolyGroups defined by Active PolyGroup property to define initial UV islands. */
	PolyGroups = 0 UMETA(DisplayName = "PolyGroups"),
	/** Use existing UV Layer to define UV islands, i.e. re-solve UV flattening based on existing UVs */
	ExistingUVs = 1
};


UENUM()
enum class ERecomputeUVsToolOrientationMode
{
	/** Do not rotate UV islands */
	None,
	/** Automatically rotate each UV island to reduce its bounding box size */
	MinBounds
};


UENUM()
enum class ERecomputeUVsPropertiesLayoutType
{
	/** Do not apply additional layout options to unwrapped UVs */
	None,
	/** Uniformly scale and translate UV islands collectively to pack them into the unit square, i.e. fit between 0 and 1 with no overlap */
	Repack,
	/** Scale and center all islands to fit within their original bounding boxes. Only applicable if using existing UVs. */
	NormalizeToExistingBounds,
	/** Uniformly scale UV islands such that they have constant relative area, relative to object bounds */
	NormalizeToBounds,
	/** Uniformly scale UV islands such that they have constant relative area, relative to world space */
	NormalizeToWorld
};


UCLASS(MinimalAPI)
class URecomputeUVsToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:

	UPROPERTY(meta = (TransientToolProperty))
	bool bEnablePolygroupSupport = true;

	/** Generation method for initial UV islands.*/
	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (EditCondition = "bEnablePolygroupSupport", EditConditionHides, HideEditConditionToggle = true))
	ERecomputeUVsPropertiesIslandMode IslandGeneration = ERecomputeUVsPropertiesIslandMode::ExistingUVs;

	/** Type of UV flattening algorithm to use */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap")
	ERecomputeUVsPropertiesUnwrapType UnwrapType = ERecomputeUVsPropertiesUnwrapType::SpectralConformal;

	/** Type of automatic rotation applied to each UV island */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap")
	ERecomputeUVsToolOrientationMode AutoRotation = ERecomputeUVsToolOrientationMode::MinBounds;

	/**  If enabled, reduces distortion for meshes with triangles of vastly different sizes, This is only enabled if the Unwrap Type is set to Spectral Conformal. */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (EditCondition = "UnwrapType == ERecomputeUVsPropertiesUnwrapType::SpectralConformal", EditConditionHides, HideEditConditionToggle))
	bool bPreserveIrregularity = true;

	/** Number of smoothing steps to apply; this slightly increases distortion but produces more stable results. This is only enabled if the Unwrap Type is set to ExpMap or Island Merging. */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (UIMin = "0", UIMax = "25", ClampMin = "0", ClampMax = "1000",
		EditCondition = "UnwrapType == ERecomputeUVsPropertiesUnwrapType::ExpMap || UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging", EditConditionHides, HideEditConditionToggle))
	int SmoothingSteps = 5;

	/** Smoothing parameter; larger values result in faster smoothing in each step. This is only enabled if the Unwrap Type is set to ExpMap or Island Merging. */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "1.0",
		EditCondition = "UnwrapType == ERecomputeUVsPropertiesUnwrapType::ExpMap || UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging", EditConditionHides, HideEditConditionToggle))
	float SmoothingAlpha = 0.25f;

	/** Threshold for stretching and distortion below which island merging is allowed; larger values increase the allowable UV distortion. This is only enabled if the Unwrap Type is set to Island Merging. */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (DisplayName = "Distortion Threshold", UIMin = "1.0", UIMax = "5.0", ClampMin = "1.0",
		EditCondition = "UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging", EditConditionHides, HideEditConditionToggle))
	float MergingDistortionThreshold = 1.5f;

	/** Threshold for the average face normal deviation below  which island merging is allowed. This is only enabled if the Unwrap Type is set to Island Merging. */
	UPROPERTY(EditAnywhere, Category = "UV Unwrap", meta = (DisplayName = "Angle Threshold", UIMin = "0.0", UIMax = "90.0", ClampMin = "0.0", ClampMax = "180.0",
		EditCondition = "UnwrapType == ERecomputeUVsPropertiesUnwrapType::IslandMerging", EditConditionHides, HideEditConditionToggle))
	float MergingAngleThreshold = 45.0f;

	/** Uniformly scale and translate UV islands collectively to pack them into the unit square, i.e. fit between 0 and 1 with no overlap. */
	UPROPERTY(EditAnywhere, Category = "UV Layout")
	ERecomputeUVsPropertiesLayoutType LayoutType = ERecomputeUVsPropertiesLayoutType::Repack;

	/** Expected resolution of the output textures; this controls spacing left between UV islands to avoid interpolation artifacts. This is only enabled when the Layout Type is set to Repack. */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (UIMin = "64", UIMax = "2048", ClampMin = "2", ClampMax = "4096",
		EditCondition = "LayoutType == ERecomputeUVsPropertiesLayoutType::Repack", EditConditionHides, HideEditConditionToggle))
	int TextureResolution = 1024;

	/** Scaling factor used for UV island normalization/scaling. This is only enabled when the Layout Type is set to Normalize to Bounds or Normalize to World. */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (UIMin = "0.001", UIMax = "10", ClampMin = "0.00001", ClampMax = "1000000.0",
		EditCondition = "LayoutType == ERecomputeUVsPropertiesLayoutType::NormalizeToBounds || LayoutType == ERecomputeUVsPropertiesLayoutType::NormalizeToWorld", EditConditionHides, HideEditConditionToggle))
	float NormalizeScale = 1.0f;

	/** Enable UDIM aware layout and keep islands within their originating UDIM tiles when laying out.*/
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (DisplayName = "Preserve UDIMs", EditCondition = "bUDIMCVAREnabled", EditConditionHides, HideEditConditionToggle = true))
	bool bEnableUDIMLayout = false;

	UPROPERTY(Transient)
	bool bUDIMCVAREnabled = false;
};
