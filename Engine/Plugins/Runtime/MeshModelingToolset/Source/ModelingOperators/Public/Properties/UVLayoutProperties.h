// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"

#include "UVLayoutProperties.generated.h"

/**
 * UV Layout Strategies for the UV Layout Tool
 */
UENUM()
enum class EUVLayoutType
{
	/** Apply Scale and Translation properties to all UV values */
	Transform,
	/** Uniformly scale and translate each UV island individually to pack it into the unit square, i.e. fit between 0 and 1 with overlap */
	Stack,
	/** Uniformly scale and translate UV islands collectively to pack them into the unit square, i.e. fit between 0 and 1 with no overlap */
	Repack,
	/** Scale and translate UV islands to normalize the UV islands' area to match an average texel density. */
	Normalize
};


/**
 * UV Layout Settings
 */
UCLASS(MinimalAPI)
class UUVLayoutProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	/** Type of layout applied to input UVs */
	UPROPERTY(EditAnywhere, Category = "UV Layout")
	EUVLayoutType LayoutType = EUVLayoutType::Repack;

	/** Expected resolution of the output textures; this controls spacing left between UV islands to avoid interpolation artifacts */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (UIMin = "64", UIMax = "2048", ClampMin = "2", ClampMax = "4096"))
	int TextureResolution = 1024;

	/** Uniform scale applied to UVs after packing */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (UIMin = "0.1", UIMax = "5.0", ClampMin = "0.0001", ClampMax = "10000") )
	float Scale = 1;

	/** Translation applied to UVs after packing, and after scaling */
	UPROPERTY(EditAnywhere, Category = "UV Layout")
	FVector2D Translation = FVector2D(0,0);

	/** Force the Repack layout type to preserve existing scaling of UV islands. Note, this might lead to the packing not fitting within a unit square, and therefore is disabled by default. */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (EditCondition = "LayoutType == EUVLayoutType::Repack"))
	bool bPreserveScale = false;

	/** Force the Repack layout type to preserve existing rotation of UV islands. Note, this might lead to the packing not being as space efficient as possible, and therefore is disabled by default. */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (EditCondition = "LayoutType == EUVLayoutType::Repack"))
	bool bPreserveRotation = false;

	/** Allow the Repack layout type to flip the orientation of UV islands to save space. Note that this may cause problems for downstream operations, and therefore is disabled by default. */
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (EditCondition = "LayoutType == EUVLayoutType::Repack && bPreserveRotation == false"))
	bool bAllowFlips = false;

	/** Enable UDIM aware layout and keep islands within their originating UDIM tiles when laying out.*/
	UPROPERTY(EditAnywhere, Category = "UV Layout", meta = (DisplayName = "Preserve UDIMs", EditCondition="bUDIMCVAREnabled", EditConditionHides, HideEditConditionToggle = true))
	bool bEnableUDIMLayout = false;

	UPROPERTY(Transient)
	bool bUDIMCVAREnabled = false;
};
