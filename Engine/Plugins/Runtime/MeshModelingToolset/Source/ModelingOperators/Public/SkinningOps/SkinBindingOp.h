// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelingOperators.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMeshAttributes.h"
#include "Operations/SkinWeightBinding.h"

#define UE_API MODELINGOPERATORS_API


struct FReferenceSkeleton;

namespace UE::Geometry
{

template<typename ParentType> class TDynamicVertexSkinWeightsAttribute;
class FDynamicMesh3;
using FDynamicMeshVertexSkinWeightsAttribute = TDynamicVertexSkinWeightsAttribute<FDynamicMesh3>;


class FSkinBindingOp : public FDynamicMeshOperator
{
public:

	// note we disable deprecation warnings here due to the deprecated TransformHierarchy member below
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FSkinBindingOp() override = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	UE_DEPRECATED(5.8, "TransformHierarchy is now unused and should not be accessed.")
	TArray<TPair<FTransform, FMeshBoneInfo>> TransformHierarchy;

	FName ProfileName = FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
	
	ESkinBindingType BindType = ESkinBindingType::DirectDistance;
	float Stiffness = 0.2f;
	int32 MaxInfluences = 5;
	int32 VoxelResolution = 256;

	UE_API void SetTransformHierarchyFromReferenceSkeleton(const FReferenceSkeleton& InRefSkeleton);
	
	UE_API virtual void CalculateResult(FProgressCancel* InProgress) override;

private:
	TArray<SkinBinding::FBonePoseInfo> BoneTransformHierarchy;
};


} // namespace UE::Geometry

#undef UE_API
