// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SmoothingOpBase.h"
#include "CoreMinimal.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class  FCotanSmoothingOp : public FSmoothingOpBase
{
public:
	UE_API FCotanSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn);
	
	// Support for smoothing only selected geometry
	UE_API FCotanSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn, const FDynamicSubmesh3& Submesh);

	~FCotanSmoothingOp() override = default;

	UE_API void CalculateResult(FProgressCancel* Progress) override;

private:
	// Compute the smoothed result by using Cotan Biharmonic
	UE_API void Smooth(FProgressCancel* Progress);

	UE_API double GetSmoothPower(int32 VertexID, bool bIsBoundary);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
