// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SmoothingOpBase.h"
#include "CoreMinimal.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class  FIterativeSmoothingOp : public FSmoothingOpBase
{
public:
	UE_API FIterativeSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn);

	// Support for smoothing only selected geometry
	UE_API FIterativeSmoothingOp(const FDynamicMesh3* Mesh, const FSmoothingOpBase::FOptions& OptionsIn,  const FDynamicSubmesh3& Submesh);

	~FIterativeSmoothingOp() override = default;

	// Apply smoothing. results in an updated ResultMesh
    //Note: if canceled using the optional FProgressCancel, the results may be in an unusable state.
	UE_API void CalculateResult(FProgressCancel* Progress) override;

private:

	UE_API double GetSmoothAlpha(int32 VertexID, bool bIsBoundary);

	// uniform iterative smoothing
	UE_API void Smooth_Forward(bool bUniform, FProgressCancel* Progress = nullptr);

	// cotan smoothing iterations
	UE_API void Smooth_Implicit_Cotan(FProgressCancel* Progress = nullptr);

	// mean value smoothing iterations
	UE_API void Smooth_MeanValue(FProgressCancel* Progress = nullptr);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
