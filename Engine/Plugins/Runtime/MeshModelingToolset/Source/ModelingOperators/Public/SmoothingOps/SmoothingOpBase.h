// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicSubmesh3.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "VectorTypes.h"
#include "WeightMapTypes.h"

#define UE_API MODELINGOPERATORS_API


namespace UE
{
namespace Geometry
{

class FDynamicMesh3;
class FMeshNormals;

class FSmoothingOpBase : public FDynamicMeshOperator
{
public:
	struct FOptions
	{
		// value in range [0,1] where 0 is no smoothing and 1 is full smoothing
		float SmoothAlpha = 1.0;

		// value in range [0,1] where 0 is no smoothing and 1 is full smoothing
		float BoundarySmoothAlpha = 1.0;

		// number of iterations for iterative smoothing 
		int32 Iterations = 1;

		// Unconstrained value in range [0,FMathf::MaxValue] with 0=NoSmoothing and MaxValue=FullySmoothed
		// Used by weighted implicit smoothing where weight is somewhat arbitrary...
		float SmoothPower = 1.0;

		// if true use implicit smoothing (where that is possible - depends on smoother?)
		bool bUseImplicit = false;

		// if true smooth the boundary, otherwise keep it fixed
		bool bSmoothBoundary = true;

		// if true use uniform weights, otherwise use something better
		bool bUniform = false;

		// use this value to clamp weights (eg for clamped mean value)
		double WeightClamp = FMathf::MaxReal;

		// mesh normals calculated for input mesh
		TSharedPtr<UE::Geometry::FMeshNormals> BaseNormals;

		// offset used by some smoothers
		double NormalOffset = 0.0;

		TSharedPtr<UE::Geometry::FIndexedWeightMap1f> WeightMap;
		bool bUseWeightMap = false;
		float WeightMapMinMultiplier = 0.0;
	};


	UE_API FSmoothingOpBase(const FDynamicMesh3* Mesh, const FOptions& OptionsIn);

	// Support for smoothing only selected geometry
	UE_API FSmoothingOpBase(const FDynamicMesh3* Mesh, const FOptions& OptionsIn, const FDynamicSubmesh3& Submesh);

	virtual ~FSmoothingOpBase() override = default;

	// set ability on protected transform.
	UE_API void SetTransform(const FTransformSRT3d& XForm);

	// base class overrides this.  Results in updated ResultMesh.
	virtual void CalculateResult(FProgressCancel* Progress) override = 0;

	// copy the PositionBuffer locations back to the ResultMesh and recompute normal if it exists.
	UE_API void UpdateResultMesh();

protected:
	FOptions SmoothOptions;

	TArray<FVector3d> PositionBuffer;

	// a copy of the original mesh to save information regarding the non-selected (/non-smoothed) mesh when applicable
	TUniquePtr<FDynamicMesh3> SavedMesh = nullptr;

	// maps the VertexId in the smoothed mesh to the VertexId in the original mesh
	TArray<int32> SmoothedToOriginalMap;

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
