// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshSpaceDeformerOp.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FFlareMeshOp : public FMeshSpaceDeformerOp
{
public:
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

	/** 0% does nothing, 100% moves 2x away from Z axis at extremal point, -100% squishes down to Z axis at extremal point. */
	double FlarePercentX = 100;
	double FlarePercentY = 100;

	enum class EFlareType
	{
		SinFlare = 0,     // Flaring is the curve sin(Pi y)  y in [0, 1]
		SinSqrFlare = 1,  // Flaring is the curve sin(Pi y)**2  y in  [0, 1]  which makes the ends smooth out back into shape
		LinearFlare = 2   // Flaring is  2*y  y in [0, 1/2] and 2(1-y) y in (1/2, 1] 
	};

	EFlareType FlareType = EFlareType::SinFlare;

protected:

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
