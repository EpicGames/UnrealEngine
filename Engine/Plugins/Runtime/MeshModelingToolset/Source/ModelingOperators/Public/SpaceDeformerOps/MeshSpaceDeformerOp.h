// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FMeshSpaceDeformerOp : public FDynamicMeshOperator
{
public:

	// Inputs
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	FFrame3d GizmoFrame;
	UE_API void SetTransform(const FTransformSRT3d& Transform);

	// The lower bound of the region of space that the operator affects, relative to the gizmo position.
	double LowerBoundsInterval;

	// The upper bound of the region of space that the operator affects, relative to the gizmo position.
	double UpperBoundsInterval;

	/**
	 * Copies over the original mesh into result mesh, and initializes ObjectToGizmo in preparation to whatever work the base class does.
	 * Note that the function will return if OriginalMesh was null but doesn't have a way to log the error, so the base class should
	 * check OriginalMesh itself as well.
	 */
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


protected:
	// transform, including translation, to gizmo space
	FMatrix ObjectToGizmo;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
