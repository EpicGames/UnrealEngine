// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshSpaceDeformerOp.h"

#define UE_API MODELINGOPERATORS_API

namespace UE
{
namespace Geometry
{

class FTwistMeshOp : public FMeshSpaceDeformerOp
{
public:
	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

	double TwistDegrees = 180;
	bool bLockBottom = false;
protected:

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
