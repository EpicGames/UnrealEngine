// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ModelingOperatorsEditorOnly : ModuleRules
{
	public ModelingOperatorsEditorOnly(ReadOnlyTargetRules Target) : base(Target)
	{		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "InteractiveToolsFramework",
                "MeshDescription",
				"StaticMeshDescription",
				"ModelingOperators",
				"GeometryCore",
				"DynamicMesh",
				"MeshConversion",
				"GeometryAlgorithms", // required for constrained Delaunay triangulation, and MikkT
				"MeshUtilitiesCommon", // required by uvlayoutop
				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",

				"MeshBuilder",
				"MeshReductionInterface", // for UE standard simplification 
				"MeshUtilities",			// for tangents calculation
			}
			);

		bool bWithProxyLOD = Target.Platform == UnrealTargetPlatform.Win64;
		PrivateDefinitions.Add("WITH_PROXYLOD=" + (bWithProxyLOD ? '1' : '0'));
		if (bWithProxyLOD)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ProxyLODMeshReduction", // currently Win64-only
				}
				);
		}
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);


		AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");

	}
}
