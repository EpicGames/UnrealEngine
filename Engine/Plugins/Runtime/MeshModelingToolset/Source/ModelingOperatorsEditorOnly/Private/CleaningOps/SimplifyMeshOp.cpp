// Copyright Epic Games, Inc. All Rights Reserved.

#include "CleaningOps/SimplifyMeshOp.h"

#include "MeshConstraints.h"
#include "MeshDescriptionToDynamicMesh.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshAttributeUtil.h"
#include "MeshReductionSettings.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "ProjectionTargets.h"

#include "IMeshReductionInterfaces.h"
#include "MeshDescription.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "GroupTopology.h"
#include "Operations/PolygroupRemesh.h"
#include "Operations/MeshClusterSimplifier.h"
#include "ConstrainedDelaunay2.h"
#include "OverlappingCorners.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimplifyMeshOp)

using namespace UE::Geometry;

template <typename SimplificationType>
void ComputeSimplify(FDynamicMesh3* TargetMesh, const bool bReproject,
					 int OriginalTriCount, FDynamicMesh3& OriginalMesh, FDynamicMeshAABBTree3& OriginalMeshSpatial,
					 EEdgeRefineFlags MeshBoundaryConstraint,
					 EEdgeRefineFlags GroupBoundaryConstraint,
					 EEdgeRefineFlags MaterialBoundaryConstraint,
					 bool bPreserveSharpEdges, bool bAllowSeamCollapse, bool bPreventNormalFlips, bool bPreventTinyTriangles,
					 const ESimplifyTargetType TargetMode,
					 const float TargetPercentage, const int TargetCount, const float TargetEdgeLength, const float TargetQuadricError,
					 const float AngleThreshold,
	                 typename SimplificationType::ESimplificationCollapseModes CollapseMode,
					 bool bUseQuadricMemory,
					 float GeometricTolerance,
					 float RegularizeWeight,
	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomGeometricErrorScaleF,
	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomEdgeLengthScaleF,
	TFunction<double(const FDynamicMesh3&, int Vertex)> CustomQuadricErrorScaleF)
{
	SimplificationType Reducer(TargetMesh);

	Reducer.ProjectionMode = (bReproject) ?
		SimplificationType::ETargetProjectionMode::AfterRefinement : SimplificationType::ETargetProjectionMode::NoProjection;

	Reducer.DEBUG_CHECK_LEVEL = 0;
	//Reducer.ENABLE_PROFILING = true;

	Reducer.bAllowSeamCollapse = bAllowSeamCollapse;
	Reducer.bRetainQuadricMemory = bUseQuadricMemory;

	Reducer.CustomGeometricErrorScaleF = CustomGeometricErrorScaleF;
	Reducer.CustomQuadricErrorScaleF = CustomQuadricErrorScaleF;
	Reducer.CustomEdgeLengthScaleF = CustomEdgeLengthScaleF;

	Reducer.RegularizeWeight = RegularizeWeight;

	if (bAllowSeamCollapse)
	{
		Reducer.SetEdgeFlipTolerance(1.e-5);

		// eliminate any bowties that might have formed on UV seams.
		if (TargetMesh->Attributes())
		{
			TargetMesh->Attributes()->SplitAllBowties();
		}
	}
	if (!bPreventNormalFlips)
	{
		Reducer.SetEdgeFlipTolerance(-1.1);
	}

	FMeshConstraints constraints;
	FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(constraints, *TargetMesh,
														 MeshBoundaryConstraint,
														 GroupBoundaryConstraint,
														 MaterialBoundaryConstraint,
														 true, !bPreserveSharpEdges, bAllowSeamCollapse);
	Reducer.SetExternalConstraints(MoveTemp(constraints));
	
	// transfer constraint setting to the simplifier, these are used to update the constraints as edges collapse.	
	Reducer.MeshBoundaryConstraint = MeshBoundaryConstraint;
	Reducer.GroupBoundaryConstraint = GroupBoundaryConstraint;
	Reducer.MaterialBoundaryConstraint = MaterialBoundaryConstraint;
	
	if (TargetMode == ESimplifyTargetType::MinimalPlanar)
	{
		Reducer.CollapseMode = SimplificationType::ESimplificationCollapseModes::AverageVertexPosition;
		GeometricTolerance = 0;		// MinimalPlanar does not allow vertices to move off the input surface
	}
	else
	{
		Reducer.CollapseMode = CollapseMode;
	}

	// use projection target if we are reprojecting or doing geometric error checking
	FMeshProjectionTarget ProjTarget(&OriginalMesh, &OriginalMeshSpatial);
	if (bReproject || GeometricTolerance > 0)
	{
		Reducer.SetProjectionTarget(&ProjTarget);
	}

	// configure geometric error settings
	if (GeometricTolerance > 0)
	{
		Reducer.GeometricErrorConstraint = SimplificationType::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
		Reducer.GeometricErrorTolerance = GeometricTolerance;
	}

	if (TargetMode == ESimplifyTargetType::Percentage)
	{
		double Ratio = (double)TargetPercentage / 100.0;
		int UseTarget = FMath::Max(4, (int)(Ratio * (double)OriginalTriCount));
		Reducer.SimplifyToTriangleCount(UseTarget);
	}
	else if (TargetMode == ESimplifyTargetType::TriangleCount)
	{
		Reducer.SimplifyToTriangleCount(TargetCount);
	}
	else if (TargetMode == ESimplifyTargetType::VertexCount)
	{
		Reducer.SimplifyToVertexCount(TargetCount);
	}
	else if (TargetMode == ESimplifyTargetType::EdgeLength)
	{
		Reducer.SimplifyToEdgeLength(TargetEdgeLength);
	}
	else if (TargetMode == ESimplifyTargetType::MinimalPlanar)
	{
		Reducer.SimplifyToMinimalPlanar(AngleThreshold);
	}
	else if (TargetMode == ESimplifyTargetType::QuadricError)
	{
		Reducer.SimplifyToMaxError(TargetQuadricError);
	}
}

bool FSimplifyMeshOp::ComputeStandardSimplifier(IMeshReduction* MeshReduction, const FMeshDescription& SrcMeshDescription, FDynamicMesh3& OutResult, 
							float PercentReduction, bool bTriBasedReduction, bool bDiscardAttributes, FProgressCancel* Progress)
{
	// Method must be called with a valid mesh reduction interface
	if (!ensure(MeshReduction))
	{
		return false;
	}

	// Spot-check the mesh description attributes, to catch mesh descriptions w/out expected static mesh attributes
	{
		const FStaticMeshConstAttributes SrcAttrs(SrcMeshDescription);
		if (!SrcAttrs.GetVertexInstanceUVs().IsValid())
		{
			UE_LOGF(LogGeometry, Error,
				"FSimplifyMeshOp::ComputeStandardSimplifier -- mesh description is missing standard static mesh attributes expected by the simplifier");
			return false;
		}
	}

	// Note: Simplifier cannot run in place, so we always need to make this copy (even if the SrcMeshDescription was temporary ...)
	FMeshDescription DstMeshDescription(SrcMeshDescription);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	FOverlappingCorners OverlappingCorners;
	FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, SrcMeshDescription, 1.e-5f);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	FMeshReductionSettings ReductionSettings;
	ReductionSettings.TerminationCriterion = bTriBasedReduction ? EStaticMeshReductionTerimationCriterion::Triangles : EStaticMeshReductionTerimationCriterion::Vertices;
	ReductionSettings.PercentTriangles = PercentReduction;
	ReductionSettings.PercentVertices = PercentReduction;

	float FoundMaxDeviation_Unused = 0.f;
	MeshReduction->ReduceMeshDescription(DstMeshDescription, FoundMaxDeviation_Unused, SrcMeshDescription, OverlappingCorners, ReductionSettings);

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Put the reduced mesh into the target...
	FMeshDescriptionToDynamicMesh Converter;
	Converter.Convert(&DstMeshDescription, OutResult);
	if (bDiscardAttributes)
	{
		OutResult.DiscardAttributes();
	}


	bool bFailedModifyNeedsRegen = false;
	// The UEStandard simplifier will split the UV boundaries.  Need to weld this.
	{
		FDynamicMesh3* ComponentMesh = &OutResult;

		FMergeCoincidentMeshEdges Merger(ComponentMesh);
		Merger.MergeSearchTolerance = 10.0f * FMathf::ZeroTolerance;
		Merger.OnlyUniquePairs = false;
		if (Merger.Apply() == false)
		{
			bFailedModifyNeedsRegen = true;
		}

		if (Progress && Progress->Cancelled())
		{
			return false;
		}

		// in the fallback case where merge edges failed, give up and reset it to what it was before the attempted merger (w/ split UV boundaries everywhere, oh well)
		if (bFailedModifyNeedsRegen)
		{
			OutResult.Clear();
			Converter.Convert(&DstMeshDescription, OutResult);
			if (bDiscardAttributes)
			{
				OutResult.DiscardAttributes();
			}
		}
	}

	return true;
}



void FSimplifyMeshOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Need access to the source mesh:
	FDynamicMesh3* TargetMesh = ResultMesh.Get();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	int OriginalTriCount = OriginalMesh->TriangleCount();
	double UseGeometricTolerance = (bGeometricDeviationConstraint) ? GeometricTolerance : 0.0;
	if (SimplifierType == ESimplifyType::QEM)
	{
		bool bUseQuadricMemory = true;
		ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
		ComputeSimplify<FQEMSimplification>(TargetMesh, bReproject, OriginalTriCount, *OriginalMesh, *OriginalMeshSpatial,
											MeshBoundaryConstraint,
											GroupBoundaryConstraint,
											MaterialBoundaryConstraint,
											bPreserveSharpEdges, bAllowSeamCollapse, bPreventNormalFlips, bPreventTinyTriangles,
											TargetMode, TargetPercentage, TargetCount, TargetEdgeLength, TargetQuadricError, MinimalPlanarAngleThresh,
											FQEMSimplification::ESimplificationCollapseModes::MinimalQuadricPositionError, bUseQuadricMemory, 
											UseGeometricTolerance,
											RegularizeWeight,
											CustomGeometricErrorScaleF,
											CustomEdgeLengthScaleF,
											CustomQuadricErrorScaleF);
	}
	else if (SimplifierType == ESimplifyType::Attribute)
	{
		bool bUseQuadricMemory = false;
		ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
		if (!ResultMesh->HasAttributes() && !ResultMesh->HasVertexNormals())
		{
			FMeshNormals::QuickComputeVertexNormals(*ResultMesh, false);
		}
		ComputeSimplify<FAttrMeshSimplification>(TargetMesh, bReproject, OriginalTriCount, *OriginalMesh, *OriginalMeshSpatial,
													MeshBoundaryConstraint,
													GroupBoundaryConstraint,
													MaterialBoundaryConstraint,
													bPreserveSharpEdges, bAllowSeamCollapse, bPreventNormalFlips, bPreventTinyTriangles,
													TargetMode, TargetPercentage, TargetCount, TargetEdgeLength, TargetQuadricError, MinimalPlanarAngleThresh,
													FAttrMeshSimplification::ESimplificationCollapseModes::MinimalQuadricPositionError, bUseQuadricMemory,
													UseGeometricTolerance,
													RegularizeWeight,
													CustomGeometricErrorScaleF,
													CustomEdgeLengthScaleF,
													CustomQuadricErrorScaleF);
	}
	else if (SimplifierType == ESimplifyType::MinimalPlanar)
	{
		bool bUseQuadricMemory = false;
		ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
		if (!ResultMesh->HasAttributes() && !ResultMesh->HasVertexNormals())
		{
			FMeshNormals::QuickComputeVertexNormals(*ResultMesh, false);
		}
		ComputeSimplify<FQEMSimplification>(TargetMesh, bReproject, OriginalTriCount, *OriginalMesh, *OriginalMeshSpatial,
			MeshBoundaryConstraint,
			GroupBoundaryConstraint,
			MaterialBoundaryConstraint,
			bPreserveSharpEdges, bAllowSeamCollapse, bPreventNormalFlips, bPreventTinyTriangles,
			ESimplifyTargetType::MinimalPlanar, TargetPercentage, TargetCount, TargetEdgeLength, TargetQuadricError, MinimalPlanarAngleThresh,
			FQEMSimplification::ESimplificationCollapseModes::MinimalQuadricPositionError, bUseQuadricMemory,
			UseGeometricTolerance,
			RegularizeWeight,
			CustomGeometricErrorScaleF,
			CustomEdgeLengthScaleF,
			CustomQuadricErrorScaleF);
	}
	else if (SimplifierType == ESimplifyType::MinimalExistingVertex)
	{
		bool bUseQuadricMemory = true;
		ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
		ComputeSimplify<FQEMSimplification>(TargetMesh, bReproject, OriginalTriCount, *OriginalMesh, *OriginalMeshSpatial,
			MeshBoundaryConstraint,
			GroupBoundaryConstraint,
			MaterialBoundaryConstraint,
			bPreserveSharpEdges, bAllowSeamCollapse, bPreventNormalFlips, bPreventTinyTriangles,
			TargetMode, TargetPercentage, TargetCount, TargetEdgeLength, TargetQuadricError, MinimalPlanarAngleThresh,
			FQEMSimplification::ESimplificationCollapseModes::MinimalExistingVertexError, bUseQuadricMemory,
			UseGeometricTolerance,
			RegularizeWeight,
			CustomGeometricErrorScaleF,
			CustomEdgeLengthScaleF,
			CustomQuadricErrorScaleF);
	}
	else if (SimplifierType == ESimplifyType::MinimalPolygroup)
	{
		ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
		FGroupTopology Topology(ResultMesh.Get(), true);
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		FPolygroupRemesh Remesh(ResultMesh.Get(), &Topology, ConstrainedDelaunayTriangulate<double>);
		Remesh.SimplificationAngleTolerance = PolyEdgeAngleTolerance;
		Remesh.Compute();
	}
	else if (SimplifierType == ESimplifyType::ClusterBased)
	{
		MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
		SimplifyOptions.TargetEdgeLength = TargetEdgeLength;
		SimplifyOptions.FixBoundaryAngleTolerance = BoundaryEdgeAngleTolerance;
		SimplifyOptions.OptionalTargetEdgeLengthScale = ClusterTargetEdgeLengthScaleF;

		auto RefineFlagsToConstraintLevel = [](EEdgeRefineFlags Flags)
		{
			if (bool(int(Flags) & (int)EEdgeRefineFlags::NoCollapse))
			{
				return MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Fixed;
			}
			else if (bool(int(Flags) & (int)EEdgeRefineFlags::NoFlip))
			{
				return MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Constrained;
			}
			return MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;
		};
		SimplifyOptions.PreserveEdges.Boundary = RefineFlagsToConstraintLevel(MeshBoundaryConstraint);
		SimplifyOptions.PreserveEdges.PolyGroup = RefineFlagsToConstraintLevel(GroupBoundaryConstraint);
		SimplifyOptions.PreserveEdges.Material = RefineFlagsToConstraintLevel(MaterialBoundaryConstraint);

		if (bDiscardAttributes)
		{
			// if discarding attributes, also discard constraints from the attribute layer
			SimplifyOptions.PreserveEdges.SetSeamConstraints(MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free);
			SimplifyOptions.PreserveEdges.Material = MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;
			SimplifyOptions.bTransferAttributes = false;
		}
		else
		{
			SimplifyOptions.PreserveEdges.SetSeamConstraints(MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Constrained);
			// typically don't actually want tangent seams to prevent simplification
			SimplifyOptions.PreserveEdges.TangentSeam = MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;
			SimplifyOptions.bTransferAttributes = true;
		}

		// drive normal seams by the bPreserveSharpEdges flag, rather than the more general bDiscardAttributes
		SimplifyOptions.PreserveEdges.NormalSeam = bPreserveSharpEdges ? 
			MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Constrained :
			MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free;
		
		MeshClusterSimplify::Simplify(*OriginalMesh, *ResultMesh, SimplifyOptions);
	}
	else // SimplifierType == ESimplifyType::UEStandard
	{
		if (!MeshReduction)
		{
			// no reduction possible, failed to load the required interface
			ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);
			return;
		}

		const FMeshDescription* SrcMeshDescription = OriginalMeshDescription.Get();

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		float PercentReduction = 1.f;
		if (TargetMode == ESimplifyTargetType::Percentage)
		{
			PercentReduction = FMath::Max(TargetPercentage / 100., .001);
		}
		else if (TargetMode == ESimplifyTargetType::TriangleCount)
		{
			int32 NumTris = SrcMeshDescription->Polygons().Num();
			PercentReduction = NumTris > 0 ? (float)TargetCount / (float)NumTris : 0.f;
		}
		else if (TargetMode == ESimplifyTargetType::VertexCount)
		{
			int32 NumVerts = SrcMeshDescription->Vertices().Num();
			PercentReduction = NumVerts > 0 ? (float)TargetCount / (float)NumVerts : 0.f;
		}
		bool bTargetIsTriangleCount = TargetMode != ESimplifyTargetType::VertexCount;
		
		if (!ComputeStandardSimplifier(MeshReduction, *SrcMeshDescription, *ResultMesh, PercentReduction, bTargetIsTriangleCount, 
			bDiscardAttributes, Progress))
		{
			return;
		}
	}


	if (Progress && Progress->Cancelled())
	{
		return;
	}

	if (!ResultMesh->HasAttributes())
	{
		FMeshNormals::QuickComputeVertexNormals(*ResultMesh);
	}

	if (!TargetMesh->HasAttributes() && bResultMustHaveAttributesEnabled)
	{
		TargetMesh->EnableAttributes();
		if (TargetMesh->HasVertexUVs())
		{
			CopyVertexUVsToOverlay(*TargetMesh, *TargetMesh->Attributes()->PrimaryUV());
		}
		if (TargetMesh->HasVertexNormals())
		{
			CopyVertexNormalsToOverlay(*TargetMesh, *TargetMesh->Attributes()->PrimaryNormals());
		}
	}
}

