// Copyright Epic Games, Inc. All Rights Reserved.

#include "CompositionOps/VoxelBooleanMeshesOp.h"

#include "MeshDescription.h"
#include "StaticMeshAttributes.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "CleaningOps/EditNormalsOp.h"


// The ProxyLOD plugin is currently only available on Windows. On other platforms we will make this a no-op.
#if WITH_PROXYLOD
#include "ProxyLODVolume.h"
#endif	// WITH_PROXYLOD

using namespace UE::Geometry;

void FVoxelBooleanMeshesOp::CalculateResult(FProgressCancel* Progress)
{
#if WITH_PROXYLOD
	
	struct FVoxelBoolInterrupter : IVoxelBasedCSG::FInterrupter
	{
		FVoxelBoolInterrupter(FProgressCancel* ProgressCancel) : Progress(ProgressCancel) {}
		FProgressCancel* Progress;
		virtual ~FVoxelBoolInterrupter(){}
		virtual bool wasInterrupted(int percent = -1) override final
		{
			bool Cancelled = Progress && Progress->Cancelled();
			return Cancelled;
		}

	} Interrupter(Progress);


	FMeshDescription ResultMeshDescription;
	FStaticMeshAttributes Attributes(ResultMeshDescription);
	Attributes.Register();

	if (VoxelCount < 16)
	{
		VoxelCount = 16;
	}

	// Use the world space bounding box of each mesh to compute the voxel size
	VoxelSizeD = ComputeVoxelSize();

	// give this an absolute min since the user might scale both objects to zero..
	VoxelSizeD = FMath::Max(VoxelSizeD, 0.001);

	// world space units.
	const double MaxIsoOffset = 2 * VoxelSizeD;

	// IsoSurfaceD is in world units

	// Note after the min/max operations of voxel-based CSG the interior distances or exterior 
	// distances can be pretty f'd up so we need to do the initial post-CSG remesh with a "safe"
	// value.
	double CSGIsoSurface = 0.;

	
	// Create BooleanTool and Boolean the meshes.
	TUniquePtr<IVoxelBasedCSG> VoxelCSGTool = IVoxelBasedCSG::CreateCSGTool(VoxelSizeD);
	FVector MergedOrigin;

	// VoxelCSGTool needs MeshDescription
	FMeshDescription MeshDescriptions[2];
	for (int i = 0; i < 2; ++i)
	{

		FStaticMeshAttributes AddAttributes(MeshDescriptions[i]);
		AddAttributes.Register();
		
		FConversionToMeshDescriptionOptions ToMeshDescriptionOptions;
		ToMeshDescriptionOptions.bSetPolyGroups = false;
		FDynamicMeshToMeshDescription DynamicMeshToMeshDescription(ToMeshDescriptionOptions);

		DynamicMeshToMeshDescription.Convert(Meshes[i].Get(), MeshDescriptions[i]);
	}

	IVoxelBasedCSG::FPlacedMesh A(&MeshDescriptions[0], Transforms[0]);
	IVoxelBasedCSG::FPlacedMesh B(&MeshDescriptions[1], Transforms[1]);

	switch (Operation)
	{
	case EBooleanOperation::DifferenceAB:
		CSGIsoSurface = 0.;
		VoxelCSGTool->ComputeDifference(Interrupter, A, B, ResultMeshDescription, MergedOrigin, AdaptivityD, CSGIsoSurface);
		break;
	case EBooleanOperation::DifferenceBA:
		CSGIsoSurface = 0.;
		VoxelCSGTool->ComputeDifference(Interrupter, B, A, ResultMeshDescription, MergedOrigin, AdaptivityD, CSGIsoSurface);
		break;
	case EBooleanOperation::Intersect:
		CSGIsoSurface = FMath::Clamp(IsoSurfaceD, -MaxIsoOffset, MaxIsoOffset);
		VoxelCSGTool->ComputeIntersection(Interrupter, A, B, ResultMeshDescription, MergedOrigin, AdaptivityD, CSGIsoSurface);
		break;
	case EBooleanOperation::Union:
		CSGIsoSurface = FMath::Clamp(IsoSurfaceD, 0., MaxIsoOffset);
		VoxelCSGTool->ComputeUnion(Interrupter, A, B, ResultMeshDescription, MergedOrigin, AdaptivityD, CSGIsoSurface);
		break;
	default:
		check(0);
	}

	ResultTransform = FTransform3d(MergedOrigin);

	if (Interrupter.wasInterrupted())
	{
		return;
	}

	const bool bNeedAdditionalRemeshing = (CSGIsoSurface != IsoSurfaceD);

	// iteratively expand (or contract) the surface
	if ( bNeedAdditionalRemeshing )
	{
		// how additional remeshes do we need?
		int32 NumRemeshes = FMath::CeilToInt( FMath::Abs(IsoSurfaceD - CSGIsoSurface) / MaxIsoOffset );

		double DeltaIsoSurface = (IsoSurfaceD - CSGIsoSurface) / NumRemeshes;

		for (int32 i = 0; i < NumRemeshes; ++i)
		{
			if (Interrupter.wasInterrupted())
			{
				return;
			}

			IVoxelBasedCSG::FPlacedMesh PlacedMesh;
			PlacedMesh.Mesh = &ResultMeshDescription;
			PlacedMesh.Transform = static_cast<FTransform>(ResultTransform);

			TArray< IVoxelBasedCSG::FPlacedMesh> SourceMeshes;
			SourceMeshes.Add(PlacedMesh);

			// union of only one mesh. we are using this to voxelize and remesh with and offset.
			VoxelCSGTool->ComputeUnion(Interrupter, SourceMeshes, ResultMeshDescription, MergedOrigin, AdaptivityD, DeltaIsoSurface);

			ResultTransform = FTransform3d(MergedOrigin);
		}
	}
	
	if(!Interrupter.wasInterrupted())
	{
		// Convert to dynamic mesh
		FMeshDescriptionToDynamicMesh Converter;
		Converter.Convert(&ResultMeshDescription, *ResultMesh);
	}
	else
	{
		return;
	}

	// for now we automatically fix up the normals if simplificaiton was requested.
	bool bFixNormals = bAutoSimplify;

	if (bAutoSimplify)
	{
		FQEMSimplification Reducer(ResultMesh.Get());
		FMeshConstraints constraints;
		FMeshConstraintsUtil::ConstrainAllSeams(constraints, *ResultMesh, true, false);
		Reducer.SetExternalConstraints(MoveTemp(constraints));
		Reducer.Progress = Progress;

		const double MaxDisplacementSqr = 3. * VoxelSizeD * VoxelSizeD;

		Reducer.SimplifyToMaxError(MaxDisplacementSqr);

	}


	if (bFixNormals && !Interrupter.wasInterrupted())
	{
		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OpResultMesh(ExtractResult().Release()); // moved the unique pointer

		// Recompute the normals
		FEditNormalsOp EditNormalsOp;
		EditNormalsOp.OriginalMesh = OpResultMesh; // the tool works on a deep copy of this mesh.
		EditNormalsOp.bFixInconsistentNormals = true;
		EditNormalsOp.bInvertNormals = false;
		EditNormalsOp.bRecomputeNormals = true;
		EditNormalsOp.NormalCalculationMethod = ENormalCalculationMethod::AreaAngleWeighting;
		EditNormalsOp.SplitNormalMethod = ESplitNormalMethod::FaceNormalThreshold;
		EditNormalsOp.bAllowSharpVertices = true;
		EditNormalsOp.NormalSplitThreshold = 60.f;

		EditNormalsOp.SetTransform(FTransform(ResultTransform));

		EditNormalsOp.CalculateResult(Progress);

		ResultMesh = EditNormalsOp.ExtractResult(); // return the edit normals operator copy to this tool.
	}
	
#else	// WITH_PROXYLOD
		// just copy the first input
		*ResultMesh = *Meshes[0];
		ResultTransform = Transforms[0];
#endif	// WITH_PROXYLOD

}

float FVoxelBooleanMeshesOp::ComputeVoxelSize() const 
{ 
	float Size = 0.f;
	auto GrowSize = [&Size, this](const FDynamicMesh3& DynamicMesh, const FTransformSRT3d& Xform)
	{
		FAxisAlignedBox3d AABB = DynamicMesh.GetBounds(true);
		FVector Scale = Xform.GetScale3D();
		FVector Extents = 2. * AABB.Extents();
		// Scale with the local space scale.
		Extents.X = Extents.X * FMath::Abs(Scale.X);
		Extents.Y = Extents.Y * FMath::Abs(Scale.Y);
		Extents.Z = Extents.Z * FMath::Abs(Scale.Z);

		float MajorAxisSize = FMath::Max3(Extents.X, Extents.Y, Extents.Z);
		Size = FMath::Max(MajorAxisSize / VoxelCount, Size);
	};

	GrowSize(*Meshes[0], Transforms[0]);
	GrowSize(*Meshes[1], Transforms[1]);
	
	
	return Size;

}
