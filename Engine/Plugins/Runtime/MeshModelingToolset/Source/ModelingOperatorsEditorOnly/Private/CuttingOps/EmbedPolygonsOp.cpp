// Copyright Epic Games, Inc. All Rights Reserved.

#include "CuttingOps/EmbedPolygonsOp.h"

#include "DynamicMeshEditor.h"
#include "Selections/MeshFaceSelection.h"
#include "MeshQueries.h"
#include "Operations/EmbedSurfacePath.h"
#include "Operations/MinimalHoleFiller.h"
#include "Generators/SweepGenerator.h"

#include "MeshDescriptionToDynamicMesh.h"
#include "MeshSimplification.h"
#include "Operations/MeshPlaneCut.h"
#include "Operations/MeshBoolean.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Curve/GeneralPolygon2.h"
#include "Curve/PolygonIntersectionUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EmbedPolygonsOp)

using namespace UE::Geometry;

void CollapseDegenerateEdgesOnVertexPath(FDynamicMesh3& Mesh, TArray<int>& VertexIDsIO, TArray<int>& PathVertCorrespondIO)
{
	// similar to the CollapseDegenerateEdges in FMeshPlaneCut, but tailored to this use case
	// maintains the vertex ID correspondence to original verts across the update
	TArray<int> VertexIDs = VertexIDsIO; // copy of inputs IDs
	TMultiMap<int, int> VertexIDToPathVertIdx;
	for (int PathIdx = 0; PathIdx < PathVertCorrespondIO.Num(); PathIdx++)
	{
		VertexIDToPathVertIdx.Add(VertexIDs[PathVertCorrespondIO[PathIdx]], PathIdx);
	}

	// build edge set directly rather than use FEdgeLoop because (1) we want a set, (2) we want to forgive edges not being there (not check() on that case)
	TSet<int> Edges;
	for (int LastIdx = VertexIDs.Num() - 1, Idx = 0; Idx < VertexIDs.Num(); LastIdx = Idx++)
	{
		int EID = Mesh.FindEdge(VertexIDs[LastIdx], VertexIDs[Idx]);
		if (EID >= 0)
		{
			Edges.Add(EID);
		}
	}

	const double DegenerateEdgeTol = .1;
	double Tol2 = DegenerateEdgeTol * DegenerateEdgeTol;
	FVector3d A, B;
	int Collapsed = 0;
	TArray<int> FoundValues;
	do
	{
		Collapsed = 0;
		for (int EID : Edges)
		{
			if (!Mesh.IsEdge(EID))
			{
				continue;
			}
			Mesh.GetEdgeV(EID, A, B);
			double DSq = DistanceSquared(A,B);
			if (DSq > Tol2)
			{
				continue;
			}

			FIndex2i EV = Mesh.GetEdgeV(EID);
			// if the vertex we'd remove is on a seam, try removing the other one instead
			if (Mesh.HasAttributes() && Mesh.Attributes()->IsSeamVertex(EV.B, false))
			{
				Swap(EV.A, EV.B);
				// if they were both on seams, then collapse should not happen?  (& would break OnCollapseEdge assumptions in overlay)
				if (Mesh.HasAttributes() && Mesh.Attributes()->IsSeamVertex(EV.B, false))
				{
					continue;
				}
			}
			FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
			EMeshResult Result = Mesh.CollapseEdge(EV.A, EV.B, CollapseInfo);
			if (Result == EMeshResult::Ok)
			{
				// move everything mapped on the removed vertex over to the kept vertex
				if (VertexIDToPathVertIdx.Contains(CollapseInfo.RemovedVertex))
				{
					FoundValues.Reset();
					VertexIDToPathVertIdx.MultiFind(CollapseInfo.RemovedVertex, FoundValues);
					for (int V : FoundValues)
					{
						VertexIDToPathVertIdx.Add(CollapseInfo.KeptVertex, V);
					}
					VertexIDToPathVertIdx.Remove(CollapseInfo.RemovedVertex);
				}
				Collapsed++;
			}
		}
	}
	while (Collapsed > 0);

	// update vertex ids array and correspondence from input path indices to vertex ids array indices
	VertexIDsIO.Reset();
	for (int VID : VertexIDs)
	{
		if (Mesh.IsVertex(VID))
		{
			int NewIdx = VertexIDsIO.Num();
			VertexIDsIO.Add(VID);
			FoundValues.Reset();
			VertexIDToPathVertIdx.MultiFind(VID, FoundValues);
			for (int V : FoundValues)
			{
				PathVertCorrespondIO[V] = NewIdx;
			}
		}
	}
}

TArray<FPolygon2d> FEmbedPolygonsOp::CleanPolygon(const FPolygon2d& Input)
{
	TArray<FPolygon2d> Result;
	// Attempt to clean self-intersections by applying a 'self-union' operation
	FGeneralPolygon2d GeneralPolygon(Input);
	FGeneralPolygon2d Empty;
	TUnionPolygon2Polygon2<FGeneralPolygon2d, double> Union(GeneralPolygon, Empty);
	bool bResolveSuccess = Union.ComputeResult();
	if (bResolveSuccess && !Union.Result.IsEmpty())
	{
		// Just take the outer polygons for now, as the embed code can't handle holes
		for (int32 Idx = 0; Idx < Union.Result.Num(); ++Idx)
		{
			Result.Add(Union.Result[Idx].GetOuter());
		}
	}
	else
	{
		// If we didn't get a result, just fall back to using the input polygon
		Result.Add(Input);
	}
	return Result;
}

void FEmbedPolygonsOp::BooleanPath(FProgressCancel* Progress)
{
	if (ResultMesh->TriangleCount() == 0 || ResultMesh->VertexCount() == 0)
	{
		return;
	}

	FAxisAlignedBox3d Bounds = OriginalMesh->GetBounds();
	double MeshDiameter = Bounds.MaxDim();

	FFrame3d Frame = PolygonFrame;

	FVector3d Dir = Frame.Z();
	FInterval1d Range;
	for (int CornerIdx = 0; CornerIdx < 8; CornerIdx++)
	{
		FVector3d Corner = Bounds.GetCorner(CornerIdx);
		Range.Contain(Dir.Dot(Corner - Frame.Origin));
	}
	// expand a little beyond bounds to avoid creating coplanar cases
	Range.Expand(1.0);

	Frame.Origin = Frame.Origin + Range.Min * Dir;
	
	TArray<FPolygon2d> CutPolygons = CleanPolygon(GetPolygon());
	
	FDynamicMesh3 ExtrudedPolygons;
	ExtrudedPolygons.EnableMeshComponents(ResultMesh->GetComponentsFlags());
	if (ResultMesh->HasAttributes())
	{
		ExtrudedPolygons.EnableAttributes();
		ExtrudedPolygons.Attributes()->EnableMatchingAttributes(*ResultMesh->Attributes());
	}
	FDynamicMeshEditor AppendEditor(&ExtrudedPolygons);
	FMeshIndexMappings IndexMaps_Unused;
	
	for (const FPolygon2d& CutPolygon : CutPolygons)
	{
		FGeneralizedCylinderGenerator ExtrudePolyGen;
		ExtrudePolyGen.CrossSection = CutPolygon;
		ExtrudePolyGen.InitialFrame = Frame;
		ExtrudePolyGen.Path.Add(Frame.Origin);
		ExtrudePolyGen.Path.Add(PolygonFrame.Origin + Range.Max * Dir);
		ExtrudePolyGen.bCapped = true;
		ExtrudePolyGen.bUVScaleRelativeWorld = true;
		ExtrudePolyGen.UnitUVInWorldCoordinates = MeshDiameter;
		FDynamicMesh3 ExtrudePoly;
		ExtrudePoly.EnableMeshComponents(ResultMesh->GetComponentsFlags());
		if (ResultMesh->HasAttributes())
		{
			ExtrudePoly.EnableAttributes();
			ExtrudePoly.Attributes()->EnableMatchingAttributes(*ResultMesh->Attributes());
		}
		ExtrudePoly.Copy(&ExtrudePolyGen.Generate());
		AppendEditor.AppendMesh(&ExtrudePoly, IndexMaps_Unused);
	}
	FMeshBoolean::EBooleanOp BoolOp = FMeshBoolean::EBooleanOp::Difference;
	switch (Operation)
	{
	case EEmbeddedPolygonOpMethod::InsertPolygon:
		BoolOp = FMeshBoolean::EBooleanOp::NewGroupInside;
		break;
	case EEmbeddedPolygonOpMethod::TrimInside:
		BoolOp = FMeshBoolean::EBooleanOp::TrimInside;
		break;
	case EEmbeddedPolygonOpMethod::TrimOutside:
		BoolOp = FMeshBoolean::EBooleanOp::TrimOutside;
		break;
	case EEmbeddedPolygonOpMethod::CutThrough:
		BoolOp = FMeshBoolean::EBooleanOp::Difference;
		break;
	case EEmbeddedPolygonOpMethod::CutOutside:
		BoolOp = FMeshBoolean::EBooleanOp::Intersect;
		break;
	default:
		unimplemented();
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	int MaxGroupID = ResultMesh->MaxGroupID();
	FMeshBoolean Boolean(ResultMesh.Get(), &ExtrudedPolygons, ResultMesh.Get(), BoolOp);
	Boolean.Progress = Progress;
	Boolean.bSimplifyAlongNewEdges = true;
	Boolean.bPutResultInInputSpace = true;
	Boolean.Compute();

	// One might think that we would want to set this to the return value of Boolean.Compute() above,
	// but that call currently returns false whenever any boundary edges are generated in a non-trim op. That type of
	// "failure" is frequently ok for the user, for instance when cutting an open mesh, like a simple rectangle. 
	// We should probably change FMeshBoolean to differentiate between types of errors so that we can still
	// respond appropriately to other types of failures. For now, this is what we do here (and to some
	// extent, in BooleanMeshesOp.cpp).
	bOperationSucceeded = true;

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	// Deal with created boundary edges
	if (Operation == EEmbeddedPolygonOpMethod::TrimInside || Operation == EEmbeddedPolygonOpMethod::TrimOutside)
	{
		EmbeddedEdges = Boolean.CreatedBoundaryEdges;
	}
	else if (Boolean.CreatedBoundaryEdges.Num() > 0 && bAttemptFixHolesOnBoolean)
	{
		FMeshBoundaryLoops OpenBoundary(Boolean.Result, false);
		TSet<int> ConsiderEdges(Boolean.CreatedBoundaryEdges);
		OpenBoundary.EdgeFilterFunc = [&ConsiderEdges](int EID)
		{
			return ConsiderEdges.Contains(EID);
		};
		OpenBoundary.Compute();

		if (Progress && Progress->Cancelled())
		{
			return;
		}

		// This hole filling code should probably be put into some common utility function since it's used in
		// BooleanMeshesOp.h, and we do similar things (with more options) for cap filling in revolves.
		const bool bNeedsNormalsAndUVs = Boolean.Result->HasAttributes();
		// UV scale based on the mesh bounds, used for hole-filled regions.
		const double MeshUVScaleFactor = (1.0 / Boolean.Result->GetBounds().MaxDim());

		for (FEdgeLoop& Loop : OpenBoundary.Loops)
		{
			FMinimalHoleFiller Filler(Boolean.Result, Loop);
			Filler.Fill(ResultMesh->AllocateTriangleGroup());

			if (bNeedsNormalsAndUVs)
			{
				// Compute a best-fit plane of the boundary vertices
				TArray<FVector3d> VertexPositions;
				Loop.GetVertices(VertexPositions);

				FVector3d PlaneOrigin, PlaneNormal;
				PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);
				PlaneNormal *= -1.0;	// Previous function seems to orient the normal opposite to what's expected elsewhere

				FDynamicMeshEditor Editor(ResultMesh.Get());
				FFrame3d ProjectionFrame(PlaneOrigin, PlaneNormal);
				Editor.SetTriangleNormals(Filler.NewTriangles);
				Editor.SetTriangleUVsFromProjection(Filler.NewTriangles, ProjectionFrame, MeshUVScaleFactor);
			}
		}
		for (int EID : Boolean.CreatedBoundaryEdges)
		{
			if (Boolean.Result->IsEdge(EID) && Boolean.Result->IsBoundaryEdge(EID))
			{
				EdgesOnFailure.Add(EID);
			}
		}
	}
	else
	{
		EdgesOnFailure = Boolean.CreatedBoundaryEdges;
	}

	// Non-mesh-boundary cut boundary edges can be detected by the group change across the cut. Not relevant to trim
	// operations, where we get them from the created boundary edges above.
	if (Operation == EEmbeddedPolygonOpMethod::CutOutside || Operation == EEmbeddedPolygonOpMethod::CutThrough || Operation == EEmbeddedPolygonOpMethod::InsertPolygon)
	{
		for (int EID : ResultMesh->EdgeIndicesItr())
		{
			FDynamicMesh3::FEdge Edge = ResultMesh->GetEdge(EID);
			if (ResultMesh->GetTriangleGroup(Edge.Tri[0]) < MaxGroupID !=
				ResultMesh->GetTriangleGroup(Edge.Tri[1]) < MaxGroupID)
			{
				EmbeddedEdges.Add(EID);
			}
		}
	}
}

void FEmbedPolygonsOp::CalculateResult(FProgressCancel* Progress)
{
	bOperationSucceeded = false;

	if (Progress && Progress->Cancelled())
	{
		return;
	}
	ResultMesh->Copy(*OriginalMesh, true, true, true, !bDiscardAttributes);

	if (bCutWithBoolean)
	{
		BooleanPath(Progress);
		return;
	}

	double MeshDiameter = OriginalMesh->GetBounds().MaxDim();
	double UVScaleFactor = 1.0 / MeshDiameter;

	bool bCollapseDegenerateEdges = true; // TODO make this optional?
	
	FFrame3d Frame = PolygonFrame;
	Frame.Origin = Frame.Origin + (2*MeshDiameter*Frame.Z());

	TArray<FPolygon2d> CutPolygons = CleanPolygon(GetPolygon());

	if (!ensure(!CutPolygons.IsEmpty()))
	{
		return;
	}
	
	// Note: This (non-boolean) path cannot handle multiple cutting polygons (it will almost always fail to cut) so we just take the first polygon here
	FPolygon2d Polygon = CutPolygons[0];
	
	TArray<TPair<float, int>> SortedHitTriangles;
	TMeshQueries<FDynamicMesh3>::FindHitTriangles_LinearSearch(*ResultMesh, FRay3d(Frame.FromPlaneUV(Polygon[0]), -Frame.Z()), SortedHitTriangles);

	if (SortedHitTriangles.Num() < 1)
	{
		// didn't hit the mesh 
		return;
	}

	int SecondHit = 1;
	if (Operation == EEmbeddedPolygonOpMethod::CutThrough || Operation == EEmbeddedPolygonOpMethod::CutOutside)
	{
		while (SecondHit < SortedHitTriangles.Num() && FMath::IsNearlyEqual(SortedHitTriangles[SecondHit].Key, SortedHitTriangles[0].Key))
		{
			SecondHit++;
		}
		if (SecondHit >= SortedHitTriangles.Num())
		{
			// failed to find a second surface to connect to
			SecondHit = -1;
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	enum EDeleteMethod
	{
		DeleteNone,
		DeleteInside,
		DeleteOutside
	};

	auto CutAllHoles = [](FDynamicMesh3& Mesh, const FFrame3d& F, const TArray<int>& TriStarts, const FPolygon2d& Polygon, EDeleteMethod DeleteMethod, TArray<TArray<int>>& AllPathVertIDs, TArray<TArray<int>>& AllPathVertCorrespond, bool bCollapseDegenerateEdges)
	{
		for (int TriStart : TriStarts)
		{
			if (!Mesh.IsTriangle(TriStart))
			{
				return false;
			}
		}
		FMeshFaceSelection Selection(&Mesh);

		TArray<TArray<FVector2d>> PolygonPaths;
		for (int PathIdx = 0; PathIdx < TriStarts.Num(); PathIdx++)
		{
			PolygonPaths.Add(Polygon.GetVertices());
		}
		bool bDidEmbed = EmbedProjectedPaths(&Mesh, TriStarts, F, PolygonPaths, AllPathVertIDs, AllPathVertCorrespond, true, &Selection);
		if (!bDidEmbed)
		{
			return false;
		}

		check(PolygonPaths.Num() == AllPathVertIDs.Num());
		check(AllPathVertIDs.Num() == AllPathVertCorrespond.Num());

		FDynamicMeshEditor MeshEditor(&Mesh);
		if (DeleteMethod != EDeleteMethod::DeleteNone)
		{
			bool bDidRemove;
			if (DeleteMethod == EDeleteMethod::DeleteOutside)
			{
				if (Selection.Num() == 0)
				{
					return false; // refuse to delete the entire mesh w/ a hole cut
				}
				TArray<int> InvSelection;
				InvSelection.Reserve(Mesh.TriangleCount() - Selection.Num());
				for (int TID : Mesh.TriangleIndicesItr())
				{
					if (!Selection.IsSelected(TID))
					{
						InvSelection.Add(TID);
					}
				}
				bDidRemove = MeshEditor.RemoveTriangles(InvSelection, true);
			}
			else
			{
				if (Selection.Num() == Mesh.TriangleCount())
				{
					return false; // refuse to delete the entire mesh w/ a hole cut
				}
				bDidRemove = MeshEditor.RemoveTriangles(Selection.AsArray(), true);
			}
			if (!bDidRemove)
			{
				return false;
			}
		}
		else
		{
			int GID = Mesh.AllocateTriangleGroup();
			for (int TID : Selection)
			{
				Mesh.SetTriangleGroup(TID, GID);
			}
		}

		// remove triangles could have removed a path vertex entirely in weird cases; just consider that a failure
		for (const TArray<int>& PathVertIDs : AllPathVertIDs)
		{
			for (int VID : PathVertIDs)
			{
				if (!Mesh.IsVertex(VID))
				{
					return false;
				}
			}
		}

		// collapse degenerate edges if we got em
		if (bCollapseDegenerateEdges)
		{
			for (int PathIdx = 0; PathIdx < AllPathVertIDs.Num(); PathIdx++)
			{
				CollapseDegenerateEdgesOnVertexPath(Mesh, AllPathVertIDs[PathIdx], AllPathVertCorrespond[PathIdx]);
			}
		}

		// For hole cut to be counted as success, hole cut vertices must be valid, unique, and correspond to valid boundary edges
		for (const TArray<int>& PathVertIDs : AllPathVertIDs)
		{
			TSet<int> SeenVIDs;
			for (int LastIdx = PathVertIDs.Num() - 1, Idx = 0; Idx < PathVertIDs.Num(); LastIdx = Idx++)
			{
				check(Mesh.IsVertex(PathVertIDs[Idx])); // collapse shouldn't leave invalid verts in, and we check + fail out on invalid verts above that, so seeing them here should be impossible
				int EID = Mesh.FindEdge(PathVertIDs[LastIdx], PathVertIDs[Idx]);
				if (!Mesh.IsEdge(EID) || (!Mesh.IsBoundaryEdge(EID) && DeleteMethod != EDeleteMethod::DeleteNone))
				{
					return false;
				}
				if (SeenVIDs.Contains(PathVertIDs[Idx]))
				{
					return false;
				}
				SeenVIDs.Add(PathVertIDs[Idx]);
			}
		}

		return true;
	};

	auto CutHole = [CutAllHoles](FDynamicMesh3& Mesh, const FFrame3d& F, int TriStart, const FPolygon2d& PolygonArg, EDeleteMethod DeleteMethod, TArray<int>& PathVertIDs, TArray<int>& PathVertCorrespond, bool bCollapseDegenerateEdges)
	{
		TArray<int> TriStarts;
		TriStarts.Add(TriStart);
		TArray<TArray<int>> AllPathVertIDs, AllPathVertCorrespond;
		bool bResult = CutAllHoles(Mesh, F, TriStarts, PolygonArg, DeleteMethod, AllPathVertIDs, AllPathVertCorrespond, bCollapseDegenerateEdges);
		if (bResult && ensure(AllPathVertIDs.Num() == 1 && AllPathVertCorrespond.Num() ==1))
		{
			PathVertIDs = AllPathVertIDs[0];
			PathVertCorrespond = AllPathVertCorrespond[0];
		}
		return bResult;
	};

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	EDeleteMethod DeleteMethod = EDeleteMethod::DeleteInside;
	if (Operation == EEmbeddedPolygonOpMethod::InsertPolygon)
	{
		DeleteMethod = EDeleteMethod::DeleteNone;
	}
	else if (Operation == EEmbeddedPolygonOpMethod::TrimOutside || Operation == EEmbeddedPolygonOpMethod::CutOutside)
	{
		DeleteMethod = EDeleteMethod::DeleteOutside;
	}

	bool bIsCutOperation = Operation == EEmbeddedPolygonOpMethod::CutThrough || Operation == EEmbeddedPolygonOpMethod::CutOutside;
	if (!bIsCutOperation || SecondHit == -1)
	{
		TArray<int> PathVertIDs, PathVertCorrespond;
		bool bCutSide1 = CutHole(*ResultMesh, Frame, SortedHitTriangles[0].Value, Polygon, DeleteMethod, PathVertIDs, PathVertCorrespond, bCollapseDegenerateEdges);
		RecordEmbeddedEdges(PathVertIDs);
		if (!bCutSide1 || PathVertIDs.Num() < 2)
		{
			EdgesOnFailure = EmbeddedEdges;
			return;
		}
	}
	else // bIsCutOperation == true
	{
		TArray<int> HitTris;
		HitTris.Add(SortedHitTriangles[0].Value);
		HitTris.Add(SortedHitTriangles[SecondHit].Value);
		TArray<TArray<int>> AllPathVertIDs, AllPathVertCorrespond;
		bool bCutSide2 = CutAllHoles(*ResultMesh, Frame, HitTris, Polygon, DeleteMethod, AllPathVertIDs, AllPathVertCorrespond, bCollapseDegenerateEdges);
		RecordEmbeddedEdges(AllPathVertIDs[0]);
		RecordEmbeddedEdges(AllPathVertIDs[1]);
		if (!bCutSide2 || AllPathVertIDs[0].Num() < 2 || AllPathVertIDs[1].Num() < 2)
		{
			EdgesOnFailure = EmbeddedEdges;
			return;
		}
		FDynamicMeshEditor MeshEditor(ResultMesh.Get());
		FDynamicMeshEditResult ResultOut;
		bool bReverseOrientation = Operation == EEmbeddedPolygonOpMethod::CutOutside;
		bool bStitched = MeshEditor.StitchSparselyCorrespondedVertexLoops(AllPathVertIDs[0], AllPathVertCorrespond[0], AllPathVertIDs[1], AllPathVertCorrespond[1], ResultOut, bReverseOrientation);
		if (!bStitched)
		{
			// Don't set bOperationSucceeded to true
			return;
		}
		if (ResultMesh->HasAttributes())
		{
			MeshEditor.SetTubeNormals(ResultOut.NewTriangles, AllPathVertIDs[0], AllPathVertCorrespond[0], AllPathVertIDs[1], AllPathVertCorrespond[1], bReverseOrientation);
			TArray<float> UValues; UValues.SetNumUninitialized(AllPathVertCorrespond[1].Num() + 1);
			FVector3f ZVec = -(FVector3f)Frame.Z();
			float Along = 0;
			for (int UIdx = 0; UIdx < UValues.Num(); UIdx++)
			{
				UValues[UIdx] = Along;
				Along += Distance(Polygon[UIdx % Polygon.VertexCount()], Polygon[(UIdx + 1) % Polygon.VertexCount()]);
			}

			for (int UVIdx = 0, NumUVLayers = ResultMesh->Attributes()->NumUVLayers(); UVIdx < NumUVLayers; UVIdx++)
			{
				MeshEditor.SetGeneralTubeUVs(ResultOut.NewTriangles,
					AllPathVertIDs[0], AllPathVertCorrespond[0], AllPathVertIDs[1], AllPathVertCorrespond[1],
					UValues, ZVec,
					UVScaleFactor, FVector2f::Zero(), UVIdx
				);
			}
		}
	}
	// TODO: later perhaps revive this hole fill code?
	//       But for now CutAndFill has been conceptually replaced with "embed polygon," which is much more useful
	//else if (Operation == EEmbeddedPolygonOpMethod::CutAndFill)
	//{
		//FDynamicMeshEditor MeshEditor(ResultMesh.Get());
		//FEdgeLoop Loop(ResultMesh.Get());
		//Loop.InitializeFromVertices(PathVertIDs1);

		//
		//FSimpleHoleFiller Filler(ResultMesh.Get(), Loop);
		//int GID = ResultMesh->AllocateTriangleGroup();
		//if (Filler.Fill(GID))
		//{
		//	if (ResultMesh->HasAttributes())
		//	{
		//		MeshEditor.SetTriangleNormals(Filler.NewTriangles, (FVector3f)Frame.Z()); // TODO: should we use the best fit plane instead of the projection plane for the normal? ... probably.
		//		
		//		for (int UVIdx = 0, NumUVLayers = ResultMesh->Attributes()->NumUVLayers(); UVIdx < NumUVLayers; UVIdx++)
		//		{
		//			MeshEditor.SetTriangleUVsFromProjection(Filler.NewTriangles, Frame, UVScaleFactor, FVector2f::Zero(), true, UVIdx);
		//		}
		//	}
		//}
	//}

	bOperationSucceeded = true;
}

void FEmbedPolygonsOp::RecordEmbeddedEdges(TArray<int>& PathVertIDs)
{
	for (int LastIdx = PathVertIDs.Num() - 1, Idx = 0; Idx < PathVertIDs.Num(); LastIdx = Idx++)
	{
		int A = PathVertIDs[LastIdx];
		int B = PathVertIDs[Idx];
		if (ResultMesh->IsVertex(A) && ResultMesh->IsVertex(B))
		{
			int EID = ResultMesh->FindEdge(A, B);
			if (ResultMesh->IsEdge(EID))
			{
				EmbeddedEdges.Add(EID);
			}
		}
	}
}
