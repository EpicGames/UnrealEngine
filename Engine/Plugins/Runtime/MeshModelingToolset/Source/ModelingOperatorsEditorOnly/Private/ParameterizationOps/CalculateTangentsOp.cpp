// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizationOps/CalculateTangentsOp.h"
#include "DynamicMeshMikkTWrapper.h"
#include "VectorUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CalculateTangentsOp)

using namespace UE::Geometry;

void FCalculateTangentsOp::CalculateResult(FProgressCancel* Progress)
{
	if (Progress && Progress->Cancelled())
	{
		return;
	}

	TUniquePtr<FMeshTangentsd> Tangents = MakeUnique<FMeshTangentsd>(SourceMesh.Get());

	if (SourceMesh->HasAttributes() == false
		|| SourceMesh->Attributes()->NumUVLayers() < 1
		|| SourceMesh->Attributes()->NumNormalLayers() == 0)
	{
		bNoAttributesError = true;
		Tangents->InitializeTriVertexTangents(true);
		SetResult(MoveTemp(Tangents));
		return;
	}

	if (TargetUVLayer < 0 || TargetUVLayer >= SourceMesh->Attributes()->NumUVLayers())
	{
		int32 ClampedLayer = FMath::Clamp(TargetUVLayer, 0, SourceMesh->Attributes()->NumUVLayers() - 1);
		UE_LOGF(LogGeometry, Warning, "Tangents compute requested invalid UV layer: %d. Using layer %d instead", TargetUVLayer, ClampedLayer);
		TargetUVLayer = ClampedLayer;
	}

	switch (CalculationMethod)
	{

	default:
	case EMeshTangentsType::FastMikkTSpace:
		CalculateStandard(Progress, Tangents);
		break;

	case EMeshTangentsType::PerTriangle:
		CalculateStandard(Progress, Tangents);
		break;

	case EMeshTangentsType::MikkTSpace:
#if WITH_MIKKTSPACE
		CalculateMikkT(Progress, Tangents);
#else
		CalculateStandard(Progress, Tangents);
#endif
		break;


	case EMeshTangentsType::CopyExisting:
		CopyFromSource(Progress, Tangents);
		break;
	}

	SetResult(MoveTemp(Tangents));
}


void FCalculateTangentsOp::CalculateStandard(FProgressCancel* Progress, TUniquePtr<FMeshTangentsd>& Tangents)
{
	const FDynamicMeshNormalOverlay* NormalOverlay = SourceMesh->Attributes()->PrimaryNormals();
	const FDynamicMeshUVOverlay* UVOverlay = SourceMesh->Attributes()->GetUVLayer(TargetUVLayer);

	FComputeTangentsOptions Options;
	Options.bAveraged = (CalculationMethod != EMeshTangentsType::PerTriangle);

	Tangents->ComputeTriVertexTangents(NormalOverlay, UVOverlay, Options);
}


void FCalculateTangentsOp::CopyFromSource(FProgressCancel* Progress, TUniquePtr<FMeshTangentsd>& Tangents)
{
	int32 NumTangents = SourceTangents->GetTangents().Num();
	if (NumTangents != SourceMesh->MaxTriangleID() * 3)
	{
		ensure(NumTangents == 0); // Expect this case to happen only when there aren't any tangents
		bNoAttributesError = true;
		Tangents->InitializeTriVertexTangents(true);
		return;
	}

	Tangents->CopyTriVertexTangents(*SourceTangents);
}


void FCalculateTangentsOp::CalculateMikkT(FProgressCancel* Progress, TUniquePtr<FMeshTangentsd>& Tangents)
{
	DynamicMeshMikkTWrapper::ComputeTangents(*Tangents, TargetUVLayer);
}
