// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParameterizationOps/ParameterizeMeshOp.h"
#include "Properties/ParameterizeMeshProperties.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Selections/MeshConnectedComponents.h"

#include "Parameterization/MeshLocalParam.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/PatchBasedMeshUVGenerator.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicSubmesh3.h"
#include "XAtlasWrapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ParameterizeMeshOp)

// The ProxyLOD plugin is currently only available on Windows.
#if WITH_PROXYLOD
#include "ProxyLODParameterization.h"
#endif	// WITH_PROXYLOD

// We do not have UVAtlas available if ProxyLOD is not available.
#define NO_UVATLAS (WITH_PROXYLOD ? 0 : 1)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "ParameterizeMeshOp"

FParameterizeMeshOp::FLinearMesh::FLinearMesh(const FDynamicMesh3& Mesh, UE::Geometry::FPolygroupSet* InputGroups)
{
	TArray<FVector3f>& Positions = this->VertexBuffer;

	// Temporary maps used during construction.

	TArray<int32> TriFromID;
	TArray<int32> TriToID;
	TArray<int32> VertFromID;

	// Compute the mapping from triangle ID to triangle number
	{
		const int32 MaxTriID = Mesh.MaxTriangleID(); // really +1.. all TriID < MaxTriID
		const int32 NumTris = Mesh.TriangleCount();

		// reserve space and add elements 
		TriFromID.SetNum(MaxTriID);

		// reserve space
		TriToID.Empty(NumTris);

		int32 count = 0;
		for (int TriID : Mesh.TriangleIndicesItr())
		{
			TriToID.Add(TriID);
			TriFromID[TriID] = count;
			count++;
		}
	}

	// Compute the mapping from vertex ID to vertex number
	{
		const int32 MaxVertID = Mesh.MaxVertexID();
		const int32 NumVerts = Mesh.VertexCount();

		// reserve space and add elements
		VertFromID.Empty(MaxVertID);
		VertFromID.AddUninitialized(MaxVertID);

		// reserve space
		VertToID.Empty(NumVerts);

		int32 count = 0;
		for (int VtxID : Mesh.VertexIndicesItr())
		{
			VertToID.Add(VtxID);
			VertFromID[VtxID] = count;
			count++;
		}
	}

	// Fill the vertex buffer
	{
		int32 NumVerts = Mesh.VertexCount();
		Positions.Empty(NumVerts);

		for (const FVector3d& Vertex : Mesh.VerticesItr())
		{
			Positions.Add(static_cast<FVector3f>(Vertex));
		}
	}

	const int32 NumTris = Mesh.TriangleCount();

	// Fill the index buffer
	{
		IndexBuffer.Empty(NumTris * 3);
		for (const FIndex3i& Tri : Mesh.TrianglesItr())
		{
			for (int i = 0; i < 3; ++i)
			{
				int VtxID = Tri[i];
				int32 RemapVtx = VertFromID[VtxID];
				IndexBuffer.Add(RemapVtx);
			}
		}

	}


	// For each edge on each triangle.
	AdjacencyBuffer.Empty(NumTris * 3);

	// Create Adjacency - create boundaries in adjacency at polygroup boundaries if requested.
	if (InputGroups)
	{
		for (int TriID : Mesh.TriangleIndicesItr())
		{
			FIndex3i NbrTris = Mesh.GetTriNeighbourTris(TriID);
			for (int32 i = 0; i < 3; ++i)
			{
				const int32 NbrID = NbrTris[i];

				int32 RemapNbrID = FDynamicMesh3::InvalidID;
				if (NbrID != FDynamicMesh3::InvalidID)
				{
					const int32 CurTriGroup = InputGroups->GetGroup(TriID);
					const int32 NbrTriGroup = InputGroups->GetGroup(NbrID);

					RemapNbrID = (CurTriGroup == NbrTriGroup) ? TriFromID[NbrID] : FDynamicMesh3::InvalidID;
				}

				AdjacencyBuffer.Add(RemapNbrID);
			}
		}

	}
	else  // compute the adjacency using only the mesh connectivity.
	{
		for (int TriID : Mesh.TriangleIndicesItr())
		{
			FIndex3i NbrTris = Mesh.GetTriNeighbourTris(TriID);
			for (int32 i = 0; i < 3; ++i)
			{
				int32 NbrID = NbrTris[i];

				int32 RemapNbrID = (NbrID != FDynamicMesh3::InvalidID) ? TriFromID[NbrID] : FDynamicMesh3::InvalidID;
				AdjacencyBuffer.Add(RemapNbrID);
			}
		}
	}
}



void FParameterizeMeshOp::CopyNewUVsToMesh(
	FDynamicMesh3& Mesh,
	const FLinearMesh& LinearMesh,
	const FDynamicMesh3& FlippedMesh,
	const TArray<FVector2D>& UVVertexBuffer,
	const TArray<int32>& UVIndexBuffer,
	const TArray<int32>& VertexRemapArray,
	bool bReverseOrientation)
{
	if (!Mesh.HasAttributes())
	{
		Mesh.EnableAttributes();
	}
	FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(UVLayer);
	if (UVOverlay == nullptr)
	{
		Mesh.Attributes()->SetNumUVLayers(UVLayer + 1);
		UVOverlay = Mesh.Attributes()->GetUVLayer(UVLayer);
	}

	// delete any existing UVs
	UVOverlay->ClearElements();
	checkSlow(UVOverlay->ElementCount() == 0);

	// Add the UVs to the overlay
	int32 NumUVs = UVVertexBuffer.Num();
	TArray<int32> UVOffsetToElID;  UVOffsetToElID.Reserve(NumUVs);

	for (int32 i = 0; i < NumUVs; ++i)
	{
		FVector2D UV = UVVertexBuffer[i];

		// add the UV to the mesh overlay
		const int32 NewID = UVOverlay->AppendElement(FVector2f(UV));
		UVOffsetToElID.Add(NewID);
	}

	int32 NumUVTris = UVIndexBuffer.Num() / 3;
	for (int32 i = 0; i < NumUVTris; ++i)
	{
		int32 t = i * 3;
		// The triangle in UV space
		FIndex3i UVTri(UVIndexBuffer[t], UVIndexBuffer[t + 1], UVIndexBuffer[t + 2]);

		// the triangle in terms of the VertIDs in the DynamicMesh
		FIndex3i TriVertIDs;
		for (int c = 0; c < 3; ++c)
		{
			// the offset for this vertex in the LinearMesh
			int32 Offset = VertexRemapArray[UVTri[c]];

			int32 VertID = LinearMesh.VertToID[Offset];

			TriVertIDs[c] = VertID;
		}

		// NB: this could be slow.. 
		int32 TriID = FlippedMesh.FindTriangle(TriVertIDs[0], TriVertIDs[1], TriVertIDs[2]);

		checkSlow(TriID != FDynamicMesh3::InvalidID);

		FIndex3i ElTri;
		if (bReverseOrientation)
		{
			ElTri = FIndex3i(UVOffsetToElID[UVTri[1]], UVOffsetToElID[UVTri[0]], UVOffsetToElID[UVTri[2]]);
		}
		else
		{
			ElTri = FIndex3i(UVOffsetToElID[UVTri[0]], UVOffsetToElID[UVTri[1]], UVOffsetToElID[UVTri[2]]);
		}

		// add the triangle to the overlay
		UVOverlay->SetTriangle(TriID, ElTri);
	}
}

void FParameterizeMeshOp::LayoutToUDIMByPolygroup(FDynamicMesh3& InOutMesh, UE::Geometry::FPolygroupSet& PolygroupSet)
{
	FDynamicMeshUVEditor UVEditor(&InOutMesh, UVLayer, true);
	TMap<int32, TArray<int32> > TidsPerTile;

	for (int32 Tid : InOutMesh.TriangleIndicesItr())
	{
		if (UVEditor.GetOverlay()->IsSetTriangle(Tid))
		{
			int32 TidGroup = PolygroupSet.GetGroup(Tid);
			TidsPerTile.FindOrAdd(TidGroup).Add(Tid);
		}
	}

	TArray<int32> Tiles;
	TidsPerTile.GetKeys(Tiles);
	Tiles.Sort();
	for (int32 FlatGroupID = 0; FlatGroupID < Tiles.Num(); ++FlatGroupID)
	{
		FVector2i TilePos;
		TilePos.X = FlatGroupID % 10;
		TilePos.Y = FlatGroupID / 10;

		ensure(UVEditor.UDIMPack(Width, Gutter, TilePos, TidsPerTile.Find(Tiles[FlatGroupID])));
	}
}


bool FParameterizeMeshOp::ComputeUVs_UVAtlas(FDynamicMesh3& Mesh, TFunction<bool(float)>& Interrupter)
{
#if NO_UVATLAS

	ensureMsgf(false, TEXT("UVAtlas not available; this should not be called!"));
	return false;

#else	// NO_UVATLAS

	TRACE_CPUPROFILER_EVENT_SCOPE(ParameterizeMeshOp_ComputeUVs_UVAtlas);

	// the UVAtlas code is unhappy if you feed it a single degenerate triangle
	bool bNonDegenerate = true;
	if (Mesh.TriangleCount() == 1)
	{
		for (int32 TriID : Mesh.TriangleIndicesItr())
		{
			double Area = Mesh.GetTriArea(TriID);
			bNonDegenerate = bNonDegenerate && FMath::Abs(Area) > 1.e-5;
		}
	}

	if (!bNonDegenerate)
	{
		NewResultInfo.AddError(FGeometryError(0, LOCTEXT("DegenerateMeshError", "Mesh Contains Degenerate Triangles, Cannot be processed by UVAtlas")));
		return false;
	}

	// IProxyLODParameterization module calls into UVAtlas, which (appears to) assume mesh orientation that is opposite
	// if what we use in UE. As a result the UV islands come back mirrored. So we send a reverse-orientation mesh instead,
	// and then fix up the UVs when we set them below. 
	const bool bFixOrientation = true;
	FDynamicMesh3 FlippedMesh(EMeshComponents::FaceGroups);
	FlippedMesh.Copy(Mesh, false, false, false, false);
	if (bFixOrientation)
	{
		FlippedMesh.ReverseOrientation(false);
	}

	// Convert to a dense form.
	FLinearMesh LinearMesh(FlippedMesh, (bRespectInputGroups && InputGroups.IsValid()) ? InputGroups.Get() : nullptr);

	// Data to be populated by the UV generation tool
	TArray<FVector2D> UVVertexBuffer;
	TArray<int32>     UVIndexBuffer;
	TArray<int32>     VertexRemapArray; // This maps the UV vertices to the original position vertices.  Note multiple UV vertices might share the same positional vertex (due to UV boundaries)


	float MaxStretch = Stretch;
	int32 MaxChartNumber = NumCharts;

	TUniquePtr<IProxyLODParameterization> ParameterizationTool = IProxyLODParameterization::CreateTool();
	bool bSuccess = ParameterizationTool->GenerateUVs(Width, Height, Gutter, LinearMesh.VertexBuffer,
		LinearMesh.IndexBuffer, LinearMesh.AdjacencyBuffer, Interrupter,
		UVVertexBuffer, UVIndexBuffer, VertexRemapArray, MaxStretch,
		MaxChartNumber);

	// Add the UVs to the FDynamicMesh
	if (bSuccess)
	{
		CopyNewUVsToMesh(Mesh, LinearMesh, FlippedMesh, UVVertexBuffer, UVIndexBuffer, VertexRemapArray, bFixOrientation);
	}
	else
	{
		NewResultInfo.AddError(FGeometryError(0, LOCTEXT("UVAtlasFailed", "UVAtlas failed")));
	}

	if (bPackToUDIMSByOriginPolygroup && bRespectInputGroups && InputGroups.IsValid())
	{
		LayoutToUDIMByPolygroup(Mesh, *InputGroups);
	}

	return bSuccess;

#endif	// NO_UVATLAS
}









bool FParameterizeMeshOp::ComputeUVs_XAtlas(FDynamicMesh3& Mesh, TFunction<bool(float)>& Interrupter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParameterizeMeshOp_ComputeUVs_XAtlas);

	// reverse mesh orientation
	const bool bFixOrientation = true;
	FDynamicMesh3 FlippedMesh(EMeshComponents::FaceGroups);
	FlippedMesh.Copy(Mesh, false, false, false, false);
	if (bFixOrientation)
	{
		FlippedMesh.ReverseOrientation(false);
	}

	// Convert to a dense form.
	FLinearMesh LinearMesh(FlippedMesh, (bRespectInputGroups && InputGroups.IsValid()) ? InputGroups.Get() : nullptr);

	// Data to be populated by the UV generation tool
	TArray<FVector2D> UVVertexBuffer;
	TArray<int32>     UVIndexBuffer;
	TArray<int32>     VertexRemapArray; // This maps the UV vertices to the original position vertices.  Note multiple UV vertices might share the same positional vertex (due to UV boundaries)

	XAtlasWrapper::XAtlasChartOptions ChartOptions;
	ChartOptions.MaxIterations = XAtlasMaxIterations;
	XAtlasWrapper::XAtlasPackOptions PackOptions;
	bool bSuccess = XAtlasWrapper::ComputeUVs(LinearMesh.IndexBuffer,
		LinearMesh.VertexBuffer,
		ChartOptions,
		PackOptions,
		UVVertexBuffer,
		UVIndexBuffer,
		VertexRemapArray);

	// Add the UVs to the FDynamicMesh
	if (bSuccess)
	{
		CopyNewUVsToMesh(Mesh, LinearMesh, FlippedMesh, UVVertexBuffer, UVIndexBuffer, VertexRemapArray, bFixOrientation);
	}
	else
	{
		NewResultInfo.AddError(FGeometryError(0, LOCTEXT("XAtlasFailed", "XAtlas failed")));
	}

	return bSuccess;
}




bool FParameterizeMeshOp::ComputeUVs_PatchBuilder(FDynamicMesh3& InOutMesh, FProgressCancel* ProgressCancel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParameterizeMeshOp_ComputeUVs_PatchBuilder);

	FPatchBasedMeshUVGenerator UVGenerator;

	TUniquePtr<FPolygroupSet> PolygroupConstraint;
	if (bRespectInputGroups && InputGroups.IsValid())
	{
		UVGenerator.GroupConstraint = InputGroups.Get();
	}

	UVGenerator.TargetPatchCount = FMath::Max(1, InitialPatchCount);
	UVGenerator.bNormalWeightedPatches = true;
	UVGenerator.PatchNormalWeight = FMath::Clamp(PatchCurvatureAlignmentWeight, 0.0, 999999.0);
	UVGenerator.MinPatchSize = 2;

	UVGenerator.MergingThreshold = FMath::Clamp(PatchMergingMetricThresh, 0.001, 9999.0);
	UVGenerator.MaxNormalDeviationDeg = FMath::Clamp(PatchMergingAngleThresh, 0.0, 180.0);

	UVGenerator.NormalSmoothingRounds = FMath::Clamp(ExpMapNormalSmoothingSteps, 0, 9999);
	UVGenerator.NormalSmoothingAlpha = FMath::Clamp(ExpMapNormalSmoothingAlpha, 0.0, 1.0);

	UVGenerator.bAutoAlignPatches = true;
	UVGenerator.bAutoPack = bEnablePacking;
	UVGenerator.PackingTextureResolution = FMath::Clamp(this->Width, 16, 4096);
	UVGenerator.PackingGutterWidth = this->Gutter;


	FDynamicMeshUVEditor UVEditor(&InOutMesh, UVLayer, true);
	FGeometryResult Result = UVGenerator.AutoComputeUVs(*UVEditor.GetMesh(), *UVEditor.GetOverlay(), ProgressCancel);

	if (bPackToUDIMSByOriginPolygroup && bRespectInputGroups && InputGroups.IsValid())
	{
		LayoutToUDIMByPolygroup(InOutMesh, *InputGroups);
	}

	SetResultInfo(NewResultInfo);
	return (Result.HasFailed() == false);
}



void FParameterizeMeshOp::CalculateResult(FProgressCancel* Progress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParameterizeMeshOp_CalculateResult);

	if (!InputMesh.IsValid())
	{
		SetResultInfo(FGeometryResult::Failed());
		return;
	}

	ResultMesh = MakeUnique<FDynamicMesh3>(*InputMesh);
	if (Progress && Progress->Cancelled())
	{
		SetResultInfo(FGeometryResult::Cancelled());
		return;
	}

	// if using PatchBuilder, it handles all the output stuff
	if (Method == EParamOpBackend::PatchBuilder)
	{
		ComputeUVs_PatchBuilder(*ResultMesh, Progress);
		return;
	}

	// The UV atlas callback uses a float progress-based interrupter. 
	TFunction<bool(float)> Interrupter = [Progress](float)->bool {return !(Progress && Progress->Cancelled()); };

#if NO_UVATLAS
	constexpr bool bUseXAtlas = true;
	auto LogUVAtlasWarning = []
	{
		UE_LOGF(LogGeometry, Warning, "AutoUV method UVAtlas not available; falling back to XAtlas instead.");
	};
	UE_CALL_ONCE(LogUVAtlasWarning);
#else	// NO_UVATLAS
	const bool bUseXAtlas = (Method == EParamOpBackend::XAtlas);
#endif	// NO_UVATLAS

	NewResultInfo = FGeometryResult(EGeometryResultType::InProgress);

	// Single call to UVAtlas - perhaps respecting the poly groups.
	bool bOK;
	if (bUseXAtlas)
	{
		bOK = ComputeUVs_XAtlas(*ResultMesh, Interrupter);
	}
	else
	{
		bOK = ComputeUVs_UVAtlas(*ResultMesh, Interrupter);
	}

	NewResultInfo.SetSuccess(bOK, Progress);
	SetResultInfo(NewResultInfo);
}



TUniquePtr<FDynamicMeshOperator> UParameterizeMeshOperatorFactory::MakeNewOperator()
{
	TUniquePtr<FParameterizeMeshOp> Op = MakeUnique<FParameterizeMeshOp>();

	Op->InputMesh = OriginalMesh;
	Op->InputGroups = GetPolygroups();

	switch (Settings->Method)
	{
	case EParameterizeMeshUVMethod::PatchBuilder:
		Op->Method = EParamOpBackend::PatchBuilder;
		Op->InitialPatchCount = PatchBuilderProperties->InitialPatches;
		Op->bRespectInputGroups = PatchBuilderProperties->bUsePolygroups;
		Op->PatchCurvatureAlignmentWeight = PatchBuilderProperties->CurvatureAlignment;
		Op->PatchMergingMetricThresh = PatchBuilderProperties->MergingDistortionThreshold;
		Op->PatchMergingAngleThresh = PatchBuilderProperties->MergingAngleThreshold;
		Op->bPackToUDIMSByOriginPolygroup = PatchBuilderProperties->bLayoutUDIMPerPolygroup;
		Op->ExpMapNormalSmoothingSteps = PatchBuilderProperties->SmoothingSteps;
		Op->ExpMapNormalSmoothingAlpha = PatchBuilderProperties->SmoothingAlpha;
		Op->bEnablePacking = PatchBuilderProperties->bRepack;
		Op->Width = PatchBuilderProperties->TextureResolution;
		Op->Height = PatchBuilderProperties->TextureResolution;
		break;
	case EParameterizeMeshUVMethod::UVAtlas:
		Op->Method = EParamOpBackend::UVAtlas;
		Op->Stretch = UVAtlasProperties->IslandStretch;
		Op->bRespectInputGroups = UVAtlasProperties->bUsePolygroups;
		Op->bPackToUDIMSByOriginPolygroup = UVAtlasProperties->bLayoutUDIMPerPolygroup;
		Op->NumCharts = UVAtlasProperties->NumIslands;
		Op->Width = UVAtlasProperties->TextureResolution;
		Op->Height = UVAtlasProperties->TextureResolution;
		break;
	case EParameterizeMeshUVMethod::XAtlas:
		Op->Method = EParamOpBackend::XAtlas;
		Op->XAtlasMaxIterations = XAtlasProperties->MaxIterations;
		break;
	}

	Op->UVLayer = GetSelectedUVChannel();

	FTransformSRT3d Transform(TargetTransform);
	Op->SetTransform(Transform);

	return Op;
}


#undef LOCTEXT_NAMESPACE
