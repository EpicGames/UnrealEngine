// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshConstraints.h"
#include "ModelingOperators.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Util/ProgressCancel.h"

#include "SimplifyMeshOp.generated.h"

#define UE_API MODELINGOPERATORSEDITORONLY_API

struct FMeshDescription;
class IMeshReduction;

UENUM()
enum class ESimplifyTargetType : uint8
{
	/** Percentage of input triangles */
	Percentage = 0 UMETA(DisplayName = "Percentage"),

	/** Target triangle count */
	TriangleCount = 1 UMETA(DisplayName = "Triangle Count"),

	/** Target vertex count */
	VertexCount = 2 UMETA(DisplayName = "Vertex Count"),

	/** Target edge length */
	EdgeLength = 3 UMETA(DisplayName = "Edge Length"),

	/** Quadric error threshold */
	QuadricError = 5 UMETA(DisplayName = "Quadric Error"),

	/** Apply all allowable edge collapses that do not change the shape */
	MinimalPlanar = 4 UMETA(Hidden)
};

UENUM()
enum class ESimplifyType : uint8
{
	/** Fastest. Standard quadric error metric.*/
	QEM = 0 UMETA(DisplayName = "QEM"),

	/** Potentially higher quality. Takes the normal into account. */
	Attribute = 1 UMETA(DisplayName = "Normal Aware"),

	/** Highest quality reduction. */
	UEStandard = 2 UMETA(DisplayName = "UE Standard"),

	/** Edge collapse to existing vertices only.  Quality may suffer.*/
	MinimalExistingVertex = 3 UMETA(DisplayName = "Existing Positions"),

	/** Collapse any spurious edges but do not change the 3D shape. */
	MinimalPlanar = 4 UMETA(DisplayName = "Minimal Shape-Preserving"),

	/** Only preserve polygroup boundaries; ignore all other shape features */
	MinimalPolygroup = 5 UMETA(DisplayName = "Minimal PolyGroup-Preserving"),

	/** Simplify by locally clustering vertices, and re-creating the mesh triangles from the cluster connectivity */
	ClusterBased = 6,

};

namespace UE
{
namespace Geometry
{


class FSimplifyMeshOp : public FDynamicMeshOperator
{
public:
	virtual ~FSimplifyMeshOp() = default;

	//
	// Inputs
	// 
	ESimplifyTargetType TargetMode;
	ESimplifyType SimplifierType;
	int TargetPercentage, TargetCount;
	float TargetEdgeLength;
	float TargetQuadricError;
	bool bDiscardAttributes, bReproject, bPreventNormalFlips, bPreserveSharpEdges, bAllowSeamCollapse, bPreventTinyTriangles;
	// When true, result will have attributes object regardless of whether attributes 
	// were discarded or present initially.
	bool bResultMustHaveAttributesEnabled = false;
	UE::Geometry::EEdgeRefineFlags MeshBoundaryConstraint, GroupBoundaryConstraint, MaterialBoundaryConstraint;
	/** Angle threshold in degrees used for testing if two triangles should be considered coplanar, or two lines collinear */
	float MinimalPlanarAngleThresh = 0.01f;

	// For minimal polygroup-preserving simplification: Threshold angle change (in degrees) along a polygroup edge, above which a vertex must be added
	float PolyEdgeAngleTolerance = 0.1f;

	// For cluster-based simplification, equivalent to PolyEdgeAngleTolerance but only for boundary edges. Helpful for preserving open boundary shape.
	float BoundaryEdgeAngleTolerance = 30.0f;

	bool bGeometricDeviationConstraint = false;
	float GeometricTolerance = 0.0f;

	// Weight of optional regularization term, for QEM-based simplifiers
	float RegularizeWeight = 0.f;

	// QEM simplifier custom scales to apply to errors and edge lengths. Note this is in effect the inverse of scaling the tolerances / target lengths.
	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomGeometricErrorScaleF;
	TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomEdgeLengthScaleF;
	// Custom per-vertex quadric error scaling. Values should be greater than zero.
	TFunction<double(const FDynamicMesh3&, int Vertex)> CustomQuadricErrorScaleF;

	// Cluster simplifier custom scales apply to the *target* edge lengths, and so is conceptually the inverse of the above scales.
	TFunction<double(int32)> ClusterTargetEdgeLengthScaleF = nullptr;

	// stored for the UEStandard path
	TSharedPtr<FMeshDescription, ESPMode::ThreadSafe> OriginalMeshDescription;
	// stored for the GeometryProcessing custom simplifier paths (currently precomputed once in tool setup)
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TSharedPtr<FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial;

	IMeshReduction* MeshReduction;

	/**
	 * Simple helper to compute UE Standard simplified mesh
	 * 
	 * @param MeshReduction Interface for mesh reduction; must be non-null
	 * @param SrcMeshDescription Input mesh
	 * @param OutResult Simplified mesh will be written to this mesh
	 * @param PercentReduction Percent to reduce
	 * @param bTriBasedReduction Whether to reduce based on triangles; otherwise, will reduce based on vertices
	 * @param bDiscardAttributes Whether to discard attributes when simplifying
	 * @param Progress If non-null, can early-out if this is set to request cancellation
	 * @return true if succeeded, false otherwise (e.g. if cancelled)
	 */
	static UE_API bool ComputeStandardSimplifier(IMeshReduction* MeshReduction, const FMeshDescription& SrcMeshDescription, FDynamicMesh3& OutResult,
		float PercentReduction, bool bTriBasedReduction, bool bDiscardAttributes,
		FProgressCancel* Progress = nullptr);

	// set ability on protected transform.
	void SetTransform(const FTransform& Transform)
	{
		ResultTransform = (FTransform3d)Transform;
	}

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
