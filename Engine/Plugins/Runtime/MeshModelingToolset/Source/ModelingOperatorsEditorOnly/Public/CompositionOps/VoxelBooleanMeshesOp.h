// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "TransformTypes.h"

#define UE_API MODELINGOPERATORSEDITORONLY_API

struct FMeshDescription;

namespace UE
{
namespace Geometry
{


class FVoxelBooleanMeshesOp : public FDynamicMeshOperator
{
public:
	virtual ~FVoxelBooleanMeshesOp() = default;

	enum class EBooleanOperation
	{
		DifferenceAB = 0,
		DifferenceBA = 1,
		Intersect = 2,
		Union = 3
	};

	
	// inputs
	TArray<TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>> Meshes;
	TArray<FTransformSRT3d> Transforms; // 1:1 with Meshes

	int32 VoxelCount = 128;
	double VoxelSizeD = 1.0;
	double AdaptivityD = 0;
	double IsoSurfaceD = 0;
	bool   bAutoSimplify = false;
	EBooleanOperation Operation = EBooleanOperation::DifferenceBA;

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

private:

	// compute the voxel size based on the voxel count and the input geometry
	UE_API float ComputeVoxelSize() const;

};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
