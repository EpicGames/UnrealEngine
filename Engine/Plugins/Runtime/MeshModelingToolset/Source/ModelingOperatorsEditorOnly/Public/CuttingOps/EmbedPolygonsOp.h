// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Polygon2.h"

#include "EmbedPolygonsOp.generated.h"

#define UE_API MODELINGOPERATORSEDITORONLY_API


UENUM()
enum class EEmbeddedPolygonOpMethod : uint8
{
	TrimOutside,
	TrimInside,
	InsertPolygon,
	CutThrough,
	CutOutside
};

namespace UE
{
namespace Geometry
{


class FEmbedPolygonsOp : public FDynamicMeshOperator
{
public:
	virtual ~FEmbedPolygonsOp() = default;

	
	// inputs
	FFrame3d PolygonFrame;
	FPolygon2d EmbedPolygon;
	bool bCutWithBoolean;
	bool bAttemptFixHolesOnBoolean;

	// TODO: switch to FGeneralPolygon2d?
	FPolygon2d GetPolygon()
	{
		return EmbedPolygon;
	}

	bool bDiscardAttributes;

	EEmbeddedPolygonOpMethod Operation;

	//float ExtrudeDistance; // TODO if we support extrude

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


	// Outputs
	TArray<int> EdgesOnFailure; // edges to highlight on failure, to help visualize what happened (partial cut edges or hole edges)
	TArray<int> EmbeddedEdges;
	bool bOperationSucceeded;

private:
	UE_API void RecordEmbeddedEdges(TArray<int>& PathVertIDs);

	UE_API void BooleanPath(FProgressCancel* Progress);

	// Fix issues like self-intersection on an input polygon
	// Note: We could change this to return an array of general polygons in the future, to handle the most general case,
	// but currently the embedding process does not support holes
	UE_API TArray<FPolygon2d> CleanPolygon(const FPolygon2d& Input);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
