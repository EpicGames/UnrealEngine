// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "DynamicMesh/MeshTangents.h"
#include "CalculateTangentsOp.generated.h"

#define UE_API MODELINGOPERATORSEDITORONLY_API


UENUM()
enum class EMeshTangentsType : uint8
{
	/** Standard MikkTSpace tangent calculation */
	MikkTSpace = 0,
	/** MikkTSpace-like blended per-triangle tangents, with the blending being based on existing mesh, normals, and UV topology */
	FastMikkTSpace = 1,
	/** Project per-triangle tangents onto normals */
	PerTriangle = 2,
	/** Use existing source mesh tangents */
	CopyExisting = 3
};

namespace UE
{
namespace Geometry
{

class FCalculateTangentsOp : public TGenericDataOperator<FMeshTangentsd>
{
public:
	virtual ~FCalculateTangentsOp() = default;

	// inputs
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> SourceMesh;
	TSharedPtr<FMeshTangentsf, ESPMode::ThreadSafe> SourceTangents;

	// parameters
	EMeshTangentsType CalculationMethod;
	// Compute tangents with respect to this UV layer
	int32 TargetUVLayer = 0;

	// error flags
	bool bNoAttributesError = false;

	//
	// TGenericDataOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;

	UE_API virtual void CalculateStandard(FProgressCancel* Progress, TUniquePtr<FMeshTangentsd>& Tangents);
	UE_API virtual void CalculateMikkT(FProgressCancel* Progress, TUniquePtr<FMeshTangentsd>& Tangents);
	UE_API virtual void CopyFromSource(FProgressCancel* Progress, TUniquePtr<FMeshTangentsd>& Tangents);
};

} // end namespace UE::Geometry
} // end namespace UE

#undef UE_API
