// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Util/ProgressCancel.h"
#include "ModelingOperators.h"
#include "Polygroups/PolygroupSet.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "ParameterizeMeshOp.generated.h"

#define UE_API MODELINGOPERATORSEDITORONLY_API

class UParameterizeMeshToolProperties;
class UParameterizeMeshToolUVAtlasProperties;
class UParameterizeMeshToolXAtlasProperties;
class UParameterizeMeshToolPatchBuilderProperties;

namespace UE
{
namespace Geometry
{

enum class EParamOpBackend
{
	PatchBuilder = 0,
	UVAtlas = 1,
	XAtlas = 2
};

class FParameterizeMeshOp : public FDynamicMeshOperator
{
public:
	virtual ~FParameterizeMeshOp() = default;

	//
	// Inputs
	// 

	// source mesh
	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> InputMesh;

	// source groups (optional)
	TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> InputGroups;

	// UVAtlas generation parameters
	float Stretch = 0.11f;
	int32 NumCharts = 0;

	// XAtlas generation parameters
	int32 XAtlasMaxIterations = 1;

	// PatchBuilder generation parameters
	int32 InitialPatchCount = 100;
	bool bRespectInputGroups = false;
	float PatchCurvatureAlignmentWeight = 1.0f;
	float PatchMergingMetricThresh = 1.5f;
	float PatchMergingAngleThresh = 45.0f;
	int ExpMapNormalSmoothingSteps = 5;
	float ExpMapNormalSmoothingAlpha = 0.25f;

	// UV layer
	int32 UVLayer = 0;

	// UDIM support
	bool bPackToUDIMSByOriginPolygroup = false;

	// Atlas Packing parameters
	bool bEnablePacking = true;
	int32 Height = 512;
	int32 Width = 512;
	float Gutter = 2.5f;

	EParamOpBackend Method = EParamOpBackend::UVAtlas;

	// set ability on protected transform.
	void SetTransform(const FTransformSRT3d& XForm)
	{
		ResultTransform = XForm;
	}

	//
	// FDynamicMeshOperator implementation
	// 

	UE_API virtual void CalculateResult(FProgressCancel* Progress) override;


protected:

	// dense index/vertex buffer based representation of the data needed for parameterization
	struct FLinearMesh
	{
		FLinearMesh(const FDynamicMesh3& Mesh, UE::Geometry::FPolygroupSet* InputGroups);

		// Stripped down mesh
		TArray<int32>   IndexBuffer;
		TArray<FVector3f> VertexBuffer;

		// Map from offset in the VertexBuffer to the VertexID in the FDynamicMesh
		TArray<int32> VertToID;

		// Adjacency[tri], Adjacency[tri+1], Adjacency[tri+2] 
		// are the three tris adjacent to tri.
		TArray<int32>   AdjacencyBuffer;
		
	};

	FGeometryResult NewResultInfo;

	UE_API bool ComputeUVs_UVAtlas(FDynamicMesh3& InOutMesh, TFunction<bool(float)>& Interrupter);
	UE_API bool ComputeUVs_XAtlas(FDynamicMesh3& InOutMesh, TFunction<bool(float)>& Interrupter);
	UE_API bool ComputeUVs_PatchBuilder(FDynamicMesh3& InOutMesh, FProgressCancel* Progress);

	UE_API void CopyNewUVsToMesh(
		FDynamicMesh3& Mesh,
		const FLinearMesh& LinearMesh,
		const FDynamicMesh3& FlippedMesh,
		const TArray<FVector2D>& UVVertexBuffer,
		const TArray<int32>& UVIndexBuffer,
		const TArray<int32>& VertexRemapArray,
		bool bReverseOrientation);

	UE_API void LayoutToUDIMByPolygroup(FDynamicMesh3& InOutMesh, UE::Geometry::FPolygroupSet& InputGroups);
};

} // end namespace UE::Geometry
} // end namespace UE

/**
 * Can be hooked up to a UMeshOpPreviewWithBackgroundCompute to perform UV parameterization operations.
 *
 * Inherits from UObject so that it can hold a strong pointer to the settings UObject, which
 * needs to be a UObject to be displayed in the details panel.
 */
UCLASS(MinimalAPI)
class UParameterizeMeshOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	// IDynamicMeshOperatorFactory API
	UE_API virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolProperties> Settings = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolUVAtlasProperties> UVAtlasProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolXAtlasProperties> XAtlasProperties = nullptr;

	UPROPERTY()
	TObjectPtr<UParameterizeMeshToolPatchBuilderProperties> PatchBuilderProperties = nullptr;

	//TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe> InputGroups;
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	TUniqueFunction<int32()> GetSelectedUVChannel = []() { return 0; };
	TUniqueFunction<TSharedPtr<UE::Geometry::FPolygroupSet, ESPMode::ThreadSafe>()> GetPolygroups = []() { return nullptr; };
	FTransform TargetTransform;
};

#undef UE_API
