// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"

#include "ParameterizeMeshProperties.generated.h"


UENUM()
enum class EParameterizeMeshUVMethod
{
	// Keep values the same as UE::Geometry::EParamOpBackend!

	/** Compute automatic UVs using the Patch Builder technique */
	PatchBuilder = 0,
	/** Compute automatic UVs using the UVAtlas technique */
	UVAtlas = 1,
	/** Compute automatic UVs using the XAtlas technique */
	XAtlas = 2,
};


UCLASS(MinimalAPI)
class UParameterizeMeshToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Automatic UV generation technique to use */
	UPROPERTY(EditAnywhere, Category = AutoUV)
	EParameterizeMeshUVMethod Method = EParameterizeMeshUVMethod::PatchBuilder;
};


/**
 * Settings for the UVAtlas Automatic UV Generation Method
 */
UCLASS(MinimalAPI)
class UParameterizeMeshToolUVAtlasProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Maximum amount of stretch, from none to unbounded. If zero stretch is specified, each triangle will likely be its own UV island. */
	UPROPERTY(EditAnywhere, Category = UVAtlas, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float IslandStretch = 0.11f;

	/** Hint at number of UV islands. The default of 0 means it is determined automatically. */
	UPROPERTY(EditAnywhere, Category = UVAtlas, meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "10000"))
	int NumIslands = 0;

	/** Expected resolution of the output textures; this controls spacing left between UV islands to avoid interpolation artifacts. */
	UPROPERTY(EditAnywhere, Category = UVAtlas, meta = (UIMin = "64", UIMax = "2048", ClampMin = "2", ClampMax = "4096"))
	int TextureResolution = 1024;

	/** Generate new UVs based on polygroups from specified layer. */
	UPROPERTY(EditAnywhere, Category = UVAtlas, meta = (DisplayName = "Constrain to PolyGroups", EditCondition = bPolygroupsEnabled, EditConditionHides, HideEditConditionToggle))
	bool bUsePolygroups;

	/** Layout resulting islands on UDIMs based on polygroups. */
	UPROPERTY(EditAnywhere, Category = UVAtlas, meta = (EditCondition = "bUsePolygroups && bUDIMsEnabled", EditConditionHides, HideEditConditionToggle))
	bool bLayoutUDIMPerPolygroup;

	/** Controls if polygroup options are available to the user. */
	UPROPERTY()
	bool bPolygroupsEnabled;

	/** Controls if UDIM options are available to the user. */
	UPROPERTY()
	bool bUDIMsEnabled;
};


UCLASS(MinimalAPI)
class UParameterizeMeshToolXAtlasProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Number of solve iterations; higher values generally result in better UV islands. */
	UPROPERTY(EditAnywhere, Category = XAtlas, meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "1000"))
	int MaxIterations = 1;
};


UCLASS(MinimalAPI)
class UParameterizeMeshToolPatchBuilderProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Number of initial patches the mesh will be split into before island merging. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (UIMin = "1", UIMax = "1000", ClampMin = "1", ClampMax = "99999999"))
	int InitialPatches = 100;

	/** Alignment of the initial patches to creases in the mesh.*/
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (UIMin = "0.1", UIMax = "2.0", ClampMin = "0.01", ClampMax = "100.0"))
	float CurvatureAlignment = 1.0f;

	/** Threshold for stretching and distortion below which island merging is allowed; larger values increase the allowable UV distortion. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (DisplayName = "Distortion Threshold", UIMin = "1.0", UIMax = "5.0", ClampMin = "1.0"))
	float MergingDistortionThreshold = 1.5f;

	/** Threshold for the average face normal deviation below which island merging is allowed. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (DisplayName = "Angle Threshold", UIMin = "0.0", UIMax = "90.0", ClampMin = "0.0", ClampMax = "180.0"))
	float MergingAngleThreshold = 45.0f;

	/** Number of smoothing steps to apply; this slightly increases distortion but produces more stable results. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (UIMin = "0", UIMax = "25", ClampMin = "0", ClampMax = "1000"))
	int SmoothingSteps = 5;

	/** Smoothing parameter; larger values result in faster smoothing in each step. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (UIMin = "0", UIMax = "1.0", ClampMin = "0", ClampMax = "1.0"))
	float SmoothingAlpha = 0.25f;

	/** Automatically pack result UVs into the unit square, i.e. fit between 0 and 1 with no overlap. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder)
	bool bRepack = true;

	/** Expected resolution of the output textures; this controls spacing left between UV islands to avoid interpolation artifacts. This is only enabled when Repack is enabled. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (UIMin = "64", UIMax = "2048", ClampMin = "2", ClampMax = "4096", EditCondition = bRepack))
	int TextureResolution = 1024;

	/** Generate new UVs based on polygroups from specified layer. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (DisplayName = "Constrain to PolyGroups", EditCondition = bPolygroupsEnabled, EditConditionHides, HideEditConditionToggle))
	bool bUsePolygroups;

	/** Layout resulting islands on UDIMs based on polygroups. */
	UPROPERTY(EditAnywhere, Category = PatchBuilder, meta = (EditCondition = "bUsePolygroups && bUDIMsEnabled", EditConditionHides, HideEditConditionToggle))
	bool bLayoutUDIMPerPolygroup;

	/** Controls if polygroup options are available to the user. */
	UPROPERTY()
	bool bPolygroupsEnabled;

	/** Controls if UDIM options are available to the user. */
	UPROPERTY()
	bool bUDIMsEnabled;
};
