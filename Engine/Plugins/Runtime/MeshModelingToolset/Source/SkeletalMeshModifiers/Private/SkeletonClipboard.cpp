// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletonClipboard.h"

#include "SkeletonModifier.h"
#include "Algo/AllOf.h"

#include "HAL/PlatformApplicationMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonClipboard)

namespace SkeletonClipboard
{

void CopyToClipboard(const USkeletonModifier& InModifier, const TArray<FName>& InBonesToCopy)
{
	CopyBonesToClipboard(InModifier, InBonesToCopy);
}

void CopyBonesToClipboard(const USkeletonModifier& InModifier, const TArray<FName>& InBonesToCopy)
{
	const FReferenceSkeleton& RefSkeleton = InModifier.GetReferenceSkeleton();

	// Copy children as well
	TFunction<void(const int32, TArray<FName>&)> GetChildrenRecursive = 
		[&](const int32 BoneIndex, TArray<FName>& InOutBoneAndChildren)
			{
				if (!ensureMsgf(BoneIndex != INDEX_NONE, TEXT("Unexpected cannot find bone index in skeleton, cannot copy bone")))
				{
					return;
				}					
					
				TArray<int32> DirectChildIndices;
				RefSkeleton.GetDirectChildBones(BoneIndex, DirectChildIndices);
					
				for (int32 ChildIndex : DirectChildIndices)
				{
					if (ChildIndex != INDEX_NONE)
					{
						InOutBoneAndChildren.AddUnique(RefSkeleton.GetBoneName(ChildIndex));
						GetChildrenRecursive(ChildIndex, InOutBoneAndChildren);
					}
				}
			};
			
	TArray<FName> BonesToCopyWithChildren;
	for (const FName& BoneToCopy : InBonesToCopy)
	{
		BonesToCopyWithChildren.AddUnique(BoneToCopy);
				
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneToCopy);
		if (BoneIndex != INDEX_NONE)
		{
			GetChildrenRecursive(BoneIndex, BonesToCopyWithChildren);
		}
	}

	// ensure ordering
	BonesToCopyWithChildren.StableSort([&](const FName Bone0, const FName Bone1)
	{
		const int32 Index0 = RefSkeleton.FindBoneIndex(Bone0);
		const int32 Index1 = RefSkeleton.FindBoneIndex(Bone1);
		return Index0 < Index1;
	});

	// convert bone names to clipboard data
	FHierarchyClipboardData ClipboardData;
	ClipboardData.Type = EBoneClipboardDataType::Bones;
	ClipboardData.Bones.Reserve(InBonesToCopy.Num());
	
	for (const FName& BoneName : BonesToCopyWithChildren)
	{
		const FName ParentName = InModifier.GetParentName(BoneName);
		
		FBoneClipboardData BoneData;
		{
			BoneData.BoneName = BoneName;
			
			BoneData.ParentBoneName = ParentName;
			
			constexpr bool bGlobal = true;
			BoneData.Global = InModifier.GetBoneTransform(BoneName, bGlobal);
			
			constexpr bool bLocal = false;
			BoneData.Local = InModifier.GetBoneTransform(BoneName, bLocal);
			
			// DEPRECATED 5.8
			PRAGMA_DISABLE_DEPRECATION_WARNINGS 		
			BoneData.ParentIndex_DEPRECATED = ParentName != NAME_None ? BonesToCopyWithChildren.IndexOfByKey(ParentName) : INDEX_NONE;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
		ClipboardData.Bones.Add(MoveTemp(BoneData));
	}
	
	// convert data to text
	FString ClipboardText;
	static const FHierarchyClipboardData DefaultData;
	FHierarchyClipboardData::StaticStruct()->ExportText(ClipboardText, &ClipboardData, &DefaultData, nullptr, PPF_None, nullptr);

	// copy to clipboard
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
}

class FHierarchyImportErrorContext : public FOutputDevice
{
public:

    int32 NumErrors;

    FHierarchyImportErrorContext(const bool bInNotify)
    	: FOutputDevice()
    	, NumErrors(0)
		, bNotify(bInNotify)
    {}

    virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const FName& Category) override
    {
    	if (bNotify)
    	{
    		UE_LOGF(LogTemp, Error, "Error Importing Bones: %ls", V);
    	}
    	NumErrors++;
    }

private:
	bool bNotify = false;
};

TArray<FName> PasteFromClipboard(USkeletonModifier& InOutModifier, const FName& InDefaultParent)
{
	// DEPRECATED 5.8

	constexpr bool bUsingGlobalTransforms = true;
	return PasteBonesFromClipboard(InOutModifier, InDefaultParent, bUsingGlobalTransforms);
}

TArray<FName> PasteBonesFromClipboard(USkeletonModifier& InOutModifier, const FName& InDefaultParent, const bool bUsingGlobalTransforms)
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	FHierarchyClipboardData ClipboardData;
	FHierarchyImportErrorContext ErrorPipe(true);
	
	UScriptStruct* DataStruct = FHierarchyClipboardData::StaticStruct();
	DataStruct->ImportText(*ClipboardText, &ClipboardData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);

	if (ErrorPipe.NumErrors > 0 || ClipboardData.Bones.IsEmpty())
	{
		return {};
	}

	TArray<FName> NewBoneNames;
	NewBoneNames.Reserve(ClipboardData.Bones.Num());

	TArray<FName> NewParentBoneNames;
	NewParentBoneNames.Reserve(ClipboardData.Bones.Num());

	TArray<FTransform> LocalTransforms;
	LocalTransforms.Reserve(ClipboardData.Bones.Num());
	
	// Generate new bone data
	{
		for (const FBoneClipboardData& BoneData : ClipboardData.Bones)
		{
			const bool bIsChildFromClipboard = ClipboardData.Bones.ContainsByPredicate(
				[&BoneData](const FBoneClipboardData& ClipboardData)
				{
					return ClipboardData.BoneName == BoneData.ParentBoneName;
				});
				
			if (bIsChildFromClipboard)
			{ 
				NewBoneNames.Add(BoneData.BoneName);
				NewParentBoneNames.Add(BoneData.ParentBoneName);
				
				// Children don't need to consider global transform
				LocalTransforms.Add(BoneData.Local);
			}
			else
			{ 
				NewBoneNames.Add(BoneData.BoneName);
				NewParentBoneNames.Add(InDefaultParent);
				
				if (bUsingGlobalTransforms)
				{
					constexpr bool bGlobal = true;
					const FTransform& ParentTransformGlobal = InOutModifier.GetBoneTransform(InDefaultParent, bGlobal);
					LocalTransforms.Add(BoneData.Global.GetRelativeTransform(ParentTransformGlobal));					
				}
				else
				{
					LocalTransforms.Add(BoneData.Local);
				}
			}
		}
	}
	
	// Make sure all new bones use unique names
	{
		check(NewBoneNames.Num() == NewParentBoneNames.Num());
		
		TArray<FName> ExistingBoneNames = InOutModifier.GetAllBoneNames();
		
		for (FName& BoneName : NewBoneNames)
		{
			if (ExistingBoneNames.Contains(BoneName))
			{
				const FName NewBoneName = InOutModifier.GetUniqueName(BoneName, ExistingBoneNames);
				for (FName& ParentBoneName : NewParentBoneNames)
				{
					const bool bUsesDefaultParent = ParentBoneName == InDefaultParent;
					if (ParentBoneName == BoneName &&
						!bUsesDefaultParent)
					{
						ParentBoneName = NewBoneName;
					}
				}
				
				BoneName = NewBoneName;
				ExistingBoneNames.Add(BoneName);
			}
		}
	}

	// add bones
	InOutModifier.AddBones(NewBoneNames, NewParentBoneNames, LocalTransforms);

	return NewBoneNames;
}

TArray<FName> ClipboardDuplicateBones(USkeletonModifier& InOutModifier, const TArray<FName>& InBonesToDuplicate)
{
	if (InBonesToDuplicate.IsEmpty())
	{
		return {}; 
	}

	SkeletonClipboard::CopyBonesToClipboard(InOutModifier, InBonesToDuplicate);

	if (!SkeletonClipboard::IsBoneClipboardValid())
	{
		return {};
	} 

	const FReferenceSkeleton& RefSkeleton = InOutModifier.GetReferenceSkeleton();

	// Find the common parent of the selection
	const int32 CommonParentIndex = [&RefSkeleton, &InBonesToDuplicate]() -> int32
		{
			const auto GetParentIndies = [&RefSkeleton](const FName& BoneName)
				{
					TArray<int32> ParentIndices;
					int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneName);
					while (BoneIndex != INDEX_NONE)
					{
						BoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
						ParentIndices.Add(BoneIndex);
					}
									
					return ParentIndices;
				};
					
			TArray<int32> CommonParentIndices = GetParentIndies(InBonesToDuplicate[0]);
			for (const FName& BoneName : InBonesToDuplicate)
			{
				if (BoneName != InBonesToDuplicate[0])
				{
					const TArray<int32> ParentIndices = GetParentIndies(BoneName);
					CommonParentIndices.RemoveAll([ParentIndices](const int32 CommonParentIndex)
						{
							return !ParentIndices.Contains(CommonParentIndex);
						});
				}
			}
					
			return CommonParentIndices.IsEmpty() ? INDEX_NONE : CommonParentIndices[0];
		}();
				
	const FName CommonParentName = CommonParentIndex == INDEX_NONE ?
		NAME_None : 
		RefSkeleton.GetBoneName(CommonParentIndex);
						
	constexpr bool bUsingGlobalTransforms = true;
	return SkeletonClipboard::PasteBonesFromClipboard(InOutModifier, CommonParentName, bUsingGlobalTransforms);
}
	
void CopyBoneTransformsToClipboard(const USkeletonModifier& InModifier, TArray<FName> InBonesToCopy)
{
	const FReferenceSkeleton& RefSkeleton = InModifier.GetReferenceSkeleton();
	
	// ensure ordering
	InBonesToCopy.StableSort([&](const FName Bone0, const FName Bone1)
	{
		const int32 Index0 = RefSkeleton.FindBoneIndex(Bone0);
		const int32 Index1 = RefSkeleton.FindBoneIndex(Bone1);
		return Index0 < Index1;
	});

	// convert bone names to clipboard data
	FHierarchyClipboardData ClipboardData;
	ClipboardData.Type = EBoneClipboardDataType::BoneTransforms;
	ClipboardData.Bones.Reserve(InBonesToCopy.Num());
	
	for (const FName& BoneName : InBonesToCopy)
	{
		const FName ParentName = InModifier.GetParentName(BoneName);
		
		FBoneClipboardData BoneData;
		{
			BoneData.BoneName = BoneName;
			
			BoneData.ParentBoneName = ParentName;
			
			constexpr bool bGlobal = true;
			BoneData.Global = InModifier.GetBoneTransform(BoneName, bGlobal);
			
			constexpr bool bLocal = false;
			BoneData.Local = InModifier.GetBoneTransform(BoneName, bLocal);
		}
		
		ClipboardData.Bones.Add(BoneData);
	}
	
	// convert data to text
	FString ClipboardText;
	static const FHierarchyClipboardData DefaultData;
	FHierarchyClipboardData::StaticStruct()->ExportText(ClipboardText, &ClipboardData, &DefaultData, nullptr, PPF_None, nullptr);

	// copy to clipboard
	FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
}

TArray<FName> PasteBoneTransformsFromClipboard(
	USkeletonModifier& InOutModifier,
	TArray<FName> InCandidateBoneNames,
	const EPasteBoneTransformsFromClipboardFlags Flags)
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	FHierarchyClipboardData ClipboardData;
	FHierarchyImportErrorContext ErrorPipe(true);
	
	UScriptStruct* DataStruct = FHierarchyClipboardData::StaticStruct();
	DataStruct->ImportText(*ClipboardText, &ClipboardData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);

	if (ErrorPipe.NumErrors > 0 || ClipboardData.Bones.IsEmpty())
	{
		return {};
	}
	
	TArray<FName> TargetBoneNames;
	TargetBoneNames.Reserve(ClipboardData.Bones.Num());

	TArray<FTransform> LocalTransforms;
	LocalTransforms.Reserve(ClipboardData.Bones.Num());
	
	const bool bTargetMatchesSource = Algo::AllOf(ClipboardData.Bones,
		[&InCandidateBoneNames](const FBoneClipboardData& BoneData)
		{
			return InCandidateBoneNames.Contains(BoneData.BoneName);
		});
		
	const auto GetTargetTransformLambda(
		[&InOutModifier, Flags](const FBoneClipboardData& BoneData, const FName& TargetBoneName)
		{			
			const FName ParentBoneName = InOutModifier.GetParentName(TargetBoneName);

			if (ParentBoneName == NAME_None)
			{
				return BoneData.Global;				
			}
			else if (EnumHasAnyFlags(Flags, EPasteBoneTransformsFromClipboardFlags::UsingGlobalTransforms))
			{
				constexpr bool bGlobal = true;
				const FTransform& ParentTransformGlobal = InOutModifier.GetBoneTransform(ParentBoneName, bGlobal);
				return BoneData.Global.GetRelativeTransform(ParentTransformGlobal);					
			}
			else
			{
				return BoneData.Local;
			}
		});
	
	if (bTargetMatchesSource)
	{
		// the clipboard data matches bone names exactly, paste transforms by name
		for (const FBoneClipboardData& BoneData : ClipboardData.Bones)
		{
			const FName* TargetBoneNamePtr = InCandidateBoneNames.FindByKey(BoneData.BoneName);
			check(TargetBoneNamePtr);
			
			TargetBoneNames.Add(*TargetBoneNamePtr);
			LocalTransforms.Add(GetTargetTransformLambda(BoneData, *TargetBoneNamePtr));
		}
	}
	else
	{
		// tagets don't match the clipboard data exactly, paste by bone index
		const FReferenceSkeleton& RefSkeleton = InOutModifier.GetReferenceSkeleton();
	
		for (int32 BoneIndex = 0; BoneIndex < ClipboardData.Bones.Num(); BoneIndex++)
		{
			if (InCandidateBoneNames.IsValidIndex(BoneIndex))
			{
				const FBoneClipboardData& BoneData = ClipboardData.Bones[BoneIndex];
				const FName& TargetBoneName = InCandidateBoneNames[BoneIndex];
				
				TargetBoneNames.Add(TargetBoneName);
				LocalTransforms.Add(GetTargetTransformLambda(BoneData, TargetBoneName));
			}
			else
			{
				break;
			}
		}
				
		// ensure ordering of targets (source was already ordered when copying)
		TargetBoneNames.StableSort([&](const FName Bone0, const FName Bone1)
		{
			const int32 Index0 = RefSkeleton.FindBoneIndex(Bone0);
			const int32 Index1 = RefSkeleton.FindBoneIndex(Bone1);
			return Index0 < Index1;
		});
	}
	
	// set transforms
	const bool bMoveChildren = EnumHasAnyFlags(Flags, EPasteBoneTransformsFromClipboardFlags::UpdateChildren);
	InOutModifier.SetBonesTransforms(TargetBoneNames, LocalTransforms, bMoveChildren);
	
	return TargetBoneNames;
}	
	
bool IsClipboardValid()
{
	// DEPRECATED 5.8
	return IsBoneClipboardValid();
}

bool IsBoneClipboardValid()
{
	// DEPRECATED 5.8
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	FHierarchyClipboardData ClipboardData;
	FHierarchyImportErrorContext ErrorPipe(false);
	
	UScriptStruct* DataStruct = FHierarchyClipboardData::StaticStruct();
	DataStruct->ImportText(*ClipboardText, &ClipboardData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);

	return 
		ErrorPipe.NumErrors == 0 && 
		ClipboardData.Type == EBoneClipboardDataType::Bones &&
		!ClipboardData.Bones.IsEmpty();
}

bool IsBoneTransformClipboardValid()
{
	// DEPRECATED 5.8
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	FHierarchyClipboardData ClipboardData;
	FHierarchyImportErrorContext ErrorPipe(false);
	
	UScriptStruct* DataStruct = FHierarchyClipboardData::StaticStruct();
	DataStruct->ImportText(*ClipboardText, &ClipboardData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);

	return 
		ErrorPipe.NumErrors == 0 && 
		ClipboardData.Type == EBoneClipboardDataType::BoneTransforms &&
		!ClipboardData.Bones.IsEmpty();
}

}
