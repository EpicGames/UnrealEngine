// Copyright Epic Games, Inc. All Rights Reserved.


#include "SkeletonModifier.h"

#include "AssetNotifications.h"
#include "BoneWeights.h"
#include "FileHelpers.h"
#include "MeshDescription.h"
#include "Animation/Skeleton.h"
#include "Misc/ITransaction.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMeshAttributes.h"
#include "Animation/AnimSequence.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimMontage.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/SkeletalMesh.h"
#include "RenderingThread.h"
#include "Animation/MirrorDataTable.h"
#include "PropertyEditorModule.h"
#include "Dialog/SCustomDialog.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Runtime/GeometryFramework/Public/UDynamicMesh.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonModifier)

#define LOCTEXT_NAMESPACE "SkeletonModifier"

namespace USkeletonModifierLocals
{
	static constexpr int32 LODIndex = 0;

	/**
	* FReferenceSkeletonCompatibilityChecker is a utility class to check whether two skeletons are compatible.
	* It follows the USkeleton compatibility pattern.
	* NOTE: there should be consistency between this, USkeleton and FSkeletonHelper so it should be merged at some point
	*/

	class FReferenceSkeletonCompatibilityChecker
	{
	public:
		FReferenceSkeletonCompatibilityChecker(const FReferenceSkeleton& InRefSkeleton)
			: ReferenceSkeleton(InRefSkeleton)
		{}

		bool DoesParentChainMatch(const int32 InStartBoneIndex, const FReferenceSkeleton& InRefSkeleton) const
		{
			// if start is root bone
			if ( InStartBoneIndex == 0 )
			{
				// verify name of root bone matches
				return (ReferenceSkeleton.GetBoneName(0) == InRefSkeleton.GetBoneName(0));
			}

			int32 SkeletonBoneIndex = InStartBoneIndex;
			// If skeleton bone is not found in mesh, fail.
			int32 OtherBoneIndex = InRefSkeleton.FindBoneIndex( ReferenceSkeleton.GetBoneName(SkeletonBoneIndex) );
			if( OtherBoneIndex == INDEX_NONE )
			{
				return false;
			}
	
			do
			{
				// verify if parent name matches
				int32 ParentSkeletonBoneIndex = ReferenceSkeleton.GetParentIndex(SkeletonBoneIndex);
				int32 ParentOtherBoneIndex = InRefSkeleton.GetParentIndex(OtherBoneIndex);

				// if one of the parents doesn't exist, make sure both end. Otherwise fail.
				if( (ParentSkeletonBoneIndex == INDEX_NONE) || (ParentOtherBoneIndex == INDEX_NONE) )
				{
					return (ParentSkeletonBoneIndex == ParentOtherBoneIndex);
				}

				// If parents are not named the same, fail.
				if( ReferenceSkeleton.GetBoneName(ParentSkeletonBoneIndex) != InRefSkeleton.GetBoneName(ParentOtherBoneIndex) )
				{
					UE_LOGF(LogAnimation, Warning, "%ls : Hierarchy does not match %ls - %ls.", *ReferenceSkeleton.GetBoneName(SkeletonBoneIndex).ToString(), *ReferenceSkeleton.GetBoneName(ParentSkeletonBoneIndex).ToString(), *InRefSkeleton.GetBoneName(ParentOtherBoneIndex).ToString());
					return false;
				}

				// move up
				SkeletonBoneIndex = ParentSkeletonBoneIndex;
				OtherBoneIndex = ParentOtherBoneIndex;
			} while ( true );
		}
		
		bool IsCompatibleReferenceSkeleton(const FReferenceSkeleton& InRefSkeleton, bool bDoParentChainCheck=true) const
		{
			// at least % of bone should match 
			int32 NumOfBoneMatches = 0;

			const int32 OtherNumBones = InRefSkeleton.GetRawBoneNum();

			// first ensure the parent exists for each bone
			for (int32 OtherBoneIndex=0; OtherBoneIndex<OtherNumBones; OtherBoneIndex++)
			{
				const FName OtherBoneName = InRefSkeleton.GetBoneName(OtherBoneIndex);
				// See if Mesh bone exists in Skeleton.
				int32 SkeletonBoneIndex = ReferenceSkeleton.FindBoneIndex( OtherBoneName );

				// if found, increase num of bone matches count
				if( SkeletonBoneIndex != INDEX_NONE )
				{
					++NumOfBoneMatches;

					// follow the parent chain to verify the chain is same
					if(bDoParentChainCheck && !DoesParentChainMatch(SkeletonBoneIndex, InRefSkeleton))
					{
						UE_LOGF(LogAnimation, Warning, "%ls : Hierarchy does not match.", *OtherBoneName.ToString());
						return false;
					}
				}
				else
				{
					int32 CurrentBoneId = OtherBoneIndex;
					// if not look for parents that matches
					while (SkeletonBoneIndex == INDEX_NONE && CurrentBoneId != INDEX_NONE)
					{
						// find Parent one see exists
						const int32 ParentOtherBoneIndex = InRefSkeleton.GetParentIndex(CurrentBoneId);
						if ( ParentOtherBoneIndex != INDEX_NONE )
						{
							// @TODO: make sure RefSkeleton's root ParentIndex < 0 if not, I'll need to fix this by checking TreeBoneIdx
							FName ParentBoneName = InRefSkeleton.GetBoneName(ParentOtherBoneIndex);
							SkeletonBoneIndex = ReferenceSkeleton.FindBoneIndex(ParentBoneName);
						}

						// root is reached
						if( ParentOtherBoneIndex == 0 )
						{
							break;
						}
						else
						{
							CurrentBoneId = ParentOtherBoneIndex;
						}
					}

					// still no match, return false, no parent to look for
					if( SkeletonBoneIndex == INDEX_NONE )
					{
						UE_LOGF(LogAnimation, Warning, "%ls : Missing joint on skeleton. Make sure to assign to the skeleton.", *OtherBoneName.ToString());
						return false;
					}

					// second follow the parent chain to verify the chain is same
					if (bDoParentChainCheck && !DoesParentChainMatch(SkeletonBoneIndex, InRefSkeleton))
					{
						UE_LOGF(LogAnimation, Warning, "%ls : Hierarchy does not match.", *OtherBoneName.ToString());
						return false;
					}
				}
			}

			// originally we made sure at least matches more than 50% 
			// but then follower components can't play since they're only partial
			// if the hierarchy matches, and if it's more then 1 bone, we allow
			return (NumOfBoneMatches > 0);
		}

	private:
		const FReferenceSkeleton& ReferenceSkeleton;
	};

	/**
	* FSkeletonOnlyBones: helps track bones that are in the USkeleton ref skeleton but not in those of the skeletal mesh.
	* This is important because the bone tree must not be recreated in this situation, but rather merged.
	*/
	
	struct FSkeletonOnlyBones
	{
		FSkeletonOnlyBones(const USkeleton* InSkeleton)
			: Skeleton(InSkeleton)
		{
			const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
 
			SkeletonOnlyBones.Reserve(RefSkeleton.GetRawBoneNum());
			for (const FMeshBoneInfo& BoneInfo: RefSkeleton.GetRawRefBoneInfo())
			{
				SkeletonOnlyBones.Add(BoneInfo.Name);
			}
 
			for (const FVirtualBone& VB : InSkeleton->GetVirtualBones())
			{
				SkeletonOnlyBones.Remove(VB.SourceBoneName);
				SkeletonOnlyBones.Remove(VB.TargetBoneName);
			}
		}
		
		void Remove(const FReferenceSkeleton& InRefSkeleton)
		{
			for (const FMeshBoneInfo& BoneInfo: InRefSkeleton.GetRawRefBoneInfo())
			{
				SkeletonOnlyBones.Remove(BoneInfo.Name);
			}
 
			const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
			for (int32 BoneIndex = RefSkeleton.GetRawBoneNum() - 1; BoneIndex >= 0; --BoneIndex)
			{
				const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
				if (!SkeletonOnlyBones.Contains(BoneName))
				{
					const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
					if (ParentIndex != INDEX_NONE)
					{
						SkeletonOnlyBones.Remove(RefSkeleton.GetBoneName(ParentIndex));
					}
				}
			}
		}
 
		int32 NumBones() const
		{
			return SkeletonOnlyBones.Num();
		}
		
	private:
		const USkeleton* Skeleton;
		TArray<FName> SkeletonOnlyBones;
	};
}

FTransform FMirrorOptions::MirrorTransform(const FTransform& InTransform) const
{
	FTransform Transform = InTransform;
	Transform.SetLocation(MirrorVector(Transform.GetLocation()));

	if (bMirrorRotation)
	{
		FRotator Rotator(ForceInitToZero);
		switch (MirrorAxis)
		{
		case EAxis::X:
			{
				Rotator.Roll = 180;
				break;
			}
		case EAxis::Y:
			{
				Rotator.Pitch = 180;
				break;
			}
		case EAxis::Z:
			{
				Rotator.Yaw = 180;
				break;
			}
		default:
			break;
		}	
		Transform.SetRotation(FQuat::MakeFromRotator(Rotator) * Transform.GetRotation());
	}

	return Transform;
}

FVector FMirrorOptions::MirrorVector(const FVector& InVector) const
{
	FVector Axis(ForceInitToZero); Axis.SetComponentForAxis(MirrorAxis, 1.f);
	return InVector.MirrorByVector(Axis);
}

FTransform FOrientOptions::OrientTransform(const FVector& InPrimaryTarget, const FTransform& InTransform) const
{
	if (Primary == EOrientAxis::None || InPrimaryTarget.IsNearlyZero())
	{
		return InTransform;
	}
	
	auto GetOrientVector = [](const EOrientAxis& InOrientAxis)
	{
		switch(InOrientAxis)
		{
		case EOrientAxis::None:	return FVector::ZeroVector;
		case EOrientAxis::PositiveX: return FVector::XAxisVector;
		case EOrientAxis::PositiveY: return FVector::YAxisVector;
		case EOrientAxis::PositiveZ: return FVector::ZAxisVector;
		case EOrientAxis::NegativeX: return -FVector::XAxisVector;
		case EOrientAxis::NegativeY: return -FVector::YAxisVector;
		case EOrientAxis::NegativeZ: return -FVector::ZAxisVector;
		default: break;
		}
		return FVector::ZeroVector;
	};
	
	FTransform Transform = InTransform;

	const FVector PrimaryOrientVector = GetOrientVector(Primary);
	const FVector PrimaryAxis = Transform.TransformVectorNoScale(PrimaryOrientVector).GetSafeNormal();
	const FVector PrimaryTarget = InPrimaryTarget.GetSafeNormal();
	
	// orient primary axis towards InPrimaryTarget
	{
		const FQuat Rotation = FQuat::FindBetweenNormals(PrimaryAxis, PrimaryTarget);
		const FQuat NewRotation = (Rotation * Transform.GetRotation()).GetNormalized();
		Transform.SetRotation(NewRotation);
	}

	// no need to use secondary axis
	if (Secondary == Primary || Secondary == EOrientAxis::None || SecondaryTarget.IsNearlyZero())
	{
		return Transform;
	}

	FVector SecondTarget = SecondaryTarget.GetSafeNormal();
	if (FMath::IsNearlyEqual(FMath::Abs(FVector::DotProduct(PrimaryTarget, SecondTarget)), 1.0f))
	{
		// both targets are parallel
		return Transform;
	}

	// orient secondary axis towards SecondaryDirection
	{
		// project on primary 
		SecondTarget = SecondTarget - FVector::DotProduct(SecondTarget, PrimaryTarget) * PrimaryTarget;
		
		if (!SecondTarget.IsNearlyZero())
		{
			SecondTarget = SecondTarget.GetSafeNormal();

			const FVector SecondaryOrientVector = GetOrientVector(Secondary);
			const FVector SecondaryAxis = Transform.TransformVectorNoScale(SecondaryOrientVector).GetSafeNormal();

			// if they are opposites, we only need to rotate 180 degrees around PrimaryTarget
			const double DotProduct = FVector::DotProduct(SecondaryAxis, SecondTarget);
			const bool bAreOpposites = (DotProduct + 1.0) < KINDA_SMALL_NUMBER;
			
			const FQuat Rotation = bAreOpposites ? FQuat(PrimaryTarget, PI) : FQuat::FindBetweenNormals(SecondaryAxis, SecondTarget);
			const FQuat NewRotation = (Rotation * Transform.GetRotation()).GetNormalized();
			Transform.SetRotation(NewRotation);
		}
	}
	
	return Transform;
}

void USkeletonModifier::ExternalUpdate(const FReferenceSkeleton& InRefSkeleton, const TArray<int32>& InIndexTracker)
{
	if (!ReferenceSkeleton)
	{
		return;
	}
	
	*ReferenceSkeleton = InRefSkeleton;
	TransformComposer.Reset(new FTransformComposer(*ReferenceSkeleton));
	BoneIndexTracker = InIndexTracker;
}

void USkeletonModifier::ExternalUpdate(const FReferenceSkeleton& InRefSkeleton)
{
	if (!ReferenceSkeleton)
	{
		return;
	}
	
	*ReferenceSkeleton = InRefSkeleton;
	
	TransformComposer.Reset(new FTransformComposer(*ReferenceSkeleton));

	int32 NumBones = InRefSkeleton.GetRawBoneNum();
	BoneIndexTracker.Reset(NumBones);
	for (int32 Index = 0; Index < NumBones; Index++)
	{
		BoneIndexTracker.Add(Index);
	}
}

bool USkeletonModifier::SetSkeletalMesh(USkeletalMesh* InSkeletalMesh)
{
	SkeletalMesh = nullptr;
	MeshDescription.Reset();
	ReferenceSkeleton.Reset();
	TransformComposer.Reset();
	BoneIndexTracker.Reset();

#if WITH_EDITORONLY_DATA
	// validate supplied skeletal mesh exists
	if (!InSkeletalMesh)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier: No skeletal mesh supplied to load.");
		return false;
	}

	const USkeleton* Skeleton = InSkeletalMesh->GetSkeleton();
	if (!Skeleton)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier: Skeletal Mesh supplied has no skeleton.");
		return false;
	}

	// verify user is not trying to modify one of the core engine assets
	if (InSkeletalMesh->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier: Cannot modify built-in engine asset.");
		return false;
	}
	
	// store pointer to mesh and instantiate a mesh description for commiting changes
	SkeletalMesh = InSkeletalMesh;

	// store mesh description to edit
	MeshDescription = MakeUnique<FMeshDescription>();
	SkeletalMesh->CloneMeshDescription(USkeletonModifierLocals::LODIndex, *MeshDescription);
	
	if (MeshDescription->IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier: mesh description is emtpy.");
		return false;
	}

	// store reference skeleton to edit
	SetReferenceSkeleton(SkeletalMesh->GetRefSkeleton());
	
	return true;
	
#else
	ensureMsgf(false, TEXT("Skeleton Modifier is an editor only feature."));
	return false;
#endif
}

bool USkeletonModifier::SetDynamicMesh(UDynamicMesh* InDynamicMesh)
{
	WeakDynamicMesh = nullptr;
	ReferenceSkeleton.Reset();
	TransformComposer.Reset();
	BoneIndexTracker.Reset();
	
#if WITH_EDITORONLY_DATA
	
	if (!InDynamicMesh)
	{
		return false;
	}
	
	WeakDynamicMesh = InDynamicMesh;
	FReferenceSkeleton MeshSkeleton;
	{
		const FDynamicMesh3& Mesh = InDynamicMesh->GetMeshRef();
		
		const int32 NumBones = Mesh.Attributes()->GetNumBones();
		FReferenceSkeletonModifier RefSkeletonModifier(MeshSkeleton, nullptr);

		using namespace UE::Geometry;
		const FDynamicMeshBoneNameAttribute* BoneNameAttrib = Mesh.Attributes()->GetBoneNames();
		const FDynamicMeshBoneParentIndexAttribute* BoneParentIndexAttrib = Mesh.Attributes()->GetBoneParentIndices();
		const FDynamicMeshBonePoseAttribute* BonePoses = Mesh.Attributes()->GetBonePoses();

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const int32 ParentBoneIndex = BoneParentIndexAttrib->GetValue(BoneIndex);
			FName BoneName = BoneNameAttrib->GetValue(BoneIndex);
			FMeshBoneInfo BoneInfo(BoneName, BoneName.ToString(), ParentBoneIndex);

			FTransform Transform = BonePoses->GetValue(BoneIndex);
			RefSkeletonModifier.Add(BoneInfo, Transform);
		}	
	}	
		
	SetReferenceSkeleton(MeshSkeleton);
	

	return true;
	
#else
	ensureMsgf(false, TEXT("Skeleton Modifier is an editor only feature."));
	return false;
#endif

}

bool USkeletonModifier::SetReferenceSkeleton(const FReferenceSkeleton& InReferenceSkeleton)
{
	ReferenceSkeleton.Reset();
	TransformComposer.Reset();
	BoneIndexTracker.Reset();

	// store reference skeleton to read from
	ReferenceSkeleton = MakeUnique<FReferenceSkeleton>();
	*ReferenceSkeleton = InReferenceSkeleton;
	TransformComposer.Reset(new FTransformComposer(*ReferenceSkeleton));

	// store initial bones indices to track for changes
	const int32 NumBones = ReferenceSkeleton->GetRawBoneNum();
	BoneIndexTracker.Reserve(NumBones);
	for (int32 Index = 0; Index < NumBones; Index++)
	{
		BoneIndexTracker.Add(Index);
	}

	return true;
}

void USkeletonModifier::SetReadOnly(bool bInReadOnly)
{
	bReadOnly = bInReadOnly;
}

bool USkeletonModifier::IsReadOnly() const
{
	return bReadOnly;
}

bool USkeletonModifier::CommitSkeletonToDynamicMesh()
{
	if (!WeakDynamicMesh.IsValid())
	{
		return false;
	}
	
	UDynamicMesh* DynamicMesh = WeakDynamicMesh.Get();
	
	TSharedPtr<FDynamicMesh3> OldMeshShared = MakeShared<FDynamicMesh3>();
	DynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { *OldMeshShared = ReadMesh; });

	EnsureSingleRoot();
	
	DynamicMesh->EditMesh([&](FDynamicMesh3& Mesh)
	{
		const int32 NumBones = ReferenceSkeleton->GetNum();
		
		using namespace UE::Geometry;
		Mesh.Attributes()->EnableBones(NumBones);
		
		FDynamicMeshBoneNameAttribute* BoneNameAttrib = Mesh.Attributes()->GetBoneNames();
		FDynamicMeshBoneParentIndexAttribute* BoneParentIndexAttrib = Mesh.Attributes()->GetBoneParentIndices();
		FDynamicMeshBonePoseAttribute* BonePoses = Mesh.Attributes()->GetBonePoses();

		for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			FName BoneName = ReferenceSkeleton->GetBoneName(BoneIndex);
			int32 ParentIndex = ReferenceSkeleton->GetParentIndex(BoneIndex);
			const FTransform& RefBonePose = ReferenceSkeleton->GetRefBonePose()[BoneIndex];

			BoneNameAttrib->SetValue(BoneIndex, BoneName);
			BoneParentIndexAttrib->SetValue(BoneIndex, ParentIndex);
			BonePoses->SetValue(BoneIndex, RefBonePose);
		}	
	});

	// Update skin weights
	DynamicMesh->EditMesh([&](FDynamicMesh3& Mesh)
	{
		using namespace UE::Geometry;
		using namespace UE::AnimationCore;
		
		FBoneWeightsSettings BoneSettings; BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::None);
		
		TArray<FName> SkinWeightProfiles;
		Mesh.Attributes()->GetSkinWeightsAttributes().GenerateKeyArray(SkinWeightProfiles);

		for (const FName SkinWeightProfile: SkinWeightProfiles)
		{
			FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = Mesh.Attributes()->GetSkinWeightsAttribute(SkinWeightProfile);

			for (int32 VertexID : Mesh.VertexIndicesItr())
			{
				FBoneWeights BoneWeights;
				SkinWeights->GetValue(VertexID, BoneWeights);
				
				if (const int32 NumBoneWeights = BoneWeights.Num())
				{
					TArray<FBoneWeight> NewWeights;
					for (int32 Idx = 0; Idx < NumBoneWeights; ++Idx)
					{
						const FBoneWeight& OldBoneWeight = BoneWeights[Idx];
						const int32 BoneIndex = OldBoneWeight.GetBoneIndex();

						int32 NewBoneIndex = 0;
						if (ensure(BoneIndexTracker.IsValidIndex(BoneIndex)))
						{
							NewBoneIndex = BoneIndexTracker[BoneIndex];							
						}
						else
						{
							UE_LOGF(LogAnimation, Warning, "Skeleton Modifier - Commit: Invalid bone index provided (%d); falling back to 0 as bone index.", BoneIndex);
						}

						if (NewBoneIndex == INDEX_NONE)
						{
							// Bone was deleted: fall back to root so the vertex stays visible instead of collapsing to the origin.
							// Note: ideally we should dump the weights of the deleted bone to its parent at the same time the bone
							// is deleted
							NewBoneIndex = 0;
						}

						const uint16 AddedRawWeight = OldBoneWeight.GetRawWeight();
						if (FBoneWeight* Existing = NewWeights.FindByPredicate(
							[NewBoneIndex](const FBoneWeight& W){ return W.GetBoneIndex() == NewBoneIndex; }))
						{
							// Multiple source weights collapsed to the same target bone (e.g. deleted bone + existing root weight on the same vert): sum, don't duplicate.
							Existing->SetRawWeight(static_cast<uint16>(
								FMath::Min<uint32>(uint32(Existing->GetRawWeight()) + uint32(AddedRawWeight), FBoneWeight::GetMaxRawWeight())));
						}
						else
						{
							NewWeights.Add(FBoneWeight(NewBoneIndex, AddedRawWeight));
						}
					}
					SkinWeights->SetValue(VertexID, FBoneWeights::Create(NewWeights, BoneSettings));
				}
			}
		}	
	});

	
	TSharedPtr<FDynamicMesh3> NewMeshShared = MakeShared<FDynamicMesh3>();
	DynamicMesh->ProcessMesh([&](const FDynamicMesh3& ReadMesh) { *NewMeshShared = ReadMesh; });
	
	TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(OldMeshShared, NewMeshShared);

	FScopedTransaction Transaction(LOCTEXT("SkeletonModifierCommitDynamicMesh", "Update Skeleton"));
	check(GUndo != nullptr);
	DynamicMesh->Modify();
	GUndo->StoreUndo(DynamicMesh, MoveTemp(ReplaceChange));
	
	return true;
}

bool USkeletonModifier::IsReferenceSkeletonValid(const bool bLog) const
{
	if (!ReferenceSkeleton.IsValid())
	{
		if(bLog)
		{
			UE_LOGF(LogAnimation, Error, "Skeleton Modifier: No valid reference skeleton provided.");
		}
		return false;		
	}
	return true;
}

ESkeletalMeshModificationType USkeletonModifier::PreCommitSkeletalMesh()
{
	EnsureSingleRoot();

	// check topological change from this modifier
	auto HasBoneIndexesChanged = [this]()
	{
		for (int32 Index = 0; Index < BoneIndexTracker.Num(); Index++)
		{
			if (BoneIndexTracker[Index] != Index)
			{
				return true;
			}
		}
		return false;
	};

	auto BasicModificationCheck = [this, HasBoneIndexesChanged](const FReferenceSkeleton& InRefSkeleton)
	{
		const TArray<FMeshBoneInfo>& OtherBoneInfos = InRefSkeleton.GetRawRefBoneInfo();
		const TArray<FTransform>& OtherBonePoses = InRefSkeleton.GetRawRefBonePose();
		const int32 NumOtherBones = OtherBoneInfos.Num();

		const TArray<FMeshBoneInfo>& NewBoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
		const TArray<FTransform>& NewBonePoses = ReferenceSkeleton->GetRawRefBonePose();
		const int32 NumNewBones = NewBoneInfos.Num();

		ESkeletalMeshModificationType Modifications = ESkeletalMeshModificationType::None;
		if (NumNewBones > NumOtherBones)
		{
			EnumAddFlags(Modifications, ESkeletalMeshModificationType::BonesAdded);
			if (HasBoneIndexesChanged())
			{
				EnumAddFlags(Modifications, ESkeletalMeshModificationType::HierarchyChanged);
			}
		}
		else if (NumNewBones < NumOtherBones)
		{
			EnumAddFlags(Modifications, ESkeletalMeshModificationType::BonesRemoved);

			for (int32 NewBoneIndex = 0; NewBoneIndex < NumNewBones; NewBoneIndex++)
			{
				// check names
				const FName NewBoneName = ReferenceSkeleton->GetBoneName(NewBoneIndex);
				const int32 OtherBoneIndex = InRefSkeleton.FindBoneIndex(NewBoneName);
				if (OtherBoneIndex == INDEX_NONE)
				{
					EnumAddFlags(Modifications, ESkeletalMeshModificationType::BonesRenamed);
				}
				else
				{
					// check parents names
					const FMeshBoneInfo& NewBoneInfo = NewBoneInfos[NewBoneIndex];
					const FMeshBoneInfo& OtherBoneInfo = OtherBoneInfos[OtherBoneIndex];

					const int32 NewParentIndex = NewBoneInfo.ParentIndex;
					const int32 OldParentIndex = OtherBoneInfo.ParentIndex;
					const FName NewParentName = NewParentIndex != INDEX_NONE ? NewBoneInfos[NewParentIndex].Name : NAME_None; 
					const FName OldParentName = OldParentIndex != INDEX_NONE ? OtherBoneInfos[OldParentIndex].Name : NAME_None;
					
					if ((NewParentIndex != INDEX_NONE || OldParentIndex != INDEX_NONE) && NewParentName != OldParentName)
					{
						EnumAddFlags(Modifications, ESkeletalMeshModificationType::HierarchyChanged);
					}
				}
			}
		}
		else
		{
			for (int32 NewBoneIndex = 0; NewBoneIndex < NumNewBones; NewBoneIndex++)
			{
				// check names
				const FName NewBoneName = ReferenceSkeleton->GetBoneName(NewBoneIndex);
				const int32 OtherBoneIndex = InRefSkeleton.FindBoneIndex(NewBoneName);
				if (OtherBoneIndex == INDEX_NONE)
				{
					EnumAddFlags(Modifications, ESkeletalMeshModificationType::BonesRenamed);
				}
				else
				{
					// check index
					if (OtherBoneIndex != NewBoneIndex)
					{
						EnumAddFlags(Modifications, ESkeletalMeshModificationType::HierarchyChanged);
					}
					
					// check parents
					const FMeshBoneInfo& NewBoneInfo = NewBoneInfos[NewBoneIndex];
					const FMeshBoneInfo& OtherBoneInfo = OtherBoneInfos[OtherBoneIndex];
					if (NewBoneInfo.ParentIndex != OtherBoneInfo.ParentIndex)
					{
						EnumAddFlags(Modifications, ESkeletalMeshModificationType::HierarchyChanged);
					}

					// check transforms
					const FTransform& NewBoneTransform = NewBonePoses[NewBoneIndex];
					const FTransform& OtherBoneTransform = OtherBonePoses[OtherBoneIndex];
					if (!NewBoneTransform.Equals(OtherBoneTransform))
					{
						EnumAddFlags(Modifications, ESkeletalMeshModificationType::TransformChanged);
					}
					
					// case-sensitive change only
					if (OtherBoneInfo.Name == NewBoneName && !OtherBoneInfo.Name.IsEqual(NewBoneName, ENameCase::CaseSensitive))
					{
						EnumAddFlags(Modifications, ESkeletalMeshModificationType::BonesRenamed);
					}
				}
			}
		}
		
		return Modifications;
	};
	
	return BasicModificationCheck(SkeletalMesh->GetRefSkeleton());
}

ESkeletonModificationType USkeletonModifier::PreCommitSkeleton(const ESkeletalMeshModificationType InSkeletalMeshModifications) const
{
	const bool bNeedSkeletonUpdate = EnumHasAnyFlags(InSkeletalMeshModifications, ESkeletalMeshModificationType::SkeletonUpdated);
	if (!bNeedSkeletonUpdate)
	{
		return ESkeletonModificationType::None;
	}

	USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	USkeletonModifierLocals::FReferenceSkeletonCompatibilityChecker Checker(Skeleton->GetReferenceSkeleton());
	if (Checker.IsCompatibleReferenceSkeleton(*ReferenceSkeleton))
	{
		// skeleton is compatible
		return ESkeletonModificationType::SimpleMerge;
	}

	USkeletalMeshMergeOptions* Options = NewObject<USkeletalMeshMergeOptions>();
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	static constexpr bool bForceRefresh = true;
	DetailsView->SetObject(Options, bForceRefresh);

	TSharedRef<SCustomDialog> OptionsDialog = SNew(SCustomDialog)
		.Title(NSLOCTEXT("SkeletonModifier", "SkeletonModifierMergeDialog", "SkeletonModifier Merge Options"))
		.Content()
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4.f, 2.f)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("SkeletonModifier", "SkeletonModifierMergeText",
				"The current changes to the edited bone hierarchy are incompatible with the assigned skeleton asset.\n"
				"'Commit' to commit the current changes using the merge type below.\n"
				"'Cancel' to cancel the current changes and revert to the previous skeleton.\n"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f, 2.f)
			[
				SNew(SBox)
				.MinDesiredWidth(450)
				[
					DetailsView
				]
			]
		]
		.Buttons({
			SCustomDialog::FButton(NSLOCTEXT("SkeletonModifier", "CommitButton", "Commit")),
			SCustomDialog::FButton(NSLOCTEXT("SkeletonModifier", "CancelButton", "Cancel"))
		});

	const int32 Choice = OptionsDialog->ShowModal();
	if (Choice == 1 || Choice == -1)
	{
		return ESkeletonModificationType::Cancel;
	}

	switch (Options->MergeType)
	{
		case ESKeletalMeshMergeType::New:
			return ESkeletonModificationType::DuplicateAndMerge;
		case ESKeletalMeshMergeType::Merge:
			return Options->bMergeAll ? ESkeletonModificationType::FullMergeAll : ESkeletonModificationType::FullMerge;
		default:
			break;
	}

	return ESkeletonModificationType::None;
}

bool USkeletonModifier::CommitSkeletonToSkeletalMesh()
{
	if (!SkeletalMesh.IsValid() || !ReferenceSkeleton || !MeshDescription)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier: No mesh loaded. Cannot apply skeleton edits.");
		return false;
	}

#if WITH_EDITORONLY_DATA
	
	// check modifications at the skeletal mesh level
	const ESkeletalMeshModificationType Modifications = PreCommitSkeletalMesh();
	if (Modifications == ESkeletalMeshModificationType::None)
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton Modifier: No modification needed.");
		return false;
	}

	// check modifications at the skeleton level
	ESkeletonModificationType SkeletonModifications = PreCommitSkeleton(Modifications);
	if (SkeletonModifications == ESkeletonModificationType::Cancel)
	{
		UE_LOGF(LogAnimation, Warning, "Skeleton Modifier: Skeleton can't be modified.");
		return false;
	}

	// duplicate the skeleton if needed
	if (SkeletonModifications == ESkeletonModificationType::DuplicateAndMerge)
	{
		TArray<UObject*> SavedObjects;
		FEditorFileUtils::SaveAssetsAs({SkeletalMesh->GetSkeleton()}, SavedObjects);

		USkeleton* NewSkeleton = SavedObjects.IsEmpty() ? nullptr : Cast<USkeleton>(SavedObjects[0]);
		if (!NewSkeleton)
		{
			UE_LOGF(LogAnimation, Warning, "Skeleton Modifier: Skeleton can't be duplicated.");
			return false;
		}

		SkeletalMesh->SetSkeleton(NewSkeleton);
		SkeletalMesh->MarkPackageDirty();

		if (NewSkeleton->GetPreviewMesh() != SkeletalMesh)
		{
			NewSkeleton->SetPreviewMesh(SkeletalMesh.Get());
		}
	}

	const bool bHasSkeletonOnlyBones = [Modifications, SkeletonModifications, SkeletalMeshPtr = SkeletalMesh.Get()]()
	{
		if (SkeletonModifications == ESkeletonModificationType::SimpleMerge && Modifications == ESkeletalMeshModificationType::BonesAdded)
		{
			USkeletonModifierLocals::FSkeletonOnlyBones SkeletonOnlyBones(SkeletalMeshPtr->GetSkeleton());
			SkeletonOnlyBones.Remove(SkeletalMeshPtr->GetRefSkeleton());
			return SkeletonOnlyBones.NumBones() > 0;
		}
		return false;
	}();
	
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	
	// update mesh description
	CommitChangesToMeshDescription(Modifications);

	// store retargeting modes
	USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	
	TArray<EBoneTranslationRetargetingMode::Type> RetargetingModes;
	RetargetingModes.Init(EBoneTranslationRetargetingMode::Animation, BoneInfos.Num());
	
	for (int32 OldBoneIndex = 0, NumOldBones = BoneIndexTracker.Num(); OldBoneIndex < NumOldBones; ++OldBoneIndex)
	{
		const int32 NewBoneIndex = BoneIndexTracker[OldBoneIndex];
		if (RetargetingModes.IsValidIndex(NewBoneIndex))
		{
			RetargetingModes[NewBoneIndex] = Skeleton->GetBoneTranslationRetargetingMode(OldBoneIndex);	
		}
	}

	// update skeletal mesh
	FlushRenderingCommands();

	// call modify on the skeleton first as post undo will re-register components so it must be done once both
	// skeletal mesh and skeleton are up to date, so it must be done after the skeletal mesh has been undone
	if (EnumHasAnyFlags(SkeletonModifications, ESkeletonModificationType::DoUpdate))
	{
		Skeleton->Modify();
	}

	SkeletalMesh->SetFlags(RF_Transactional);
	SkeletalMesh->Modify();

	// update the ref skeleton
	SkeletalMesh->SetRefSkeleton(*ReferenceSkeleton);
	SkeletalMesh->GetRefBasesInvMatrix().Reset();
	SkeletalMesh->CalculateInvRefMatrices();
	
	// update skeletal mesh LOD (cf. USkeletalMesh::CommitMeshDescription)
	SkeletalMesh->ModifyMeshDescription(USkeletonModifierLocals::LODIndex);
	SkeletalMesh->CreateMeshDescription(USkeletonModifierLocals::LODIndex, MoveTemp(*MeshDescription));
	SkeletalMesh->CommitMeshDescription(USkeletonModifierLocals::LODIndex);

	// update skeleton
	if (EnumHasAnyFlags(SkeletonModifications, ESkeletonModificationType::DoUpdate))
	{
		NotifyFromSkeletonChanges();
		
		auto UpdateSkeleton = [this, &SkeletonModifications, &RetargetingModes, bHasSkeletonOnlyBones]()
		{
			USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
			if (SkeletonModifications == ESkeletonModificationType::SimpleMerge)
			{
				constexpr bool bSkeletalMeshReferencesOnly = true;
				// merge the skeleton if there are several references or if the USkeleton's ref skeleton has bones that did not exist in the
				// non-modified ref skeleton.
				if (HasAnyOtherReferences(bSkeletalMeshReferencesOnly) || bHasSkeletonOnlyBones)
				{
					return Skeleton->MergeAllBonesToBoneTree(SkeletalMesh.Get());
				}
				SkeletonModifications = ESkeletonModificationType::FullMerge;
			}
		
			bool bSkeletonModified = false;	
			if (EnumHasAnyFlags(SkeletonModifications, ESkeletonModificationType::DeepMerge))
			{
				bSkeletonModified = Skeleton->RecreateBoneTree(SkeletalMesh.Get());
			}

			if (!bSkeletonModified)
			{
				return false;
			}

			// restore retargeting modes
			const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
			for (int32 BoneIndex = 0, NumBones = BoneInfos.Num(); BoneIndex < NumBones; ++BoneIndex)
			{
				Skeleton->SetBoneTranslationRetargetingMode(BoneIndex, RetargetingModes[BoneIndex]);
			}

			return true;
		};
	
		if (UpdateSkeleton())
		{
			PostCommitSkeleton(SkeletonModifications);
			Skeleton->MarkPackageDirty();
			FAssetNotifications::SkeletonNeedsToBeSaved(Skeleton);
		}
		else
		{
			ensure(false);
		}
	}
	
	// must be done once the skeleton is up to date
	SkeletalMesh->PostEditChange();

	return true;
#else
	ensureMsgf(false, TEXT("Skeleton Modifier is an editor only feature."));
	return false;
#endif
}

void USkeletonModifier::CommitChangesToMeshDescription(const ESkeletalMeshModificationType InSkeletalMeshModifications)
{
	if (!SkeletalMesh.IsValid() || !ReferenceSkeleton || !MeshDescription)
	{
		// this is supposed to be tested earlier
		return;
	}
	
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	const TArray<FTransform>& Transforms = ReferenceSkeleton->GetRawRefBonePose();
	
	FSkeletalMeshAttributes MeshAttributes(*MeshDescription);

	// update bone data
	if (!MeshAttributes.HasBones())
	{
		MeshAttributes.Register(true);
	}

	MeshAttributes.Bones().Reset(BoneInfos.Num());

	FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = MeshAttributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = MeshAttributes.GetBoneParentIndices();
	FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = MeshAttributes.GetBonePoses();

	for (int Index = 0; Index < BoneInfos.Num(); ++Index)
	{
		const FMeshBoneInfo& Info = BoneInfos[Index];
		const FBoneID BoneID = MeshAttributes.CreateBone();
		BoneNames.Set(BoneID, Info.Name);
		BoneParentIndices.Set(BoneID, Info.ParentIndex);
		BonePoses.Set(BoneID, Transforms[Index]);
	}
		
	// update skin weight data if needed
	if (EnumHasAnyFlags(InSkeletalMeshModifications, ESkeletalMeshModificationType::IndicesUpdated))
	{
		using namespace UE::AnimationCore;
		FBoneWeightsSettings BoneSettings; BoneSettings.SetNormalizeType(EBoneWeightNormalizeType::None);

		for (const FName SkinWeightProfile: MeshAttributes.GetSkinWeightProfileNames())
		{
			FSkinWeightsVertexAttributesRef SkinWeights = MeshAttributes.GetVertexSkinWeights(SkinWeightProfile);
			if (SkinWeights.IsValid())
			{
				for (const FVertexID& VertexID: MeshDescription->Vertices().GetElementIDs())
				{
					FVertexBoneWeights BoneWeights = SkinWeights.Get(VertexID);
					if (const int32 NumBoneWeights = BoneWeights.Num())
					{
						TArray<FBoneWeight> NewWeights;
						for (int32 Idx = 0; Idx < NumBoneWeights; ++Idx)
						{
							const FBoneWeight& OldBoneWeight = BoneWeights[Idx];
							const int32 BoneIndex = OldBoneWeight.GetBoneIndex();

							int32 NewBoneIndex = 0;
							if (ensure(BoneIndexTracker.IsValidIndex(BoneIndex)))
							{
								NewBoneIndex = BoneIndexTracker[BoneIndex];
							}
							else
							{
								UE_LOGF(LogAnimation, Warning, "Skeleton Modifier - Commit: Invalid bone index provided (%d); falling back to 0 as bone index.", BoneIndex);
							}

							if (NewBoneIndex == INDEX_NONE)
							{
								// Bone was deleted: fall back to root so the vertex stays visible instead of collapsing to the origin.
								// Note: ideally we should dump the weights of the deleted bone to its parent at the same time the bone
								// is deleted
								NewBoneIndex = 0;
							}

							const uint16 AddedRawWeight = OldBoneWeight.GetRawWeight();
							if (FBoneWeight* Existing = NewWeights.FindByPredicate(
								[NewBoneIndex](const FBoneWeight& W){ return W.GetBoneIndex() == NewBoneIndex; }))
							{
								// Multiple source weights collapsed to the same target bone (e.g. deleted bone + existing root weight on the same vert): sum, don't duplicate.
								Existing->SetRawWeight(static_cast<uint16>(
									FMath::Min<uint32>(uint32(Existing->GetRawWeight()) + uint32(AddedRawWeight), FBoneWeight::GetMaxRawWeight())));
							}
							else
							{
								NewWeights.Add(FBoneWeight(NewBoneIndex, AddedRawWeight));
							}
						}
						SkinWeights.Set(VertexID, FBoneWeights::Create(NewWeights, BoneSettings));
					}
				}
			}
		}
	}
}

void USkeletonModifier::PostCommitSkeleton(const ESkeletonModificationType InSkeletonModifications) const
{
	if (InSkeletonModifications != ESkeletonModificationType::FullMergeAll)
	{
		return;
	}

	USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	
	TArray<const USkeletalMesh*> OtherSkeletalMeshUsingSkeleton;

	const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	
	FARFilter ARFilter;
	ARFilter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	ARFilter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(Skeleton).GetExportTextName());

	TArray<FAssetData> SkeletalMeshAssetData;
	if (AssetRegistry.GetAssets(ARFilter, SkeletalMeshAssetData))
	{
		for (const FAssetData& AssetData: SkeletalMeshAssetData)
		{
			const USkeletalMesh* ExtraSkeletalMesh = Cast<USkeletalMesh>(AssetData.GetAsset());
			if (IsValid(ExtraSkeletalMesh) && ExtraSkeletalMesh != SkeletalMesh)
			{
				OtherSkeletalMeshUsingSkeleton.Add(ExtraSkeletalMesh);
			}
		}
	}
	
	for (const USkeletalMesh* ExtraSkeletalMesh : OtherSkeletalMeshUsingSkeleton)
	{
		// merge still can fail
		if (!Skeleton->MergeAllBonesToBoneTree(ExtraSkeletalMesh))
		{
			FMessageDialog::Open(EAppMsgType::Ok,
				FText::Format(NSLOCTEXT("SkeletonModifier", "SkeletonModifier_RemergingBones", "Failed to merge SkeletalMesh '{0}'."), FText::FromString(ExtraSkeletalMesh->GetName())));
		}
	}
}

void USkeletonModifier::EnsureSingleRoot()
{
	// before commiting, we have to reparent non-root bones with no parent as the animation pipeline
	// doesn't support multi-roots
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	if (!BoneInfos.IsEmpty())
	{
		TArray<FName> BonesToParent;
		for (int32 Index = 1; Index < BoneInfos.Num(); Index++)
		{
			const FMeshBoneInfo& BoneInfo = BoneInfos[Index];
			if (BoneInfo.ParentIndex == INDEX_NONE)
			{
				BonesToParent.Add(BoneInfo.Name);
			}
		}
		
		if (!BonesToParent.IsEmpty())
		{
			for (const FName& BoneName: BonesToParent)
			{
				UE_LOGF(LogAnimation, Warning, "Skeleton Modifier: %ls will be parented to the root bone before commiting.", *BoneName.ToString());
			}
			ParentBones(BonesToParent, {BoneInfos[0].Name});
		}
	}	
}

bool USkeletonModifier::HasAnyOtherReferences(const bool bSkeletalMeshOnly) const
{
	USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	UPackage* SkeletonPackage = Skeleton->GetPackage();
	UPackage* SkeletalMeshPackage = SkeletalMesh->GetPackage();

	if (SkeletonPackage && SkeletalMeshPackage)
	{
		TArray<FSoftObjectPath> Result;
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		IAssetRegistry& AssetRegistry = AssetRegistryModule.GetRegistry();

		TArray<FName> References;
		AssetRegistry.GetReferencers(SkeletonPackage->GetFName(), References);

		const FName SkeletalMeshPackageName = SkeletalMeshPackage->GetFName();
		
		TArray<FName> Dependencies;
		for (const FName& Reference : References)
		{
			if (SkeletalMeshPackageName != Reference)
			{
				const FString PackageString = Reference.ToString();
				const FSoftObjectPath FullAssetPath(FTopLevelAssetPath(*PackageString, *FPackageName::GetLongPackageAssetName(PackageString)));
				FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FullAssetPath);
				if (UClass* Class = AssetData.GetClass())
				{
					if (!bSkeletalMeshOnly || Class == USkeletalMesh::StaticClass())
					{
						Dependencies.AddUnique(*AssetData.GetObjectPathString());
					}
				}
			}
		}
		
		return !Dependencies.IsEmpty();
	}
	
	return false;
}

void USkeletonModifier::NotifyFromSkeletonChanges() const
{
	// check assets using this skeleton ?
	if (bDebug)
	{
		// TODO avoid certain changes when the skeleton is referenced by other assets (i.e. changing the skeletal mesh's
		// reference skeleton poses is fine, re-parenting/removing bones, etc. is not)
		static const TArray<FTopLevelAssetPath> AssetPaths({
		   UAnimSequence::StaticClass()->GetClassPathName(),
		   UAnimMontage::StaticClass()->GetClassPathName(),
		   UPoseAsset::StaticClass()->GetClassPathName(),
		   USkeletalMesh::StaticClass()->GetClassPathName()});
	
		FARFilter Filter;
		Filter.ClassPaths = AssetPaths;
	
		const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		TArray<FAssetData> Assets; AssetRegistry.GetAssets(Filter, Assets);
	
		const FAssetData SkeletonAssetData(SkeletalMesh->GetSkeleton());
		const FString SkeletonPath = SkeletonAssetData.GetExportTextName();
	
		static const FName Tag("Skeleton");
		for (const FAssetData& AssetData: Assets)
		{
			const FString TagValue = AssetData.GetTagValueRef<FString>(Tag);
			if (TagValue == SkeletonPath)
			{
				UE_LOGF(LogAnimation, Warning, "%ls references that skeleton.", *AssetData.GetExportTextName());
			}
		}
	}
}

bool USkeletonModifier::AddBone(const FName InBoneName, const FName InParentName, const FTransform& InTransform)
{
	if (InBoneName == NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Add: Cannot add bone with no name.");
		return false;
	}
	
	return AddBones({InBoneName}, {InParentName}, {InTransform});
}

bool USkeletonModifier::AddBones(
	const TArray<FName>& InBoneNames, const TArray<FName>& InParentNames, const TArray<FTransform>& InTransforms)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}
	
	const int32 NumBonesToAdd = InBoneNames.Num();
	if (NumBonesToAdd == 0)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Add: The provided bone names array is empty.");
		return false;
	}
	
	struct FBoneData
	{
		FMeshBoneInfo BoneInfo;
		int32 TransformOffset;
	};

	TArray<FBoneData> BonesToAdd;
	BonesToAdd.Reserve(NumBonesToAdd);

	auto GetParentName = [&](int32 Index)
	{
		if (InBoneNames.Num() == InParentNames.Num())
		{
			return InParentNames[Index];
		}
		return InParentNames.IsEmpty() ? NAME_None : InParentNames[0];  
	};

	const int NumBonesBefore = ReferenceSkeleton->GetRawBoneNum();

	for (int32 Index = 0; Index < NumBonesToAdd; Index++)
	{
		const FName& BoneName = InBoneNames[Index];		
		if (ReferenceSkeleton->FindBoneIndex(InBoneNames[Index]) == INDEX_NONE)
		{
			const FName ParentName = GetParentName(Index);

			// look for parent index in the ref skeleton
			int32 ParentIndex = ReferenceSkeleton->FindBoneIndex(ParentName);
			if (ParentIndex == INDEX_NONE && Index > 0)
			{
				// otherwise, check if one of the new bone is going to be the parent
				ParentIndex = InBoneNames.IndexOfByKey(ParentName);

				if (ParentIndex > INDEX_NONE && ParentIndex < Index)
				{
					ParentIndex += NumBonesBefore;
				}
			}
			const FMeshBoneInfo NewBoneInfo(BoneName, BoneName.ToString(), ParentIndex);
			BonesToAdd.Add({NewBoneInfo, Index});
		}
	}

	if (BonesToAdd.IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Add: None of the provided names is avalable to be added.");
		return false;
	}

	auto GetTransform = [&](int32 Index)
	{
		if (InBoneNames.Num() == InTransforms.Num())
		{
			return InTransforms[Index];
		}
		return InTransforms.IsEmpty() ? FTransform::Identity : InTransforms[0];  
	};
	
	//update reference skeleton
	{
		static constexpr bool bAllowMultipleRoots = true;
		FReferenceSkeletonModifier Modifier(*ReferenceSkeleton, nullptr);
		for (FBoneData& BoneData: BonesToAdd)
		{
			Modifier.Add(BoneData.BoneInfo, GetTransform(BoneData.TransformOffset), bAllowMultipleRoots);
		}
	}

	// invalidate composer
	TransformComposer->Invalidate(INDEX_NONE);
	
	// update index tracker: nothing to do as those new indices do not represent any bone in the initial skinning data
	
	return true;
}

bool USkeletonModifier::MirrorBone(const FName InBoneName, const FMirrorOptions& InOptions)
{
	if (InBoneName == NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Mirror: Cannot mirror bone with no name.");
		return false;
	}
	
	return MirrorBones({InBoneName}, InOptions);
}

bool USkeletonModifier::MirrorBones(const TArray<FName>& InBonesName, const FMirrorOptions& InOptions)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}

	// get bones to mirror
	TArray<int32> BonesToMirror; GetBonesToMirror(InBonesName, InOptions, BonesToMirror);
	
	const int32 NumBonesToMirror = BonesToMirror.Num();
	if (NumBonesToMirror == 0)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Mirror: None of the provided names has been found.");
		return false;
	}

	// get mirrored names
	TArray<FName> MirroredNames; GetMirroredNames(BonesToMirror, InOptions, MirroredNames);

	// add bones first if they are missing
	TArray<int32> MirroredBones; GetMirroredBones(BonesToMirror, MirroredNames, MirroredBones);
	if (MirroredBones.Num() != NumBonesToMirror)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Mirror: Couldn't find mirrored bones.");
		return false;
	}

	// compute mirrored transforms
	TArray<FTransform> MirroredTransforms; GetMirroredTransforms(BonesToMirror, MirroredBones, InOptions, MirroredTransforms);

	// update reference skeleton
	{
		FReferenceSkeletonModifier Modifier(*ReferenceSkeleton, nullptr);
		for (int32 Index = 0; Index < NumBonesToMirror; Index++)
		{
			Modifier.UpdateRefPoseTransform(MirroredBones[Index], MirroredTransforms[Index]);
		}
	}

	// invalidate composer
	TransformComposer->Invalidate(INDEX_NONE);
	
	return true;
}

void USkeletonModifier::GetBonesToMirror(
	const TArray<FName>& InBonesName, const FMirrorOptions& InOptions, TArray<int32>& OutBonesToMirror) const
{
	OutBonesToMirror.Reset();
	
	TSet<int32> IndicesToMirror;
	auto GetBonesToMirror = [&](const int32 BoneIndex, auto&& GetBonesToMirror2) -> void
	{
		if (BoneIndex == INDEX_NONE)
		{
			return;
		}
		
		IndicesToMirror.Add(BoneIndex);

		if (InOptions.bMirrorChildren)
		{
			TArray<int32> Children; ReferenceSkeleton->GetRawDirectChildBones(BoneIndex, Children);
			for (int32 ChildIndex: Children)
			{
				GetBonesToMirror2(ChildIndex, GetBonesToMirror2);
			}
		}
	};
	
	for (const FName& BoneName: InBonesName)
	{
		GetBonesToMirror(ReferenceSkeleton->FindRawBoneIndex(BoneName), GetBonesToMirror);
	}

	const int32 NumBonesToMirror = IndicesToMirror.Num();
	if (NumBonesToMirror == 0)
	{
		return;
	}

	IndicesToMirror.Sort([](const int32 Index0, const int32 Index1) {return Index0 < Index1;});
	OutBonesToMirror = IndicesToMirror.Array();
}

void USkeletonModifier::GetMirroredNames(
	const TArray<int32>& InBonesToMirror, const FMirrorOptions& InOptions, TArray<FName>& OutBonesName) const
{	
	// Similar to UMirrorDataTable::GetMirrorName but doing a search from the end if the find string is contained

	OutBonesName.Reset();
	if (InBonesToMirror.IsEmpty())
	{
		return;
	}

	const FName LeftFindRegex = *FString::Printf(TEXT("([^}]*)%s([^}]*)"), *InOptions.LeftString);
	const FName RightReplaceRegex = *FString::Printf(TEXT("$1%s$2"), *InOptions.RightString);

	const FName RightFindRegex = *FString::Printf(TEXT("([^}]*)%s([^}]*)"), *InOptions.RightString);
	const FName LeftReplaceRegex = *FString::Printf(TEXT("$1%s$2"), *InOptions.LeftString);

	const TArray MirrorFindReplaceExpressions({
		FMirrorFindReplaceExpression(LeftFindRegex, RightReplaceRegex, EMirrorFindReplaceMethod::RegularExpression),
		FMirrorFindReplaceExpression(RightFindRegex, LeftReplaceRegex, EMirrorFindReplaceMethod::RegularExpression)
	});
		
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	
	const auto FindReplaceFromEnd = [](const FName& InBoneName, const FString& InFind, const FString& InReplace, FName& OutResult)
		{
			if (InBoneName.IsNone() ||
				InFind.IsEmpty() ||
				InReplace.IsEmpty())
			{
				// Since replacing can not occur, result to NAME_None implying a new name should be generated
				OutResult = NAME_None;
				return true;
			}

			const FString BoneNameString = InBoneName.ToString();

			const int32 ReplaceIndex = BoneNameString.Find(InFind, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (ReplaceIndex != INDEX_NONE)
			{
				FString ReplaceBoneName = BoneNameString;
				ReplaceBoneName.RemoveAt(ReplaceIndex, InFind.Len());
				ReplaceBoneName.InsertAt(ReplaceIndex, InReplace);

				OutResult = *ReplaceBoneName;
				return true;
			}

			return false;
		};

	OutBonesName.Reserve(InBonesToMirror.Num());
	Algo::Transform(InBonesToMirror, OutBonesName, [&](const int32 BoneIndex)
		{
			const FName BoneName = BoneInfos[BoneIndex].Name;

			const FName MirrorName = [&BoneName, &InOptions, &FindReplaceFromEnd, &MirrorFindReplaceExpressions]()
			{
				// Try to replace with an exact match from end, and if successful prefer it over 
				// UMirrorDataTable::GetMirrorName that does look ups from the front.
				// Still use UMirrorDataTable::GetMirrorName to support regular expressions.
				FName MirrorName;
				if (!FindReplaceFromEnd(BoneName, InOptions.LeftString, InOptions.RightString, MirrorName) &&
					!FindReplaceFromEnd(BoneName, InOptions.RightString, InOptions.LeftString, MirrorName))
				{
					MirrorName = UMirrorDataTable::GetMirrorName(BoneName, MirrorFindReplaceExpressions);
				}

				return MirrorName;
			}(); 

			if (MirrorName.IsNone() || MirrorName == BoneName)
			{
				return GetUniqueName(BoneName, OutBonesName);
			}

			return MirrorName;
		});
}

void USkeletonModifier::GetMirroredBones(
	const TArray<int32>& InBonesToMirror, const TArray<FName>& InMirroredNames, TArray<int32>& OutMirroredBones)
{
	const int32 NumBones = InBonesToMirror.Num();
	
	OutMirroredBones.Reset();
	if (InBonesToMirror.IsEmpty() || NumBones != InMirroredNames.Num())
	{
		return;
	}

	// check mirrored names uniqueness 
	const TSet<FName> UniqueMirroredNames(InMirroredNames);
	if (UniqueMirroredNames.Num() != NumBones)
	{
		return;
	}
	
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	const TArray<FTransform>& BoneTransforms = ReferenceSkeleton->GetRawRefBonePose();
	
	TArray<FName> BonesToAdd, ParentNames;
	TArray<FTransform> Transforms;

	for (int32 Index = 0; Index < NumBones; Index++)
	{
		const FName& MirroredName = InMirroredNames[Index];
		const int32 MirroredIndex = ReferenceSkeleton->FindRawBoneIndex(MirroredName);
		if (MirroredIndex == INDEX_NONE)
		{
			const int32 RefBoneIndex = InBonesToMirror[Index];
				
			// name
			BonesToAdd.Add(MirroredName);

			// parent
			int32 ParentIndex = BoneInfos[RefBoneIndex].ParentIndex;
			FName ParentName = NAME_None;
			if (ParentIndex != INDEX_NONE)
			{
				ParentName = BoneInfos[ParentIndex].Name;
				// is that parent being mirrored?
				const int32 ParentIndexInMirrored = InBonesToMirror.IndexOfByKey(ParentIndex);
				if (ParentIndexInMirrored != INDEX_NONE)
				{
					ParentName = InMirroredNames[ParentIndexInMirrored];
				}
			}
			ParentNames.Add(ParentName);

			// transform
			Transforms.Add(BoneTransforms[RefBoneIndex]);
		}
		else
		{
			OutMirroredBones.Add(MirroredIndex);
		}
	}

	if (BonesToAdd.IsEmpty())
	{
		return;
	}

	// add missing bones and get their index
	OutMirroredBones.Reset();
	if (!AddBones(BonesToAdd, ParentNames, Transforms))
	{
		return;
	}

	OutMirroredBones.Reserve(NumBones);
	for (int32 Index = 0; Index < NumBones; Index++)
	{
		const FName& MirroredName = InMirroredNames[Index];
		const int32 MirroredIndex = ReferenceSkeleton->FindRawBoneIndex(MirroredName);
		if (MirroredIndex == INDEX_NONE)
		{
			OutMirroredBones.Reset();
			return;
		}
		OutMirroredBones.Add(MirroredIndex);
	}
}

void USkeletonModifier::GetMirroredTransforms(
	const TArray<int32>& InBonesToMirror, const TArray<int32>& InMirroredBones,
	const FMirrorOptions& InOptions, TArray<FTransform>& OutMirroredTransforms) const
{
	OutMirroredTransforms.Reset();
	
	const int32 NumBonesToMirror = InBonesToMirror.Num();
	if (NumBonesToMirror == 0)
	{
		return;
	}

	TArray<int32> BonesAndBonesToMirror;
	BonesAndBonesToMirror.Append(InBonesToMirror);
	BonesAndBonesToMirror.Append(InMirroredBones);

	// compute global mirrored transforms
	TArray<FTransform> MirroredGlobal; 
	MirroredGlobal.Reserve(NumBonesToMirror);
	for (int32 Index = 0; Index < NumBonesToMirror; Index++)
	{
		// bone global
		const int32 RefBoneIndex = InBonesToMirror[Index];
		const FTransform GlobalTransform = TransformComposer->GetGlobalTransform(RefBoneIndex);

		// mirror
		const FTransform MirroredGlobalTransform = InOptions.MirrorTransform(GlobalTransform);

		MirroredGlobal.Add(MirroredGlobalTransform);
	}

	// switch back to local
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	OutMirroredTransforms.Reserve(NumBonesToMirror);
	for (int32 Index = 0; Index < NumBonesToMirror; Index++)
	{
		const int32 RefBoneIndex = InBonesToMirror[Index];		
		const int32 RefParentIndex = BoneInfos[RefBoneIndex].ParentIndex;
		const int32 ParentMirroredIndex = InBonesToMirror.IndexOfByKey(RefParentIndex);
		const int32 ParentIndex = BoneInfos[InMirroredBones[Index]].ParentIndex;
		const FTransform& ParentGlobal = ParentMirroredIndex != INDEX_NONE ? MirroredGlobal[ParentMirroredIndex] :
							TransformComposer->GetGlobalTransform(ParentIndex);
		OutMirroredTransforms.Add(MirroredGlobal[Index].GetRelativeTransform(ParentGlobal));
	}
}

// NOTE : that function might take a bUpdateChildren to decide whether we want to compensate the children transforms
// atm, we update the bone's local ref transform so children's global transforms are changed (we just need to cache the
// global transforms then restore them back)
// orienting the bone for example should change the children's global transform
bool USkeletonModifier::SetBoneTransform( const FName InBoneName, const FTransform& InNewTransform, const bool bMoveChildren)
{
	if (InBoneName == NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Move: Cannot move bone with no name.");
		return false;
	}
	
	return SetBonesTransforms({InBoneName}, {InNewTransform}, bMoveChildren);
}

bool USkeletonModifier::SetBonesTransforms(
	const TArray<FName>& InBoneNames, const TArray<FTransform>& InNewTransforms, const bool bMoveChildren)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}
	
	const int32 NumBonesToMove = InBoneNames.Num();
	if (NumBonesToMove == 0 || NumBonesToMove != InNewTransforms.Num())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Move: Discrepancy between bones and transforms (%d / %d).", NumBonesToMove, InNewTransforms.Num());
		return false;
	}
	
	TArray<int32> BoneIndices, Offsets;
	BoneIndices.Reserve(NumBonesToMove);
	Offsets.Reserve(NumBonesToMove);

	for (int32 Index = 0; Index < NumBonesToMove; Index++)
	{
		const int32 BoneIndex = ReferenceSkeleton->FindRawBoneIndex(InBoneNames[Index]);
		if (BoneIndex != INDEX_NONE)
		{
			BoneIndices.Add(BoneIndex);
			Offsets.Add(Index);
		}
	}

	if (BoneIndices.IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Move: None of the provided bones has been found.");
		return false;
	}

	// compute global transforms if needed
	TArray<int32> ChildrenToFix;
	TArray<FTransform> GlobalTransforms;
	
	if (!bMoveChildren)
	{
		// get children
		for (int32 Index = 0; Index < BoneIndices.Num(); Index++)
		{
			TArray<int32> Children;
			ReferenceSkeleton->GetRawDirectChildBones(BoneIndices[Index], Children);
			for (int32 ChildIndex: Children)
			{
				if (!BoneIndices.Contains(ChildIndex))
				{
					ChildrenToFix.Add(ChildIndex);
				}
			}
		}

		// sort them from highest index to lowest
		ChildrenToFix.Sort([](const int32 Index0, const int32 Index1) {return Index0 > Index1;} );
		const int32 NumChildren = ChildrenToFix.Num();

		// compute global transforms (note that we could cache them for faster implementation) 
		GlobalTransforms.AddUninitialized(NumChildren);

		for (int32 Index = 0; Index < NumChildren; Index++)
		{
			GlobalTransforms[Index] = TransformComposer->GetGlobalTransform(ChildrenToFix[Index]);
		}
	}
	
	// update reference skeleton
	{
		FReferenceSkeletonModifier Modifier(*ReferenceSkeleton, nullptr);
		for (int32 Index = 0; Index < BoneIndices.Num(); Index++)
		{
			FTransform NewTransform = InNewTransforms[Offsets[Index]];
			NewTransform.NormalizeRotation();
			Modifier.UpdateRefPoseTransform(BoneIndices[Index], NewTransform);

			// invalidate cached global transform
			TransformComposer->Invalidate(BoneIndices[Index]);
		}

		if (!bMoveChildren)
		{
			const int32 NumChildren = ChildrenToFix.Num();
			for (int32 Index = 0; Index < NumChildren; Index++)
			{
				const int32 ChildrenIndex = ChildrenToFix[Index];
				const int32 ParentIndex = ReferenceSkeleton->GetRawParentIndex(ChildrenIndex);
				const FTransform& NewParentGlobal = TransformComposer->GetGlobalTransform(ParentIndex);
				FTransform NewLocal = GlobalTransforms[Index].GetRelativeTransform(NewParentGlobal);
				NewLocal.NormalizeRotation();
				Modifier.UpdateRefPoseTransform(ChildrenIndex, NewLocal);
				TransformComposer->Invalidate(ChildrenIndex);
			}
		}
	}

	// update index tracker: no modification on bone indices to track when changing transforms
	
	return true;
}

bool USkeletonModifier::RemoveBone(const FName InBoneName, const bool bRemoveChildren)
{
	if (InBoneName == NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Remove: Cannot remove bone with no name.");
		return false;
	}
	
	return RemoveBones({InBoneName}, bRemoveChildren);
}

bool USkeletonModifier::RemoveBones(const TArray<FName>& InBoneNames, const bool bRemoveChildren)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}

	if (InBoneNames.IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Remove: No bone provided.");
		return false;
	}
	
	// store initial data
	const TArray<FMeshBoneInfo> InfosBeforeRemoval = ReferenceSkeleton->GetRawRefBoneInfo();

	TArray<FName> BonesToRemove;
	if (bRemoveChildren)
	{
		const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
		auto IsParentToBeRemoved = [&](const FName BoneName)
		{
			const int32 BoneIndex = ReferenceSkeleton->FindRawBoneIndex(BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				int32 ParentIndex = BoneInfos[BoneIndex].ParentIndex;
				while (ParentIndex != INDEX_NONE)
				{
					if (InBoneNames.Contains(BoneInfos[ParentIndex].Name))
					{
						return true;
					}
					ParentIndex = BoneInfos[ParentIndex].ParentIndex;
				}
			}
			return false;
		};
		
		for (const FName& BoneName: InBoneNames)
		{
			if (!IsParentToBeRemoved(BoneName))
			{
				BonesToRemove.Add(BoneName);
			}
		}
	}
	else
	{
		BonesToRemove = InBoneNames;
	}

	// update reference skeleton
	{
		FReferenceSkeletonModifier Modifier(*ReferenceSkeleton, nullptr);
		for (const FName& BoneName: BonesToRemove)
		{
			Modifier.Remove(BoneName, bRemoveChildren);
		}
	}

	if (InfosBeforeRemoval.Num() == ReferenceSkeleton->GetRawRefBoneInfo().Num())
	{ // no bone has been removed
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Remove: No bone has been removed.");
		return false;
	}
	
	// invalidate composer
	TransformComposer->Invalidate(INDEX_NONE);

	// update index tracker
	UpdateBoneTracker(InfosBeforeRemoval);

	return true;		
}

bool USkeletonModifier::RenameBone(const FName InOldBoneName, const FName InNewBoneName)
{
	if (InOldBoneName == NAME_None || InNewBoneName == NAME_None || InNewBoneName == InOldBoneName)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Rename: cannot rename %ls with %ls.", *InOldBoneName.ToString(), *InNewBoneName.ToString());
		return false;
	}
	
	return RenameBones({InOldBoneName}, {InNewBoneName});
}

bool USkeletonModifier::RenameBones(const TArray<FName>& InOldBoneNames, const TArray<FName>& InNewBoneNames)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}
	
	if (InOldBoneNames.IsEmpty() || InNewBoneNames.Num() != InOldBoneNames.Num())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Rename: Discrepancy between old and new names (%d / %d).", InOldBoneNames.Num(), InNewBoneNames.Num());
		return false;
	}

	// update reference skeleton
	{
		FReferenceSkeletonModifier Modifier(*ReferenceSkeleton, nullptr);

		TArray<FName> NewBoneNames;
		
		const int32 NumBonesToRename = InOldBoneNames.Num();
		for (int32 Index = 0; Index < NumBonesToRename; ++Index)
		{
			const FName& OldName = InOldBoneNames[Index];
			const FName NewName = GetUniqueName(InNewBoneNames[Index], NewBoneNames);
			if (OldName != NAME_None && !OldName.IsEqual(NewName, ENameCase::CaseSensitive))
			{
				NewBoneNames.Add(NewName);
				Modifier.Rename(OldName, NewName);
			}
		}
	}

	// update index tracker: no modification on bone indices to track when renaming

	return true;
}

bool USkeletonModifier::ParentBone(const FName InBoneName, const FName InParentName)
{
	if (InBoneName == NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Parent: Cannot parent a bone with no name.");
		return false;
	}
	
	return ParentBones({InBoneName}, {InParentName});
}

bool USkeletonModifier::ParentBones(const TArray<FName>& InBoneNames, const TArray<FName>& InParentNames)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}
	
	if (InBoneNames.IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Parent: No bone provided.");
		return false;
	}

	// store initial data
	const TArray<FMeshBoneInfo> InfosBeforeParenting = ReferenceSkeleton->GetRawRefBoneInfo();
	
	// update reference skeleton
	{
		auto GetParentName = [&](int32 Index)
		{
			if (InBoneNames.Num() == InParentNames.Num())
			{
				return InParentNames[Index];
			}
			return InParentNames.IsEmpty() ? NAME_None : InParentNames[0];  
		};

		static constexpr bool bAllowMultipleRoots = true;
		
		FReferenceSkeletonModifier Modifier(*ReferenceSkeleton, nullptr);
		for (int32 Index = 0; Index < InBoneNames.Num(); ++Index)
		{
			const int32 BoneIndex = ReferenceSkeleton->FindRawBoneIndex(InBoneNames[Index]);
			if (BoneIndex != INDEX_NONE)
			{
				const FName NewParentName = GetParentName(Index);
				
				// change parent
				const int32 NewIndex = Modifier.SetParent(InBoneNames[Index], NewParentName, bAllowMultipleRoots);
				if (NewIndex > INDEX_NONE)
				{
					// invalidate composer
					TransformComposer->Invalidate(INDEX_NONE);
				}
			}
		}
	}

	// update index tracker
	UpdateBoneTracker(InfosBeforeParenting);
	
	return true;
}

bool USkeletonModifier::OrientBone(const FName InBoneName, const FOrientOptions& InOptions)
{
	if (InBoneName == NAME_None)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Orient: Cannot orient a bone with no name.");
		return false;
	}
	
	return OrientBones({InBoneName}, InOptions);
}

bool USkeletonModifier::OrientBones(const TArray<FName>& InBoneNames, const FOrientOptions& InOptions)
{
	if (!IsReferenceSkeletonValid())
	{
		return false;
	}
	
	if (InBoneNames.IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Orient: No bone provided.");
		return false;
	}

	// get bones to mirror
	TArray<int32> BonesToOrient; GetBonesToOrient(InBoneNames, InOptions, BonesToOrient);
	const int32 NumBonesToOrient = BonesToOrient.Num();
	if (NumBonesToOrient == 0)
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Orient: None of the provided names has been found.");
		return false;
	}
	
	auto GetAlignedTransform = [&](const int32 BoneIndex) -> FTransform
	{
		const FTransform& BoneGlobal = TransformComposer->GetGlobalTransform(BoneIndex);
		
		const int32 ParentIndex = BoneIndex != INDEX_NONE ? ReferenceSkeleton->GetRawParentIndex(BoneIndex) : INDEX_NONE;

		const FTransform& ParentGlobal = TransformComposer->GetGlobalTransform(ParentIndex);
		FVector Direction = (BoneGlobal.GetLocation() - ParentGlobal.GetLocation()).GetSafeNormal();
			
		TArray<int32> Children; 
		ReferenceSkeleton->GetRawDirectChildBones(BoneIndex, Children);
	
		if (InOptions.bOrientChildren &&
			Children.Num() == 1)
		{
			const FTransform& ChildGlobal = TransformComposer->GetGlobalTransform(Children[0]);
			Direction = (ChildGlobal.GetLocation() - BoneGlobal.GetLocation()).GetSafeNormal();
		}
		
		if (Direction.IsNearlyZero())
		{
			return BoneGlobal;
		}

		// compute the secondary target based on the plane formed by the bones if needed
		if (InOptions.bUsePlaneAsSecondary)
		{
			const FVector SecondaryDirection = (BoneGlobal.GetLocation() - ParentGlobal.GetLocation()).GetSafeNormal();
			if (!SecondaryDirection.IsNearlyZero())
			{
				const bool bComputePlane = (FMath::Abs(FVector::DotProduct(Direction, SecondaryDirection)) - 1.0f) < KINDA_SMALL_NUMBER;
				if (bComputePlane)
				{ 
					// use the plane normal as the secondary target, otherwise use InOptions' SecondaryTarget
					FOrientOptions Options = InOptions;
					Options.SecondaryTarget = FVector::CrossProduct(Direction, SecondaryDirection);
					return Options.OrientTransform(Direction, BoneGlobal);
				}
			}
		}

		return InOptions.OrientTransform(Direction, BoneGlobal);
	};

	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	
	TArray<FName> BonesToAlign; BonesToAlign.Reserve(NumBonesToOrient);
	TArray<FTransform> OrientedGlobal; OrientedGlobal.Reserve(NumBonesToOrient);
	for (int32 Index = 0; Index < NumBonesToOrient; ++Index)	
	{
		const int32 BoneIndex = BonesToOrient[Index];
		BonesToAlign.Add(BoneInfos[BoneIndex].Name);
		OrientedGlobal.Add(GetAlignedTransform(BoneIndex));
	}

	// switch back to local
	TArray<FTransform> Transforms; Transforms.Reserve(NumBonesToOrient);
	for (int32 Index = 0; Index < BonesToAlign.Num(); Index++)
	{
		const FName& BoneName = BonesToAlign[Index];
		const int32 BoneIndex = ReferenceSkeleton->FindRawBoneIndex(BoneName);
		const int32 ParentIndex = BoneInfos[BoneIndex].ParentIndex;
		const int32 ParentOrientedIndex = ParentIndex != INDEX_NONE ? BonesToAlign.IndexOfByKey(BoneInfos[ParentIndex].Name) : INDEX_NONE;
		const FTransform& ParentGlobal = ParentOrientedIndex != INDEX_NONE ? OrientedGlobal[ParentOrientedIndex] :
					TransformComposer->GetGlobalTransform(ParentIndex);
		Transforms.Add(OrientedGlobal[Index].GetRelativeTransform(ParentGlobal));
	}

	if (BonesToAlign.IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skeleton Modifier - Orient: No bone to orient.");
		return false;
	}

	static constexpr bool bMoveChildren = false;
	return SetBonesTransforms(BonesToAlign, Transforms, bMoveChildren);
}

void USkeletonModifier::GetBonesToOrient(
	const TArray<FName>& InBonesName, const FOrientOptions& InOptions, TArray<int32>& OutBonesToOrient) const
{
	OutBonesToOrient.Reset();
	
	TSet<int32> IndicesToOrient;
	auto GetBonesToOrient = [&](const int32 BoneIndex, auto&& GetBonesToOrient2) -> void
	{
		if (BoneIndex == INDEX_NONE)
		{
			return;
		}
		
		IndicesToOrient.Add(BoneIndex);

		if (InOptions.bOrientChildren)
		{
			TArray<int32> Children; ReferenceSkeleton->GetRawDirectChildBones(BoneIndex, Children);
			for (int32 ChildIndex: Children)
			{
				GetBonesToOrient2(ChildIndex, GetBonesToOrient2);
			}
		}
	};
	
	for (const FName& BoneName: InBonesName)
	{
		GetBonesToOrient(ReferenceSkeleton->FindRawBoneIndex(BoneName), GetBonesToOrient);
	}

	const int32 NumBonesToOrient = IndicesToOrient.Num();
	if (NumBonesToOrient == 0)
	{
		return;
	}

	IndicesToOrient.Sort([](const int32 Index0, const int32 Index1) {return Index0 < Index1;});
	OutBonesToOrient = IndicesToOrient.Array();
}

void USkeletonModifier::UpdateBoneTracker(const TArray<FMeshBoneInfo>& InOtherInfos)
{
	for (int32 Index = 0; Index < BoneIndexTracker.Num(); ++Index)
	{
		const int32 IndexToTrack = BoneIndexTracker[Index];
		if (IndexToTrack > INDEX_NONE)
		{
			check(InOtherInfos.IsValidIndex(IndexToTrack));
			const int32 NewIndex = ReferenceSkeleton->FindRawBoneIndex(InOtherInfos[IndexToTrack].Name);
			BoneIndexTracker[Index] = NewIndex;
		}
	}
}

FName USkeletonModifier::GetUniqueName(const FName InBoneName, const TArray<FName>& InBoneNames) const
{
	if (!ReferenceSkeleton || InBoneName == NAME_None)
	{
		return NAME_None;
	}

	static constexpr TCHAR Underscore = '_';
	static constexpr TCHAR Hashtag = '#';

	int32 LastHashtag = INDEX_NONE, LastDigit = INDEX_NONE;
	
	FString InBoneNameStr = InBoneName.ToString();

	// 1. Remove white spaces from start / end
	InBoneNameStr.TrimStartAndEndInline();
	const int32 NameLength = InBoneNameStr.Len();

	// 2. Sanitize name: remove unwanted characters and get padding info from hashtags or digits
	auto SanitizedName = [&]()
	{
		bool bHasAnyGoodChar = false;
		for (int32 Index = 0; Index < NameLength; ++Index)
		{
			TCHAR& C = InBoneNameStr[Index];

			const bool bIsAlphaOrUnderscore = FChar::IsAlpha(C) || (C == Underscore);
			
			const bool bIsDigit = FChar::IsDigit(C);
			if (bIsDigit)
			{
				LastDigit = Index;
			}
			
			const bool bIsHashtag = C == Hashtag;
			if (bIsHashtag)
			{
				LastHashtag = Index;
			}
			
			const bool bGoodChar = bIsAlphaOrUnderscore || bIsHashtag || bIsDigit;
			bHasAnyGoodChar |= bGoodChar;
			if (!bGoodChar)
			{
				C = Underscore;
			}
		}
		return bHasAnyGoodChar;
	};

	// 3. Early exit if none of the character is good to use
	const bool bHasAnyGoodChar = SanitizedName();
	if (!bHasAnyGoodChar)
	{
		return NAME_None;
	}

	// 4. if we found a padding, check if there are digits before/after to grow it if needed (ei. joint_##10_left)

	// note that # takes priority here
	int32 EndPadding = LastHashtag != INDEX_NONE ? LastHashtag : LastDigit != INDEX_NONE ? LastDigit : INDEX_NONE;
	int32 StartPadding = EndPadding;
	FString PaddingStr;

	if (EndPadding != INDEX_NONE)
	{
		// find # or digit before EndPadding
		bool bIsHashtag = InBoneNameStr[StartPadding] == Hashtag, bIsDigit = FChar::IsDigit(InBoneNameStr[StartPadding]);

		while (StartPadding > INDEX_NONE && (bIsHashtag || bIsDigit))
		{
			--StartPadding;
			if (StartPadding > INDEX_NONE)
			{
				bIsHashtag = InBoneNameStr[StartPadding] == Hashtag;
				bIsDigit = FChar::IsDigit(InBoneNameStr[StartPadding]);
			}
		}
		
		// find # or digit after StartPadding
		StartPadding = StartPadding+1;
		EndPadding = StartPadding;

		bIsHashtag = InBoneNameStr[EndPadding] == Hashtag, bIsDigit = FChar::IsDigit(InBoneNameStr[EndPadding]);

		while (EndPadding < NameLength && (bIsHashtag || bIsDigit))
		{
			++EndPadding;
			if (EndPadding < NameLength)
			{
				bIsHashtag = InBoneNameStr[EndPadding] == Hashtag;
				bIsDigit = FChar::IsDigit(InBoneNameStr[EndPadding]);
			}
		}
		EndPadding -= 1;

		// store the padding string
		if (EndPadding >= StartPadding)
		{
			PaddingStr = InBoneNameStr.Mid(StartPadding, EndPadding-StartPadding+1);
		}

		// replace any # with zeros
		static constexpr TCHAR Zero = '0';
		InBoneNameStr.ReplaceInline(&Hashtag, &Zero);
		PaddingStr.ReplaceInline(&Hashtag, &Zero);
	}

	// 5. prepare prefix, suffix and padding
	FString Prefix = InBoneNameStr;
	FString Suffix;
	int32 CurrentIndex = 1;

	if (StartPadding != INDEX_NONE)
	{
		CurrentIndex = PaddingStr.IsEmpty() ? INDEX_NONE : FCString::Atoi(*PaddingStr);
		if (CurrentIndex == 0)
		{
			PaddingStr[PaddingStr.Len()-1] = '1';
			CurrentIndex = 1;
		}
		Prefix = InBoneNameStr.Left(StartPadding);
		Suffix = InBoneNameStr.RightChop(EndPadding+1);
	}

	// check for availability in both the reference skeleton and the names that are going to be added 
	auto IsNameAvailable = [&](const FString& InNameStr)
	{
		const FName Name(*InNameStr);
		
		const int32 Index = ReferenceSkeleton->FindRawBoneIndex(Name);
        if (Index != INDEX_NONE)
        {
        	if (!ReferenceSkeleton->GetBoneName(Index).IsEqual(Name, ENameCase::CaseSensitive))
        	{
        		const bool bContainsByPredicate = InBoneNames.ContainsByPredicate([Name](const FName& BoneName)
        		{
        			return BoneName.IsEqual(Name, ENameCase::CaseSensitive);
        		});
        		
        		if (!bContainsByPredicate)
        		{
        			return true;
        		}
        	}
        	
        	return false;
        }
		
		if (InBoneNames.Contains(Name))
		{
			return false;
		}
		
		return true;
	};

	// 6. build the new unique name 
	FString OutBoneNameStr = Prefix + PaddingStr + Suffix;
	while (!IsNameAvailable(OutBoneNameStr))
	{
		// increment the index
		FString NewIncrement = PaddingStr.FromInt(CurrentIndex++);

		// switch this new index into a padding str
		const int32 IncrementLen = PaddingStr.Len();
		const int32 NewIncrementLen = NewIncrement.Len();
		if (NewIncrementLen < IncrementLen)
		{
			int32 Index = 0;
			while (Index < NewIncrementLen)
			{
				PaddingStr[IncrementLen-1-Index] = NewIncrement[NewIncrementLen-1-Index];
				Index++;
			}
		}
		else
		{
			PaddingStr = NewIncrement;
		}

		// form the new name
		OutBoneNameStr = Prefix + PaddingStr + Suffix;
	}

	return FName(*OutBoneNameStr);
}

const FReferenceSkeleton& USkeletonModifier::GetReferenceSkeleton() const
{
	static const FReferenceSkeleton Dummy;
	return ReferenceSkeleton ? *ReferenceSkeleton.Get() : Dummy;
}

const TArray<int32>& USkeletonModifier::GetBoneIndexTracker() const
{
	return BoneIndexTracker;	
}

FTransform USkeletonModifier::GetBoneTransform(const FName InBoneName, const bool bGlobal) const
{
	if (!ReferenceSkeleton.IsValid())
	{
		return FTransform::Identity;
	}
	return GetTransform(ReferenceSkeleton->FindRawBoneIndex(InBoneName), bGlobal);
}

FName USkeletonModifier::GetParentName(const FName InBoneName) const
{
	if (!IsReferenceSkeletonValid())
	{
		return NAME_None;
	}
	
	const int32 BoneIndex = ReferenceSkeleton->FindBoneIndex(InBoneName);
	if (BoneIndex == INDEX_NONE)
	{
		return NAME_None; 
	}
	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	const int32 ParentIndex = BoneInfos[BoneIndex].ParentIndex;
	return (ParentIndex > INDEX_NONE) ? BoneInfos[ParentIndex].Name : NAME_None;
}

TArray<FName> USkeletonModifier::GetChildrenNames(const FName InBoneName, const bool bRecursive) const
{
	TArray<FName> ChildrenNames;

	if (!IsReferenceSkeletonValid())
	{
		return ChildrenNames;
	}
	
	const int32 BoneIndex = ReferenceSkeleton->FindRawBoneIndex(InBoneName);
	if (BoneIndex == INDEX_NONE)
	{
		return ChildrenNames; 
	}

	TArray<int32> ChildrenIndices;
	if (bRecursive)
	{
		auto GetChildren = [&](const int32 InBoneIndex, auto&& GetChildren) -> void
		{
			TArray<int32> Children; ReferenceSkeleton->GetRawDirectChildBones(InBoneIndex, Children);
			ChildrenIndices.Append(Children);
			for (int32 ChildIndex: Children)
			{
				GetChildren(ChildIndex, GetChildren);
			}
		};
		GetChildren(BoneIndex, GetChildren);
	}
	else
	{
		ReferenceSkeleton->GetRawDirectChildBones(BoneIndex, ChildrenIndices);
	}

	ChildrenNames.Reserve(ChildrenIndices.Num());

	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	Algo::Transform(ChildrenIndices, ChildrenNames, [&BoneInfos](int32 BoneIndex) { return BoneInfos[BoneIndex].Name; });

	return ChildrenNames;
}

TArray<FName> USkeletonModifier::GetAllBoneNames() const
{
	TArray<FName> BoneNames;
	if (!IsReferenceSkeletonValid())
	{
		return BoneNames;
	}

	const TArray<FMeshBoneInfo>& BoneInfos = ReferenceSkeleton->GetRawRefBoneInfo();
	BoneNames.Reserve(BoneInfos.Num());

	Algo::Transform(BoneInfos, BoneNames, [](const FMeshBoneInfo& BoneInfo) { return BoneInfo.Name; });

	return BoneNames;
}

const FTransform& USkeletonModifier::GetTransform(const int32 InBoneIndex, const bool bGlobal) const
{
	if (!ReferenceSkeleton.IsValid())
	{
		return FTransform::Identity;
	}
	
	if (bGlobal)
	{
		return TransformComposer.IsValid() ? TransformComposer->GetGlobalTransform(InBoneIndex) : FTransform::Identity;  
	}

	const TArray<FTransform>& LocalTransforms = ReferenceSkeleton->GetRawRefBonePose();
	return LocalTransforms.IsValidIndex(InBoneIndex) ? LocalTransforms[InBoneIndex] : FTransform::Identity;
}


#undef LOCTEXT_NAMESPACE
