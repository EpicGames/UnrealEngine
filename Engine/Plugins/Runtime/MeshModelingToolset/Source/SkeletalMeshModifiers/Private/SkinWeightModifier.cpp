// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinWeightModifier.h"

#include "MeshDescription.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshModel.h"
#include "EngineLogs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinWeightModifier)

#define LOCTEXT_NAMESPACE "SkinWeightModifier"

bool USkinWeightModifier::SetSkeletalMesh(USkeletalMesh* InMesh)
{
	// prepare to load mesh data
	Reset();
	
#if WITH_EDITORONLY_DATA
	
	// validate supplied skeletal mesh exists
	if (!InMesh)
	{
		UE_LOGF(LogAnimation, Error, "Skin Weight Modifier: No skeletal mesh supplied to load.");
		return false;
	}

	// verify user is not trying to modify one of the core engine assets
	if (InMesh->GetPathName().StartsWith(TEXT("/Engine/")))
	{
		UE_LOGF(LogAnimation, Error, "Skin Weight Modifier: Cannot modify built-in engine asset.");
		return false;
	}

	// store pointer to mesh 
	Mesh = InMesh;
	
	// store mesh description to edit
	MeshDescription = MakeUnique<FMeshDescription>();
	InMesh->CloneMeshDescription(LODIndex, *MeshDescription);
	if (MeshDescription->IsEmpty())
	{
		UE_LOGF(LogAnimation, Error, "Skin Weight Modifier: mesh description is empty.");
		return false;
	}

	// load weights into memory for subsequent editing
	const FSkeletalMeshConstAttributes MeshAttribs(*MeshDescription);
	const FSkinWeightsVertexAttributesConstRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
	const int32 NumVertices = MeshDescription->Vertices().Num();
	Weights.SetNum(NumVertices);
	const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
	{
		const FVertexID VertexID(VertexIndex);
		int32 InfluenceIndex = 0;
		for (UE::AnimationCore::FBoneWeight BoneWeight: VertexSkinWeights.Get(VertexID))
		{
			const int32 BoneIndex = BoneWeight.GetBoneIndex();
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
			const float Weight = BoneWeight.GetWeight();
			Weights[VertexIndex].Add(BoneName, Weight);
			++InfluenceIndex;
		}

		if (InfluenceIndex > MAX_TOTAL_INFLUENCES)
		{
			UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied skeletal mesh has a vertex with too many influences.");
		}
	}
	return true;
	
#else
	ensureMsgf(false, TEXT("Skin Weight Modifier: is an editor only feature."));
	return false;
#endif
}

bool USkinWeightModifier::CommitWeightsToSkeletalMesh()
{
	if (!Mesh)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot apply weights to skeletal mesh.");
		return false;
	}

#if WITH_EDITORONLY_DATA
	// update weights in the mesh description
	{
		FSkeletalMeshAttributes MeshAttribs(*MeshDescription);
		FSkinWeightsVertexAttributesRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
	
		UE::AnimationCore::FBoneWeightsSettings Settings;
		Settings.SetNormalizeType(UE::AnimationCore::EBoneWeightNormalizeType::Always);

		TArray<UE::AnimationCore::FBoneWeight> SourceBoneWeights;
		SourceBoneWeights.Reserve(UE::AnimationCore::MaxInlineBoneWeightCount);

		const int32 NumVertices = MeshDescription->Vertices().Num();
		const TArray<FName> AllBoneNames = GetAllBoneNames();
		for (int32 VertexIndex = 0; VertexIndex < NumVertices; VertexIndex++)
		{
			SourceBoneWeights.Reset();

			const TMap<FName, float>& VertexWeights = Weights[VertexIndex];
			if (VertexWeights.IsEmpty())
			{
				// in case where user has removed ALL influences from a vertex, revert back to the root bone
				const FName RootName = Mesh->GetRefSkeleton().GetBoneName(0);
				const int32 BoneIndex = AllBoneNames.IndexOfByKey(RootName);
				constexpr float FullWeight = 1.0f;
				SourceBoneWeights.Add(UE::AnimationCore::FBoneWeight(BoneIndex, FullWeight));
			}
			else
			{
				for (const TTuple<FName, float>& SingleBoneWeight : VertexWeights)
				{
					const FName BoneName = SingleBoneWeight.Key;
					const int32 BoneIndex = AllBoneNames.IndexOfByKey(BoneName);
					const float Weight = SingleBoneWeight.Value;
					SourceBoneWeights.Add(UE::AnimationCore::FBoneWeight(BoneIndex, Weight));
				}
			}

			VertexSkinWeights.Set(FVertexID(VertexIndex), UE::AnimationCore::FBoneWeights::Create(SourceBoneWeights, Settings));
		}
	}
	
	// push changes to the skeletal mesh asset
	{
		// flush any pending rendering commands, which might touch a component while we are rebuilding it's mesh
		FlushRenderingCommands();
		
		// update skeletal mesh LOD (cf. USkeletalMesh::CommitMeshDescription)
		Mesh->ModifyMeshDescription(LODIndex);
		Mesh->CreateMeshDescription(LODIndex, MoveTemp(*MeshDescription));
		Mesh->CommitMeshDescription(LODIndex);
		// rebuilds mesh
		Mesh->PostEditChange();
	}

	return true;

#else
	ensureMsgf(false, TEXT("Skin Weight Modifier: is an editor only feature."));
	return false;
#endif
}

TMap<FName, float> USkinWeightModifier::GetVertexWeights(int32 VertexID)
{
	if (!Weights.IsValidIndex(VertexID))
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied vertex index is not valid for this mesh.");
		return TMap<FName, float>();
	}

	return Weights[VertexID];
}

bool USkinWeightModifier::SetVertexWeights(
	const int32 VertexID,
	const TMap<FName, float>& InWeights,
	const bool bReplaceAll)
{
	if (!Mesh)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot set weights.");
		return false;
	}

	if (!Weights.IsValidIndex(VertexID))
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied vertex index is not valid on this mesh. Cannot set weights.");
		return false;
	}

	TMap<FName, float>& VertexWeights = Weights[VertexID];
	if (bReplaceAll)
	{
		// replace weights entirely
		VertexWeights = InWeights;
	}
	else
	{
		// replace only the input weights
		for (const TTuple<FName, float>& InWeight : InWeights)
		{
			VertexWeights.FindOrAdd(InWeight.Key) = InWeight.Value;
		}
	}
	
	return true;
}

bool USkinWeightModifier::NormalizeVertexWeights(const int32 VertexID)
{
	if (!Mesh)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot normalize weights.");
		return false;
	}

	if (!Weights.IsValidIndex(VertexID))
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied vertex ID was invalid. Cannot normalize vertex weights.");
		return false;
	}

	return NormalizeVertexInternal(VertexID);
}

bool USkinWeightModifier::NormalizeAllWeights()
{
	if (!(Mesh && MeshDescription))
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot normalize weights.");
		return false;
	}
	
	const int32 NumVertices = MeshDescription->Vertices().Num();
	for (int32 VertexID=0; VertexID<NumVertices; ++VertexID)
	{
		NormalizeVertexInternal(VertexID);
	}

	return true;
}

bool USkinWeightModifier::EnforceMaxInfluences(const int32 MaxInfluences)
{
	if (!MeshDescription)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot enforce max influences.");
		return false;
	}
	
	int32 Max = MaxInfluences;
	if (Max < 0)
	{
		Max = UE::AnimationCore::MaxInlineBoneWeightCount;
	}

	const int32 NumVertices = MeshDescription->Vertices().Num();
	for (int32 VertexID=0; VertexID<NumVertices; ++VertexID)
	{
		EnforceMaxInfluenceInternal(VertexID, Max);
	}

	return true;
}

bool USkinWeightModifier::PruneVertexWeights(const int32 VertexID, const float WeightThreshold)
{
	if (!Mesh)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot prune weights.");
		return false;
	}

	if (!Weights.IsValidIndex(VertexID))
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied vertex ID was invalid. Cannot prune weights.");
		return false;
	}

	PruneVertexWeightInternal(VertexID, WeightThreshold);
	
	return true;
}

bool USkinWeightModifier::PruneAllWeights(const float WeightThreshold)
{
	if (!Mesh)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot prune weights.");
		return false;
	}

	const int32 NumVertices = MeshDescription->Vertices().Num();
	for (int32 i=0; i<NumVertices; ++i)
	{
		PruneVertexWeightInternal(i, WeightThreshold);
	}
	
	return true;
}

int32 USkinWeightModifier::GetNumVertices()
{
	if (!MeshDescription)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot get vertex count.");
		return INDEX_NONE;
	}

	return MeshDescription->Vertices().Num();
}

TArray<FName> USkinWeightModifier::GetAllBoneNames()
{
	if (!Mesh)
	{
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: No mesh loaded. Cannot get bones.");
		return TArray<FName>();
	}

	const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	const int32 NumBones = RefSkeleton.GetNum();
	TArray<FName> AllBoneNames;
	for (int32 i=0; i<NumBones; ++i)
	{
		AllBoneNames.Add(RefSkeleton.GetBoneName(i));
	}
	return AllBoneNames;
}

bool USkinWeightModifier::NormalizeVertexInternal(const int32 VertexID)
{
	// handle case where user has removed all weights on a vertex (probably a mistake so warn them)
	TMap<FName, float>& VertexWeights = Weights[VertexID];
	if (VertexWeights.IsEmpty())
	{
		const FName RootBone = Mesh->GetRefSkeleton().GetBoneName(0);
		VertexWeights.Add(RootBone, 1.f);
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied vertex did not have any bone influences. Applied all weight to root bone.");
		return true;
	}

	// calculate total weight on all vertices
	float TotalWeight = 0.f;
	for (const TTuple<FName, float>& Weight : VertexWeights)
	{
		TotalWeight += Weight.Value;
	}

	// handle case where total weight is near zero
	if (TotalWeight <= KINDA_SMALL_NUMBER)
	{
		// in this case, we just split the weight evenly across all available influences
		const float SplitWeight = 1.f / VertexWeights.Num();
		for (TTuple<FName, float>& Weight : VertexWeights)
		{
			Weight.Value = SplitWeight;
		}
		UE_LOGF(LogAnimation, Warning, "Skin Weight Modifier: Supplied vertex had near zero total weight.");
		return true;
	}

	// scale weights to achieve total value of 1.0
	for (TTuple<FName, float>& Weight : VertexWeights)
	{
		Weight.Value /= TotalWeight;
	}

	return true;
}

void USkinWeightModifier::EnforceMaxInfluenceInternal(const int32 VertexID, const int32 Max)
{
	// get weights for this vertex
	TMap<FName, float>& VertexWeights = Weights[VertexID];

	// already under max influences?
	if (VertexWeights.Num() <= Max)
	{
		return;
	}

	// remove smallest influences until we reach max influences
	while(VertexWeights.Num() > Max)
	{
		FName SmallestInfName = NAME_None;
		float SmallestInf = TNumericLimits<float>::Max();
		for (const TTuple<FName, float> Weight : VertexWeights)
		{
			if (Weight.Value <= SmallestInf)
			{
				SmallestInf = Weight.Value;
				SmallestInfName = Weight.Key;
			}
		}

		VertexWeights.Remove(SmallestInfName);
	}

	NormalizeVertexInternal(VertexID);
}

bool USkinWeightModifier::PruneVertexWeightInternal(const int32 VertexID, const float WeightThreshold)
{
	// get weights for this vertex
	TMap<FName, float>& VertexWeights = Weights[VertexID];
	TArray<FName> InfluencesToPrune;
	for (const TTuple<FName, float>& Weight : VertexWeights)
	{
		if (Weight.Value < WeightThreshold)
		{
			InfluencesToPrune.Add(Weight.Key);
		}
	}
	
	for (const FName InfluenceToPrune : InfluencesToPrune)
	{
		VertexWeights.Remove(InfluenceToPrune);
	}
	
	return true;
}

void USkinWeightModifier::Reset()
{
	Mesh = nullptr;
	MeshDescription.Reset();
	Weights.Reset();
}

#undef LOCTEXT_NAMESPACE
