// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/NameTypes.h"
#include "Math/MathFwd.h"

#include "SkeletonClipboard.generated.h"

class USkeletonModifier;

/**
 * The functions below are helper functions that simplify usage of copying/pasting bones on a FReferenceSkeleton
 * using a USkeletonModifier.
 */

namespace SkeletonClipboard
{
	/** DEPRECATED 5.8, instead use CopyBonesToClipboard */
	UE_DEPRECATED(5.8, "Instead use CopyBonesToClipboard to avoid ambiguity with CopyBoneTransformsToClipboard")
	SKELETALMESHMODIFIERS_API void CopyToClipboard(
		const USkeletonModifier& InModifier, 
		const TArray<FName>& InBonesToCopy);

	/**
	 * Copies InBonesToCopy data (name, parent, transform) into the clipboard. Also copies children of these bones.
	 *
	 * @param InModifier				The skeleton modifier that provides the skeleton and the bones to copy.
	 * @param InBonesToCopy				The bones which are copied
	 */
	SKELETALMESHMODIFIERS_API void CopyBonesToClipboard(
		const USkeletonModifier& InModifier, 
		const TArray<FName>& InBonesToCopy);

	/** DEPRECATED 5.8, instead use PasteBonesFromClipboard	 */
	UE_DEPRECATED(5.8, "Instead use the PasteBonesFromClipboard that offers the bUsingGlobalTransforms argument")
	SKELETALMESHMODIFIERS_API TArray<FName> PasteFromClipboard(
		USkeletonModifier& InOutModifier, 
		const FName& InDefaultParent);
	
	/**
	 * Paste new bones from the clipboard InBonesToCopy data (name, parent, transform) into the InOutModifier.
	 * 
	 * @param InOutModifier				The skeleton modifier that provides the skeleton to which the bones are pasted
	 * @param InDefaultParent			The parent to which bones are pasted
	 * @param bUsingGlobalTransforms	If true, pastes global transforms
	 */
	 SKELETALMESHMODIFIERS_API TArray<FName> PasteBonesFromClipboard(
		USkeletonModifier& InOutModifier, 
		const FName& InDefaultParent, 
		const bool bUsingGlobalTransforms);
	
	/**
	 * Duplicates InBonesToCopy data (name, parent, transform) using the clipboard. Also duplicates children of these bones.
	 * 
	 * @param InOutModifier				The skeleton modifier that provides the skeleton and the bones to duplicate.
	 * @param InBonesToDuplicate		The bones that are being duplicated
	 */
	SKELETALMESHMODIFIERS_API TArray<FName> ClipboardDuplicateBones(
		USkeletonModifier& InOutModifier, 
		const TArray<FName>& InBonesToDuplicate);
	
	/**
	 * Copies transforms of InBonesToCopy into the clipboard. 
	 * Hierarchy info & transforms are retrieved via the InModifier.
	 * 
	 * @param InModifier				The skeleton modifier that provides the skeleton and the bones from which transforms are copied.
	 * @param InBonesToCopy				The bones from which transforms are copied
	 */
	SKELETALMESHMODIFIERS_API void CopyBoneTransformsToClipboard(
		const USkeletonModifier& InModifier, 
		TArray<FName> InBonesToCopy);

	/** Flags for PasteBoneTransformsFromClipboard */
	enum class EPasteBoneTransformsFromClipboardFlags
	{
		None					= 0,
		UpdateChildren			= 1 << 0,
		UsingGlobalTransforms	= 1 << 1,
	};
	ENUM_CLASS_FLAGS(EPasteBoneTransformsFromClipboardFlags);

	/**
	 * Paste transforms from the clipboard data into the InOutModifier.
	 * 
	 * @param InOutModifier				The skeleton modifier that provides the skeleton and the bones to which transforms are pasted.
	 * @param InCandidateBoneNames		Bones where the transforms should be pasted, not necessarily considered
	 * @param Flags						Flags to define how bone transforms should be pasted
	 * @return							The bones that were affected by the paste action
	 */
	SKELETALMESHMODIFIERS_API TArray<FName> PasteBoneTransformsFromClipboard(
		USkeletonModifier& InOutModifier,
		TArray<FName> InCandidateBoneNames,
		const EPasteBoneTransformsFromClipboardFlags Flags);
		
	/** DEPRECATED 5.8, instead use IsBoneClipboardValid and IsBoneTransformClipboardValid */
	UE_DEPRECATED(5.8, "Instead use IsBoneClipboardValid and IsBoneTransformClipboardValid")
	SKELETALMESHMODIFIERS_API bool IsClipboardValid();
	
	/**
	 * Checks whether bone clipboard data is valid. 
	 */
	SKELETALMESHMODIFIERS_API bool IsBoneClipboardValid();

	/**
	 * Checks whether bone transform clipboard data is valid. 
	 */
	SKELETALMESHMODIFIERS_API bool IsBoneTransformClipboardValid();

}

USTRUCT()
struct FBoneClipboardData
{
	GENERATED_USTRUCT_BODY()
	
	FBoneClipboardData() = default;
	
	// removing deprecation for default copy operator/constructor to avoid deprecation warnings
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FBoneClipboardData(const FBoneClipboardData&) = default;
	FBoneClipboardData(FBoneClipboardData&&) = default;
	FBoneClipboardData& operator=(const FBoneClipboardData&) = default;
	FBoneClipboardData& operator=(FBoneClipboardData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	FName BoneName = NAME_None;
	
	UPROPERTY()
	FName ParentBoneName = NAME_None;
	
	UE_DEPRECATED(5.8, "Instead refer to ParentBoneName")
	UPROPERTY()
	int32 ParentIndex_DEPRECATED = INDEX_NONE;
	
	UPROPERTY()
	FTransform Global = FTransform::Identity;
	
	UPROPERTY()
	FTransform Local = FTransform::Identity;
};

UENUM()
enum class EBoneClipboardDataType : uint8
{
	Invalid,
	Bones,
	BoneTransforms
};

USTRUCT()
struct FHierarchyClipboardData
{
	GENERATED_USTRUCT_BODY()
	
	UPROPERTY()
	TArray<FBoneClipboardData> Bones;
	
	UPROPERTY()
	EBoneClipboardDataType Type = EBoneClipboardDataType::Invalid;
};
