// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "MeshDescription.h"
#include "ReferenceSkeleton.h"
#include "Misc/EnumClassFlags.h"

#include "SkeletonModifier.generated.h"

#define UE_API SKELETALMESHMODIFIERS_API

class USkeletalMesh;
class FName;
struct FMeshDescription;
struct FReferenceSkeleton;
struct FTransformComposer;
class UDynamicMesh;

enum class ESkeletalMeshModificationType : uint32
{
	None = 0x000,

	BonesAdded = 0x001,
	BonesRemoved = 0x002,
	BonesRenamed = 0x004,
	TransformChanged = 0x008,
	HierarchyChanged = 0x010,

	IndicesUpdated = HierarchyChanged | BonesRemoved,

	SkeletonUpdated = BonesAdded | BonesRenamed | HierarchyChanged
};
ENUM_CLASS_FLAGS(ESkeletalMeshModificationType)

enum class ESkeletonModificationType : uint32
{
	Cancel = 0x000,
	None = 0x001,
	
	SimpleMerge = 0x002,
	DuplicateAndMerge = 0x004,
	FullMerge = 0x008,
	FullMergeAll = 0x010,

	DeepMerge = DuplicateAndMerge | FullMerge | FullMergeAll,
	
	DoUpdate = SimpleMerge | DeepMerge
};
ENUM_CLASS_FLAGS(ESkeletonModificationType)

/**
 * FMirrorOptions
 */

USTRUCT(BlueprintType)
struct FMirrorOptions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mirror")
	TEnumAsByte<EAxis::Type> MirrorAxis = EAxis::X;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mirror")
	bool bMirrorRotation = true;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mirror")
	FString LeftString = TEXT("_l");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mirror")
	FString RightString = TEXT("_r");

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Mirror")
	bool bMirrorChildren = true;

	UE_API FTransform MirrorTransform(const FTransform& InGlobalTransform) const;
	UE_API FVector MirrorVector(const FVector& InVector) const;
};

/**
 * FOrientOptions
 */

UENUM()
enum class EOrientAxis : uint8
{
	None,
	PositiveX	UMETA(DisplayName = "+X", ToolTip = "Orients axis in the positive X direction"),
	PositiveY	UMETA(DisplayName = "+Y", ToolTip = "Orients axis in the positive Y direction"),
	PositiveZ	UMETA(DisplayName = "+Z", ToolTip = "Orients axis in the positive Z direction"),
	NegativeX	UMETA(DisplayName = "-X", ToolTip = "Orients axis in the negative X direction"),
	NegativeY	UMETA(DisplayName = "-Y", ToolTip = "Orients axis in the negative Y direction"),
	NegativeZ	UMETA(DisplayName = "-Z", ToolTip = "Orients axis in the negative Z direction"),
};

USTRUCT(BlueprintType)
struct FOrientOptions
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Orient")
	EOrientAxis Primary = EOrientAxis::PositiveX;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Orient", meta=(EditCondition="Primary != EOrientAxis::None"))
	EOrientAxis Secondary = EOrientAxis::PositiveY;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Orient", meta=(EditCondition="Primary != EOrientAxis::None"))
	bool bUsePlaneAsSecondary = true;
	
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Orient", meta=(EditCondition="Primary != EOrientAxis::None && !bUsePlaneAsSecondary"))
	FVector SecondaryTarget = FVector::YAxisVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Orient", meta=(EditCondition="Primary != EOrientAxis::None"))
	bool bOrientChildren = true;

	UE_API FTransform OrientTransform(const FVector& InPrimaryTarget, const FTransform& InTransform) const;
};

/**
 * FSkeletalMeshSkeletonModifier
 */

UCLASS(MinimalAPI, BlueprintType)
class USkeletonModifier : public UObject
{
	GENERATED_BODY()
	
public:

	UFUNCTION(BlueprintCallable, Category = "Mesh")
	UE_API bool SetSkeletalMesh(USkeletalMesh* InSkeletalMesh);
	
	/**
	 * Applies the skeleton modifications to the skeletal mesh.
	 * @return true if commit succeeded.
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh")
	UE_API bool CommitSkeletonToSkeletalMesh();


	UE_API bool SetDynamicMesh(UDynamicMesh* InDynamicMesh);

	UE_API bool SetReferenceSkeleton(const FReferenceSkeleton& InReferenceSkeleton);
	
	UE_API void SetReadOnly(bool bInReadOnly);
	UE_API bool IsReadOnly() const;
	
	/**
	 * Applies the skeleton modifications to the dynamic mesh.
	 * @return true if commit succeeded.
	 */
	UE_API bool CommitSkeletonToDynamicMesh();

	
	/** Creates a new bone in the skeleton hierarchy at desired transform
	 *  @param InBoneName The new bone's name.
	 *  @param InParentName The new bone parent's name. 
	 *  @param InTransform The default local transform in the parent space.
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool AddBone(const FName InBoneName, const FName InParentName, const FTransform& InTransform);
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool AddBones(const TArray<FName>& InBonesName, const TArray<FName>& InParentsName, const TArray<FTransform>& InTransforms);

	/** Mirror bones
	 *  @param InBoneName The new bone's name.
	 *  @param InOptions The mirroring options
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool MirrorBone(const FName InBoneName, const FMirrorOptions& InOptions = FMirrorOptions());
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool MirrorBones(const TArray<FName>& InBonesName, const FMirrorOptions& InOptions = FMirrorOptions());

	/** Sets the bone the desired local transform
	 *  @param InBoneName The new bone's name that needs to be moved.
	 *  @param InNewTransform The new local transform in the bone's parent space.
	 *  @param bMoveChildren Propagate new transform to children
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool SetBoneTransform(const FName InBoneName, const FTransform& InNewTransform, const bool bMoveChildren);
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool SetBonesTransforms(const TArray<FName>& InBoneNames, const TArray<FTransform>& InNewTransforms, const bool bMoveChildren);

	/** Remove a bone in the skeleton hierarchy
	 *  @param InBoneName The new bone's name.
	 *  @param bRemoveChildren Remove children recursively.
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool RemoveBone(const FName InBoneName, const bool bRemoveChildren);
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool RemoveBones(const TArray<FName>& InBoneNames, const bool bRemoveChildren);

	/** Rename bones
	 *  @param InOldBoneName The current bone's name.
	 *  @param InNewBoneName The new bone's name.
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool RenameBone(const FName InOldBoneName, const FName InNewBoneName);
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool RenameBones(const TArray<FName>& InOldBoneNames, const TArray<FName>& InNewBoneNames);
	
	/** Parent bones
	 *  @param InBoneName The current bone's name.
	 *  @param InParentName The new parent's name (Name_NONE to unparent).
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool ParentBone(const FName InBoneName, const FName InParentName);
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool ParentBones(const TArray<FName>& InBoneNames, const TArray<FName>& InParentNames);

	/** Align bones
	 *  @param InBoneName The current bone's name.
	 *  @param InOptions The orienting options
	 *  @return \c true if the operation succeeded, false otherwise. 
	 */
	 UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool OrientBone(const FName InBoneName, const FOrientOptions& InOptions = FOrientOptions());
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API bool OrientBones(const TArray<FName>& InBoneNames, const FOrientOptions& InOptions = FOrientOptions());

	/** Get Bone Transform
	 *  @param InBoneName The current bone's name.
	 *  @param bGlobal Whether it should return the parent space or global transform
	 *  @return \c The current bone's transform 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API FTransform GetBoneTransform(const FName InBoneName, const bool bGlobal = false) const;

	/** Get Parent Name
	 *  @param InBoneName The current bone's name.
	 *  @return \c The current bone's parent name 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API FName GetParentName(const FName InBoneName) const;

	/** Get Children Names
	 *  @param InBoneName The parent's name.
	 *  @param bRecursive If set to true grand-children will also be added recursively
	 *  @return \c The children names list 
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API TArray<FName> GetChildrenNames(const FName InBoneName, const bool bRecursive = false) const;

	/** Get All Bone Names
	 *  @return \c All bone names list
	 */
	UFUNCTION(BlueprintCallable, Category = "Skeleton")
	UE_API TArray<FName> GetAllBoneNames() const;
	


	UE_API void ExternalUpdate(const FReferenceSkeleton& InRefSkeleton, const TArray<int32>& InIndexTracker);
	UE_API void ExternalUpdate(const FReferenceSkeleton& InRefSkeleton);
	
	UE_API FName GetUniqueName(const FName InBoneName, const TArray<FName>& InBoneNames = {}) const;
	
	UE_API const FReferenceSkeleton& GetReferenceSkeleton() const;
	UE_API const TArray<int32>& GetBoneIndexTracker() const;
	
	UE_API const FTransform& GetTransform(const int32 InBoneIndex, const bool bGlobal = false) const;
	
private:

	UE_API bool IsReferenceSkeletonValid(const bool bLog = true) const;

	UE_API void UpdateBoneTracker(const TArray<FMeshBoneInfo>& InOtherInfos);
	
	// mirroring functions
	UE_API void GetBonesToMirror(const TArray<FName>& InBonesName, const FMirrorOptions& InOptions, TArray<int32>& OutBonesToMirror) const;
	UE_API void GetMirroredNames(const TArray<int32>& InBonesToMirror, const FMirrorOptions& InOptions, TArray<FName>& OutBonesName) const;
	UE_API void GetMirroredBones(const TArray<int32>& InBonesToMirror, const TArray<FName>& InMirroredNames, TArray<int32>& OutMirroredBones);
	UE_API void GetMirroredTransforms(	const TArray<int32>& InBonesToMirror, const TArray<int32>& InMirroredBones,
								const FMirrorOptions& InOptions, TArray<FTransform>& OutMirroredTransforms) const;

	// orient function
	UE_API void GetBonesToOrient(const TArray<FName>& InBonesName, const FOrientOptions& InOptions, TArray<int32>& OutBonesToOrient) const;

	// pre/post commit
	UE_API ESkeletalMeshModificationType PreCommitSkeletalMesh();
	UE_API ESkeletonModificationType PreCommitSkeleton(const ESkeletalMeshModificationType InSkeletalMeshModifications) const;
	UE_API void PostCommitSkeleton(const ESkeletonModificationType InSkeletonModifications) const;

	UE_API void EnsureSingleRoot();
	
	// does the skeleton have any references other than the skeletal mesh currently being edited
	UE_API bool HasAnyOtherReferences(const bool bSkeletalMeshOnly) const;

	// update mesh description on commit
	UE_API void CommitChangesToMeshDescription(const ESkeletalMeshModificationType InSkeletalMeshModifications);
	
	// notification
	UE_API void NotifyFromSkeletonChanges() const;
	
	UPROPERTY(transient)
	TWeakObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;
	
	TUniquePtr<FMeshDescription> MeshDescription;

	UPROPERTY(transient)
	TWeakObjectPtr<UDynamicMesh> WeakDynamicMesh = nullptr;

	TUniquePtr<FReferenceSkeleton> ReferenceSkeleton;
	TUniquePtr<FTransformComposer> TransformComposer;
	TArray<int32> BoneIndexTracker;
	bool bReadOnly = false;

public:
	
	// UPROPERTY(EditAnywhere, Category = Debug)
	bool bDebug = false;
};

struct FTransformComposer
{
	FTransformComposer(const FReferenceSkeleton& InRefSkeleton)
		: RefSkeleton(InRefSkeleton)
		, Transforms(InRefSkeleton.GetRawRefBonePose())
	{
		TransformCached.Init(false, Transforms.Num());
	}

	const FTransform& GetGlobalTransform(const uint32 Index)
	{
		if (!Transforms.IsValidIndex(Index))
		{
			return FTransform::Identity;
		}

		if (TransformCached[Index])
		{
			return Transforms[Index];
		}

		const int32 ParentIndex = RefSkeleton.GetRawParentIndex(Index);
		if (ParentIndex > INDEX_NONE)
		{
			Transforms[Index] *= GetGlobalTransform(ParentIndex);
		}

		TransformCached[Index] = true;
		return Transforms[Index];
	}

	void Invalidate(const uint32 Index = INDEX_NONE)
	{
		if (!TransformCached.IsValidIndex(Index))
		{
			Transforms = RefSkeleton.GetRawRefBonePose();
			TransformCached.Init(false, Transforms.Num());
			return;
		}
		
		Transforms[Index] = RefSkeleton.GetRawRefBonePose()[Index];
		TransformCached[Index] = false;

		TArray<int32> Children; RefSkeleton.GetRawDirectChildBones(Index, Children);
		for (const int32 ChildIndex: Children)
		{
			Invalidate(ChildIndex);
		}
	}

	const FReferenceSkeleton& RefSkeleton;
	TArray<FTransform> Transforms;
	TBitArray<> TransformCached;
};

UENUM()
enum class ESKeletalMeshMergeType : uint8
{
	New		UMETA(ToolTip = "Create a new skeleton asset from the current changes."),
	Merge	UMETA(ToolTip = "Merge the current changes to the existing skeleton asset. Note that this may invalidate dependent assets, such as animation clips, pose assets and animation blueprints."),
};

UCLASS(HideCategories=Object, MinimalAPI)
class USkeletalMeshMergeOptions : public UObject
{
	GENERATED_BODY()

public:

	/** Changes merge type (New or Merge) */
	UPROPERTY(EditAnywhere, Category="Merge")
	ESKeletalMeshMergeType MergeType = ESKeletalMeshMergeType::New;

	/** Also apply the changes to the skeletal meshes referencing the same skeleton */
	UPROPERTY(EditAnywhere, Category="Merge", meta=(EditCondition="MergeType==ESKeletalMeshMergeType::Merge"))
	bool bMergeAll = false;
};

#undef UE_API
