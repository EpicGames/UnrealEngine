// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "MeshDescription.h"
#include "Engine/SkeletalMesh.h"

#include "SkinWeightModifier.generated.h"

#define UE_API SKELETALMESHMODIFIERS_API

class USkeletalMesh;

/**
 * API used to modify skin weights on a Skeletal Mesh asset.
 * 
 * To use:
 * 1. Instantiate an instance of USkinWeightModifier
 * 2. Call "SetSkeletalMesh(MyMeshAsset)", passing in the skeletal mesh you want to edit weights on
 * 3. Use Get/Set weights functions, and Normalize/Prune to edit the weights as desired
 * 4. When ready to commit to the asset, call CommitWeightsToSkeletalMesh() and save the asset if desired
 *
 * This API can be used from C++, Blueprint or Python. Here is a sample usage of the API in Python:

import unreal

# create a weight modifier for a given skeletal mesh
skel_mesh = unreal.EditorAssetLibrary().load_asset("/Game/Characters/Wolf/Meshes/SK_Wolf")
weight_modifier = unreal.SkinWeightModifier()
weight_modifier.set_skeletal_mesh(skel_mesh)

# get weight of vertex 1234
vertex_weights = weight_modifier.get_vertex_weights(1234)
print(vertex_weights)

# remove neck2 as an influence on this vertex
vertex_weights.pop("neck2")
weight_modifier.set_vertex_weights(vertex_weights, True)
print(vertex_weights)

# commit change to the skeletal mesh
weight_modifier.commit_weights_to_skeletal_mesh()

 * OUTPUT:
 * {"head": 0.6, "neck1": 0.3, "neck2": 0.1}
 * {"head": 0.6, "neck1": 0.3}
 * 
 * In Python, the per-vertex weights are stored as a dictionary mapping Bone Names to float weight values.
 *
 * The "SetVertexWeights()" function expects the same data structure. You can add/remove/edit influences as needed.
 * The SetVertexWeights() function does not normalize the weights. So you can make multiple modifications
 * and call NormalizeVertexWeights() or NormalizeAllWeights() as desired.
 *
 * The PruneVertexWeights() and EnforceMaxInfluences() functions can be used to trim small influences
 * and clamp the total number of influences per vertex as needed.
 * 
 * Note that it is not required to normalize the weights by calling any of the normalize functions. Or manually before
 * calling SetVertexWeights(). Committing the weights to the skeletal mesh will always enforce normalization.
 *
 * Though it may be useful to normalize while editing.
 */
UCLASS(MinimalAPI, BlueprintType)
class USkinWeightModifier : public UObject
{
	GENERATED_BODY()

public:
	
	/**
	 * Call this first to load the weights for a skeletal mesh for fast editing.
	 * @param InMesh - The skeletal mesh asset to load for weight editing
	 * @return bool - true if the mesh weights were successfully loaded.
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh")
	UE_API bool SetSkeletalMesh(USkeletalMesh* InMesh);

	/**
	 * Actually applies the weight modifications to the skeletal mesh. This action creates an undo transaction.
	 * The skeletal mesh asset will be dirtied, but it is up to the user to save the asset if required.
	 * @return true if weights were applied to a skeletal mesh, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh")
	UE_API bool CommitWeightsToSkeletalMesh();

	/**
	 * Get a reference to the skeletal mesh that was loaded
	 * @return USkeletalMesh* - the mesh that was loaded, or null
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh")
	USkeletalMesh* GetSkeletalMesh() { return Mesh; }

	/**
	 * Get the total number of vertices in the skeletal mesh.
	 * @return int, number of vertices
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh Query")
	UE_API int32 GetNumVertices();

	/**
	 * Get an array of all bone names in the skeletal mesh.
	 * @return array of bone names
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh Query")
	UE_API TArray<FName> GetAllBoneNames();

	/**
	 * Get all bone weights for a single vertex.
	 * @param VertexID the index of the vertex
	 * @return a map of Bone Name to weight values for all bones that influence the specified vertex.
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API TMap<FName, float> GetVertexWeights(const int32 VertexID);

	/**
	 * Set bone weights for a single vertex. The weights are stored as supplied and not normalized until
	 * either "CommitWeightsToSkeletalMesh()" or "NormalizeVertexWeights()" is called.
	 * @param VertexID the index of the vertex
	 * @param InWeights a map of Bone-Name to Weight for all bones that influence the specified vertex, ie {"Head": 0.6, "Neck": 0.4}
	 * @param bReplaceAll if true, all weights on this vertex will be replaced with the input weights. Default is false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API bool SetVertexWeights(
		const int32 VertexID,
		const TMap<FName, float>& InWeights,
		const bool bReplaceAll=false);

	/**
	 * Normalize weights on the specified vertex.
	 * @param VertexID the index of the vertex to normalize weights on
	 * @return true if normalization was performed, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API bool NormalizeVertexWeights(const int32 VertexID);

	/**
	 * Normalize weights on all vertices in the mesh.
	 * @return true if normalization was performed, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API bool NormalizeAllWeights();

	/**
	 * Strips out smallest influences to ensure each vertex does not have weight on more influences than MaxInfluences.
	 * Influences with the smallest weight are culled first.
	 * @param MaxInfluences The maximum number of influences to allow for each vertex in the mesh. If -1 is passed, will use the project-wide MaxInfluences amount.
	 * @return true if influences were removed, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API bool EnforceMaxInfluences(const int32 MaxInfluences=-1);

	/**
	 * Remove all weights below the given threshold value, on the given vertex.
	 * Influences that are pruned will no longer receive weight from normalization.
	 * @param VertexID the index of the vertex to prune weights on
	 * @param WeightThreshold vertex weights below this value will be removed. 
	 * @return true if influences were removed, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API bool PruneVertexWeights(const int32 VertexID, const float WeightThreshold);

	/**
	 * Remove all weights below the given threshold value, on all vertices.
	 * @param WeightThreshold vertex weights below this value will be removed. 
	 * @return true if influences were removed, false otherwise
	 */
	UFUNCTION(BlueprintCallable, Category = "Weights")
	UE_API bool PruneAllWeights(const float WeightThreshold);

private:

	// the skeletal mesh that was loaded by SetSkeletalMesh
	UPROPERTY()
	TObjectPtr<USkeletalMesh> Mesh;
	TUniquePtr<FMeshDescription> MeshDescription;

	// weights stored in-memory for editing before being committed to the skeletal mesh
	TArray<TMap<FName, float>> Weights;

	// fast internal procedures (do not validate inputs)
	UE_API bool NormalizeVertexInternal(const int32 VertexID);
	UE_API void EnforceMaxInfluenceInternal(const int32 VertexID, const int32 Max);
	UE_API bool PruneVertexWeightInternal(const int32 VertexID, const float WeightThreshold);

	// reset all internal data when loading a new mesh
	UE_API void Reset();

	static constexpr int32 LODIndex = 0;
};

#undef UE_API
