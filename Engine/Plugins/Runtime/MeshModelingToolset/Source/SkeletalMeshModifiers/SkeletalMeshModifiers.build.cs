// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SkeletalMeshModifiers : ModuleRules
{
	public SkeletalMeshModifiers(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
            );
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"MeshDescription",
				"SkeletalMeshDescription",
				"SkeletalMeshUtilitiesCommon",
				"RenderCore",
				"CoreUObject",
				"Engine",
				"ApplicationCore",
				"SlateCore",
				"Slate",
				"UnrealEd",
				"ToolWidgets",
				"GeometryFramework",
				"GeometryCore"
			}
			);


		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
